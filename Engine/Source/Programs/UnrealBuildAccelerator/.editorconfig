# EditorConfig is awesome: https://EditorConfig.org

# top-most EditorConfig file
root = true

# Baseline
[*]
charset = utf-8
indent_style = tab
indent_size = 4
tab_width = 4
trim_trailing_whitespace = false
max_line_length = 120

# MSBuild
[*.{csproj,proj,projitems,shproj,fsproj,target,props}]
indent_style = space
indent_size = 2

# XML config files
[*.{config,nuspec,resx,natvis}]
indent_style = space
indent_size = 2

# Python files
[*.py]
indent_style = space
indent_size = 4

# YAML files
[*.yaml]
indent_style = space
indent_size = 2
