.sarif/
**/.vs/
vcpkg_installed/
x64/
arm64/
Bin/

**/*.vcxproj.user
**/obj/
**/bin/
**/*.slnx
**/*.unrealvs
**/published/