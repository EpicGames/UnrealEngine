// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaApplication.h"
#include "UbaAWS.h"
#include "UbaConfig.h"
#include "UbaDirectoryIterator.h"
#include "UbaNetworkBackendMemory.h"
#include "UbaNetworkBackendQuic.h"
#include "UbaNetworkBackendTcp.h"
#include "UbaNetworkMessage.h"
#include "UbaNetworkServer.h"
#include "UbaProcessStartInfo.h"
#include "UbaSessionClient.h"
#include "UbaStorageClient.h"
#include "UbaStorageProxy.h"
#include "UbaSentry.h"
#include "UbaVersion.h"

#if defined(UBA_USE_SENTRY)
#pragma comment (lib, "WinHttp.lib")
#pragma comment (lib, "Version.lib")
#pragma comment (lib, "sentry.lib")
#pragma comment (lib, "crashpad_client.lib")
#pragma comment (lib, "crashpad_compat.lib")
#pragma comment (lib, "crashpad_getopt.lib")
#pragma comment (lib, "crashpad_handler_lib.lib")
#pragma comment (lib, "crashpad_minidump.lib")
#pragma comment (lib, "crashpad_snapshot.lib")
#pragma comment (lib, "crashpad_tools.lib")
#pragma comment (lib, "crashpad_util.lib")
#pragma comment (lib, "mini_chromium.lib")
#endif

#if PLATFORM_WINDOWS
#define UBA_AUTO_UPDATE 1
#else
#define UBA_AUTO_UPDATE 0
#endif
//#include <dbghelp.h>
//#pragma comment (lib, "Dbghelp.lib")

namespace uba
{
	const tchar*		Version = GetVersionString();
	constexpr u32	DefaultCapacityGb = 20;
	constexpr u32	DefaultListenTimeout = 5;
	const tchar*	DefaultRootDir = [](){
		static tchar buf[256];
		if constexpr (IsWindows)
			ExpandEnvironmentStringsW(TC("%ProgramData%\\Epic\\" UE_APP_NAME), buf, sizeof(buf));
		else
			GetFullPathNameW(TC("~/" UE_APP_NAME), sizeof_array(buf), buf, nullptr);
		return buf;
	}();
	u32				DefaultProcessorCount = []() { return GetLogicalProcessorCount(); }();
	u32				DefaultMaxConnectionCount = 4;

	bool PrintHelp(const tchar* message)
	{
		LoggerWithWriter logger(g_consoleLogWriter, TC(""));
		if (*message)
		{
			logger.Info(TC(""));
			logger.Error(TC("%s"), message);
		}
		StringBuffer<> name;
		GetComputerNameW(name);
		logger.Info(TC(""));
		logger.Info(TC("-------------------------------------------"));
		logger.Info(TC("   UbaAgent v%s%s"), Version, (IsArmBinary ? TC(" (ARM64)") : TC("")));
		logger.Info(TC("-------------------------------------------"));
		logger.Info(TC(""));
		logger.Info(TC("  When started UbaAgent will keep trying to connect to provided host address."));
		logger.Info(TC("  Once connected it will start helping out. Nothing else is needed :)"));
		logger.Info(TC(""));
		logger.Info(TC("  -dir=<rootdir>          The directory used to store data. Defaults to \"%s\""), DefaultRootDir);
		logger.Info(TC("  -host=<host>[:<port>]   The ip/name and port (default: %u) of the machine we want to help"), DefaultPort);
		logger.Info(TC("  -listen[=port]          Agent will listen for connections on port (default: %u) and help when connected"), DefaultPort);
		logger.Info(TC("  -listenTimeout=<sec>    Number of seconds agent will listen for host before giving up (default: %u)"), DefaultListenTimeout);
		logger.Info(TC("  -proxyport=<port>       Which port that agent will use if being assigned to be proxy for other agents (default: %u)"), DefaultStorageProxyPort);
		logger.Info(TC("  -proxyaddr=<addr>       Which address that agent will use if being assigned to be proxy for other agents. If not set it will automatically fetch"));
		logger.Info(TC("  -maxcpu=<number>        Max number of processes that can be started. Defaults to \"%u\" on this machine"), DefaultProcessorCount);
		logger.Info(TC("  -mulcpu=<number>        This value multiplies with number of cpu to figure out max cpu. Defaults to 1.0"));
		logger.Info(TC("  -mincon=<number>        Min number of connections for agent. Defaults to 1 (amount up to max will depend on ping)"));
		logger.Info(TC("  -maxcon=<number>        Max number of connections that can be started by agent. Defaults to \"%u\" (amount up to max will depend on ping)"), DefaultMaxConnectionCount);
		logger.Info(TC("  -maxworkers=<number>    Max number of workers is started by agent. Defaults to \"%u\""), DefaultProcessorCount);
		logger.Info(TC("  -capacity=<gigaby>      Capacity of local store. Defaults to %u gigabytes"), DefaultCapacityGb);
		logger.Info(TC("  -config=<file>          Config file that contains options for various systems"));
		logger.Info(TC("  -quic                   Use Quic instead of tcp backend."));
		logger.Info(TC("  -name=<name>            The identifier of this agent. Defaults to \"%s\" on this machine"), name.data);
		logger.Info(TC("  -verbose                Print debug information to console"));
		logger.Info(TC("  -log                    Log all processes detouring information to file (only works with debug builds)"));
		logger.Info(TC("  -nocustomalloc          Disable custom allocator for processes. If you see odd crashes this can be tested"));
		logger.Info(TC("  -storeraw               Disable compression of storage. This will use more storage and might improve performance"));
		logger.Info(TC("  -sendraw                Disable compression of send. This will use more bandwidth but less cpu"));
		logger.Info(TC("  -sendsize               Max size of messages being sent from client to server (does not affect server to client)"));
		logger.Info(TC("  -named=<name>           Use named events and file mappings by providing the base name in this option"));
		logger.Info(TC("  -nopoll                 Does not keep polling for work; attempts to connect once then exits"));
		logger.Info(TC("  -nostore                Does not use storage to store files (with a few exceptions such as binaries)"));
		logger.Info(TC("  -nodetoursdownload      Does not download UbaDetours library from server and instead use local."));
		logger.Info(TC("  -resetstore             Delete all cas"));
		logger.Info(TC("  -quiet                  Does not output any logging in console"));
		logger.Info(TC("  -maxidle=<seconds>      Max time agent will idle before disconnecting. Ignored if -nopoll is not set"));
		logger.Info(TC("  -binasversion           Will use binaries as version. This will cause updates everytime binaries change on host side"));
		logger.Info(TC("  -summary                Print summary at the end of a session"));
		logger.Info(TC("  -eventfile=<file>       File containing external events to agent. Things like machine is about to be terminated etc"));
		logger.Info(TC("  -sentry                 Enable sentry"));
		logger.Info(TC("  -zone                   Set the zone this machine exists in. This info is used to figure out if proxies should be created."));
		logger.Info(TC("  -version                Prints the version for this executable."));
		logger.Info(TC("  -noproxy                Does not allow this agent to be a storage proxy for other agents"));
		logger.Info(TC("  -proxyuselocalstorage   Storage proxy will use local storage to see if files exist"));
		logger.Info(TC("  -nocloud                Will not try to connect to cloud meta data server (this can take time during first startup)"));
		logger.Info(TC("  -notips                 Disable printing of tips at startup"));
		logger.Info(TC("  -killrandom             Kills random process and exit session"));
		logger.Info(TC("  -memwait=<percent>      The amount of memory needed to spawn a process. Set this to 100 to disable. Defaults to 80%%"));
		logger.Info(TC("  -memkill=<percent>      The amount of memory needed before processes starts to be killed. Set this to 100 to disable. Defaults to 90%%"));
		logger.Info(TC("  -crypto=<key>           32 character (16 bytes) crypto key used for secure network transfer"));
		logger.Info(TC("  -resendcas              Will try to send same cas multiple times (set this to true if server is allowed to remove cas files)"));
		logger.Info(TC("  -populateCas=<dir>      Prepopulate cas database with files in dir. If files needed exists on machine this can be an optimization"));
		logger.Info(TC("  -description            Add more info about the agent that will show in the trace log when hovering over session"));
		logger.Info(TC("  -disablecrawler         Disables file crawler for known process types (clang/msvc)"));
		logger.Info(TC("  -killtcphogs            If failing to bind listen socket UbaAgent will attempt to kill processes holding it and then retry"));
		#if PLATFORM_MAC
		logger.Info(TC("  -populateCasFromXcodeVersion=<version>   Prepopulate cas database with files from local xcode installation that matches the version."));
		logger.Info(TC("  -populateCasFromAllXcodes   Prepopulate cas database with files from local xcode installation that matches the version."));
		#elif PLATFORM_WINDOWS
		logger.Info(TC("  -useOverlappedSend      Enable/Disable overlapped send for tcp"));
		logger.Info(TC("  -useIocp[=workerCount]  Enable/Disable iocp for tcp. Defaults to 4 workers is not set"));
		#endif
		logger.Info(TC(""));
		return false;
	}

	bool UsingNativeSockets();

	ReaderWriterLock* g_exitLock = new ReaderWriterLock();
	LoggerWithWriter* g_logger;
	SessionClient* g_sessionClient;
	Atomic<bool> g_shouldExit = false;
	Atomic<bool> g_ctrlPressed = false;

	bool ShouldExit()
	{
		return g_shouldExit || IsEscapePressed();
	}

	void CtrlBreakPressed()
	{
		if (g_ctrlPressed)
			FatalError(13, TC("Force terminate"));

		g_shouldExit = true;
		g_ctrlPressed = true;

		g_exitLock->Enter();
		if (g_logger)
			g_logger->Info(TC("  Exiting..."));
		if (g_sessionClient)
			g_sessionClient->Stop(false);
		g_exitLock->Leave();
	}

	#if PLATFORM_WINDOWS
	BOOL ConsoleHandler(DWORD signal)
	{
		CtrlBreakPressed();
		return TRUE;
	}
	#else
	void ConsoleHandler(int sig)
	{
		CtrlBreakPressed();
	}
	#endif

	StringBuffer<> g_rootDir(DefaultRootDir);

#if UBA_AUTO_UPDATE
	const tchar* g_ubaAgentBinaries[] = { UBA_AGENT_EXECUTABLE }; //, UBA_DETOURS_LIBRARY };

	bool DownloadBinaries(StorageClient& storageClient, CasKey* keys)
	{
		StringBuffer<256> binDir(g_rootDir);
		binDir.Append(TCV("\\binaries\\"));
		storageClient.CreateDirectory(binDir.data);
		u32 index = 0;
		for (auto file : g_ubaAgentBinaries)
		{
			Storage::RetrieveResult result;
			if (!storageClient.RetrieveCasFile(result, keys[index++], {}, file))
				return false;
			StringBuffer<256> fullFile(binDir);
			fullFile.Append(file);
			if (!storageClient.CopyOrLink(result.casKey, fullFile.data, DefaultAttributes()))
				return false;
		}
		return true;
	}

	bool LaunchProcess(tchar* args)
	{
		STARTUPINFOW si;
		ZeroMemory(&si, sizeof(si));
		si.cb = sizeof(si);
		PROCESS_INFORMATION pi;
		ZeroMemory(&pi, sizeof(pi));
		if (!CreateProcessW(NULL, args, NULL, NULL, false, 0, NULL, NULL, &si, &pi))
			return false;
		CloseHandle(pi.hProcess);
		CloseHandle(pi.hThread);
		return true;
	}

	bool LaunchTemp(Logger& logger, int argc, tchar* argv[])
	{
		StringBuffer<256> currentDir;
		if (!GetDirectoryOfCurrentModule(logger, currentDir))
			return false;

		StringBuffer<> args;
		args.Append(g_rootDir).Append(TC("\\binaries\\") UBA_AGENT_EXECUTABLE);
		args.Append(TCV(" -relaunch=\"")).Append(currentDir).Append(TCV("\""));
		args.Appendf(TC(" -waitid=%u"), GetCurrentProcessId());

		for (int i = 1; i != argc; ++i)
			args.Append(' ').Append(argv[i]);

		return LaunchProcess(args.data);
	}

	bool WaitForProcess(u32 procId)
	{
		HANDLE ph = OpenProcess(SYNCHRONIZE, TRUE, procId);
		if (!ph)
			return true;
		bool success = WaitForSingleObject(ph, 10000) == WAIT_OBJECT_0;
		CloseHandle(ph);
		return success;
	}

	bool LaunchReal(Logger& logger, StringBufferBase& relaunchPath, int argc, tchar* argv[])
	{
		StringBuffer<256> currentDir;
		if (!GetDirectoryOfCurrentModule(logger, currentDir))
			return false;
		logger.Info(TC("Copying new binaries..."));
		for (auto file : g_ubaAgentBinaries)
		{
			StringBuffer<256> from(currentDir);
			from.Append('\\').Append(file);

			StringBuffer<256> to(relaunchPath.data);
			to.Append('\\').Append(file);

			if (!uba::CopyFileW(from.data, to.data, false))
				return logger.Error(TC("Failed to copy file for relaunch"));
		}

		StringBuffer<> args;
		args.Append(relaunchPath).Append(PathSeparator).Append(UBA_AGENT_EXECUTABLE);
		//args.Appendf(TC(" -waitid=%u"), GetCurrentProcessId());

		for (int i = 1; i != argc; ++i)
			if (!StartsWith(argv[i], TC("-relaunch")) && !StartsWith(argv[i], TC("-waitid")))
				args.Append(' ').Append(argv[i]);
		logger.Info(TC("Relaunching new %s..."), UBA_AGENT_EXECUTABLE);
		logger.Info(TC(""));
		return LaunchProcess(args.data);
	}
#endif // UBA_AUTO_UPDATE

	bool IsTerminating(Logger& logger, const tchar* eventFile, StringBufferBase& outReason, u64& outTerminationTimeMs)
	{
		if (!*eventFile)
			return false;

		u64 fileSize;
		if (!FileExists(logger, eventFile, &fileSize))
			return false;

		outTerminationTimeMs = 0;
		Sleep(1000);

		FileHandle fileHandle;
		if (!OpenFileSequentialRead(logger, eventFile, fileHandle))
			return true; // Fail to open the file we treat as instant termination

		auto g = MakeGuard([&]() { CloseFile(eventFile, fileHandle); });
		char buffer[2048];
		u64 toRead = Min(fileSize, u64(sizeof(buffer) - 1));
		if (!ReadFile(logger, eventFile, fileHandle, buffer, toRead))
			return true; // Fail to read the file we treat as instant termination

		buffer[toRead] = 0;
		StringBuffer<> reason;
		u64 terminateTimeMsUtc = 0;

		char* lineBegin = buffer;
		u32 lineIndex = 0;
		bool loop = true;
		while (loop)
		{
			char* lineEnd = strchr(lineBegin, '\n');
			if (lineEnd)
			{
				if (lineBegin < lineEnd && lineEnd[-1] == '\r')
					lineEnd[-1] = 0;
				else
					lineEnd[0] = 0;
			}
			else
				loop = false;

			switch (lineIndex)
			{
			case 0: // version
				if (strcmp(lineBegin, "v1") != 0)
					loop = false;
				break;
			case 1: // Relative time
				//strtoull(lineBegin, nullptr, 10);
				break;
			case 2: // Absolute time
				terminateTimeMsUtc = strtoull(lineBegin, nullptr, 10);
				break;
			case 3: // reason
				outReason.Appendf(PERCENT_HS, lineBegin);
				break;
			}

			lineBegin = lineEnd + 1;
			++lineIndex;
		}

		if (terminateTimeMsUtc != 0)
		{
			u64 nowMsUtc = time(0) * 1000;
			if (terminateTimeMsUtc > nowMsUtc)
			{
				u64 relativeTime = terminateTimeMsUtc - nowMsUtc;
				outTerminationTimeMs = relativeTime;
			}
		}
		return true;
	}

	bool WrappedMain(int argc, tchar*argv[])
	{
		u32 maxProcessCount = DefaultProcessorCount;
		u32 maxWorkerCount = DefaultProcessorCount;
		float mulProcessValue = 1.0f;
		u32 minConnectionCount = 1;
		u32 maxConnectionCount = DefaultMaxConnectionCount;
		u32 storageCapacityGb = DefaultCapacityGb;
		StringBuffer<256> host;
		TString named;
		StringBuffer<512> relaunchPath;
		StringBuffer<256> eventFile;
		TString configFile;
		TString command;
		u16 port = DefaultPort;
		u16 proxyPort = DefaultStorageProxyPort;
		TString proxyAddr;
		TString agentName;
		bool useListen = false;
		bool logToFile = false;
		bool storeCompressed = true;
		bool sendCompressed = true;
		bool disableCustomAllocator = false;
		bool useBinariesAsVersion = false;
		bool useQuic = false;
		bool poll = true;
		bool allowProxy = true;
		bool proxyUseLocalStorage = false;
		bool couldBeCloud = true;
		bool printTips = true;
		bool useStorage = true;
		bool resetStore = false;
		bool quiet = false;
		bool verbose = false;
		bool printSummary = false;
		bool killRandom = false;
		bool useCrawler = true;
		bool useOverlappedSend = true;
		u32 iocpWorkerCount = 0;
		bool downloadDetoursLib = true;
		bool useExceptionHandler = true;
		bool resendCas = false;
		TString sentryUrl;
		StringBuffer<128> zone;
		u32 maxIdleSeconds = ~0u;
		u32 sendSize = SendDefaultSize;
		u32 waitProcessId = ~0u;
		u32 memWaitLoadPercent = 80;
		u32 memKillLoadPercent = 90;
		u32 listenTimeoutSec = DefaultListenTimeout;
		u8 crypto[16];
		bool hasCrypto = false;
		Vector<TString> populateCasDirs;
		TString description;

		#if PLATFORM_MAC
		StringBuffer<32> populateCasFromXcodeVersion;
		bool populateCasFromAllXcodes = false;
		#endif
		bool killTcpHogs = false;

		for (int i=1; i!=argc; ++i)
		{
			StringBuffer<> name;
			StringBuffer<> value;

			auto parseValue = [&](auto& out, bool allowEmpty = false)
				{
					if (value.IsEmpty())
						return allowEmpty ? true : PrintHelp(StringBuffer().Appendf(TC("%s needs a value"), name.data).data);
					if (!ExpandEnvironmentVariables(value, PrintHelp))
						return false;
					std::decay_t<decltype(out)> temp;
					if (value.Parse(temp))
						out = temp;
					else
						LoggerWithWriter(g_consoleLogWriter, TC("")).Warning(TC("Invalid value for %s, ignoring and will use default!"), name.data);
					return true;
				};

			if (const tchar* equals = TStrchr(argv[i],'='))
			{
				name.Append(argv[i], equals - argv[i]);
				value.Append(equals+1);
			}
			else
			{
				name.Append(argv[i]);
			}
		
			if (name.Equals(TCV("-verbose")))
			{
				verbose = true;
			}
			else if (name.Equals(TCV("-relaunch")))
			{
				relaunchPath.Append(value);
			}
			else if (name.Equals(TCV("-waitid")))
			{
				value.Parse(waitProcessId);
			}
			else if (name.Equals(TCV("-maxcpu")))
			{
				if (!parseValue(maxProcessCount))
					return false;
			}
			else if (name.Equals(TCV("-mulcpu")))
			{
				if (!parseValue(mulProcessValue))
					return false;
			}
			else if (name.Equals(TCV("-mincon")))
			{
				if (!parseValue(minConnectionCount))
					return false;
				if (minConnectionCount == 0)
					return PrintHelp(TC("Invalid value for -mincon"));
			}
			else if (name.Equals(TCV("-maxcon")) || name.Equals(TCV("-maxtcp")))
			{
				if (!parseValue(maxConnectionCount))
					return false;
				if (maxConnectionCount == 0)
					return PrintHelp(TC("Invalid value for -maxcon"));
			}
			else if (name.Equals(TCV("-maxworkers")))
			{
				if (!parseValue(maxWorkerCount))
					return false;
			}
			else if (name.Equals(TCV("-capacity")))
			{
				if (!parseValue(storageCapacityGb))
					return false;
			}
			else if (name.Equals(TCV("-config")))
			{
				if (!parseValue(configFile))
					return false;
			}
			else if (name.Equals(TCV("-host")))
			{
				if (const tchar* portIndex = value.First(':'))
				{
					StringBuffer<> portStr(portIndex + 1);
					if (!portStr.Parse(port))
						return PrintHelp(TC("Invalid value for port in -host"));
					value.Resize(portIndex - value.data);
				}
				if (value.IsEmpty())
					return PrintHelp(TC("-host needs a name/ip"));
				host.Append(value);
			}
			else if (name.Equals(TCV("-listen")))
			{
				if (!parseValue(port))
					return false;
				useListen = true;
			}
			else if (name.Equals(TCV("-listenTimeout")))
			{
				if (!parseValue(listenTimeoutSec))
					return false;
			}
			else if (name.Equals(TCV("-named")))
			{
				if (!parseValue(named))
					return false;
			}
			else if (name.Equals(TCV("-log")))
			{
				logToFile = true;
			}
			else if (name.Equals(TCV("-quiet")))
			{
				quiet = true;
			}
			else if (name.Equals(TCV("-nocustomalloc")))
			{
				disableCustomAllocator = true;
			}
			else if (name.Equals(TCV("-storeraw")))
			{
				storeCompressed = false;
			}
			else if (name.Equals(TCV("-sendraw")))
			{
				sendCompressed = false;
			}
			else if (name.Equals(TCV("-sendsize")))
			{
				if (!parseValue(sendSize))
					return false;
			}
			else if (name.Equals(TCV("-dir")))
			{
				if (value.IsEmpty())
					return PrintHelp(TC("-dir needs a value"));
				if (!ExpandEnvironmentVariables(value, PrintHelp))
					return false;
				if ((g_rootDir.count = GetFullPathNameW(value.Replace('\\', PathSeparator).data, g_rootDir.capacity, g_rootDir.data, nullptr)) == 0)
					return PrintHelp(StringBuffer<>().Appendf(TC("-dir has invalid path %s"), value.data).data);
			}
			else if (name.Equals(TCV("-name")))
			{
				if (!parseValue(agentName))
					return false;
			}
			else if (name.Equals(TCV("-nopoll")))
			{
				poll = false;
			}
			else if (name.Equals(TCV("-nostore")))
			{
				if constexpr (IsWindows) // Only supported on windows atm.
					useStorage = false;
			}
			else if (name.Equals(TCV("-nohandler")))
			{
				useExceptionHandler = false;
			}
			else if (name.Equals(TCV("-nodetoursdownload")))
			{
				downloadDetoursLib = false;
			}
			else if (name.Equals(TCV("-resetstore")))
			{
				resetStore = true;
			}
			else if (name.Equals(TCV("-binasversion")))
			{
				useBinariesAsVersion = true;
			}
			else if (name.Equals(TCV("-quic")))
			{
				#if !UBA_USE_QUIC
				return PrintHelp(TC("-quic not supported. Quic is not compiled into this binary"));
				#else
				useQuic = true;
				#endif
			}
			else if (name.Equals(TCV("-maxidle")))
			{
				if (!parseValue(maxIdleSeconds))
					return false;
			}
			else if (name.Equals(TCV("-proxyport")))
			{
				if (!parseValue(proxyPort))
					return false;
			}
			else if (name.Equals(TCV("-proxyaddr")))
			{
				if (!parseValue(proxyAddr))
					return false;
			}
			else if (name.Equals(TCV("-summary")))
			{
				printSummary = true;
			}
			else if (name.Equals(TCV("-eventfile")))
			{
				if (value.IsEmpty())
					return PrintHelp(TC("-eventfile needs a value"));
				if (!ExpandEnvironmentVariables(value, PrintHelp))
					return false;
				if ((eventFile.count = GetFullPathNameW(value.Replace('\\', PathSeparator).data, eventFile.capacity, eventFile.data, nullptr)) == 0)
					return PrintHelp(StringBuffer<>().Appendf(TC("-eventfile has invalid path %s"), value.data).data);
			}
			else if (name.Equals(TCV("-killrandom")))
			{
				killRandom = true;
			}
			else if (name.Equals(TCV("-disablecrawler")))
			{
				useCrawler = false;
			}
			else if (name.Equals(TCV("-usecrawler"))) // Ignored, flag has been inverted
			{
				useCrawler = true;
			}
			#if PLATFORM_WINDOWS
			else if (name.Equals(TCV("-useOverlappedSend")))
			{
				if (!parseValue(useOverlappedSend, true))
					return false;
			}
			else if (name.Equals(TCV("-useIocp")))
			{
				iocpWorkerCount = 4;
				if (!parseValue(iocpWorkerCount, true))
					return false;
			}
			#endif
			else if (name.Equals(TCV("-memwait")))
			{
				if (!parseValue(memWaitLoadPercent))
					return false;
				if (memWaitLoadPercent > 100)
					return PrintHelp(TC("Invalid value for -memwait"));
			}
			else if (name.Equals(TCV("-memkill")))
			{
				if (!parseValue(memKillLoadPercent))
					return false;
				if (memKillLoadPercent > 100)
					return PrintHelp(TC("Invalid value for -memkill"));
			}
			else if (name.Equals(TCV("-crypto")))
			{
				if (value.count != 32)
					return PrintHelp(TC("Invalid number of characters in crypto string. Should be 32"));
				((u64*)crypto)[0] = StringToValue(value.data, 16);
				((u64*)crypto)[1] = StringToValue(value.data + 16, 16);
				hasCrypto = true;
			}
			else if (name.Equals(TCV("-resendcas")))
			{
				resendCas = true;
			}
			else if (name.Equals(TCV("-populateCas")))
			{
				TString temp;
				if (!parseValue(temp))
					return false;
				populateCasDirs.push_back(temp);
			}
			#if PLATFORM_MAC
			else if (name.Equals(TCV("-populateCasFromXcodeVersion")))
			{
				TString temp;
				if (!parseValue(temp))
					return false;
				populateCasFromXcodeVersion.Append(temp);
			}
			else if (name.Equals(TCV("-populateCasFromAllXcodes")))
			{
				populateCasFromAllXcodes = true;
			}
			#endif
			else if (name.Equals(TCV("-killtcphogs")))
			{
				killTcpHogs = true;
			}
			else if (name.Equals(TCV("-sentry")))
			{
				if (!parseValue(sentryUrl))
					return false;
			}
			else if (name.Equals(TCV("-zone")))
			{
				if (!parseValue(zone))
					return false;
			}
			else if (name.Equals(TCV("-version")))
			{
				const tchar* dbgStr = TC("");
				#if UBA_DEBUG
				dbgStr = TC(" (DEBUG)");
				#endif
				LoggerWithWriter(g_consoleLogWriter, TC("")).Info(TC("v%s%s (Network: %u, Storage: %u, Session: %u, Cache: %u)"), Version, dbgStr, SystemNetworkVersion, StorageNetworkVersion, SessionNetworkVersion, CacheNetworkVersion);
				return true;
			}
			else if (name.Equals(TCV("-noproxy")))
			{
				allowProxy = false;
			}
			else if (name.Equals(TCV("-proxyuselocalstorage")))
			{
				proxyUseLocalStorage = true;
			}
			else if (name.Equals(TCV("-description")))
			{
				if (value.IsEmpty())
					return PrintHelp(TC("-description"));
				if (value[value.count - 1] == '\"')
					value.Resize(value.count - 1);
				description = value.data + (value[0] == '\"');
			}
			else if (name.Equals(TCV("-nocloud")))
			{
				couldBeCloud = false;
			}
			else if (name.Equals(TCV("-notips")))
			{
				printTips = false;
			}
			else if (name.Equals(TCV("-command")))
			{
				if (!parseValue(command))
					return false;
				poll = false;
				quiet = true;
			}
			else if (name.Equals(TCV("-?")) || name.Equals(TCV("-help")))
			{
				return PrintHelp(TC(""));
			}
			else if (relaunchPath.IsEmpty())
			{
				StringBuffer<> msg;
				msg.Appendf(TC("Unknown argument '%s'"), name.data);
				return PrintHelp(msg.data);
			}
		}

		InitMemory();
		SetMemoryWorkingSet(DefaultMemoryWorkingSet);

		#if !PLATFORM_WINDOWS
		SetupOrphanReaper();
		#endif

		if (useExceptionHandler)
			AddExceptionHandler();

		if (!named.empty()) // We only run once with named connection
			poll = false;

		maxProcessCount = u32(float(maxProcessCount) * mulProcessValue);

		if (poll) // no point disconnect on idle since agent will just reconnect immediately again
			maxIdleSeconds = ~0u;

		if (memKillLoadPercent < memWaitLoadPercent)
			memKillLoadPercent = memWaitLoadPercent;

		if (maxConnectionCount < minConnectionCount)
			maxConnectionCount = minConnectionCount;

		FilteredLogWriter logWriter(g_consoleLogWriter, verbose ? LogEntryType_Debug : LogEntryType_Detail);
		LoggerWithWriter logger(logWriter, TC(""));

		g_exitLock->Enter();
		g_logger = &logger;
		g_exitLock->Leave();
		auto glg = MakeGuard([]() { g_exitLock->Enter(); g_logger = nullptr; g_exitLock->Leave(); });

#if UBA_AUTO_UPDATE
		if (waitProcessId != ~0u)
			if (!WaitForProcess(waitProcessId))
				return false;
		if (relaunchPath.count)
			return LaunchReal(logger, relaunchPath, argc, argv);
#endif // UBA_AUTO_UPDATE

		if (host.IsEmpty() && named.empty() && !useListen)
			return PrintHelp(TC("No host provided. Add -host=<host> (or use -listen)"));

		StringBuffer<256> extraInfo;

		#if defined(UBA_USE_SENTRY)
		if (!sentryUrl.empty())
		{
			char release[128];
			char url[512];
			size_t urlLen;
			sprintf_s(release, sizeof_array(release), "BoxAgent@%ls", Version);
			wcstombs_s(&urlLen, url, sizeof_array(url), sentryUrl.c_str(), sizeof_array(url) - 1);
			sentry_options_t* options = sentry_options_new();
			sentry_options_set_dsn(options, url);
			sentry_options_set_database_path(options, ".sentry-native");
			sentry_options_set_release(options, release);
			//sentry_options_set_debug(options, 1);
			sentry_init(options);
			extraInfo.Append(TCV(", SentryEnabled"));
		}
		
		auto sentryGuard = MakeGuard([&]() { if (!sentryUrl.empty()) sentry_close(); });
		#endif


		// Check if cloud (AWS/google cloud etc)
		#if UBA_USE_CLOUD
		Cloud cloud;
		if (couldBeCloud)
		{
			DirectoryCache dirCache;
			dirCache.CreateDirectory(logger, g_rootDir.data);
			cloud.QueryInformation(logger, extraInfo, g_rootDir.data);
			if (zone.IsEmpty())
				zone.Append(cloud.GetAvailabilityZone());
		}
		#endif

		if (agentName.empty())
		{
			StringBuffer<128> temp;
			if (GetComputerNameW(temp))
				agentName = temp.ToString();
		}

		if (!zone.count)
			GetZone(zone);

		u32 osVersion;
		StringBuffer<32> osVersionStr;
		if (GetOsVersion(osVersionStr, osVersion))
			extraInfo.Append(TCV(", ")).Append(osVersionStr);
		if (useQuic)
			extraInfo.Append(TCV(", MsQuic"));
		if (hasCrypto)
			extraInfo.Append(TCV(", Encrypted"));
		#if PLATFORM_WINDOWS
		if (IsRunningWine() && UsingNativeSockets())
			extraInfo.Append(TCV(", NativeSock"));
		#endif

		if (!description.empty())
			extraInfo.Append(TCV(", ")).Append(description);

		//logger.Info(TC("\033[39m\n"));
		const tchar* dbgStr = TC("");
		#if UBA_DEBUG
		dbgStr = TC(" (DEBUG)");
		#endif
		logger.Info(TC("UbaAgent v%s%s (Cpu%s: %u, MaxCon: %u, Dir: \"%s\", StoreCapacity: %uGb, Zone: %s%s)"), Version, dbgStr, (IsArmBinary ? TC("[Arm]") : TC("")), maxProcessCount, maxConnectionCount, g_rootDir.data, storageCapacityGb, zone.count ? zone.data : TC("none"), extraInfo.data);
		
		Config config;
		if (!configFile.empty())
			config.LoadFromFile(logger, configFile.c_str());
				
		if (!eventFile.IsEmpty())
			logger.Info(TC("  Will poll for external events in file %s"), eventFile.data);

		if constexpr (!IsArmBinary)
			if (IsRunningArm())
				logger.Warning(TC("  Running x64 binary on arm64 system. Use arm binaries instead"));

		if (!NetworkBackendTcp::CheckEnvironment(logger, printTips))
			return false;

		logger.Info(TC(""));


#if PLATFORM_WINDOWS
		{
			StringBuffer<256> consoleTitle;
			consoleTitle.Appendf(TC("UbaAgent v%s%s"), Version, dbgStr);
			SetConsoleTitleW(consoleTitle.data);
		}

		// Uncomment to log winsock behavior in wine.. requires WINEDEBUG to be allowed though
		//if (IsRunningWine())
		//	SetEnvironmentVariable(TC("WINEDEBUG"), TC("+timestamp,+winsock"));
#endif

		u64 storageCapacity = u64(storageCapacityGb)*1000*1000*1000;

		if (command.empty() && useStorage)
		{
			// Create a uba storage quickly just to fix non-graceful shutdowns
			WorkManagerImpl workManager(maxProcessCount);
			StorageCreateInfo info(g_rootDir.data, logWriter, workManager);
			info.Apply(config);
			info.rootDir = g_rootDir.data;
			info.casCapacityBytes = storageCapacity;
			info.storeCompressed = storeCompressed;
			StorageImpl storage(info);
			if (resetStore)
			{
				if (!storage.Reset())
					return false;
			}
			else if (!storage.LoadCasTable(false))
				return false;
		}

		StringBuffer<512> terminationReason;

#if PLATFORM_MAC
		
		Vector<TString> xcodeDirectories;
		
		if (populateCasFromXcodeVersion.count > 0 || populateCasFromAllXcodes)
		{
			// look for all xcodes in /Applications (is there a function to get Applications dir location for other locales?)
			StringBuffer<> applicationsDir;
			applicationsDir.Append("/Applications");
			
			TraverseDir(logger, applicationsDir,
						[&](const DirectoryEntry& e)
						{
				if (IsDirectory(e.attributes) && StartsWith(e.name, "Xcode"))
				{
					StringBuffer<128> xcodeDir("/Applications/");
					xcodeDir.Append(e.name).Append("/Contents/Developer/");
					if (FileExists(logger, xcodeDir.data))
					{
						if (populateCasFromAllXcodes)
						{
							xcodeDirectories.push_back(xcodeDir.data);
						}
						else
						{
							StringBuffer<512> command;
							StringBuffer<32> xcodeVer;
							
							// look for short version like 15.1 or 15, or BuildVersion like 15C610
							bool bUseShortVersion = (populateCasFromXcodeVersion.Contains('.')) || populateCasFromXcodeVersion.count <= 3;
							const char* key = bUseShortVersion ? "CFBundleShortVersionString" : "ProductBuildVersion";
							command.Append("/usr/bin/defaults read /Applications/").Append(e.name).Append("/Contents/version.plist ").Append(key);
							
							FILE* getver = popen(command.data, "r");
							if (getver == nullptr || fgets(xcodeVer.data, xcodeVer.capacity, getver) == nullptr)
							{
								pclose(getver);
								logger.Error("Failed to get DTXcodeBuild from /Applications/%s", e.name);
								return;
							}
							pclose(getver);
							xcodeVer.count = strlen(xcodeVer.data);
							while (isspace(xcodeVer.data[xcodeVer.count-1]))
							{
								xcodeVer.data[xcodeVer.count-1] = 0;
								xcodeVer.count--;
							}
							
							logger.Info("/Applications/%s has version '%s' (looking for %s)", e.name, xcodeVer.data, populateCasFromXcodeVersion.data);
							
							if (xcodeVer.Equals(populateCasFromXcodeVersion.data))
							{
								xcodeDirectories.push_back(xcodeDir.data);
							}
						}
					}
				}
			});

			if (xcodeDirectories.empty())
			{
				//terminationReason.Append("Unable to populate from any Xcodes. Agent is unusable.");
				logger.Warning(TC("Unable to populate from any Xcodes and host might not be able to share sdk files"));
			}
		}
		// if we didn't want a single version, or all xcodes, then use active xcode (useful for user running their own agents)
		else
		{
			StringBuffer<512> xcodeSelectOutput;
			FILE* xcodeSelect = popen("/usr/bin/xcode-select -p", "r");
			if (xcodeSelect == nullptr || fgets(xcodeSelectOutput.data, xcodeSelectOutput.capacity, xcodeSelect) == nullptr || pclose(xcodeSelect) != 0)
				terminationReason.Append("Failed to get an Xcode from xcode-select");
			else
			{
				xcodeSelectOutput.count = strlen(xcodeSelectOutput.data);
				while (isspace(xcodeSelectOutput.data[xcodeSelectOutput.count-1]))
					xcodeSelectOutput.data[--xcodeSelectOutput.count] = 0;
				xcodeDirectories.push_back(xcodeSelectOutput.data);
			}
		}

		for (TString& xcodeDir : xcodeDirectories)
		{
			logger.Info("Populating cas with %s", xcodeDir.data());
			
			const char* subDirs[] = { "/Toolchains", "/Platforms" };
			for (auto subDir : subDirs)
			{
				TString populateDir(xcodeDir);
				populateDir.append(subDir);
				populateCasDirs.push_back(populateDir);
			}
		}
#endif


		Vector<ProcessLogLine> logLines[2];
		u32 logLinesIndex = 0;
		Futex logLinesLock;
		Event logLinesAvailable(EventResetType_Auto);

		auto processFinished = [&](const ProcessHandle& process)
		{
			u32 errorCode = process.GetExitCode();
			if (errorCode == ProcessCancelExitCode)
				return;

			const Vector<ProcessLogLine>& processLogLines = process.GetLogLines();
			if (!processLogLines.empty())
			{
				SCOPED_FUTEX(logLinesLock, lock);
				for (auto& line : processLogLines)
					logLines[logLinesIndex].push_back(line);
				if (errorCode)
				{
					StringBuffer<> errorMsg;
					errorMsg.Appendf(TC(" (exit code: %u)"), errorCode);
					logLines[logLinesIndex].back().text += errorMsg.data;
				}
			}
			else
			{
				const TString& desc = process.GetStartInfo().GetDescription();
				StringBuffer<> name;
				if (!desc.empty())
					name.Append(desc);
				else
					GenerateNameForProcess(name, process.GetStartInfo().arguments, 0);
				LogEntryType entryType = LogEntryType_Info;
				if (errorCode)
				{
					name.Appendf(TC(" (exit code: %u)"), errorCode);
					entryType = LogEntryType_Error;
				}
				SCOPED_FUTEX(logLinesLock, lock);
				logLines[logLinesIndex].push_back({ name.ToString(), entryType });
			}

			logLinesAvailable.Set();
		};

		#if PLATFORM_WINDOWS
		SetConsoleCtrlHandler(ConsoleHandler, TRUE);
		#else
		signal(SIGINT, ConsoleHandler);
		signal(SIGTERM, ConsoleHandler);
		#endif

		bool relaunch = false;
		u64 terminationTimeMs = 0;

		#if UBA_USE_CLOUD
		if (couldBeCloud && terminationReason.IsEmpty())
			cloud.IsTerminating(logger, terminationReason, terminationTimeMs);
		#endif

		bool isTerminating = !terminationReason.IsEmpty();
		
		do
		{
			NetworkBackendMemory networkBackendMem(logWriter);
			NetworkBackend* networkBackend;
			#if UBA_USE_QUIC
			if (useQuic)
				networkBackend = new NetworkBackendQuic(logWriter);
			else
			#endif
			{
				NetworkBackendTcpCreateInfo info(logWriter);
				info.useOverlappedSend = useOverlappedSend;
				info.iocpWorkerCount = iocpWorkerCount;
				//info.statusUpdateSeconds = 2;
				networkBackend = new NetworkBackendTcp(info);
			}

			auto backendGuard = MakeGuard([networkBackend]() { delete networkBackend; });

			NetworkClientCreateInfo ncci(logWriter);
			//ncci.Apply(config);
			ncci.sendSize = sendSize;
			//ncci.receiveTimeoutSeconds = 0; // Use default 10 minutes
			ncci.workerCount = maxWorkerCount;
			if (hasCrypto)
				ncci.cryptoKey128 = crypto;
			bool ctorSuccess = true;
			NetworkClient* client = new NetworkClient(ctorSuccess, ncci);
			auto csg = MakeGuard([&]() { client->Disconnect(); delete client; });
			if (!ctorSuccess)
				return false;

			if (useListen)
			{
				while (true)
				{
					if (client->StartListen(*networkBackend, port))
						break;
					if (killTcpHogs)
					{
						killTcpHogs = false;
						if (!NetworkBackendTcp::KillTcpHogs(logger, port))
							return false;
						Sleep(2000);
						continue;
					}
					return logger.Error(TC("Failed to get start listening on port %u"), u32(port));
				}

				u64 startTime = GetTime();
				while (!client->IsOrWasConnected(200))
				{
					if (ShouldExit())
						return true;

					u64 waitTime = GetTime() - startTime;
					if (!poll)
					{
						if (TimeToMs(waitTime) > listenTimeoutSec*1000)
							return logger.Error(TC("Failed to get connection while listening for %s"), TimeToText(waitTime).str);
						continue;
					}

					#if UBA_USE_CLOUD
					if (couldBeCloud && !isTerminating && cloud.IsTerminating(logger, terminationReason, terminationTimeMs))
						isTerminating = true;
					#endif
					if (isTerminating)
						return logger.Error(TC("Terminating.. (%s)"), terminationReason.data);
				}
			}
			else
			{
				logger.Info(TC("Waiting to connect to %s:%u"), host.data, port);
				int retryCount = 5;
				u64 startTime = GetTime();
				bool timedOut = false;
				while (!client->Connect(*networkBackend, host.data, port, &timedOut))
				{
					if (ShouldExit())
						return true;

					if (!timedOut)
						return false;

					if (!poll)
					{
						if (!--retryCount)
							return logger.Error(TC("Failed to connect to %s:%u (after %s)"), host.data, port, TimeToText(GetTime() - startTime).str);
						continue;
					}

					#if UBA_USE_CLOUD
					if (couldBeCloud && !isTerminating && cloud.IsTerminating(logger, terminationReason, terminationTimeMs))
						isTerminating = true;
					#endif
					if (isTerminating)
						return logger.Error(TC("Terminating.. (%s)"), terminationReason.data);
				}
			}

			if (!command.empty())
			{
				StackBinaryWriter<128> writer;
				NetworkMessage msg(*client, SessionServiceId, SessionMessageType_Command, writer);
				writer.WriteString(command);
				StackBinaryReader<8*1024> reader;
				if (!msg.Send(reader))
					return logger.Error(TC("Failed to send command to host"));
				LoggerWithWriter commandLogger(g_consoleLogWriter, TC(""));
				commandLogger.Info(TC("----------------------------------"));
				while (true)
				{
					auto logType = (LogEntryType)reader.ReadByte();
					if (logType == 255)
						break;
					TString result = reader.ReadString();
					commandLogger.Log(logType, result.c_str(), u32(result.size()));
				}
				commandLogger.Info(TC("----------------------------------"));
				return true;
			}

			if (!client->FetchConfig(config))
				continue;

			SessionClient* sessionClient = nullptr;

			Event wakeupSessionWait(EventResetType_Auto);
			Atomic<u32> targetConnectionCount = minConnectionCount;
			StorageClient* storageClient = nullptr;


			StorageProxy* storageProxy = nullptr;
			Atomic<NetworkServer*> proxyNetworkServer = nullptr;
			auto psg = MakeGuard([&]() { delete proxyNetworkServer; });
			auto pg = MakeGuard([&]() { delete storageProxy; });
			TString proxyNetworkServerPrefix;

			auto startProxy = [&](u16 proxyPort, const Guid& storageServerUid)
				{
					NetworkServerCreateInfo nsci(g_consoleLogWriter);
					nsci.workerCount = 192;
					nsci.receiveTimeoutSeconds = 5*60; // 5min timeout

					proxyNetworkServerPrefix = StringBuffer<256>().Append(TCV("UbaProxyServer (")).Append(GuidToString(client->GetUid()).str).Append(')').data;
					bool ctorSuccess = true;
					auto proxyServer = new NetworkServer(ctorSuccess, nsci, proxyNetworkServerPrefix.c_str());
					if (!ctorSuccess)
					{
						delete proxyServer;
						return false;
					}

					StorageProxyCreateInfo proxyInfo { *proxyServer, *client, storageServerUid, TC("Wooohoo"), storageClient };
					proxyInfo.useLocalStorage = proxyUseLocalStorage;
					proxyInfo.Apply(config);

					storageProxy = new StorageProxy(proxyInfo);

					proxyServer->RegisterOnClientConnected(0, [&wakeupSessionWait](const Guid& clientUid, u32 clientId) { wakeupSessionWait.Set(); return true; });
					proxyServer->SetWorkTracker(client->GetWorkTracker());
					proxyServer->StartListen(networkBackendMem, proxyPort);
					

					if (!proxyServer->StartListen(*networkBackend, proxyPort))
					{
						if (sessionClient)
							sessionClient->SendNotification(TCV("Failed to create proxy server"));
						//NetworkBackendTcp::KillTcpHogs(logger, proxyPort)
						delete storageProxy;
						storageProxy = nullptr;
						delete proxyServer;
						proxyServer = nullptr;
						return false;
					}

					proxyNetworkServer = proxyServer;
					wakeupSessionWait.Set();
					targetConnectionCount = maxConnectionCount;
					return true;
				};

			Atomic<bool> isDisconnected = false;
			client->RegisterOnDisconnected([&]() { isDisconnected = true; networkBackend->StopListen(); if (auto proxyServer = proxyNetworkServer.load()) proxyServer->DisconnectClients(); });

			struct NetworkBackends
			{
				NetworkBackend& tcp;
				NetworkBackend& mem;
			} backends { *networkBackend, networkBackendMem };

			static auto getProxyBackend = [](void* userData, const tchar* host) -> NetworkBackend&
				{
					auto& backends = *(NetworkBackends*)userData;
					return Equals(host, TC("inprocess")) ? backends.mem : backends.tcp;
				};

			StorageClientCreateInfo storageInfo(*client, g_rootDir.data);
			storageInfo.rootDir = g_rootDir.data;
			storageInfo.casCapacityBytes = storageCapacity;
			storageInfo.storeCompressed = storeCompressed;
			storageInfo.sendCompressed = sendCompressed;
			storageInfo.resendCas = resendCas;
			storageInfo.getProxyBackendCallback = getProxyBackend;
			storageInfo.getProxyBackendUserData = &backends;
			storageInfo.allowProxy = allowProxy;
			storageInfo.startProxyCallback = [](void* userData, u16 proxyPort, const Guid& storageServerUid) { return (*(decltype(startProxy)*)userData)(proxyPort, storageServerUid); };;
			storageInfo.startProxyUserData = &startProxy;
			storageInfo.zone = zone.data;
			storageInfo.proxyPort = proxyPort;
			storageInfo.proxyAddress = proxyAddr.c_str();
			storageInfo.writeToDisk = useStorage;
			storageInfo.Apply(config);

			storageClient = new StorageClient(storageInfo);
			auto bscsg = MakeGuard([&]() { delete storageClient; });

			if (!storageClient->LoadCasTable(true))
				return false;

			CasKey keys[2];
			client->RegisterOnVersionMismatch([&](const CasKey& exeKey, const CasKey& dllKey)
				{
					keys[0] = exeKey;
					keys[1] = dllKey;
				});

			SessionClientCreateInfo info(*storageClient, *client, logWriter);
			info.maxProcessCount = maxProcessCount;
			info.dedicated = poll;
			info.maxIdleSeconds = maxIdleSeconds;
			info.name.Append(agentName);
			info.extraInfo = extraInfo.data;
			info.deleteSessionsOlderThanSeconds = 1; // Delete all old sessions
			//if (!awsInstanceId.IsEmpty())
			//	info.name.Append(TCV(" (")).Append(awsInstanceId).Append(TCV(")"));
			info.rootDir = g_rootDir.data;
			info.logToFile = logToFile;
			info.disableCustomAllocator = disableCustomAllocator;
			info.useBinariesAsVersion = useBinariesAsVersion;
			info.killRandom = killRandom;
			info.useStorage = useStorage;
			info.downloadDetoursLib = downloadDetoursLib;
			info.memWaitLoadPercent = u8(memWaitLoadPercent);
			info.memKillLoadPercent = u8(memKillLoadPercent);
			info.useDependencyCrawler = useCrawler;
			info.osVersion = osVersion;
			info.Apply(config);

			if (!quiet)
				info.processFinished = processFinished;

			sessionClient = new SessionClient(info);
			auto secsg = MakeGuard([&]() { delete sessionClient; sessionClient = nullptr; });

			Atomic<bool> loopLogging = true;
			Thread loggingThread([&]()
				{
					while (loopLogging)
					{
						logLinesAvailable.IsSet();
						u32 logLinesIndexPrev;
						{
							SCOPED_FUTEX(logLinesLock, l);
							logLinesIndexPrev = logLinesIndex;
							logLinesIndex = (logLinesIndex + 1) % 2;
						}
						logger.BeginScope();
						for (auto& s : logLines[logLinesIndexPrev])
							logger.Log(LogEntryType_Detail, s.text.c_str(), u32(s.text.size()));
						logger.EndScope();
						logLines[logLinesIndexPrev].clear();
					}
					return 0;
				}, TC("UbaLogging"));

			g_sessionClient = sessionClient;

			u32 systemError = 0;
			SetSystemErrorCallback([&](u32 error) { systemError = error; wakeupSessionWait.Set(); });

			auto disconnectAndStopLoggingThread = MakeGuard([&]()
			{
				SetSystemErrorCallback({});

				g_exitLock->Enter();
				g_sessionClient = nullptr;
				g_exitLock->Leave();

				networkBackend->StopListen();
				storageClient->StopProxy();
				sessionClient->Stop();

				auto proxyServer = proxyNetworkServer.load();
				if (proxyServer)
				{
					// Let's give the active fetches some time (60 seconds)
					u32 waitCount = 60*10;
					while (storageProxy->GetActiveFetchCount())
					{
						Sleep(100);
						if (!waitCount--)
							break;
					}
					proxyServer->DisconnectClients();
				}
				sessionClient->SendSummary([&](Logger& logger) { if (proxyServer) proxyServer->PrintSummary(logger); });
				client->Disconnect();

				loopLogging = false;
				logLinesAvailable.Set();
				loggingThread.Wait();
			});

			if (quiet)
				logger.Info(TC("Client session %s started"), sessionClient->GetId());
			else
				logger.Info(TC("----------- Session %s started -----------"), sessionClient->GetId());

			u32 connectionCount = 1;

			//#if PLATFORM_WINDOWS
			//SetPriorityClass(GetCurrentProcess(), ABOVE_NORMAL_PRIORITY_CLASS);
			//#endif

			bool needPrepopulate = !populateCasDirs.empty();
			if (needPrepopulate || isTerminating)
				sessionClient->SetAllowSpawn(false);

			storageClient->Start();
			sessionClient->Start();

			// We do population here to make sure session thread is running which will send pings to host (to prevent timeouts
			if (needPrepopulate && !isTerminating)
			{
				if (storageClient->PopulateCasFromDirs(populateCasDirs, maxProcessCount, [&]() { return isDisconnected.load(); }))
					sessionClient->SetAllowSpawn(true);
				else
					terminationReason.Append(TCV("Failed to prepopulate cas from local directories"));
			}

			if (terminationReason.count)
			{
				isTerminating = true;
				sessionClient->SetIsTerminating(terminationReason.data, 0);
			}

			while (!ShouldExit())
			{
				u32 sessionWaitTimeout = 5*1000;
				if (useListen)
				{
					if (connectionCount < targetConnectionCount)
					{
						u32 newConnectionCount = targetConnectionCount;
						logger.Info(TC("Updating desired connection count from %u to %u"), connectionCount, newConnectionCount);
						client->SetConnectionCount(newConnectionCount);
						connectionCount = newConnectionCount;
					}
				}
				else
				{
					if (connectionCount < targetConnectionCount && client->IsConnected())
					{
						bool timedOut = false;
						if (client->Connect(*networkBackend, host.data, port, &timedOut))
						{
							++connectionCount;
							sessionWaitTimeout = 0;
						}
						else
							logger.Warning(TC("Failed to connect secondary connection number %u"), connectionCount);
					}
				}

				if (systemError)
				{
					isTerminating = true;
					sessionClient->SetIsTerminating(TStrdup(LastErrorToText(systemError).data), 0); // Leak allocation
				}

				if (sessionClient->Wait(sessionWaitTimeout, &wakeupSessionWait))
				{
					// We got version mismatch and have the cas keys for the needed Agent/Detours binaries
					if (keys[0] != CasKeyZero)
					{
		#if UBA_AUTO_UPDATE
						logger.Info(TC("Downloading new binaries..."));
						if (!DownloadBinaries(*storageClient, keys))
							return false;
						isTerminating = true;
						relaunch = true;
						break;
		#else
						return false;
		#endif
					}
					break;
				}

				// If we are the proxy server and have external connections we lower max process count. Note that it will always have one connection which is itself
				if (auto proxyServer = proxyNetworkServer.load())
				{
					u32 clientCount = proxyServer->GetClientCount();
					if (clientCount > 1) // when having a proxy the agent itself is always connected to it
					{
						if (maxIdleSeconds < 30) // A bit hacky solution.. but we really want to try preventing idling out the proxy if possible when other agents are connected
							sessionClient->SetMaxIdleTime(maxIdleSeconds*2);

						u32 processToFree = (clientCount - 1)/3 + 1; // Always free one, and then one per three helpers.. (so 16 helpers connected will remove 6 process count)
						u32 newProcessCount;
						if (processToFree < maxProcessCount)
							newProcessCount = maxProcessCount - processToFree;
						else
							newProcessCount = 1;

						if (sessionClient->GetMaxProcessCount() != newProcessCount)
						{
							logger.Info(TC("Changed max process count to %u"), newProcessCount);
							sessionClient->SetMaxProcessCount(newProcessCount);
						}
					}
					else
					{
						// Restore idle time if reaching 0 connections.
						sessionClient->SetMaxIdleTime(maxIdleSeconds);
					}
				}

				if (u32 desiredConnectionCount = sessionClient->GetDesiredConnectionCount())
				{
					targetConnectionCount = desiredConnectionCount;
				}
				else
				{
					// This is an estimation based on tcp limitations (ack and sliding windows).
					// For every 15ms latency on "best ping") we increase targetConnectionCount up to maxConnectionCount
					if (!storageClient->IsUsingProxy())
						if (u64 bestPing = sessionClient->GetBestPing())
							targetConnectionCount = Max(minConnectionCount, Min(u32(TimeToMs(bestPing) / 15), maxConnectionCount));
				}

				if (!isTerminating)
				{
					if (IsTerminating(logger, eventFile.data, terminationReason, terminationTimeMs))
						isTerminating = true;
					#if UBA_USE_CLOUD
					else if (couldBeCloud && cloud.IsTerminating(logger, terminationReason, terminationTimeMs))
						isTerminating = true;
					#endif

					if (isTerminating)
					{
						sessionClient->SetIsTerminating(terminationReason.data, terminationTimeMs);
						if (quiet)
							LoggerWithWriter(g_consoleLogWriter, TC("")).Info(TC("%s"), terminationReason.data);
					}
				}
			}

			disconnectAndStopLoggingThread.Execute();

			if (quiet)
			{
				logger.Info(TC("Client session %s done"), sessionClient->GetId());
			}
			else
			{
				logger.BeginScope();
				if (printSummary)
				{
					sessionClient->PrintSummary(logger);
					storageClient->PrintSummary(logger);
					client->PrintSummary(logger);
					KernelStats::GetGlobal().Print(logger, true);
				}

				logger.Info(TC("----------- Session %s done! -----------"), sessionClient->GetId());
				logger.Info(TC(""));
				logger.EndScope();
			}

			//if (proxy.storage)
			//	proxy.storage->PrintSummary();
			//if (proxy.server)
			//	proxy.server->PrintSummary(logger);

			#if UBA_TRACK_CONTENTION
			LoggerWithWriter contLogger(g_consoleLogWriter, TC(""));
			PrintContentionSummary(contLogger);
			#endif
		}
		while (poll && !isTerminating && !ShouldExit());

#if UBA_AUTO_UPDATE
		if (relaunch)
			if (!LaunchTemp(logger, argc, argv))
				return false;
#endif

		return true;
	}
}

#if PLATFORM_WINDOWS
int wmain(int argc, wchar_t* argv[])
{
	return uba::WrappedMain(argc, argv) ? 0 : -1;
}
#else
int main(int argc, char* argv[])
{
	return uba::WrappedMain(argc, argv) ? 0 : -1;
}
#endif
