// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaAgent : ModuleRules
{
	public UbaAgent(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivatePCHHeaderFile = "../Core/Public/UbaCorePch.h";

		bRequiresPlatformSDK = true;

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		StaticAnalyzerDisabledCheckers.Clear();

		PrivateDependencyModuleNames.AddRange(new string[] {
			"UbaCommon",
		});

		PrivateDefinitions.AddRange(new string[] {
			"_CONSOLE",
		});

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && Target.Architecture.bIsX64 && (Target.Configuration == UnrealTargetConfiguration.Development || Target.Configuration == UnrealTargetConfiguration.Shipping))
		{
			PrivateDependencyModuleNames.Add("SentryNative");
			PrivateDefinitions.Add("UBA_USE_SENTRY");
		}
	}
}
