@echo off
setlocal 

pushd "%~dp0"

set config=Development

if "%~1"=="" goto blank

set config=%1

:blank

echo.
echo === Building Targets ===
pushd "../../../.."
call Engine/Build/BatchFiles/RunUBT.bat -NoUba -NoUbaLocal -NoSNDBS -NoXGE -NoVFS -Architectures=x64 ^
	-Target="UbaAgent Win64 %config%" ^
	-Target="UbaCli Win64 %config%" ^
	-Target="UbaDetours Win64 %config%" ^
	-Target="UbaHost Win64 %config%" ^
	-Target="UbaVisualizer Win64 %config%" ^
	-Target="UbaCacheService Win64 %config%" ^
	-Target="UbaObjTool Win64 %config%" ^
	-Target="UbaTest Win64 %config%" ^
	-StaticAnalyzer=Clang

popd

popd

if %ERRORLEVEL% NEQ 0 exit /b %ERRORLEVEL%

endlocal

exit /b 0
