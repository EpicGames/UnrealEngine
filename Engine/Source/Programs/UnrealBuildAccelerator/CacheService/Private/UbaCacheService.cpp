// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaCacheServer.h"
#include "UbaApplication.h"
#include "UbaConfig.h"
#include "UbaCrypto.h"
#include "UbaDirectoryIterator.h"
#include "UbaFile.h"
#include "UbaHttpServer.h"
#include "UbaNetworkBackendTcp.h"
#include "UbaNetworkServer.h"
#include "UbaPlatform.h"
#include "UbaProtocol.h"
#include "UbaStorageServer.h"
#include "UbaTrace.h"
#include "UbaVersion.h"

#if PLATFORM_WINDOWS
#include <dbghelp.h>
#include <io.h>
#pragma comment (lib, "Dbghelp.lib")
#elif PLATFORM_LINUX
#include <sys/prctl.h>
#include <sys/statvfs.h>
#endif

namespace uba
{
	const tchar*	Version = GetVersionString();
	u32				DefaultCapacityGb = 500;
	u32				DefaultExpiration = 3*24*60*60;
	u32				DefaultReportIntervalSeconds = 5*60; // Five minutes
	const tchar*	DefaultRootDir = []() {
		static tchar buf[256];
		if constexpr (IsWindows)
			ExpandEnvironmentStringsW(TC("%ProgramData%\\Epic\\" UE_APP_NAME), buf, sizeof(buf));
		else
			GetFullPathNameW(TC("~/" UE_APP_NAME), sizeof_array(buf), buf, nullptr);
		return buf;
		}();
	u32				DefaultWorkerCount = []() { return GetLogicalProcessorCount() + 4; }();

	bool PrintHelp(const tchar* message)
	{
		LoggerWithWriter logger(g_consoleLogWriter, TC(""));
		if (*message)
		{
			logger.Info(TC(""));
			logger.Error(TC("%s"), message);
		}
		logger.Info(TC(""));
		logger.Info(TC("-------------------------------------------"));
		logger.Info(TC("   UbaCacheService v%s (%u)"), Version, CacheNetworkVersion);
		logger.Info(TC("-------------------------------------------"));
		logger.Info(TC(""));
		logger.Info(TC("  -dir=<rootdir>          The directory used to store data. Defaults to \"%s\""), DefaultRootDir);
		logger.Info(TC("  -port=[<host>:]<port>   The ip/name and port (default: %u) to listen for clients on"), DefaultCachePort);
		logger.Info(TC("  -capacity=<gigaby>      Capacity of local store. Defaults to %u gigabytes"), DefaultCapacityGb);
		logger.Info(TC("  -expiration=<seconds>   Time until unused cache entries get deleted. Defaults to %s (%u seconds)"), TimeToText(MsToTime(DefaultExpiration*1000)).str, DefaultExpiration);
		logger.Info(TC("  -config=<file>          Config file that contains options for various systems"));
		logger.Info(TC("  -http=<port>            If set, a http server will be started and listen on <port>"));
		logger.Info(TC("  -httpcrypto=<crypto>    If set, all provided keys will be decrypted using this key."));
		logger.Info(TC("  -fullmaintenance        Force a full maintenance"));
		logger.Info(TC("  -nomaintenance          Skip all maintenance"));
		logger.Info(TC("  -crash                  Force a crash (for testing)"));
		logger.Info(TC("  -nosignalhandler        Will not hook up signal handler"));
		logger.Info(TC("  -notips                 Disable printing of tips at startup"));
		logger.Info(TC("  -maxworkers=<number>    Max number of workers used by cache server. Defaults to \"%u\""), DefaultWorkerCount);
		logger.Info(TC("  -reportinterval=<sec>   How often the service should report status. Defaults to \"%s\""), TimeToText(MsToTime(DefaultReportIntervalSeconds*1000)).str);
		logger.Info(TC("  -trace                  Create a trace file that is written out when cache service exits"));
#if PLATFORM_LINUX
		logger.Info(TC("  -fork                   Will handle segfaults and restart"));
#elif PLATFORM_WINDOWS
		logger.Info(TC("  -useIocp[=workerCount]  Enable/Disable iocp for tcp. Defaults to 4 workers is not set"));
#endif
		logger.Info(TC(" Example of how to register crypto key to cache server (when -http=80 is provided)"));
		logger.Info(TC("   curl http://localhost/addcrypto?3f58aa57466db9999213456789123445"));
		logger.Info(TC("   curl http://localhost/addsession?crypto=3f58aa57466db9999213456789123445,readonly=true,expiration=60"));
		logger.Info(TC(""));
		return false;
	}

	ReaderWriterLock* g_exitLock = new ReaderWriterLock();
	LoggerWithWriter* g_logger;
	Atomic<bool> g_shouldExit = false;

	bool ShouldExit()
	{
		return g_shouldExit || IsEscapePressed();
	}
	
	void CtrlBreakPressed()
	{
		g_shouldExit = true;

		g_exitLock->Enter();
		if (g_logger)
			g_logger->Info(TC("  Exiting..."));
		g_exitLock->Leave();
	}

	#if PLATFORM_WINDOWS
	BOOL ConsoleHandler(DWORD signal)
	{
		CtrlBreakPressed();
		return TRUE;
	}
	#else
	void ConsoleHandler(int sig)
	{
		CtrlBreakPressed();
	}
	#endif
	
	StringBuffer<> g_rootDir(DefaultRootDir);

	bool WrappedMain(int argc, tchar* argv[])
	{
		using namespace uba;

		float storageCapacityGb = float(DefaultCapacityGb);
		StringBuffer<256> workDir;
		StringBuffer<128> listenIp;
		Vector<CryptoKey> httpCryptos;
		u16 port = DefaultCachePort;
		u16 httpPort = 0;
		bool quiet = false;
		bool storeCompressed = true;
		bool fullMaintenance = false;
		bool maintenanceEnabled = true;
		bool allowSave = true;
		bool signalHandlerEnabled = true;
		bool shouldCrash = false;
		bool requiresCrypto = false;
		bool writeTrace = false;
		bool printTips = true;
		u32 expirationTimeSeconds = DefaultExpiration;
		u32 maxWorkerCount = DefaultWorkerCount;
		u32 reportIntervalSeconds = DefaultReportIntervalSeconds;
		TString configFile;
		u32 iocpWorkerCount = 0;

		u8 permanentCrypto[16];
		bool hasPermanentCrypto = false;

		#if PLATFORM_LINUX
		bool forkProcess = false;
		#endif

		for (int i=1; i!=argc; ++i)
		{
			StringBuffer<> name;
			StringBuffer<> value;

			if (const tchar* equals = TStrchr(argv[i],'='))
			{
				name.Append(argv[i], equals - argv[i]);
				value.Append(equals+1);
			}
			else
			{
				name.Append(argv[i]);
			}

			if (name.Equals(TCV("-port")))
			{
				if (const tchar* portIndex = value.First(':'))
				{
					StringBuffer<> portStr(portIndex + 1);
					if (!portStr.Parse(port))
						return PrintHelp(TC("Invalid value for port in -port"));
					listenIp.Append(value.data, portIndex - value.data);
				}
				else
				{
					if (!value.Parse(port))
						return PrintHelp(TC("Invalid value for -port"));
				}
			}
			else if (name.Equals(TCV("-dir")))
			{
				if (value.IsEmpty())
					return PrintHelp(TC("-dir needs a value"));
				if ((g_rootDir.count = GetFullPathNameW(value.Replace('/', PathSeparator).data, g_rootDir.capacity, g_rootDir.data, nullptr)) == 0)
					return PrintHelp(StringBuffer<>().Appendf(TC("-dir has invalid path %s"), g_rootDir.data).data);
			}
			else if (name.Equals(TCV("-capacity")))
			{
				if (!value.Parse(storageCapacityGb))
					return PrintHelp(TC("Invalid value for -capacity"));
			}
			else if (name.Equals(TCV("-expiration")))
			{
				if (!value.Parse(expirationTimeSeconds))
					return PrintHelp(TC("Invalid value for -expire"));
			}
			else if (name.Equals(TCV("-http")))
			{
				if (!value.Parse(httpPort))
					httpPort = 80;
				requiresCrypto = true; // We force crypto connections when enabling http since http is only used to add crypto keys right now
			}
			else if (name.Equals(TCV("-httpcrypto")))
			{
				if (value.count != 32)
					return PrintHelp(TC("Invalid number of characters in crypto string. Should be 32"));
				u8 crypto[16];
				((u64*)crypto)[0] = StringToValue(value.data, 16);
				((u64*)crypto)[1] = StringToValue(value.data + 16, 16);
				LoggerWithWriter tempLogger(g_consoleLogWriter, TC(""));
				CryptoKey key = Crypto::CreateKey(tempLogger, crypto);
				if (key == InvalidCryptoKey)
					return PrintHelp(TC("Failed to create http crypto key"));
				httpCryptos.emplace_back(key);
			}
			else if (name.Equals(TCV("-fullmaintenance")))
			{
				fullMaintenance = true;
			}
			else if (name.Equals(TCV("-crypto")))
			{
				if (value.count != 32)
					return PrintHelp(TC("Invalid number of characters in crypto string. Should be 32"));
				((u64*)permanentCrypto)[0] = StringToValue(value.data, 16);
				((u64*)permanentCrypto)[1] = StringToValue(value.data + 16, 16);
				hasPermanentCrypto = true;
				requiresCrypto = true;
			}
			else if (name.Equals(TCV("-crash")))
			{
				shouldCrash = true;
			}
			else if (name.Equals(TCV("-trace")))
			{
				writeTrace = true;
			}
			else if (name.Equals(TCV("-nosignalhandler")))
			{
				signalHandlerEnabled = false;
			}
			#if PLATFORM_LINUX
			else if (name.Equals(TCV("-fork")))
			{
				forkProcess = true;
			}
			#elif PLATFORM_WINDOWS
			else if (name.Equals(TCV("-useIocp")))
			{
				iocpWorkerCount = 4;
				if (!value.IsEmpty() && !value.Parse(iocpWorkerCount))
					return false;
			}
			#endif
			else if (name.Equals(TCV("-maxworkers")))
			{
				if (!value.Parse(maxWorkerCount))
					return PrintHelp(TC("Invalid value for -maxworkers"));
			}
			else if (name.Equals(TCV("-reportinterval")))
			{
				if (!value.Parse(reportIntervalSeconds))
					return PrintHelp(TC("Invalid value for -reportinterval"));
			}
			else if (name.Equals(TCV("-nomaintenance")))
			{
				maintenanceEnabled = false;
			}
			else if (name.Equals(TCV("-notips")))
			{
				printTips = false;
			}
			else if (name.Equals(TCV("-nosave")))
			{
				allowSave = false;
			}
			else if (name.Equals(TCV("-config")))
			{
				if (value.IsEmpty())
					return PrintHelp(TC("-config needs a value"));
				if (!ExpandEnvironmentVariables(value, PrintHelp))
					return false;
				configFile = value.data;
			}
			else if (name.Equals(TCV("-?")))
			{
				return PrintHelp(TC(""));
			}
			else
			{
				StringBuffer<> msg;
				msg.Appendf(TC("Unknown argument '%s'"), name.data);
				return PrintHelp(msg.data);
			}
		}

		FilteredLogWriter logWriter(g_consoleLogWriter, quiet ? LogEntryType_Info : LogEntryType_Detail);
		LoggerWithWriter logger(logWriter, TC(""));

		g_exitLock->Enter();
		g_logger = &logger;
		g_exitLock->Leave();
		auto glg = MakeGuard([]() { g_exitLock->Enter(); g_logger = nullptr; g_exitLock->Leave(); });

		Config config;
		if (!configFile.empty())
			config.LoadFromFile(logger, configFile.data());

		u64 storageCapacity = u64(storageCapacityGb*1000.0f)*1000*1000;

		const tchar* dbgStr = TC("");
		#if UBA_DEBUG
		dbgStr = TC(" (DEBUG)");
		#endif
		logger.Info(TC("UbaCacheService v%s(%u)%s (Workers: %u, Rootdir: \"%s\", Crypto: %s, StoreCapacity: %s, Expiration: %s)"), Version, CacheNetworkVersion, dbgStr, maxWorkerCount,
			g_rootDir.data, (requiresCrypto ? TC("on") : TC("off")), BytesToText(storageCapacity).str, TimeToText(MsToTime(expirationTimeSeconds)*1000, true).str);

		u64 maintenanceReserveSizeMb = 196;

		if (SupportsHugePages())
		{
			u64 hugePageCount = GetHugePageCount();
			u64 recommendedHugePageCount = (maintenanceReserveSizeMb*GetLogicalProcessorCount())/2;
			if (hugePageCount < recommendedHugePageCount && printTips)
				logger.Info(TC("  TIP: Improve maintenance performance by enabling %llu huge pages on system (%llu enabled)"), recommendedHugePageCount, hugePageCount);
		}

		if (!NetworkBackendTcp::CheckEnvironment(logger, printTips))
			return false;

		StringBuffer<512> currentDir;
		GetCurrentDirectoryW(currentDir);

		if (workDir.IsEmpty())
			workDir.Append(currentDir);

		#if PLATFORM_LINUX
		if (forkProcess)
		{
			if (prctl(PR_GET_DUMPABLE) == 0)
			{
				prctl(PR_SET_DUMPABLE, 1);
				if (prctl(PR_GET_DUMPABLE) == 0)
					logger.Info(TC("  prctl(PR_SET_DUMPABLE, 1) failed to set dumpable."));
				else
					logger.Info(TC("  Made process dumpable"));
			}

			auto ReadFirstLine = [](StringBufferBase& out, const char* cmd)
				{
					FILE* f = popen(cmd, "r");
					if (!f)
						return false;
					bool success = fgets(out.data, out.capacity, f) == out.data;
					pclose(f);
					if (!success)
						return false;
					out.count = TStrlen(out.data);
					out.data[--out.count] = 0;
					return true;
				};

			StringBuffer<256> temp;
			TString crashDumpDir;
			TString crashDumpPattern;
			if (ReadFirstLine(temp, "ulimit -c"))
			{
				if (temp.Equals(TCV("0")))
					logger.Info(TC("  Crash dumps disabled. Enable with \"ulimit -c unlimited\""));
				else if (ReadFirstLine(temp, "cat /proc/sys/kernel/core_pattern"))
				{
					if (temp[0] == '|')
						logger.Info(TC("  Crash dumps enabled but piped so can't wait (%s). use 'sudo echo \"/<path>/dump.%t\" | sudo tee /proc/sys/kernel/core_pattern > /dev/null' to write to file"), temp.data);
					else
					{
						//if (ReadFirstLine(temp.Clear(), "cat /proc/sys/fs/suid_dumpable"))
						//{
						//}


						logger.Info(TC("  Crash dumps enabled and written to file: %s (Write no other files in the same dir)"), temp.data);
						crashDumpDir = StringView(temp).GetPath().ToString();
						crashDumpPattern = StringView(temp).GetFileName().ToString();
					}
				}
			}

			Set<StringKey> existingFiles;
			if (!crashDumpPattern.empty())
				TraverseDir(logger, crashDumpDir, [&](const DirectoryEntry& e) { existingFiles.insert(ToStringKey(e.name, e.nameLen)); });

			static bool shouldExit = false;
			static pid_t actualChild = 0;
			auto SigHandler = [](int sig) { shouldExit = true; if (actualChild && sig == SIGTERM) kill(actualChild, sig); };

			logger.Info(TC("  Starting cache server fork"));
			forkProcess = false;
			while (true)
			{
				u64 startTime = GetTime();
				pid_t childPid = fork();
				if (childPid < 0)
					return logger.Error(TC("Failed to fork process. (%s)"), strerror(errno));
				if (childPid == 0)
					break;

				// Parent

				actualChild = childPid;
				signal(SIGINT, SigHandler);
				signal(SIGTERM, SigHandler);

				int status;
				if (waitpid(childPid, &status, 0) == -1)
					return logger.Error(TC("waitpid failed. (%s)"), strerror(errno));
				actualChild = 0;
				if (!shouldExit)
					logger.Info(TC("  Fork exited with . (%u)"), status);

				if (WIFSIGNALED(status) && WCOREDUMP(status))
				{
					// Wait for new file to appear
					TString crashDumpFile;
					u32 retryCount = 5;
					while (retryCount--)
					{
						TraverseDir(logger, crashDumpDir, [&](const DirectoryEntry& e) { if (existingFiles.find(ToStringKey(e.name, e.nameLen)) == existingFiles.end()) crashDumpFile = e.name; });
						if (!crashDumpFile.empty())
							break;
						Sleep(1000);
					}
					if (crashDumpFile.empty())
						logger.Info(TC("  No core dump found after 10 seconds"));
					else
					{
						existingFiles.insert(ToStringKey(crashDumpFile));
						TString crashDumpPath = crashDumpDir + '/' + crashDumpFile;
						logger.Info(TC("  Found new core dump %s"), crashDumpPath.c_str());

						temp.Clear().Appendf("/proc/%u", childPid);
						retryCount = 10*60;
						bool firstCall = true;
						while (retryCount--)
						{
							if (access(temp.data, F_OK) != 0)
								break;
							if (firstCall)
								logger.Info(TC("  Waiting (up to 10 minutes) for pid %u to be cleaned up"), childPid);
							firstCall = false;
							Sleep(1000);
							continue;
						}
					}
				}
				else
				{
					shouldExit = true;
				}

				u32 retryCount = 40;
				while (!shouldExit && --retryCount)
					Sleep(100);
				if (shouldExit)
					exit(0);
				logger.Info(TC("  Restarting cache server fork"));
			}
		}
		#endif

		logger.Info(TC(""));

		InitMemory();

		if (signalHandlerEnabled)
			AddExceptionHandler();

		if (shouldCrash)
		{
			#ifndef __clang_analyzer__
			int* ptr = nullptr;
			*ptr = 0;
			#endif
		}

		// TODO: Change workdir to make it full

		#if PLATFORM_WINDOWS
		SetConsoleCtrlHandler(ConsoleHandler, TRUE);
		#else
		signal(SIGINT, ConsoleHandler);
		signal(SIGTERM, ConsoleHandler);
		#endif

		NetworkBackendTcpCreateInfo nbtci(logWriter);
		nbtci.iocpWorkerCount = iocpWorkerCount;
		nbtci.Apply(config);
		NetworkBackendTcp networkBackend(nbtci);

		NetworkServerCreateInfo nsci(logWriter);
		nsci.Apply(config);
		nsci.workerCount = maxWorkerCount;
		nsci.logConnections = false; // Let the cache server report instead
		nsci.receiveTimeoutSeconds = 2*60*60; // Two hours timeout
		nsci.useMessagePriority = true;
		bool ctorSuccess = true;
		NetworkServer networkServer(ctorSuccess, nsci);
		if (!ctorSuccess)
			return false;

		StorageServerCreateInfo storageInfo(networkServer, g_rootDir.data, logWriter);
		storageInfo.Apply(config);
		storageInfo.casCapacityBytes = storageCapacity;
		storageInfo.storeCompressed = storeCompressed;
		storageInfo.allowHintAsFallback = false;
		storageInfo.writeReceivedCasFilesToDisk = true;
		storageInfo.allowDeleteVerified = true;
		storageInfo.allowDeferredDeletes = false;
		StorageServer storageServer(storageInfo);

		Trace trace(logger.m_writer);
		Thread traceThread;
		Event traceLoop;

		if (writeTrace)
		{
			u64 traceReserveSize = 128ull*1024*1024;
			trace.StartWrite(nullptr, traceReserveSize);
			trace.SessionAdded(0, 0, TCV("CacheServer"), {});
			trace.ProcessAdded(0, 0, TCV("Running"), {});
			networkServer.SetWorkTracker(&trace);
			traceLoop.Create(EventResetType_Manual);
			traceThread.Start([&]()
				{
					while (!traceLoop.IsSet(500))
					{
						trace.SessionUpdate(0, networkServer.GetConnectionCount(), networkServer.GetTotalSentBytes(), networkServer.GetTotalRecvBytes(), 0, 0, 0, 0);
					}
					return 0;
				}, TC("UbaCaTrc"));
		}
		auto endTrace = MakeGuard([&]()
			{
				if (!writeTrace)
					return;
				traceLoop.Set();
				traceThread.Wait();
				networkServer.SetWorkTracker(nullptr);
				trace.ProcessExited(0, 0);
				trace.WriteSessionSummary([&](Logger& logger)
					{
						storageServer.PrintSummary(logger);
						networkServer.PrintSummary(logger);
						KernelStats::GetGlobal().Print(logger, true);
						PrintContentionSummary(logger);
					});
				StringBuffer<> traceFile(g_rootDir.data);
				traceFile.EnsureEndsWithSlash().Append(TCV("UbaCacheServer.uba"));
				trace.StopWrite(traceFile.data);
			});


		bool wasTerminated = false;
		if (!storageServer.LoadCasTable(true, true, &wasTerminated))
			return false;

		CacheServerCreateInfo cacheInfo(storageServer, g_rootDir.data, logWriter);
		cacheInfo.Apply(config);
		cacheInfo.expirationTimeSeconds = expirationTimeSeconds;
		cacheInfo.maintenanceReserveSize = maintenanceReserveSizeMb * 1024 * 1024;
		CacheServer cacheServer(cacheInfo);

		if (!cacheServer.Load(wasTerminated))
			return false;

		if (fullMaintenance)
			cacheServer.SetForceFullMaintenance();

		if (maintenanceEnabled)
			if (!cacheServer.RunMaintenance(true, allowSave, ShouldExit))
				return false;


		if (hasPermanentCrypto)
		{
			logger.Info(TC("Adding permanent crypto. crypto now required"));
			networkServer.RegisterCryptoKey(permanentCrypto, ~0llu, 0);
		}

		HttpServer httpServer(logWriter, networkBackend);

		if (httpPort)
		{
			httpServer.AddCommandHandler([&](StringView command, StringBufferBase& arguments)
				{
					u8 crypto128Data[16];
					u64 expirationSeconds = 60;
					u64 userData = 0;
					auto registerCrypto = [&]()
						{
							u64 expirationTime = GetTime() + MsToTime(expirationSeconds*1000);
							if (httpCryptos.empty())
								networkServer.RegisterCryptoKey(crypto128Data, expirationTime, userData);
							else
							{
								for (auto& crypto : httpCryptos)
								{
									u8 decryptedKey[16];
									memcpy(decryptedKey, crypto128Data, 16);
									Guid initVector;
									if (!Crypto::Decrypt(logger, crypto, decryptedKey, 16, initVector))
										return "Failed to decrypt key from http addcrypto";
									networkServer.RegisterCryptoKey(decryptedKey, expirationTime, userData);
								}
							}
							return (const char*)nullptr;
						};


					if (command.Equals(TCV("addcrypto")))
					{
						if (tchar* comma  = TStrchr(arguments.data, ','))
						{
							*comma = 0;
							if (!Parse(expirationSeconds, comma+1, TStrlen(comma+1)))
								return "Failed to parse expiration seconds";
						}
						if (!CryptoFromString(crypto128Data, 16, arguments.data))
							return "Failed to read crypto argument (Needs to be 32 characters long)";
						return registerCrypto();
					}
					if (command.Equals(TCV("addsession")))
					{
						tchar* argument = arguments.data;
						while (true)
						{
							tchar* comma = TStrchr(argument, ',');
							if (comma)
								*comma = 0;

							tchar* equalSign = TStrchr(argument, '=');
							if (!equalSign)
								return "Bad argument format";
							*equalSign = 0;
							tchar* value = equalSign + 1;

							if (Equals(argument, TC("crypto")))
							{
								if (!CryptoFromString(crypto128Data, 16, value))
									return "Failed to read crypto argument (Needs to be 32 characters long)";
							}
							else if (Equals(argument, TC("readonly")))
							{
								userData = (!Equals(value, TC("false")) && !Equals(value, TC("0"))) ? 1 : 0;
							}
							else if (Equals(argument, TC("expiration")))
							{
								if (!Parse(expirationSeconds, value, TStrlen(value+1)))
									return "Failed to parse expiration seconds";
							}
							else
							{
								return "Unknown argument to addsession. (supports crypto,readonly and expiration)";
							}

							if (!comma)
								break;
							argument = comma + 1;
						}
						return registerCrypto();
					}
	
					return "Unknown command (existing commands: addcrypto, addsession)";
				});
			httpServer.StartListen(httpPort);
		}

		{
			auto stopListen = MakeGuard([&]() { networkBackend.StopListen(); });
			auto stopServer = MakeGuard([&]() { networkServer.DisconnectClients(); });

			if (!networkServer.StartListen(networkBackend, port, listenIp.data, requiresCrypto))
				return false;

			u64 lastUpdateTime = GetTime();

			while (!ShouldExit() && !cacheServer.ShouldShutdown())
			{
				Sleep(1000);

				bool force = false;
				StringBuffer<> statusInfo;

				#if PLATFORM_LINUX
				struct statvfs stat;
				if (statvfs(g_rootDir.data, &stat) == 0)
				{
					u64 available = stat.f_bsize * stat.f_bavail;
					statusInfo.Append(TC(" FreeDisk: ")).Append(BytesToText(available).str);
					if (available < 1024ull * 1024 * 1024)
					{
						logger.Warning(TC("Running low on disk space. Only %s available. Will force maintenance"), BytesToText(available).str);
						force = true;
					}
				}
				#endif

				u64 currentTime = GetTime();
				if (TimeToMs(currentTime - lastUpdateTime) > reportIntervalSeconds*1000)
				{
					lastUpdateTime = currentTime;
					cacheServer.PrintStatusLine(statusInfo.data);
				}

				if (maintenanceEnabled)
					if (!cacheServer.RunMaintenance(force, allowSave, ShouldExit))
						break;
			}
		}

		if (maintenanceEnabled)
			cacheServer.RunMaintenance(false, allowSave, {});

		storageServer.DeleteIsRunningFile();

		return true;
	}
}

#if PLATFORM_WINDOWS
int wmain(int argc, wchar_t* argv[])
{
	int res = uba::WrappedMain(argc, argv) ? 0 : 1;
	return res;
}
#else
int main(int argc, char* argv[])
{
	return uba::WrappedMain(argc, argv) ? 0 : 1;
}
#endif
