// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaHttpServer.h"
#include "UbaStringBuffer.h"

namespace uba
{
	struct HttpServer::Connection
	{
		HttpServer& server;
		void* connection;
	};

	HttpServer::HttpServer(LogWriter& logWriter, NetworkBackend& backend, const tchar* name)
	:	m_logger(logWriter, name)
	,	m_backend(backend)
	{
	}

	bool HttpServer::ReceiveHeader(void* context, const Guid& connectionUid, u8* headerData, void*& outBodyContext, u8*& outBodyData, u32& outBodySize)
	{
		outBodySize = 32*1024;
		outBodyData = new u8[outBodySize];
		return true;
	}

	bool HttpServer::ReceiveBody(void* context, bool recvError, u8* headerData, void* bodyContext, u8* bodyData, u32 bodySize)
	{
		auto& c = *(Connection*)context;

		char response[1024];
		auto sendAndExit = MakeGuard([&]()
			{
				NetworkBackend::SendContext sendContext; c.server.m_backend.Send(c.server.m_logger, c.connection, response, u32(strlen(response)), sendContext, TC("HttpServer"));
				delete[] bodyData;
			});

		auto sendFail = [&](const char* failReason)
			{
				snprintf(response, 1024, "HTTP/1.1 404 Not Found\r\n""Content-Type: text/plain\r\n""\r\n""404 %s", failReason);
				return false;
			};

		auto request = (const char*)bodyData;
		if (strncmp(request, "GET /", 5) != 0)
			return sendFail("Only support GET");

		request += 5;
		const char* commandEnd = strchr(request, '?');
		
		if (!commandEnd)
			return sendFail("Command not found (must end with '?')");

		u32 commandLen = u32(commandEnd - request);
		if (commandLen >= 128)
			return sendFail("Command not found (must end with '?')");

		StringBuffer<128> command;
		command.Append(request, commandLen);

		request = commandEnd + 1;

		const char* argumentsEnd = strchr(request, ' ');
		if (!argumentsEnd)
			return sendFail("Arguments end not found");

		if (argumentsEnd - request >= 256)
			return sendFail("arguments too long");

		StringBuffer<512> arguments;
		arguments.Append(request, u32(argumentsEnd - request));

		if (const char* result = c.server.m_handler(command, arguments))
			return sendFail(result);

		snprintf(response, 1024, "HTTP/1.1 200 OK\r\n""Content-Type: text/plain\r\n""\r\n");
		return false;
	}

	bool HttpServer::StartListen(u16 port)
	{
		bool linger = true;
		m_backend.StartListen(m_logger, port, nullptr, [this](void* connection, const sockaddr& remoteSocketAddr)
		{
			m_backend.SetAllowLessThanBodySize(connection, true);
			auto c = new Connection{*this, connection};
			m_backend.SetDisconnectCallback(connection, c, [](void* context, const Guid& connectionUid, void* connection) { delete (Connection*)context; });
			m_backend.SetRecvCallbacks(connection, c, 0, ReceiveHeader, ReceiveBody, TC("Receive"));
			return true;
		}, linger);
		return true;
	}

	void HttpServer::AddCommandHandler(const CommandHandler& handler)
	{
		m_handler = handler;
	}
}