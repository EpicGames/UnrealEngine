// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaLogger.h"
#include "UbaNetworkBackend.h"

namespace uba
{
	class HttpServer
	{
	public:
		HttpServer(LogWriter& logWriter, NetworkBackend& backend, const tchar* name = TC("UbaHttpServer"));

		using CommandHandler = Function<const char*(StringView command, StringBufferBase& arguments)>;
		void AddCommandHandler(const CommandHandler& handler);

		bool StartListen(u16 port);

	private:
		static bool ReceiveHeader(void* context, const Guid& connectionUid, u8* headerData, void*& outBodyContext, u8*& outBodyData, u32& outBodySize);
		static bool ReceiveBody(void* context, bool recvError, u8* headerData, void* bodyContext, u8* bodyData, u32 bodySize);

		struct Connection;
		MutableLogger m_logger;
		NetworkBackend& m_backend;
		CommandHandler m_handler;
	};
}