// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaNetworkBackendMemory.h"
#include "UbaNetworkServer.h"
#include "UbaSessionClient.h"
#include "UbaStorageClient.h"
#include "UbaStorageProxy.h"

namespace uba
{
	struct ClientInitInfo
	{
		LogWriter& logWriter;
		NetworkBackend& networkBackend;
		const tchar* rootDir = nullptr;
		const tchar* host = TC("localhost");
		u16 port = DefaultPort;
		const tchar* zone = TC("");
		u32 maxProcessorCount = ~0u;
		u32 index = 0;
		bool createSession = true;
		bool addDirSuffix = true;
		bool useDependencyCrawler = false;
	};

	class Client
	{
	public:
		bool Init(const ClientInitInfo& info)
		{
			networkBackend = &info.networkBackend;
			networkBackendMem = new NetworkBackendMemory(info.logWriter);
			bool ctorSuccess = true;
			NetworkClientCreateInfo ncci(info.logWriter);
			ncci.workerCount = Min(info.maxProcessorCount, 8u);
			networkClient = new NetworkClient(ctorSuccess, ncci);
			if (!ctorSuccess)
				return false;

			StringBuffer<> clientRootDir;
			clientRootDir.Append(info.rootDir);
			if (info.addDirSuffix)
				clientRootDir.Append("Agent").AppendValue(info.index);

			StorageClientCreateInfo storageClientInfo(*networkClient, clientRootDir.data);
			storageClientInfo.zone = info.zone;
			storageClientInfo.getProxyBackendCallback = [](void* ud, const tchar* h) -> NetworkBackend& { return ((Client*)ud)->GetProxyBackend(h); };
			storageClientInfo.getProxyBackendUserData = this;
			storageClientInfo.startProxyCallback = [](void* ud, u16 p, const Guid& ssu) { return ((Client*)ud)->StartProxy(p, ssu); };
			storageClientInfo.startProxyUserData = this;
			storageClient = new StorageClient(storageClientInfo);

			storageClient->LoadCasTable(false);

			storageClient->Start();

			if (info.createSession)
			{
				SessionClientCreateInfo sessionClientInfo(*storageClient, *networkClient, info.logWriter);
				sessionClientInfo.maxProcessCount = info.maxProcessorCount;
				sessionClientInfo.rootDir = clientRootDir.data;
				sessionClientInfo.deleteSessionsOlderThanSeconds = 1;
				sessionClientInfo.name.Append("Agent").AppendValue(info.index);
				sessionClientInfo.useDependencyCrawler = info.useDependencyCrawler;

				sessionClient = new SessionClient(sessionClientInfo);
				sessionClient->Start();
			}
			
			m_host = info.host;
			m_port = info.port;

			if (!networkClient->Connect(*networkBackend, m_host.c_str(), m_port))
				return false;

			m_loopDone.Create(EventResetType_Manual);
			if (sessionClient)
			{
				m_loop.Start([this]()
					{
						u32 timeoutMs = 0;
						while (!m_loopDone.IsSet(timeoutMs))
						{
							timeoutMs = 1000;
							u32 desiredConnections = sessionClient->GetDesiredConnectionCount();
							if (desiredConnections <= networkClient->GetConnectionCount())
								continue;
							networkClient->Connect(*networkBackend, m_host.c_str(), m_port);
							timeoutMs = 0;
						}
						return 0;
					}, TC("UbaClLoop"));
			}
			return true;
		}

		bool StartProxy(u16 proxyPort, const Guid& storageServerUid)
		{
			NetworkServerCreateInfo nsci(networkClient->GetLogWriter());
			nsci.workerCount = 192;
			nsci.receiveTimeoutSeconds = 60;

			StringBuffer<256> prefix;
			prefix.Append(TCV("UbaProxyServer (")).Append(GuidToString(networkClient->GetUid()).str).Append(')');
			serverPrefix = prefix.data;
			bool ctorSuccess = true;
			proxyNetworkServer = new NetworkServer(ctorSuccess, nsci, serverPrefix.c_str());
			if (!ctorSuccess)
			{
				delete proxyNetworkServer;
				return false;
			}
			StorageProxyCreateInfo proxyInfo { *proxyNetworkServer, *networkClient, storageServerUid, TC("Wooohoo"), storageClient };
			proxyStorage = new StorageProxy(proxyInfo);
			proxyNetworkServer->StartListen(*networkBackendMem, proxyPort);
			proxyNetworkServer->StartListen(*networkBackend, proxyPort);
			return true;
		}

		NetworkBackend& GetProxyBackend(const tchar* host)
		{
			return Equals(host, TC("inprocess")) ? *networkBackendMem : *networkBackend;
		}

		~Client()
		{
			if (m_loopDone.IsCreated())
			{
				m_loopDone.Set();
				m_loop.Wait();
			}
			if (proxyNetworkServer)
				proxyNetworkServer->DisconnectClients();
			if (storageClient)
				storageClient->StopProxy();
			if (sessionClient)
				sessionClient->Stop();
			if (networkClient)
				networkClient->Disconnect();
			delete proxyStorage;
			delete proxyNetworkServer;
			delete sessionClient;
			delete storageClient;
			delete networkClient;
			delete networkBackendMem;
		}

		TString m_host;
		u16 m_port;

		NetworkBackendMemory* networkBackendMem = nullptr;
		NetworkClient* networkClient = nullptr;
		StorageClient* storageClient = nullptr;
		SessionClient* sessionClient = nullptr;

		NetworkBackend* networkBackend = nullptr;
		NetworkServer* proxyNetworkServer = nullptr;
		StorageProxy* proxyStorage = nullptr;
		TString serverPrefix;

		Event m_loopDone;
		Thread m_loop;
	};
}
