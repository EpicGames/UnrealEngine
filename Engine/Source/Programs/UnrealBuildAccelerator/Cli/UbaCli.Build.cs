// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaCli : ModuleRules
{
	public UbaCli(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivatePCHHeaderFile = "../Core/Public/UbaCorePch.h";

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		StaticAnalyzerDisabledCheckers.Clear();

		bRequiresPlatformSDK = true;

		PrivateDependencyModuleNames.AddRange(new string[] {
			"UbaCommon",
		});

		PrivateDefinitions.AddRange(new string[] {
			"_CONSOLE",
		});

		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PublicFrameworks.AddRange(["CoreFoundation", "Security"]);
		}
	}
}
