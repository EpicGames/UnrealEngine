// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaCacheClient.h"
#include "UbaApplicationRules.h"
#include "UbaCacheEntry.h"
#include "UbaCompactTables.h"
#include "UbaCompressedFileHeader.h"
#include "UbaConfig.h"
#include "UbaDirectoryIterator.h"
#include "UbaFileAccessor.h"
#include "UbaNetworkMessage.h"
#include "UbaProcessStartInfo.h"
#include "UbaRootPaths.h"
#include "UbaSession.h"
#include "UbaStorage.h"
#include "UbaStorageUtils.h"

#define UBA_LOG_WRITE_CACHE_INFO 0 // 0 = Disabled, 1 = Normal, 2 = Detailed
#define UBA_LOG_FETCH_CACHE_INFO 0 // 0 = Disabled, 1 = Misses, 2 = Both misses and hits

#define UBA_TRACE_WRITE_CACHE 0
#define UBA_TRACE_FETCH_CACHE 0

#if UBA_TRACE_WRITE_CACHE
#define UBA_TRACE_WRITE_HINT(text) tws.AddHint(AsView(TC(text)));
#else
#define UBA_TRACE_WRITE_HINT(text)
#endif

#if UBA_TRACE_FETCH_CACHE
#define UBA_TRACE_FETCH_HINT(text) tws.AddHint(AsView(TC(text)));
#define UBA_TRACE_FETCH_HINT_SCOPE(text) TrackHintScope ths(tws, AsView(TC(text)));
#else
#define UBA_TRACE_FETCH_HINT(text)
#define UBA_TRACE_FETCH_HINT_SCOPE(text)
#endif

#if UBA_CACHE_USE_SEND_MAX_BUFFER_CACHE
#define UBA_CACHE_BINARY_WRITER_MAX_SIZE(name) BinaryWriterUsingCache name(m_sendBufferSlots);
#define UBA_CACHE_BINARY_READER_MAX_SIZE(name) BinaryReaderUsingCache name(m_sendBufferSlots);
#else
#define UBA_CACHE_BINARY_WRITER_MAX_SIZE(name) StackBinaryWriter<SendMaxSize> name;
#define UBA_CACHE_BINARY_READER_MAX_SIZE(name) StackBinaryReader<SendMaxSize> name;
#endif

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct CacheClient::DowngradedLogger : public LoggerWithWriter
	{
		DowngradedLogger(Atomic<bool>& c, LogWriter& writer, const tchar* prefix) : LoggerWithWriter(writer, prefix), connected(c) {}
		virtual void Log(LogEntryType type, const tchar* str, u32 strLen) override { if (connected) LoggerWithWriter::Log(Max(type, LogEntryType_Info), str, strLen); }
		Atomic<bool>& connected;
	};

	void CacheClientCreateInfo::Apply(Config& config, const tchar* tableName)
	{
		const ConfigTable* tablePtr = config.GetTable(tableName);
		if (!tablePtr)
			return;
		const ConfigTable& table = *tablePtr;
		table.GetValueAsBool(useDirectoryPreparsing, TC("UseDirectoryPreparsing"));
		table.GetValueAsBool(validateCacheWritesInput, TC("ValidateCacheWritesInput"));
		table.GetValueAsBool(validateCacheWritesOutput, TC("ValidateCacheWritesOutput"));
		table.GetValueAsBool(reportCacheKey, TC("ReportCacheKey"));
		table.GetValueAsBool(reportMissReason, TC("ReportMissReason"));
		table.GetValueAsBool(useRoots, TC("UseRoots"));
		table.GetValueAsBool(useCacheHit, TC("UseCacheHit"));
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct CacheClient::Bucket
	{
		Bucket(u32 id_)
		:	id(id_)
		,	serverPathTable(CaseInsensitiveFs, 0, 0, CacheBucketVersion)
		,	sendPathTable(CaseInsensitiveFs, 0, 0, CacheBucketVersion)
		{
		}

		u32 id = 0;

		Futex serverPathTableNetworkLock;
		CompactPathTable serverPathTable;
		Atomic<u32> serverPathTableSize = 0;

		Futex serverCasKeyTableNetworkLock;
		CompactCasKeyTable serverCasKeyTable;
		Atomic<u32> serverCasKeyTableSize = 0;

		CompactPathTable sendPathTable;
		Futex sendPathTableNetworkLock;
		u32 pathTableSizeSent = 0;
		Atomic<u32> pathTableSizeMaxToSend = 0;

		CompactCasKeyTable sendCasKeyTable;
		Futex sendCasKeyTableNetworkLock;
		u32 casKeyTableSizeSent = 0;
		Atomic<u32> casKeyTableSizeMaxToSend = 0;
		Atomic<bool> sendCorrupted = false;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct CacheClient::FetchScope
	{
		FetchScope(Session& session, bool& success, u8* memory, const tchar* desc) : m_session(session), m_success(success), m_memory(memory)
		,	storageStatsScope(storageStats)
		,	kernelStatsScope(kernelStats)
		{
			m_fetchId = m_session.CreateProcessId();
			m_session.GetTrace().CacheBeginFetch(m_fetchId, desc);
		}

		~FetchScope()
		{
			BinaryWriter writer(m_memory, 0, SendMaxSize);
			cacheStats.Write(writer);
			storageStats.Write(writer);
			kernelStats.Write(writer);
			m_session.GetTrace().CacheEndFetch(m_fetchId, m_success, m_memory, writer.GetPosition());

			KernelStats::GetGlobal().Add(kernelStats);
			m_session.GetStorage().AddStats(storageStats);
		}

		Session& m_session;
		bool& m_success;
		u8* m_memory;
		u32 m_fetchId;

		CacheFetchStats cacheStats;
		StorageStats storageStats;
		KernelStats kernelStats;

		StorageStatsScope storageStatsScope;
		KernelStatsScope kernelStatsScope;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	CacheClient::FetchContext::FetchContext(const ProcessStartInfo& i, Session& s, RootsHandle rh, MessagePriority p) : info(i), priority(p)
	{
		s.PopulateLocalToIndexRoots(rootPaths, rh);
	}

	CacheClient::FetchContext::FetchContext(const ProcessStartInfo& i, const RootPaths& rp) : info(i), priority(HasPriority)
	{
		rootPaths = rp;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	CacheClient::CacheClient(const CacheClientCreateInfo& info)
	:	m_logger(info.writer, TC("UbaCacheClient"))
	,	m_storage(info.storage)
	,	m_client(info.client)
	,	m_session(info.session)
#if UBA_CACHE_USE_SEND_MAX_BUFFER_CACHE
	,	m_sendBufferSlots(SendMaxSize)
#endif
	{
		m_reportCacheKey = info.reportCacheKey;
		m_reportMissReason = info.reportMissReason;
		#if UBA_LOG_FETCH_CACHE_INFO
		m_reportMissReason = true;
		#endif

		m_useDirectoryPreParsing = info.useDirectoryPreparsing;
		m_validateCacheWritesInput = info.validateCacheWritesInput;
		m_validateCacheWritesOutput = info.validateCacheWritesOutput;
		m_useCacheHit = info.useCacheHit;
		m_useMappingForCreatedCas = info.useMappingForCreatedCas;
		m_useRoots = info.useRoots;
		m_connected = false;

		m_client.RegisterOnConnected([this, hint = TString(info.hint)]()
			{
				u32 retryCount = 0;
				while (retryCount < 10)
				{
					StackBinaryWriter<1024> writer;
					NetworkMessage msg(m_client, CacheServiceId, CacheMessageType_Connect, writer);
					writer.WriteU32(CacheNetworkVersion);
					writer.WriteString(hint);
					StackBinaryReader<1024> reader;
					u64 sendTime = GetTime();
					if (!msg.Send(reader))
					{
						m_logger.Info(TC("Failed to send connect message to cache server (%u). Version mismatch? (%s)"), msg.GetError(), TimeToText(GetTime() - sendTime).str);
						return;
					}
					bool success = reader.ReadBool();
					if (success)
					{
						if (retryCount != 0)
							m_logger.Info(TC("Connected to cache server"));
						m_connected = true;
						return;
					}

					if (retryCount == 0)
					{
						StringBuffer<> reason;
						reader.ReadString(reason);
						m_logger.Info(TC("Cache server busy, retrying... (Reason: %s)"), reason.data);
					}
					Sleep(1000);
					++retryCount;
				}

				m_logger.Info(TC("Failed to connect to cache server after %u retries. Giving up."), retryCount);

			});

		m_client.RegisterOnDisconnected([this]()
			{
				m_connected = false;
			});

		if (m_session.HasDetailedtrace())
			m_client.SetWorkTracker(&m_session.GetTrace());
	}

	CacheClient::~CacheClient()
	{
		TraceSummary();
	}

	bool CacheClient::RegisterPathHash(const tchar* path, const CasKey& hash)
	{
		m_pathHashes.emplace_back(TString(path), AsCompressed(hash, true));
		return true;
	}

	u64 CacheClient::RegisterSpeculativeOutputs(FetchContext& context)
	{
		UBA_ASSERT(context.bucket);
		auto& bucket = *context.bucket;

		SCOPED_FUTEX(m_speculativeCasKeysLock, lock);
		u64 id = m_nextSpeculativeOutputId++;
		auto& keys = m_speculativeOutputsById[id];

		for (u32 offset : context.casKeyOffsets)
		{
			StringBuffer<MaxPath> path;
			CasKey casKey;
			if (!GetLocalPathAndCasKey(bucket, context.rootPaths, path, casKey, bucket.serverCasKeyTable, bucket.serverPathTable, offset))
			{
				UBA_ASSERT(false);
				continue;
			}
			StringBuffer<MaxPath> forKey;
			forKey.Append(path);
			if (CaseInsensitiveFs)
				forKey.MakeLower();
			StringKey fileNameKey = ToStringKey(forKey);

			m_speculativeCasKeys.emplace(fileNameKey, casKey);
			keys.push_back(fileNameKey);
		}
		return id;
	}

	void CacheClient::UnregisterSpeculativeOutputs(u64 id)
	{
		SCOPED_FUTEX(m_speculativeCasKeysLock, lock);
		auto it = m_speculativeOutputsById.find(id);
		if (it == m_speculativeOutputsById.end())
			return;
		for (const StringKey& key : it->second)
			m_speculativeCasKeys.erase(key);
		m_speculativeOutputsById.erase(it);
	}

	bool CacheClient::WriteToCache(u32 bucketId, const ProcessStartInfo& info, const u8* inputs, u64 inputsSize, const u8* outputs, u64 outputsSize, const u8* logLines, u64 logLinesSize, u32 processId)
	{
		RootPaths rootPaths;
		if (!m_session.PopulateLocalToIndexRoots(rootPaths, info.rootsHandle))
			return false;
		return WriteToCache(rootPaths, bucketId, info, inputs, inputsSize, outputs, outputsSize, logLines, logLinesSize, processId);
	}

	bool CacheClient::WriteToCache(const RootPaths& rootPaths, u32 bucketId, const ProcessStartInfo& info, const u8* inputs, u64 inputsSize, const u8* outputs, u64 outputsSize, const u8* logLines, u64 logLinesSize, u32 processId)
	{
		if (!m_connected)
			return false;

		#if UBA_TRACE_WRITE_CACHE
		TrackWorkScope tws(m_client, AsView(TC("WriteToCache")));
		#else
		TrackWorkScope tws;
		#endif

		CasKey cmdKey = GetCmdKey(rootPaths, info, false, bucketId);
		if (cmdKey == CasKeyZero)
		{
			#if UBA_LOG_WRITE_CACHE_INFO
			m_logger.Info(TC("WRITECACHE FAIL: %s"), info.GetDescription());
			#endif
			return false;
		}

		CacheSendStats stats;

		bool finished = false;
		if (processId)
			m_session.GetTrace().CacheBeginWrite(processId);
		auto tg = MakeGuard([&]()
			{
				if (!processId)
					return;
				StackBinaryWriter<128> writer;
				stats.Write(writer);
				m_session.GetTrace().CacheEndWrite(processId, finished, writer.GetData(), writer.GetPosition());
			});

		BinaryReader inputsReader(inputs, 0, inputsSize);
		BinaryReader outputsReader(outputs, 0, outputsSize);

		Map<u32, u32> inputsStringToCasKey;
		Map<u32, u32> outputsStringToCasKey;
		u32 requiredPathTableSize = 0;
		u32 requiredCasTableSize = 0;
		bool success = true;

		SCOPED_FUTEX(m_bucketsLock, bucketsLock);
		Bucket& bucket = m_buckets.try_emplace(bucketId, bucketId).first->second;
		if (!bucket.sendPathTable.GetMemory())
			bucket.sendPathTable.AddCommonStringSegments();
		bucketsLock.Leave();

		TString qualifiedPath;

		Vector<bool> handledPathHashes;
		handledPathHashes.resize(m_pathHashes.size());

		UBA_TRACE_WRITE_HINT("TraverseInputsOutputs");

		TimerScope buildTs(stats.build);

		// This is here to make sure cache client can use dropped cas entries without making them permanent
		// .. which means that it needs to ref count the cas entries until it is done with them
		Vector<StorageImpl::CasEntry*> refCountedCas;
		auto refCountedCasGuard = MakeGuard([&]()
			{
				for (auto casEntry : refCountedCas)
					m_storage.ReleaseCasEntry(*casEntry, true);
			});

		// Traverse all inputs and outputs. to create cache entry that we can send to server
		while (true)
		{
			CasKey casKey;

			StringBuffer<512> path;
			bool isOutput = outputsReader.GetLeft() != 0;
			if (isOutput)
				outputsReader.ReadString(path);
			else if (inputsReader.GetLeft())
				inputsReader.ReadString(path);
			else
				break;
			
			if (path.count < 2)
			{
				m_logger.Info(TC("Got messed up path from caller to WriteToCache: %s (%s)"), path.data, info.GetDescription());
				success = false;
			}

			// For .exe and .dll we sometimes get relative paths so we need to expand them to full
			#if PLATFORM_WINDOWS
			if (path[1] != ':' && (path.EndsWith(TCV(".dll")) || path.EndsWith(TCV(".exe"))))
			{
				tchar temp[512];
				bool res = SearchPathW(NULL, path.data, NULL, 512, temp, NULL);
				if (!res)
				{
					m_logger.Info(TC("Can't find file: %s"), path.data);
					return false;
				}
				path.Clear().Append(temp);
			}
			#endif

			// Ignore cmd.exe or sh as input. It should always exist but can be different between windows versions
			if (!isOutput && path.EndsWith(IsWindows ? TCV("\\cmd.exe") : TCV("/sh")))
				continue;

			if (ShouldNormalize(path)) // Paths can be absolute in rsp files so we need to normalize those paths
			{
				casKey = rootPaths.NormalizeAndHashFile(m_logger, path.data);
				if (casKey == CasKeyZero)
				{
					success = false;
					continue;
				}
				casKey = IsNormalized(casKey) ? AsCompressed(casKey, true) : CasKeyZero;
			}

			// Handle path hashes.
			if (!isOutput)
			{
				bool handled = false;
				for (u32 i=0, e=u32(m_pathHashes.size()); i!=e; ++i)
				{
					auto& ph = m_pathHashes[i];
					if (!path.StartsWith(StringView(ph.path), CaseInsensitiveFs))
						continue;
					if (handledPathHashes[i])
					{
						handled = true;
						break;
					}
					handledPathHashes[i] = true;
					path.Clear().Append(ph.path).Append(TCV("<PathHash>"));
					casKey = ph.hash;
					break;
				}
				if (handled)
					continue;
			}

			if (m_useRoots)
			{
				// Find root for path in order to be able to normalize it.
				auto root = rootPaths.FindRoot(path);
				if (!root)
				{
					// Temporarily disabling invalid roots warnings to mitigate a log spam until an issue with them can be fixed
					//m_logger.Info(TC("FILE WITHOUT ROOT: %s (%s)"), path.data, info.GetDescription());
					success = false;
					continue;
				}

				if (!root->includeInKey)
					continue;

				u32 rootLen = u32(root->path.size());
				qualifiedPath = path.data + rootLen - 1;
				qualifiedPath[0] = tchar(RootPaths::RootStartByte + root->index);
			}
			else
			{
				qualifiedPath = path.data;
			}


			u32 pathOffset = bucket.sendPathTable.Add(qualifiedPath.c_str(), u32(qualifiedPath.size()), requiredPathTableSize);

			if (!isOutput) // Output files should be removed from input files.. For example when cl.exe compiles pch it reads previous pch file and we don't want it to be input
			{
				if (outputsStringToCasKey.find(pathOffset) != outputsStringToCasKey.end())
					continue;
				//m_logger.Info(TC("INPUT ENTRY: %s -> %u"), qualifiedPath.c_str(), pathOffset);
			}
			else
			{
				inputsStringToCasKey.erase(pathOffset);
				//m_logger.Info(TC("OUT ENTRY: %s -> %u"), qualifiedPath.c_str(), pathOffset);
			}

			auto& stringToCasKey = isOutput ? outputsStringToCasKey : inputsStringToCasKey;
			auto insres = stringToCasKey.try_emplace(pathOffset);
			
			if (!insres.second)
			{
				//m_logger.Warning(TC("Input file %s exists multiple times"), qualifiedPath.c_str()); 
				continue;
			}

			// Get file caskey using storage
			if (casKey == CasKeyZero)
			{
				bool shouldValidate = (m_validateCacheWritesInput && !isOutput) || (m_validateCacheWritesOutput && isOutput);
				bool deferCreation = true;

				if (isOutput)
				{
					StorageImpl::CasEntry* casEntry = nullptr;
					if (!m_storage.StoreCasFile(casKey, path.data, CasKeyZero, deferCreation, &casEntry))
						return m_logger.Info(TC("Failed to store output file %s when writing to cache"), path.data).ToFalse();
					if (casEntry)
						refCountedCas.push_back(casEntry);
				}
				else
				{
					if (!m_storage.StoreCasKey(casKey, path.data, CasKeyZero))
						return m_logger.Info(TC("Failed to store input file %s when writing to cache"), path.data).ToFalse();
				}

				if (casKey == CasKeyZero) // If file is not found it was a temporary file that was deleted and is not really an output
				{
					
					if (shouldValidate && FileExists(m_logger, path.data))
						return m_logger.Warning(TC("CasDb claims file %s does not exist but it does! Will not populate cache for %s"), path.data, info.GetDescription()); 

					//m_logger.Warning(TC("Can't find file %s"), path.data); 
					stringToCasKey.erase(insres.first);
					continue; // m_logger.Info(TC("This should never happen! (%s)"), path.data);
				}


				if (shouldValidate)
				{
					FileAccessor fa(m_logger, path.data);
					if (!fa.OpenMemoryRead())
						return m_logger.Warning(TC("CasDb claims file %s does exist but can't open it. Will not populate cache for %s"), path.data, info.GetDescription()); 

					CasKey oldKey = AsCompressed(casKey, false);
					CasKey newKey;

					u64 fileSize = fa.GetSize();
					u8* fileMem = fa.GetData();

					if (fileSize > sizeof(CompressedFileHeader) && ((CompressedFileHeader*)fileMem)->IsValid())
						newKey = AsCompressed(((CompressedFileHeader*)fileMem)->casKey, false);
					else
						newKey = CalculateCasKey(fileMem, fileSize, false, nullptr, path.data);

					if (newKey != oldKey)
					{
						FileBasicInformation fileInfo;
						fa.GetFileBasicInformationByHandle(fileInfo);

						auto& fileEntry = m_storage.GetOrCreateFileEntry(CaseInsensitiveFs ? ToStringKeyLower(path) : ToStringKey(path));
						SCOPED_FUTEX_READ(fileEntry.lock, lock);

						auto ToString = [](bool b) { return b ? TC("true") : TC("false"); };
						return m_logger.Warning(TC("CasDb claims file %s has caskey %s but recalculating it gives us %s (FileEntry: %llu/%llu/%s, Real: %llu/%llu). Will not populate cache for %s"),
							path.data, CasKeyString(oldKey).str, CasKeyString(newKey).str, fileEntry.size, fileEntry.lastWritten, ToString(fileEntry.verified), fileSize, fileInfo.lastWriteTime, info.GetDescription());
					}
				}

				#if UBA_TRACK_IS_EXECUTABLE
				if (isOutput)
				{
					auto& fileEntry = m_storage.GetOrCreateFileEntry(CaseInsensitiveFs ? ToStringKeyLower(path) : ToStringKey(path));
					if (fileEntry.isExecutable)
						casKey = AsExecutable(casKey, true);
				}
				#endif
			}

			#ifndef __clang_analyzer__ // Must be bug in analyzer
			UBA_ASSERT(IsCompressed(casKey));
			#endif

			insres.first->second = bucket.sendCasKeyTable.Add(casKey, pathOffset, requiredCasTableSize);
		}

		if (!success)
			return false;

		buildTs.Leave();

		if (outputsStringToCasKey.empty())
			m_logger.Warning(TC("NO OUTPUTS FROM process %s"), info.GetDescription()); 

		UBA_TRACE_WRITE_HINT("SendPathTable");

		// Make sure server has enough of the path table to be able to resolve offsets from cache entry
		if (!SendPathTable(bucket, stats, requiredPathTableSize))
			return false;

		UBA_TRACE_WRITE_HINT("SendCasTable");

		// Make sure server has enough of the cas table to be able to resolve offsets from cache entry
		if (!SendCasTable(bucket, stats, requiredCasTableSize))
			return false;

		// Send actual cache entry now when we know server has the needed tables. A concurrent send on this
		// bucket may have failed ambiguously (shifting the server scratch table) after our tables were confirmed,
		// which would make our offsets resolve to the wrong place on the server.
		if (bucket.sendCorrupted)
			return false;

		if (!SendCacheEntry(tws, bucket, stats, rootPaths, cmdKey, inputsStringToCasKey, outputsStringToCasKey, logLines, logLinesSize, info))
			return false;


		#if UBA_LOG_WRITE_CACHE_INFO
		m_logger.BeginScope();
		m_logger.Info(TC("WRITECACHE: %s -> %u %s"), info.GetDescription(), bucketId, CasKeyString(cmdKey).str);
		#if UBA_LOG_WRITE_CACHE_INFO == 2
		for (auto& kv : inputsStringToCasKey)
		{
			StringBuffer<> path;
			CasKey casKey;
			bucket.sendCasKeyTable.GetPathAndKey(path, casKey, bucket.sendPathTable, kv.second);
			m_logger.Info(TC("   IN: %s -> %s"), path.data, CasKeyString(casKey).str);
		}
		for (auto& kv : outputsStringToCasKey)
		{
			StringBuffer<> path;
			CasKey casKey;
			bucket.sendCasKeyTable.GetPathAndKey(path, casKey, bucket.sendPathTable, kv.second);
			m_logger.Info(TC("   OUT: %s -> %s"), path.data, CasKeyString(casKey).str);
		}
		#endif // 2
		m_logger.EndScope();
		#endif

		finished = true;
		return true;
	}

	u64 CacheClient::MakeId(u32 bucketId)
	{
		constexpr u64 ForBackwardsCompatibility = 1;
		return u64(bucketId) | ((u64(!CaseInsensitiveFs) + (ForBackwardsCompatibility << 1u) + (u64(!m_useRoots) << 2u) + (CacheBucketVersion << 3u)) << 32u);
	}

	bool CacheClient::FetchFromCache(CacheResult& outResult, RootsHandle rootsHandle, u32 bucketId, const ProcessStartInfo& info)
	{
		FetchContext context(info, m_session, rootsHandle, HasPriority);
		if (!FetchEntryFromCache(outResult, context, bucketId))
			return false;
		if (!outResult.hit)
			return false;
		return FetchFilesFromCache(outResult, context, false);
	}

	bool CacheClient::FetchFromCache(CacheResult& outResult, const RootPaths& rootPaths, u32 bucketId, const ProcessStartInfo& info)
	{
		FetchContext context(info, rootPaths);
		if (!FetchEntryFromCache(outResult, context, bucketId))
			return false;
		if (!outResult.hit)
			return false;
		return FetchFilesFromCache(outResult, context, false);
	}

	bool CacheClient::FetchEntryFromCache(CacheResult& outResult, FetchContext& context, u32 bucketId)
	{
		outResult.hit = false;
		outResult.logLines.clear();

		if (!m_connected)
			return false;

		#if UBA_TRACE_FETCH_CACHE
		TrackWorkScope tws(m_client, AsView(TC("FetchFromCache")));
		#else
		TrackWorkScope tws;
		#endif

		auto& rootPaths = context.rootPaths;
		auto& info = context.info;

#if UBA_CACHE_USE_SEND_MAX_BUFFER_CACHE
		u8* memory = m_sendBufferSlots.Pop();
		auto g = MakeGuard([this, memory]() { m_sendBufferSlots.Push(memory); });
#else
		u8 memory[SendMaxSize];
#endif

		bool success = false;
		FetchScope scope(m_session, success, memory, info.GetDescription());
		auto& cacheStats = scope.cacheStats;


		CasKey cmdKey = GetCmdKey(rootPaths, info, m_reportCacheKey, bucketId);
		if (cmdKey == CasKeyZero)
			return false;

		BinaryReader reader(memory, 0, SendMaxSize);

		SCOPED_FUTEX(m_bucketsLock, bucketsLock);
		Bucket& bucket = m_buckets.try_emplace(bucketId, bucketId).first->second;
		bucketsLock.Leave();

		{
			UBA_TRACE_FETCH_HINT("FetchEntries")

			TimerScope ts(cacheStats.fetchEntries);
			// Fetch entries.. server will provide as many as fits. TODO: Should it be possible to ask for more entries?
			StackBinaryWriter<32> writer;
			NetworkMessage msg(m_client, CacheServiceId, CacheMessageType_FetchEntries, writer, context.priority);
			writer.Write7BitEncoded(MakeId(bucket.id));
			writer.WriteCasKey(cmdKey);
			if (!msg.Send(reader))
				return false;
		}

		u32 entryCount = reader.ReadU16();

		#if UBA_LOG_FETCH_CACHE_INFO
		auto mg = MakeGuard([&]()
			{
				if (!success || UBA_LOG_FETCH_CACHE_INFO == 2)
					m_logger.Info(TC("FETCHCACHE %s: %s -> %u %s (%u)"), success ? TC("SUCC") : TC("FAIL"), info.GetDescription(), bucketId, CasKeyString(cmdKey).str, entryCount);
			});
		#endif

		if (!entryCount)
		{
			if (m_reportMissReason)
				m_logger.Info(TC("Cache miss on %s because no entry with key %s was found in bucket %u (%llu)"), info.GetDescription(), CasKeyString(cmdKey).str, bucketId, MakeId(bucketId));
			return false;
		}

		struct MissInfo { TString path; u32 entryIndex; CasKey cache; CasKey local; };
		Vector<MissInfo> misses;

		#if UBA_TRACE_FETCH_CACHE
		u64 storeTime = 0;
		auto addStoreKeyHint = [&]() { tws.AddHint(AsView(TC("StoreCasKey")), GetTime() - storeTime); storeTime = 0; };
		#endif

		auto IsCasKeyMatch = [&, normalizedCasKeys = UnorderedMap<StringKey, CasKey>(), isCasKeyMatchCache = UnorderedMap<u32, bool>()](bool& outIsMatch, u32 casKeyOffset, u32 entryIndex, bool useLookup) mutable
			{
				outIsMatch = false;

				CasKey cacheCasKey;
				CasKey localCasKey;

				bool* cachedIsMatch = nullptr;
				if (useLookup)
				{
					auto insres = isCasKeyMatchCache.try_emplace(casKeyOffset);
					if (!insres.second)
					{
						outIsMatch = insres.first->second;
						return true;
					}
					cachedIsMatch = &insres.first->second;
				}

				StringBuffer<MaxPath> path;
				if (!GetLocalPathAndCasKey(bucket, rootPaths, path, cacheCasKey, bucket.serverCasKeyTable, bucket.serverPathTable, casKeyOffset))
					return false;
				UBA_ASSERTF(IsCompressed(cacheCasKey), TC("Cache entry for %s has uncompressed cache key for path %s (%s)"), info.GetDescription(), path.data, CasKeyString(cacheCasKey).str);

				if (IsNormalized(cacheCasKey)) // Need to normalize caskey for these files since they contain absolute paths
				{
					auto insres2 = normalizedCasKeys.try_emplace(ToStringKeyNoCheck(path.data, path.count));
					if (insres2.second)
					{
						UBA_TRACE_FETCH_HINT_SCOPE("NormalizeAndHash")
						TimerScope ts(cacheStats.normalizeFile);
						localCasKey = rootPaths.NormalizeAndHashFile(m_logger, path.data);
						if (localCasKey != CasKeyZero)
							localCasKey = AsCompressed(localCasKey, true);
						insres2.first->second = localCasKey;
					}
					else
						localCasKey = insres2.first->second;

				}
				else
				{
					StringBuffer<MaxPath> forKey;
					forKey.Append(path);
					if (CaseInsensitiveFs)
						forKey.MakeLower();
					StringKey fileNameKey = ToStringKey(forKey);

					if (m_useDirectoryPreParsing)
						PreparseDirectory(fileNameKey, path);

					#if UBA_TRACE_FETCH_CACHE
					u64 t = GetTime();
					#endif

					if (path.EndsWith(TCV("<PathHash>")))
					{
						path.Resize(path.count - 10);
						for (auto& ph : m_pathHashes)
							if (path.Equals(ph.path) && cacheCasKey == ph.hash)
							{
								localCasKey = cacheCasKey;
								break;
							}
					}
					else
					{
						{
							SCOPED_FUTEX_READ(m_speculativeCasKeysLock, slock);
							auto speculativeIt = m_speculativeCasKeys.find(fileNameKey);
							if (speculativeIt != m_speculativeCasKeys.end())
								localCasKey = speculativeIt->second;
						}
						if (localCasKey == CasKeyZero)
						{
							if (!m_storage.StoreCasKey(localCasKey, fileNameKey, path.data, CasKeyZero, false))
								m_logger.Info(TC("Failed to store input file %s when fetching from cache"), path.data);
						}
					}
					#if UBA_TRACE_FETCH_CACHE
					storeTime += GetTime() - t;
					#endif

					UBA_ASSERT(localCasKey == CasKeyZero || IsCompressed(localCasKey));
				}

				outIsMatch = localCasKey == cacheCasKey;
				if (useLookup)
					*cachedIsMatch = outIsMatch;

				if (!outIsMatch)
					if (m_reportMissReason && path.count) // if empty this has already been reported
						misses.push_back({path.ToString(), entryIndex, cacheCasKey, localCasKey });
				return true;
			};


		u64 sharedSize = reader.Read7BitEncoded();
		const u8* sharedPos = reader.GetPositionData();
		reader.Skip(sharedSize);

		u64 sharedLogLinesSize = reader.Read7BitEncoded();
		const u8* sharedLogLines = reader.GetPositionData();
		reader.Skip(sharedLogLinesSize);

		// Calculate max caskey offset and fetch tables (faster to do in one call)
		{
			u64 maxCasKeyOffset = 0;
			BinaryReader sharedReader(sharedPos, 0, sharedSize);
			while (sharedReader.GetLeft())
				maxCasKeyOffset = Max(maxCasKeyOffset, sharedReader.Read7BitEncoded());

			BinaryReader entryReader(reader.GetPositionData(), 0, reader.GetLeft());
			for (u32 i=0; i!=entryCount; ++i)
			{
				entryReader.Read7BitEncoded(); // Id
				u64 extraSize = entryReader.Read7BitEncoded();
				BinaryReader extraReader(entryReader.GetPositionData(), 0, extraSize);
				while (extraReader.GetLeft())
					maxCasKeyOffset = Max(maxCasKeyOffset, extraReader.Read7BitEncoded());
				entryReader.Skip(extraSize);
				entryReader.Skip(entryReader.Read7BitEncoded()); // Skip ranges
				entryReader.Skip(entryReader.Read7BitEncoded()); // Skip outs
				entryReader.ReadByte(); // Skip log lines type
			}
			if (!FetchCasTable(tws, bucket, cacheStats, u32(maxCasKeyOffset)))
				return false;
		}

		// Calculate max path offset and fetch path table
		{
			u64 maxPathTableOffset = 0;
			BinaryReader casKeyReader(bucket.serverCasKeyTable.GetMemory(), 0, bucket.serverCasKeyTableSize);
			BinaryReader sharedReader(sharedPos, 0, sharedSize);
			while (sharedReader.GetLeft())
			{
				casKeyReader.SetPosition(sharedReader.Read7BitEncoded());
				maxPathTableOffset = Max(maxPathTableOffset, casKeyReader.Read7BitEncoded());
			}

			BinaryReader entryReader(reader.GetPositionData(), 0, reader.GetLeft());
			for (u32 i=0; i!=entryCount; ++i)
			{
				entryReader.Read7BitEncoded(); // Id
				u64 extraSize = entryReader.Read7BitEncoded();
				BinaryReader extraReader(entryReader.GetPositionData(), 0, extraSize);
				while (extraReader.GetLeft())
				{
					casKeyReader.SetPosition(extraReader.Read7BitEncoded());
					maxPathTableOffset = Max(maxPathTableOffset, casKeyReader.Read7BitEncoded());
				}
				entryReader.Skip(extraSize);
				entryReader.Skip(entryReader.Read7BitEncoded()); // Skip ranges
				entryReader.Skip(entryReader.Read7BitEncoded()); // Skip outs
				entryReader.ReadByte(); // Skip log lines type
			}
			if (!FetchPathTable(tws, bucket, cacheStats, u32(maxPathTableOffset)))
				return false;
		}

		struct Range { u32 begin; u32 end; };
		Vector<Range> sharedMatchingRanges;

		// Create ranges out of shared offsets that matches local state
		{
			UBA_TRACE_FETCH_HINT("TestSharedMatch")
			TimerScope ts(cacheStats.testEntry);

			BinaryReader sharedReader(sharedPos, 0, sharedSize);

			u32 rangeBegin = 0;

			auto addRange = [&](u32 rangeEnd)
				{
					if (rangeBegin != rangeEnd)
					{
						auto& range = sharedMatchingRanges.emplace_back();
						range.begin = rangeBegin;
						range.end = rangeEnd;
					}
				};
			while (sharedReader.GetLeft())
			{
				u32 position = u32(sharedReader.GetPosition());
				bool isMatch;
				if (!IsCasKeyMatch(isMatch, u32(sharedReader.Read7BitEncoded()), 0, false))
					return false;

				if (isMatch)
				{
					if (rangeBegin != ~0u)
						continue;
					rangeBegin = position;
				}
				else
				{
					if (rangeBegin == ~0u)
						continue;
					addRange(position);
					rangeBegin = ~0u;
				}
			}
			if (rangeBegin != ~0u)
				addRange(u32(sharedReader.GetPosition()));
			if (sharedMatchingRanges.empty())
			{
				auto& range = sharedMatchingRanges.emplace_back();
				range.begin = 0;
				range.end = 0;
			}

			#if UBA_TRACE_FETCH_CACHE
			addStoreKeyHint();
			#endif
		}

		// Read entries
		{
			UBA_TRACE_FETCH_HINT("TestEntriesMatch")
			--cacheStats.testEntry.count; // Remove the shared one

			u32 entryIndex = 0;
			for (; entryIndex!=entryCount; ++entryIndex)
			{
				u32 entryId = u32(reader.Read7BitEncoded());
				u64 extraSize = reader.Read7BitEncoded();
				BinaryReader extraReader(reader.GetPositionData(), 0, extraSize);
				reader.Skip(extraSize);
				u64 rangeSize = reader.Read7BitEncoded();
				BinaryReader rangeReader(reader.GetPositionData(), 0, rangeSize);
				reader.Skip(rangeSize);
				u64 outSize = reader.Read7BitEncoded();
				BinaryReader outputsReader(reader.GetPositionData(), 0, outSize);
				reader.Skip(outSize);
				
				auto logLinesType = LogLinesType(reader.ReadByte());

				{
					TimerScope ts(cacheStats.testEntry);

					bool isMatch = true;

					// Check ranges first
					auto sharedRangeIt = sharedMatchingRanges.begin();
					while (isMatch && rangeReader.GetLeft())
					{
						u64 begin = rangeReader.Read7BitEncoded();
						u64 end = rangeReader.Read7BitEncoded();
					
						Range matchingRange = *sharedRangeIt;

						while (matchingRange.end <= begin)
						{
							++sharedRangeIt;
							if (sharedRangeIt == sharedMatchingRanges.end())
								break;
							matchingRange = *sharedRangeIt;
						}

						isMatch = matchingRange.begin <= begin && matchingRange.end >= end;
					}

					// Check extra keys after
					while (isMatch && extraReader.GetLeft())
						if (!IsCasKeyMatch(isMatch, u32(extraReader.Read7BitEncoded()), entryIndex, true))
							return false;

					if (!isMatch)
						continue;
				}

				#if UBA_TRACE_FETCH_CACHE
				addStoreKeyHint();
				#endif

				if (!m_useCacheHit)
					return false;

				UBA_TRACE_FETCH_HINT("ReportUsedEntry")

				if (logLinesType == LogLinesType_Shared)
					if (!PopulateLogLines(outResult.logLines, sharedLogLines, sharedLogLinesSize))
						return false;

				if (!ReportUsedEntry(outResult.logLines, logLinesType == LogLinesType_Owned, bucket, cmdKey, entryId))
					return false;

				while (outputsReader.GetLeft())
					context.casKeyOffsets.push_back(u32(outputsReader.Read7BitEncoded()));
				context.bucket = &bucket;

				// Make sure caskey/path tables has enough data for outputs
				if (!context.casKeyOffsets.empty())
				{
					u64 maxCasKeyOffset = 0;
					for (u64 offset : context.casKeyOffsets)
						maxCasKeyOffset = Max(maxCasKeyOffset, offset);
					if (!FetchCasTable(tws, bucket, cacheStats, u32(maxCasKeyOffset)))
						return false;

					u64 maxPathTableOffset = 0;
					BinaryReader casKeyReader(bucket.serverCasKeyTable.GetMemory(), 0, bucket.serverCasKeyTableSize);
					for (u64 offset : context.casKeyOffsets)
					{
						casKeyReader.SetPosition(offset);
						maxPathTableOffset = Max(maxPathTableOffset, casKeyReader.Read7BitEncoded());
					}
					if (!FetchPathTable(tws, bucket, cacheStats, u32(maxPathTableOffset)))
						return false;
				}

				outResult.hit = true;
				success = true;
				return true;
			}
		}

		for (auto& miss : misses)
			m_logger.Info(TC("Cache miss on %s because of mismatch of %s (entry: %u, local: %s cache: %s bucket: %u)"), info.GetDescription(), miss.path.data(), miss.entryIndex, CasKeyString(miss.local).str, CasKeyString(miss.cache).str, bucketId);

		return false;
	}

	bool CacheClient::FetchFilesFromCache(CacheResult& outResult, FetchContext& context, bool writePlaceholders)
	{
		if (context.casKeyOffsets.empty())
			return true;

		#if UBA_TRACE_FETCH_CACHE
		TrackWorkScope tws(m_client, AsView(TC("FetchFromCache")));
		#else
		TrackWorkScope tws;
		#endif

		bool success = false;
		u8 memory[1024];
		FetchScope scope(m_session, success, memory, context.info.GetDescription());

		// Go wide on multiple files
		Atomic<bool> fetchSuccess = true;
		m_storage.m_workManager.ParallelFor(u32(context.casKeyOffsets.size() - 1), context.casKeyOffsets, [&](const WorkContext&, auto& it)
			{
				KernelStatsScope _(scope.kernelStats);
				UBA_TRACE_FETCH_HINT_SCOPE("FetchFile")
				TimerScope fts(scope.cacheStats.fetchOutput);
				if (!FetchFile(*context.bucket, context.rootPaths, context.info, scope.cacheStats, scope.storageStats, *it, writePlaceholders))
					fetchSuccess = false;

			}, AsView(TC("CacheFetchOutput")));

		success = fetchSuccess;

		return success;
	}

	bool CacheClient::FetchFile(Bucket& bucket, const RootPaths& rootPaths, const ProcessStartInfo& info, CacheFetchStats& cacheStats, StorageStats& storageStats, u32 casKeyOffset, bool writePlaceholders)
	{
		StringBuffer<MaxPath> path;
		CasKey casKey;
		if (!GetLocalPathAndCasKey(bucket, rootPaths, path, casKey, bucket.serverCasKeyTable, bucket.serverPathTable, casKeyOffset))
			return false;
		UBA_ASSERT(IsCompressed(casKey));

		if (writePlaceholders)
		{
			UBA_ASSERT(info.rules);
			if (info.rules->SupportsPlaceholder(path))
			{
				FileAccessor destinationFile(m_logger, path.data);
				return destinationFile.CreateWrite() && destinationFile.Close();
			}
		}

		FileFetcher fetcher { m_storage.m_bufferSlots, storageStats };
		fetcher.m_errorOnFail = false;

		// TODO: This probably is not needed but is correct. Since there could be deferred entries of the file we are downloading and writing to
		//m_storage.ReportFileWrite(path.data, 

		u32 attributes = DefaultAttributes();

		if (IsNormalized(casKey))
		{
			DowngradedLogger logger(m_connected, m_logger.m_writer, TC("UbaCacheClientNormalizedDownload"));
			// Fetch into memory, file is in special format without absolute paths
			MemoryBlock normalizedBlock(TC("FetchFileNormalized"), 4*1024*1024);
			bool destinationIsCompressed = false;
			if (!fetcher.RetrieveFile(logger, m_client, casKey, path.data, destinationIsCompressed, &normalizedBlock))
				return logger.Debug(TC("Failed to download cache output for %s"), info.GetDescription()).ToFalse();

			MemoryBlock localBlock(TC("FetchFileDenormalize"), 4*1024*1024);

			u32 rootOffsets = *(u32*)(normalizedBlock.memory);
			char* fileStart = (char*)(normalizedBlock.memory + sizeof(u32));
			UBA_ASSERT(rootOffsets <= normalizedBlock.writtenSize);

			// "denormalize" fetched file into another memory block that will be written to disk
			u64 lastWritten = 0;
			BinaryReader reader2(normalizedBlock.memory, rootOffsets, normalizedBlock.writtenSize);
			while (reader2.GetLeft())
			{
				u64 rootOffset = reader2.Read7BitEncoded();
				if (u64 toWrite = rootOffset - lastWritten)
					memcpy(localBlock.Allocate(toWrite, 1, TC("")), fileStart + lastWritten, toWrite);
				u8 rootIndex = fileStart[rootOffset] - RootPaths::RootStartByte;
				const TString& root = rootPaths.GetRoot(rootIndex);
				if (root.empty())
					return logger.Error(TC("Cache entry uses root path index %u which is not set for this startupinfo (%s)"), rootIndex, info.GetDescription());

				#if PLATFORM_WINDOWS
				StringBuffer<> pathTemp;
				pathTemp.Append(root);
				char rootPath[512];
				u32 rootPathLen = pathTemp.Parse(rootPath, sizeof_array(rootPath));
				UBA_ASSERT(rootPathLen);
				--rootPathLen;
				#else
				const char* rootPath = root.data();
				u32 rootPathLen = root.size();
				#endif

				if (u32 toWrite = rootPathLen)
					memcpy(localBlock.Allocate(toWrite, 1, TC("")), rootPath, toWrite);
				lastWritten = rootOffset + 1;
			}

			u64 fileSize = rootOffsets - sizeof(u32);
			if (u64 toWrite = fileSize - lastWritten)
				memcpy(localBlock.Allocate(toWrite, 1, TC("")), fileStart + lastWritten, toWrite);

			FileAccessor destFile(logger, path.data);

			bool useFileMapping = true;
			if (useFileMapping)
			{
				if (!destFile.CreateMemoryWrite(false, DefaultAttributes(), localBlock.writtenSize))
					return logger.Error(TC("Failed to create file for cache output %s for %s"), path.data, info.GetDescription());
				FileMapping_CopyMem(destFile.GetData(), localBlock.memory, localBlock.writtenSize);
			}
			else
			{
				if (!destFile.CreateWrite())
					return logger.Error(TC("Failed to create file for cache output %s for %s"), path.data, info.GetDescription());
				if (!destFile.Write(localBlock.memory, localBlock.writtenSize))
					return false;
			}
			if (!destFile.Close(&fetcher.lastWritten))
				return false;

			fetcher.sizeOnDisk = localBlock.writtenSize;
			casKey = CalculateCasKey(localBlock.memory, localBlock.writtenSize, false, nullptr, path.data);
			casKey = AsCompressed(casKey, m_storage.StoreCompressed());
		}
		else
		{
			#if UBA_TRACK_IS_EXECUTABLE
			if (IsExecutable(casKey))
			{
				casKey = AsExecutable(casKey, false); // Remove exec bit when fetching from casdb
				attributes = DefaultAttributes(true);
			}
			#endif

			// TODO: Implement this.
			#if 0
			{
				StringKey fileNameKey = ToStringKeyLower(path);
				CasKey existingCasKey;
				if (m_storage.CalculateCasKey(existingCasKey, fileNameKey, path.data))
					if (existingCasKey == casKey)
						return true;
			}
			#endif

			DowngradedLogger logger(m_connected, m_logger.m_writer, TC("UbaCacheClientDownload"));
			bool writeCompressed = m_session.ShouldStoreIntermediateFilesCompressed() && g_globalRules.FileCanBeCompressed(path);
			if (!fetcher.RetrieveFile(logger, m_client, casKey, path.data, writeCompressed, nullptr, attributes))
				return logger.Debug(TC("Failed to download cache output %s for %s"), path.data, info.GetDescription()).ToFalse();
			
			if (fetcher.sizeOnDisk == 0 && info.rules->SupportsPlaceholder(path))
			{
				DeleteFileW(path.data);
				return logger.Info(TC("Fetched file that seems to be placeholder, this should never happen %s for %s"), path.data, info.GetDescription()).ToFalse();
			}
		}

		cacheStats.fetchBytesRaw += fetcher.sizeOnDisk;
		cacheStats.fetchBytesComp += fetcher.bytesReceived;

		// TODO: Remove when bug is found
#if 0
		if (path.EndsWith(TCV(".obj")) || path.EndsWith(TCV(".o")))
		{
			if (fetcher.sizeOnDisk == 0)
				return m_logger.Warning(TC("Received %s file that has size 0!"), path.data);
			else
			{
				FileAccessor file(m_logger, path.data);
				if (!file.OpenMemoryRead())
					return m_logger.Warning(TC("Failed to test open received %s file"), path.data);
				if (file.GetSize() != fetcher.sizeOnDisk)
					return m_logger.Warning(TC("Received file %s has size %llu even though it shouldn have %llu"), path.data, file.GetSize(), fetcher.sizeOnDisk);
			}
		}
#endif
		if (!m_storage.FakeCopy(casKey, path.data, fetcher.sizeOnDisk, fetcher.lastWritten, attributes, false))
			return false;
		if (!m_session.RegisterNewFile(path.data))
			return false;

		return true;
	}

	bool CacheClient::RequestServerShutdown(const tchar* reason)
	{
		StackBinaryWriter<1024> writer;
		NetworkMessage msg(m_client, CacheServiceId, CacheMessageType_RequestShutdown, writer);
		writer.WriteString(reason);
		StackBinaryReader<512> reader;
		if (!msg.Send(reader))
			return false;
		return reader.ReadBool();
	}

	bool CacheClient::ExecuteCommand(Logger& logger, const tchar* command, const tchar* destinationFile, const tchar* additionalInfo)
	{
		StackBinaryWriter<1024> writer;
		NetworkMessage msg(m_client, CacheServiceId, CacheMessageType_ExecuteCommand, writer);
		writer.WriteString(command);
		writer.WriteString(additionalInfo ? additionalInfo : TC(""));

		CasKey statusFileCasKey;
		{
			StackBinaryReader<512> reader;
			if (!msg.Send(reader))
				return false;
			statusFileCasKey = reader.ReadCasKey();
			if (statusFileCasKey == CasKeyZero)
				return false;
		}

		StorageStats storageStats;
		FileFetcher fetcher { m_storage.m_bufferSlots, storageStats };
		bool destinationIsCompressed = false;
		if (destinationFile)
		{
			if (!fetcher.RetrieveFile(m_logger, m_client, statusFileCasKey, destinationFile, destinationIsCompressed))
				return false;
		}
		else
		{
			MemoryBlock block(TC("ExecuteCommand"), 512*1024*1024);
			if (!fetcher.RetrieveFile(m_logger, m_client, statusFileCasKey, TC("CommandString"), destinationIsCompressed, &block))
				return false;
			BinaryReader reader(block.memory, 3, block.writtenSize); // Skipping bom

			tchar line[1024];
			tchar* it = line;
			while (true)
			{
				tchar c = reader.ReadUtf8Char<tchar>();
				if (c != '\n' && c != 0)
				{
					*it++ = c;
					continue;
				}

				if (c == 0 && it == line)
					break;
				*it = 0;
				logger.Log(LogEntryType_Info, line, u32(it - line));
				it = line;
				if (c == 0)
					break;
			}
		}
		return true;
	}

	void CacheClient::Disable()
	{
		m_connected = false;
	}

	void CacheClient::TraceSummary()
	{
		StackBinaryWriter<16*256> writer;
		m_session.GetTrace().WriteSummaryText(writer, [&](Logger& logger)
			{
				m_client.PrintSummary(logger);
			});
		m_session.GetTrace().CacheSummary(writer.GetData(), writer.GetPosition());
	}

	bool CacheClient::SendPathTable(Bucket& bucket, CacheSendStats& stats, u32 requiredPathTableSize)
	{
		AtomicMax(bucket.pathTableSizeMaxToSend, requiredPathTableSize);

		TimerScope waitTs(stats.sendTableWait);
		SCOPED_FUTEX(bucket.sendPathTableNetworkLock, lock);
		if (requiredPathTableSize <= bucket.pathTableSizeSent)
			return true;
		waitTs.Leave();

		requiredPathTableSize = bucket.pathTableSizeMaxToSend;

		u32 left = requiredPathTableSize - bucket.pathTableSizeSent;
		while (left)
		{
			TimerScope ts(stats.sendPathTable);
			UBA_CACHE_BINARY_WRITER_MAX_SIZE(writer);
			NetworkMessage msg(m_client, CacheServiceId, CacheMessageType_StorePathTable, writer);
			writer.Write7BitEncoded(MakeId(bucket.id));
			u32 toSend = Min(requiredPathTableSize - bucket.pathTableSizeSent, u32(m_client.GetMessageMaxSize() - 32));
			left -= toSend;
			writer.WriteBytes(bucket.sendPathTable.GetMemory() + bucket.pathTableSizeSent, toSend);

			StackBinaryReader<16> reader;
			if (!msg.Send(reader))
			{
				if (!bucket.sendCorrupted.exchange(true))
					m_logger.Debug(TC("Disabling cache stores for bucket %u this session: path table send failed (error %u; server scratch table may be out of sync)"), bucket.id, msg.GetError());
				return false;
			}

			bucket.pathTableSizeSent += toSend;
		}
		return true;
	}

	bool CacheClient::SendCasTable(Bucket& bucket, CacheSendStats& stats, u32 requiredCasTableSize)
	{
		AtomicMax(bucket.casKeyTableSizeMaxToSend, requiredCasTableSize);

		TimerScope waitTs(stats.sendTableWait);
		SCOPED_FUTEX(bucket.sendCasKeyTableNetworkLock, lock);
		if (requiredCasTableSize <= bucket.casKeyTableSizeSent)
			return true;
		waitTs.Leave();

		requiredCasTableSize = bucket.casKeyTableSizeMaxToSend; // Send as much as possible

		u32 left = requiredCasTableSize - bucket.casKeyTableSizeSent;
		while (left)
		{
			TimerScope ts(stats.sendCasTable);
			UBA_CACHE_BINARY_WRITER_MAX_SIZE(writer);
			NetworkMessage msg(m_client, CacheServiceId, CacheMessageType_StoreCasTable, writer);
			writer.Write7BitEncoded(MakeId(bucket.id));
			u32 toSend = Min(requiredCasTableSize - bucket.casKeyTableSizeSent, u32(m_client.GetMessageMaxSize() - 32));
			left -= toSend;
			writer.WriteBytes(bucket.sendCasKeyTable.GetMemory() + bucket.casKeyTableSizeSent, toSend);

			StackBinaryReader<16> reader;
			if (!msg.Send(reader))
			{
				if (!bucket.sendCorrupted.exchange(true))
					m_logger.Debug(TC("Disabling cache stores for bucket %u this session: cas key table send failed (error %u; server scratch table may be out of sync)"), bucket.id, msg.GetError());
				return false;
			}

			bucket.casKeyTableSizeSent += toSend;
		}
		return true;
	}

	bool CacheClient::SendCacheEntry(TrackWorkScope& tws, Bucket& bucket, CacheSendStats& stats, const RootPaths& rootPaths, const CasKey& cmdKey, const Map<u32, u32>& inputsStringToCasKey, const Map<u32, u32>& outputsStringToCasKey, const u8* logLines, u64 logLinesSize, const ProcessStartInfo& info)
	{
		StackBinaryReader<24*1024> reader;

		UBA_TRACE_WRITE_HINT("SendCacheEntryMessage");

		if (!SendCacheEntryMessage(reader, bucket, stats, cmdKey, inputsStringToCasKey, outputsStringToCasKey, logLines, logLinesSize))
			return false;

		// Server has all content for caskeys.. upload is done
		if (!reader.GetLeft())
			return true;

		bool success = false;
		auto doneGuard = MakeGuard([&]()
			{
				TimerScope ts(stats.sendEntry);

				// Send done.. confirm to server
				UBA_TRACE_WRITE_HINT("SendCacheDone");
				StackBinaryWriter<64> writer;
				NetworkMessage msg(m_client, CacheServiceId, CacheMessageType_StoreEntryDone, writer.Reset());
				writer.Write7BitEncoded(MakeId(bucket.id));
				writer.WriteCasKey(cmdKey);
				writer.WriteBool(success);
				return msg.Send(reader);
			});

		DowngradedLogger logger(m_connected, m_logger.m_writer, TC("UbaCacheClientUpload"));

		// There is content we need to upload to server
		while (reader.GetLeft())
		{
			u32 casKeyOffset = u32(reader.Read7BitEncoded());

			StringBuffer<MaxPath> path;
			CasKey casKey;
			if (!GetLocalPathAndCasKey(bucket, rootPaths, path, casKey, bucket.sendCasKeyTable, bucket.sendPathTable, casKeyOffset))
				return false;

			casKey = AsCompressed(casKey, true);
			
			#if UBA_TRACK_IS_EXECUTABLE
			if (IsExecutable(casKey))
				casKey = AsExecutable(casKey, false); // Remove exec bit when storing in casdb
			#endif

			TimerScope createCasTs(stats.createCas);
			StorageImpl::CasEntry* casEntry;

			if (m_storage.HasCasFile(casKey, &casEntry, m_useMappingForCreatedCas, false))
			{
				createCasTs.Leave();

				auto refCountGuard = MakeGuard([&]()
					{
						m_storage.ReleaseCasEntry(*casEntry, true); // HasCasFile increase refcount
					});

				UBA_TRACE_WRITE_HINT("SendFile");
				TimerScope ts(stats.sendFile);

				UBA_ASSERT(!IsNormalized(casKey));
				StringBuffer<> casKeyFileName;
				if (!m_storage.GetCasFileName(casKeyFileName, casKey))
					return false;

				const u8* fileData = nullptr;
				u64 fileSize = 0;

				MappedView mappedView;
				auto mapViewGuard = MakeGuard([&](){ m_storage.m_casDataBuffer.UnmapView(mappedView, path.data); });

				FileAccessor file(m_logger, casKeyFileName.data);

				auto mh = casEntry->memoryHandle;
				if (mh.IsValid()) // If file was created by helper it will be in the transient mapped memory
				{
					u64 memoryOffset = casEntry->memoryOffset;
					u64 memorySize = casEntry->memorySize;

					mappedView = m_storage.m_casDataBuffer.MapView(mh, memoryOffset, memorySize, path.data);
					fileData = mappedView.memory;
					fileSize = mappedView.size;
				}

				if (!fileData)
				{
					if (!file.OpenMemoryRead())
						return false;
					fileData = file.GetData();
					fileSize = file.GetSize();
				}

				if (fileSize == 0)
				{
					UBA_ASSERT(info.rules);
					if (info.rules->SupportsPlaceholder(path)) // TODO: This should also check if placeholder writing is enabled but not available
						return m_logger.Warning(TC("File %s has size 0 and supports placeholders.. so it is likely we are sending a placeholder file. Investigate!!"), path.data);
				}

				// TODO: Use WorkManager for this!.. flush them in the end

				if (!SendFile(logger, m_client, &m_storage.m_workManager, casKey, fileData, fileSize, path.data))
					return false;

				stats.sendFile.bytes += fileSize;
			}
			else // If we don't have the cas key it should be one of the normalized files.... otherwise there is a bug
			{
				createCasTs.Cancel();

				UBA_TRACE_WRITE_HINT("SendNormalizedFile");
				TimerScope ts(stats.sendNormalizedFile);

				if (!IsNormalized(casKey))
					return m_logger.Error(TC("Can't find output file %s to send to cache server"), path.data);

				FileAccessor file(m_logger, path.data);
				if (!file.OpenMemoryRead())
					return false;
				MemoryBlock block(TC("SendNormalizedFile"), AlignUp(file.GetSize() + 16, PageSize));
				u32& rootOffsetsStart = *(u32*)block.Allocate(sizeof(u32), 1, TC(""));
				rootOffsetsStart = 0;
				Vector<u32> rootOffsets;
				u32 rootOffsetsSize = 0;

				auto handleString = [&](const char* str, u64 strLen, u32 rootPos)
					{
						void* mem = block.Allocate(strLen, 1, TC(""));
						memcpy(mem, str, strLen);
						if (rootPos != ~0u)
						{
							rootOffsets.push_back(rootPos);
							rootOffsetsSize += Get7BitEncodedCount(rootPos);
						}
					};

				if (!rootPaths.NormalizeString<char>(m_logger, (const char*)file.GetData(), file.GetSize(), handleString, false, path.data))
					return false;

				if (rootOffsetsSize)
				{
					u8* mem = (u8*)block.Allocate(rootOffsetsSize, 1, TC(""));
					rootOffsetsStart = u32(mem - block.memory);
					BinaryWriter writer(mem, 0, rootOffsetsSize);
					for (u32 rootOffset : rootOffsets)
						writer.Write7BitEncoded(rootOffset);
				}
				else
					rootOffsetsStart = u32(block.writtenSize);


				auto& s = m_storage;
				FileSender sender { logger, m_client, s.m_bufferSlots, s.Stats(), m_sendOneAtTheTimeLock, s.m_casCompressor, s.m_casCompressionLevel };

				u8* dataToSend = block.memory;
				u64 sizeToSend = block.writtenSize;

				if (!sender.SendFileCompressed(casKey, path.data, dataToSend, sizeToSend, path.data))
					return m_logger.Warning(TC("Failed to send cas content for file %s"), path.data);

				stats.sendNormalizedFile.bytes += sender.m_bytesSent;
			}

		}

		success = true;
		return doneGuard.Execute();
	}

	bool CacheClient::SendCacheEntryMessage(BinaryReader& out, Bucket& bucket, CacheSendStats& stats, const CasKey& cmdKey, const Map<u32, u32>& inputsStringToCasKey, const Map<u32, u32>& outputsStringToCasKey, const u8* logLines, u64 logLinesSize)
	{
		TimerScope ts(stats.sendEntry);

		UBA_CACHE_BINARY_WRITER_MAX_SIZE(writer);

		NetworkMessage msg(m_client, CacheServiceId, CacheMessageType_StoreEntry, writer);
		writer.Write7BitEncoded(MakeId(bucket.id));
		writer.WriteCasKey(cmdKey);

		writer.Write7BitEncoded(inputsStringToCasKey.size());
		writer.Write7BitEncoded(outputsStringToCasKey.size());
		for (auto& kv : outputsStringToCasKey)
			writer.Write7BitEncoded(kv.second);

		for (auto& kv : inputsStringToCasKey)
			writer.Write7BitEncoded(kv.second);

		if (writer.GetPosition() >= SendMaxSize)
		{
			m_logger.Warning(TC("Something is wrong. Sending a cache entry that is too large. Output count: %llu, Input count: %llu LogLines size: %llu"), outputsStringToCasKey.size(), inputsStringToCasKey.size(), logLinesSize);
			return false;
		}

		if (logLinesSize && logLinesSize < writer.GetCapacityLeft())
			writer.WriteBytes(logLines, logLinesSize);

		if (msg.Send(out))
			return true;
		m_logger.Info(TC("Failed to send cache entry. CasTable: %u/%u PathTable: %u/%u"), bucket.casKeyTableSizeSent, bucket.sendCasKeyTable.GetSize(), bucket.pathTableSizeSent, bucket.sendPathTable.GetSize());
		return false;
	}

	bool CacheClient::FetchCasTable(TrackWorkScope& tws, Bucket& bucket, CacheFetchStats& stats, u32 requiredCasTableOffset)
	{
		if (HasEnoughCasData(bucket, requiredCasTableOffset))
			return true;

		UBA_TRACE_FETCH_HINT_SCOPE("FetchCasTable")

		if (requiredCasTableOffset > CasKeyTableMaxSize)
			return m_logger.Warning(TC("Cas table offset %u too large. Cache entry corrupt (Bucket %llu)"), bucket.id);

		TimerScope ts2(stats.fetchCasTable);

		SCOPED_FUTEX(bucket.serverCasKeyTableNetworkLock, lock);
		while (!HasEnoughCasData(bucket, requiredCasTableOffset))
		{
			if (!FetchCompactTable(bucket.id, bucket.serverCasKeyTable, requiredCasTableOffset + sizeof(CasKey) + 8, CacheMessageType_FetchCasTable2))
				return false;
			bucket.serverCasKeyTableSize = bucket.serverCasKeyTable.GetSize();
		}
		return true;
	}

	bool CacheClient::FetchPathTable(TrackWorkScope& tws, Bucket& bucket, CacheFetchStats& stats, u32 requiredPathTableOffset)
	{
		if (HasEnoughPathData(bucket, requiredPathTableOffset))
			return true;

		SCOPED_FUTEX(bucket.serverPathTableNetworkLock, lock);
		while (!HasEnoughPathData(bucket, requiredPathTableOffset))
		{
			u32 targetSize = requiredPathTableOffset + 200;
			if (!FetchCompactTable(bucket.id, bucket.serverPathTable, targetSize, CacheMessageType_FetchPathTable2))
				return false;
			bucket.serverPathTableSize = bucket.serverPathTable.GetSize();
		}
		return true;
	}

	template<typename TableType>
	bool CacheClient::FetchCompactTable(u32 bucketId, TableType& table, u32 requiredTableSize, u8 messageType)
	{
		u32 tableSize = table.GetSize();
		u32 messageFetchSize = u32(m_client.GetMessageMaxSize() - m_client.GetMessageReceiveHeaderSize());
		u32 totalFetchSize = requiredTableSize - tableSize;
		u32 messageCount = (totalFetchSize + messageFetchSize - 1) / messageFetchSize;
		u32 commitSize = messageCount * SendMaxSize;
		u8* data = table.BeginCommit(commitSize);

		struct Entry
		{
			Entry(u8* slot, u32 i, u32 messageMaxSize) : reader(slot + i * messageMaxSize, 0, SendMaxSize), done(EventResetType_Manual) {}
			NetworkMessage message;
			BinaryReader reader;
			EventSlim done;
		};

		bool success = true;

		List<Entry> entries;
		for (u32 i=0; i!=messageCount; ++i)
		{
			Entry& e = entries.emplace_back(data, i, messageFetchSize);
			StackBinaryWriter<32> writer;
			e.message.Init(m_client, CacheServiceId, messageType, writer, HasPriority);
			writer.Write7BitEncoded(MakeId(bucketId));
			writer.WriteU32(tableSize + i*messageFetchSize);
			if (e.message.SendAsync(e.reader, [](bool error, void* userData) { ((EventSlim*)userData)->Set(); }, &e.done))
				continue;
			entries.pop_back();
			success = false;
			break;
		}

		u32 timeOutTimeMs = 20*60*1000; // Must be higher than network timeout

		u64 written = 0;
		for (Entry& e : entries)
		{
			if (!e.done.IsSet(timeOutTimeMs)) // This should not be possible
				m_logger.Error(TC("FetchCompactTable timed out after 20 minutes getting async message response. This timeout will cause a crash."));
			if (!e.message.ProcessAsyncResults(e.reader))
				success = false;
			written += e.reader.GetLeft();
		}

		if (!success)
			return false;

		table.EndCommit(data, written);
		return true;
	}

	bool CacheClient::HasEnoughCasData(Bucket& bucket, u32 requiredCasTableOffset)
	{
		u32 tableSize = bucket.serverCasKeyTableSize;
		// CasKeyTable is 7bitEncoded(pathoffset) + CasKey... path table offset is minimum 1 byte
		u32 neededSizeMin = requiredCasTableOffset + 1 + sizeof(CasKey);
		if (neededSizeMin > tableSize)
			return false;
		BinaryReader r(bucket.serverCasKeyTable.GetMemory(), requiredCasTableOffset, tableSize);
		r.Read7BitEncoded();
		u32 neededSize = u32(r.GetPosition()) + sizeof(CasKey);
		return neededSize <= tableSize;
	}

	bool CacheClient::HasEnoughPathData(Bucket& bucket, u32 requiredPathTableOffset)
	{
		u32 tableSize = bucket.serverPathTableSize;
		// PathTable is 7bitEncoded(parentoffset) + 7bitEncoded(stroffset). If stroffset is 0, then string is after stroffset
		if (requiredPathTableOffset + 200 < tableSize) // Early out. no filename without path + two 7bit encoded values are larger than this
			return true;
		if (requiredPathTableOffset + 2 > tableSize) // This means that it must be at least 2 bytes
			return false;
		BinaryReader r(bucket.serverPathTable.GetMemory(), requiredPathTableOffset, tableSize);
		u64 value;
		if (!r.TryRead7BitEncoded(value)) // Parent offset
			return false;
		if (!r.TryRead7BitEncoded(value)) // stroffset
			return false;
		if (value) // non-0 means it has the string segment before required offset
			return true;
		if (!r.TryRead7BitEncoded(value)) // string length
			return false;
		if (r.GetLeft() < value) // Actual string in bytes
			return false;
		return true;
	}

	bool CacheClient::ReportUsedEntry(Vector<ProcessLogLine>& outLogLines, bool ownedLogLines, Bucket& bucket, const CasKey& cmdKey, u32 entryId)
	{
		StackBinaryWriter<128> writer;
		NetworkMessage msg(m_client, CacheServiceId, CacheMessageType_ReportUsedEntry, writer, HasPriority);
		writer.Write7BitEncoded(MakeId(bucket.id));
		writer.WriteCasKey(cmdKey);
		writer.Write7BitEncoded(entryId);

		if (!ownedLogLines)
			return msg.Send();

		UBA_CACHE_BINARY_READER_MAX_SIZE(reader);
		if (!msg.Send(reader))
			return false;

		return PopulateLogLines(outLogLines, reader.GetPositionData(), reader.GetLeft());
	}

	bool CacheClient::PopulateLogLines(Vector<ProcessLogLine>& outLogLines, const u8* mem, u64 memLen)
	{
		BinaryReader reader(mem, 0, memLen);
		while (reader.GetLeft())
		{
			auto& logLine = outLogLines.emplace_back();
			logLine.text = reader.ReadString();
			logLine.type = LogEntryType(reader.ReadByte());
		}
		return true;
	}

	CasKey CacheClient::GetCmdKey(const RootPaths& rootPaths, const ProcessStartInfo& info, bool report, u32 bucketId)
	{
		CasKeyHasher hasher;

		//if (m_reportCacheKey)
		//	m_logger.Info(TC("CACHEKEY %s: Failed to calculate cache key"), info.GetDescription());
		//if (m_reportCacheKey)
		//	m_logger.Info(TC("CACHEKEY %s: %s (bucket %u)"), info.GetDescription(), CasKeyString(cmdKey).str, bucketId);
		
		if (report)
		{
			m_logger.BeginScope();
			m_logger.Info(TC("CACHEKEY %s (bucket %u)"), info.GetDescription(), bucketId);
		}
		auto guard = MakeGuard([&]() {if (report) m_logger.EndScope(); });

		#if PLATFORM_WINDOWS
		// cmd.exe is special.. we can't hash it because it might be different on different os versions but should do the same thing regardless of version
		if (Contains(info.application, TC("cmd.exe")))
		{
			hasher.Update(TC("cmd.exe"), 7*sizeof(tchar));
		}
		else
		#endif
		{
			// Add hash of application binary to key
			CasKey applicationCasKey;
			if (!m_storage.StoreCasKey(applicationCasKey, info.application, CasKeyZero))
				return CasKeyZero;
			if (report)
				m_logger.Info(TC("   %s %s"), CasKeyString(applicationCasKey).str, info.application);
			hasher.Update(&applicationCasKey, sizeof(CasKey));
		}

		// Add arguments list to key
		if (m_useRoots)
		{
			auto hashString = [&](const tchar* str, u64 strLenIncTerm, u32 rootPos) { hasher.Update(str, strLenIncTerm*sizeof(tchar)); };
			if (!rootPaths.NormalizeString(m_logger, info.arguments, TStrlen(info.arguments), hashString, false, info.GetDescription(), TC(" calculating command line hash")))
			{
				if (report)
					m_logger.Info(TC("   Failed to normalize commandline %s"), info.arguments);
				return CasKeyZero;
			}
		}
		else
			hasher.Update(info.arguments, TStrlen(info.arguments)*sizeof(tchar));

		if (report)
			m_logger.Info(TC("   %s %s"), CasKeyString(ToCasKey(hasher, false)).str, info.arguments);


		// Add content of rsp file to key (This will cost a bit of perf since we need to normalize.. should this be part of key?)
		if (auto rspStart = TStrchr(info.arguments, '@'))
		{
			if (rspStart[1] == '"')
			{
				rspStart += 2;
				if (auto rspEnd = TStrchr(rspStart, '"'))
				{
					StringBuffer<MaxPath> workingDir(info.workingDir);
					workingDir.EnsureEndsWithSlash();
					StringBuffer<> rsp;
					rsp.Append(rspStart, rspEnd - rspStart);
					StringBuffer<> fullPath;
					FixPath(rsp.data, workingDir.data, workingDir.count, fullPath);
					if (!DevirtualizePath(fullPath, info.rootsHandle))
					{
						if (report)
							m_logger.Warning(TC("Failed to normalize rsp file path %s"), fullPath.data);
						return CasKeyZero;
					}
					CasKey rspCasKey = rootPaths.NormalizeAndHashFile(m_logger, fullPath.data, true);
					if (rspCasKey == CasKeyZero)
					{
						if (report)
							m_logger.Info(TC("   Failed to normalize rsp file %s"), fullPath.data);
						return CasKeyZero;
					}
					if (report)
						m_logger.Info(TC("   %s %s"), CasKeyString(rspCasKey).str, fullPath.data);
					hasher.Update(&rspCasKey, sizeof(CasKey));
				}
			}
		}

		return ToCasKey(hasher, false);
	}

	bool CacheClient::DevirtualizePath(StringBufferBase& inOut, RootsHandle rootsHandle)
	{
		return m_session.DevirtualizePath(inOut, rootsHandle);
	}

	bool CacheClient::ShouldNormalize(const StringBufferBase& path)
	{
		if (!m_useRoots)
			return false;
		if (path.EndsWith(TCV(".json"))) // Contains absolute paths (dep files for msvc and vfsoverlay files for clang)
			return true;
		if (path.EndsWith(TCV(".d"))) // Contains absolute paths (dep files for clang)
			return true;
		if (path.EndsWith(TCV(".tlh"))) // Contains absolute path in a comment
			return true;
		if (path.EndsWith(TCV(".rsp"))) // Contains absolute paths in some cases
			return true;
		if (path.EndsWith(TCV(".bat"))) // Contains absolute paths in some cases
			return true;
		if (path.EndsWith(TCV(".txt"))) // Contains absolute paths (ispc dependency file)
			return true;
		return false;
	}

	bool CacheClient::GetLocalPathAndCasKey(Bucket& bucket, const RootPaths& rootPaths, StringBufferBase& outPath, CasKey& outKey, CompactCasKeyTable& casKeyTable, CompactPathTable& pathTable, u32 offset)
	{
		StringBuffer<MaxPath> normalizedPath;
		casKeyTable.GetPathAndKey(normalizedPath, outKey, pathTable, offset);
		UBA_ASSERT(normalizedPath.count);

		u32 rootIndex = normalizedPath[0] - RootPaths::RootStartByte;
		const TString& root = rootPaths.GetRoot(rootIndex);

		outPath.Append(root).Append(normalizedPath.data + u32(m_useRoots)); // If we use root paths, then first byte is root path table index
		return true;
	}

	void CacheClient::PreparseDirectory(const StringKey& fileNameKey, const StringBufferBase& filePath)
	{
		const tchar* lastSep = filePath.Last(PathSeparator);
		if (!lastSep)
			return;

		StringBuffer<MaxPath> path;
		path.Append(filePath.data, lastSep - filePath.data);
		if (CaseInsensitiveFs)
			path.MakeLower();

		StringKeyHasher dirHasher;
		dirHasher.Update(path.data, path.count);
		StringKey pathKey = ToStringKey(dirHasher);

		PreparedDir* dir = nullptr;

		#if PLATFORM_WINDOWS
		SCOPED_FUTEX_READ(m_directoryPreparserLock, preparserLockRead);
		auto findIt = m_directoryPreparser.find(pathKey);
		bool found = findIt != m_directoryPreparser.end();
		preparserLockRead.Leave();
		if (found)
			dir = &findIt->second;
		else
		#endif
		{
			SCOPED_FUTEX(m_directoryPreparserLock, preparserLock);
			dir = &m_directoryPreparser.try_emplace(pathKey).first->second;
		}

		if (dir->done)
			return;

		SCOPED_FUTEX(dir->lock, lock);
		if (dir->done)
			return;
		dir->done = true;

		// It is likely this folder has already been handled by session if this file is verified
		if (m_storage.IsFileVerified(fileNameKey))
			return;

		// Traverse all files in directory and report the file information... but only if it has not been reported before.. we don't want to interfere with other reports
		TraverseDir(m_logger, path, 
			[&](const DirectoryEntry& e)
			{
				if (IsDirectory(e.attributes))
					return;

				path.Clear().Append(PathSeparator).Append(e.name, e.nameLen);
				if (CaseInsensitiveFs)
					path.MakeLower();

				StringKey fileNameKey = ToStringKey(dirHasher, path.data, path.count);
				m_storage.ReportFileInfoWeak(fileNameKey, e.lastWritten, e.size);
			});
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
