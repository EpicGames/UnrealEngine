// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaCacheEntry.h"
#include "UbaCompactTables.h"
#include "UbaFile.h"
#include "UbaHashMap.h"
#include "UbaStorage.h"
#include <algorithm>

namespace uba
{
	u64 CacheEntries::GetSharedSize()
	{
		return sizeof(u16)
			+ Get7BitEncodedCount(sharedInputCasKeyOffsets.size()) + sharedInputCasKeyOffsets.size()
			+ Get7BitEncodedCount(sharedLogLines.size()) + sharedLogLines.size();
	}

	u64 CacheEntries::GetEntrySize(CacheEntry& entry, u32 clientVersion, bool toDisk)
	{
		u64 size = 0;

		if (toDisk)
		{
			size += Get7BitEncodedCount(entry.creationTime) + Get7BitEncodedCount(entry.lastUsedTime);
			if (clientVersion >= 5 && entry.logLinesType == LogLinesType_Owned)
				size += Get7BitEncodedCount(entry.logLines.size()) + entry.logLines.size();
		}
		else
		{
			size += Get7BitEncodedCount(entry.id);
		}

		if (clientVersion >= 5)
			++size; // logLinesType

		auto& extra = entry.extraInputCasKeyOffsets;
		size += Get7BitEncodedCount(extra.size()) + extra.size();

		auto& ranges = entry.sharedInputCasKeyOffsetRanges;
		size += Get7BitEncodedCount(ranges.size()) + ranges.size();

		auto& outputs = entry.outputCasKeyOffsets;
		size += Get7BitEncodedCount(outputs.size()) + outputs.size();
		return size;
	}

	u64 CacheEntries::GetTotalSize(u32 clientVersion, bool toDisk)
	{
		u64 size = GetSharedSize();
		for (auto& entry : entries)
			size += GetEntrySize(entry, clientVersion, toDisk);
		if (toDisk)
			size += sizeof(u32) + inputsThatAreOutputs.size()*sizeof(u32);
		return size;
	}

	bool CacheEntries::Write(BinaryWriter& writer, u32 clientVersion, bool toDisk)
	{
		u16& entryCount = *(u16*)writer.AllocWrite(2);
		entryCount = 0;

		if (clientVersion == 3)
		{
			UBA_ASSERT(!toDisk);

			Vector<u8> flattenInputs;
			for (auto& entry : entries)
			{
				Flatten(flattenInputs, entry);
				auto& inputs = flattenInputs;
				auto& outputs = entry.outputCasKeyOffsets;

				u64 neededSize = Get7BitEncodedCount(inputs.size()) + inputs.size() + Get7BitEncodedCount(outputs.size()) + outputs.size();
				if (neededSize > writer.GetCapacityLeft())
					return true;

				writer.Write7BitEncoded(inputs.size());
				writer.WriteBytes(inputs.data(), inputs.size());
				writer.Write7BitEncoded(outputs.size());
				writer.WriteBytes(outputs.data(), outputs.size());
				++entryCount;
			}
			return true;
		}

		{
			auto& sharedOffsets = sharedInputCasKeyOffsets;

			if (!toDisk)
			{
				u64 neededSize = Get7BitEncodedCount(sharedOffsets.size()) + sharedOffsets.size();
				neededSize += Get7BitEncodedCount(sharedLogLines.size()) + sharedLogLines.size();
				if (neededSize > writer.GetCapacityLeft())
					return true;
			}
			writer.Write7BitEncoded(sharedOffsets.size());
			writer.WriteBytes(sharedOffsets.data(), sharedOffsets.size());
			if (clientVersion >= 5)
			{
				writer.Write7BitEncoded(sharedLogLines.size());
				writer.WriteBytes(sharedLogLines.data(), sharedLogLines.size());
			}
		}

		for (auto& entry : entries)
		{
			auto& extra = entry.extraInputCasKeyOffsets;
			auto& ranges = entry.sharedInputCasKeyOffsetRanges;
			auto& outputs = entry.outputCasKeyOffsets;

			if (toDisk)
			{
				writer.Write7BitEncoded(entry.creationTime);
				writer.Write7BitEncoded(entry.lastUsedTime);
			}
			else
			{
				u64 neededSize = Get7BitEncodedCount(entry.id) + Get7BitEncodedCount(extra.size()) + extra.size();
				neededSize += Get7BitEncodedCount(ranges.size()) + ranges.size();
				neededSize += Get7BitEncodedCount(outputs.size()) + outputs.size();
				if (clientVersion >= 5)
					neededSize += 1; // hasLogLines

				if (neededSize > writer.GetCapacityLeft())
					return true;

				writer.Write7BitEncoded(entry.id);
			}

			writer.Write7BitEncoded(extra.size());
			writer.WriteBytes(extra.data(), extra.size());

			writer.Write7BitEncoded(ranges.size());
			writer.WriteBytes(ranges.data(), ranges.size());

			writer.Write7BitEncoded(outputs.size());
			writer.WriteBytes(outputs.data(), outputs.size());

			// Log lines are not included in network data.
			if (toDisk)
			{
				writer.WriteByte(entry.logLinesType);
				if (entry.logLinesType == LogLinesType_Owned)
				{
					writer.Write7BitEncoded(entry.logLines.size());
					writer.WriteBytes(entry.logLines.data(), entry.logLines.size());
				}
			}
			else if (clientVersion >= 5)
			{
				writer.WriteByte(entry.logLinesType);
			}

			++entryCount;
		}

		if (toDisk)
		{
			writer.WriteU32(u32(inputsThatAreOutputs.size()));
			for (u32 offset : inputsThatAreOutputs)
				writer.WriteU32(offset);
		}
		return true;
	}

	bool CacheEntries::ReadFromDisk(Logger& logger, BinaryReader& reader, u32 databaseVersion, StorageImpl& storage, CompactCasKeyTable& table)
	{
		if (databaseVersion == 3)
		{
			u32 cacheEntryCount = reader.ReadU32();
			Vector<u32> temp;
			Vector<u8> temp2;
			while (cacheEntryCount--)
			{
				auto& cacheEntry = entries.emplace_back();
				cacheEntry.id = idCounter++;
				reader.ReadU64();
				cacheEntry.creationTime = GetSystemTimeAsFileTime();
				cacheEntry.lastUsedTime = GetSystemTimeAsFileTime();

				u32 inputSize = reader.ReadU32();

				u64 inputEnd = reader.GetPosition() + inputSize;
				temp.clear();
				while (reader.GetPosition() < inputEnd)
					temp.push_back(u32(reader.Read7BitEncoded()));
				BuildInputsT(cacheEntry, temp, entries.size() == 1, temp2);

				u32 outputSize = reader.ReadU32();
				cacheEntry.outputCasKeyOffsets.resize(outputSize);
				reader.ReadBytes(cacheEntry.outputCasKeyOffsets.data(), outputSize);
			}

			return true;
		}

		u16 entryCount = reader.ReadU16();
		u64 sharedSize = reader.Read7BitEncoded();
		sharedInputCasKeyOffsets.resize(sharedSize);
		reader.ReadBytes(sharedInputCasKeyOffsets.data(), sharedSize);

		if (databaseVersion >= 6)
		{
			u64 sharedLogLinesSize = reader.Read7BitEncoded();
			sharedLogLines.resize(sharedLogLinesSize);
			reader.ReadBytes(sharedLogLines.data(), sharedLogLinesSize);
		}

		while (entryCount--)
		{
			auto& entry = entries.emplace_back();
			entry.id = idCounter++;
			entry.creationTime = reader.Read7BitEncoded();
			entry.lastUsedTime = reader.Read7BitEncoded();

			u64 extraSize = reader.Read7BitEncoded();
			entry.extraInputCasKeyOffsets.resize(extraSize);
			reader.ReadBytes(entry.extraInputCasKeyOffsets.data(), extraSize);

			u64 rangeSize = reader.Read7BitEncoded();
			entry.sharedInputCasKeyOffsetRanges.resize(rangeSize);
			reader.ReadBytes(entry.sharedInputCasKeyOffsetRanges.data(), rangeSize);

			u64 outputSize = reader.Read7BitEncoded();
			entry.outputCasKeyOffsets.resize(outputSize);
			reader.ReadBytes(entry.outputCasKeyOffsets.data(), outputSize);

			if (databaseVersion >= 6)
			{
				entry.logLinesType = LogLinesType(reader.ReadByte());
				if (entry.logLinesType == LogLinesType_Owned)
				{
					u64 logLinesSize = reader.Read7BitEncoded();
					entry.logLines.resize(logLinesSize);
					reader.ReadBytes(entry.logLines.data(), logLinesSize);
				}
			}

		}

		if (databaseVersion < 8)
		{
			if (databaseVersion == 7)
				reader.ReadBool();
			PopulateInputsThatAreOutputs(sharedInputCasKeyOffsets, storage, table);
			for (auto& entry : entries)
				PopulateInputsThatAreOutputs(entry.extraInputCasKeyOffsets, storage, table);
		}

		if (databaseVersion >= 8)
		{
			u32 count = reader.ReadU32();
			inputsThatAreOutputs.reserve(count);
			while (count--)
				inputsThatAreOutputs.insert(reader.ReadU32());
		}

		return true;
	}

	template<typename Container>
	void CacheEntries::BuildInputsT(CacheEntry& entry, const Container& sortedInputs, bool populateShared, Vector<u8>& temp)
	{
		StackBinaryWriter<256*1024> rangeWriter;

		auto g = MakeGuard([&]()
			{
				entry.sharedInputCasKeyOffsetRanges.resize(rangeWriter.GetPosition());
				memcpy(entry.sharedInputCasKeyOffsetRanges.data(), rangeWriter.GetData(), rangeWriter.GetPosition());
			});

		auto writeRange = [&](u64 begin, u64 end) { rangeWriter.Write7BitEncoded(begin); rangeWriter.Write7BitEncoded(end); };

		if (populateShared)
		{
			u64 bytes = 0;
			for (u32 i : sortedInputs)
				bytes += Get7BitEncodedCount(i);
			sharedInputCasKeyOffsets.resize(bytes);
			BinaryWriter writer(sharedInputCasKeyOffsets.data(), 0, sharedInputCasKeyOffsets.size());
			for (u32 i : sortedInputs)
				writer.Write7BitEncoded(i);
			UBA_ASSERT(bytes == writer.GetPosition());
			writeRange(0, bytes);
			return;
		}

		auto inputsIt = sortedInputs.begin();
		auto inputsEnd = sortedInputs.end();

		BinaryReader sharedReader(sharedInputCasKeyOffsets);

		u32 sharedOffset = ~0u;
		u32 offset = ~0u;

		u32 rangeBegin = 0;
		bool inRange = false;
		bool previousWasExtra = false;
		u32 lastSharedPos = ~0u;

		Vector<u8>& extraOffsets = temp;
		extraOffsets.resize(sortedInputs.size()*5);
		BinaryWriter extraWriter(extraOffsets.data(), 0, extraOffsets.size());

		while (true)
		{
			u32 sharedPos = u32(sharedReader.GetPosition());

			if (!sharedReader.GetLeft())
			{
				bool extraWritten = false;

				// Handle all the extra offsets that are lower than shared offset
				if (previousWasExtra)
				{
					extraWritten = true;
					while (inputsIt != inputsEnd)
					{
						offset = *inputsIt++;
						if (offset >= sharedOffset)
						{
							extraWritten = false;
							break;
						}
						extraWriter.Write7BitEncoded(offset);
					}
				}

				// Add current range if there is one going
				if (inRange)
				{
					writeRange(rangeBegin, offset == sharedOffset ? sharedPos : lastSharedPos);
					if (previousWasExtra && offset > sharedOffset)
						extraWriter.Write7BitEncoded(offset);
				}
				else if (offset == sharedOffset) // We want to use a range just to make it easier to debug
					writeRange(lastSharedPos, sharedPos);
				else if (!extraWritten)
					extraWriter.Write7BitEncoded(offset);

				// Populate rest in extraInputCasKeyOffsets
				for (;inputsIt != inputsEnd ;++inputsIt)
					if (*inputsIt != sharedOffset)
						extraWriter.Write7BitEncoded(*inputsIt);
				break;
			}

			if (inputsIt == inputsEnd)
			{
				// Add current range if there is one going
				if (inRange)
				{
					writeRange(rangeBegin, previousWasExtra ? lastSharedPos : sharedPos);
				}
				else if (offset > sharedOffset)
				{
					lastSharedPos = sharedPos;
					sharedOffset = u32(sharedReader.Read7BitEncoded());
					continue;
				}
				else
				{
					if (sharedOffset == offset)
						writeRange(lastSharedPos, u32(sharedReader.GetPosition()));
					else if (!previousWasExtra)
						extraWriter.Write7BitEncoded(offset);
				}
				break;
			}

			previousWasExtra = false;

			if (sharedOffset < offset)
			{
				lastSharedPos = sharedPos;
				sharedOffset = u32(sharedReader.Read7BitEncoded());
			}
			else if (offset < sharedOffset)
			{
				offset = *inputsIt++;
				sharedPos = lastSharedPos;
			}
			else
			{
				lastSharedPos = sharedPos;
				sharedOffset = u32(sharedReader.Read7BitEncoded());
				offset = *inputsIt++;
			}

			if (sharedOffset == offset)
			{
				if (!inRange)
				{
					rangeBegin = sharedPos;
					inRange = true;
				}
			}
			else
			{
				if (offset < sharedOffset)
				{
					extraWriter.Write7BitEncoded(offset);
					previousWasExtra = true;
				}
				else if (inRange)
				{
					inRange = false;
					writeRange(rangeBegin, sharedPos);
				}
			}
		}

		entry.extraInputCasKeyOffsets.resize(extraWriter.GetPosition());
		memcpy(entry.extraInputCasKeyOffsets.data(), extraWriter.GetData(), extraWriter.GetPosition());
	}

	template<typename Container>
	void CacheEntries::BuildRangesFromExcludedT(CacheEntry& entry, const Container& sortedExcludedInputs)
	{
		StackBinaryWriter<256*1024> rangeWriter;

		auto g = MakeGuard([&]()
			{
				entry.sharedInputCasKeyOffsetRanges.resize(rangeWriter.GetPosition());
				memcpy(entry.sharedInputCasKeyOffsetRanges.data(), rangeWriter.GetData(), rangeWriter.GetPosition());
			});

		auto writeRange = [&](u64 begin, u64 end) { rangeWriter.Write7BitEncoded(begin); rangeWriter.Write7BitEncoded(end); };

		auto excludedInputsIt = sortedExcludedInputs.begin();
		auto excludedInputsEnd = sortedExcludedInputs.end();

		BinaryReader sharedReader(sharedInputCasKeyOffsets);

		u32 sharedOffset = ~0u;
		u32 offset = ~0u;

		u32 excludeRangeEnd = 0;
		u32 excludeRangeBegin = 0;
		bool inExcludeRange = false;
		u32 lastSharedPos = ~0u;

		while (true)
		{
			u32 sharedPos = u32(sharedReader.GetPosition());

			if (!sharedReader.GetLeft())
			{
				if (!inExcludeRange)
					writeRange(excludeRangeEnd, excludeRangeBegin);
				break;
			}

			if (offset <= sharedOffset && excludedInputsIt == excludedInputsEnd)
			{
				if (!inExcludeRange)
					writeRange(excludeRangeEnd, sharedInputCasKeyOffsets.size());
				else
					writeRange(sharedPos, sharedInputCasKeyOffsets.size());
				break;
			}

			if (sharedOffset < offset)
			{
				lastSharedPos = sharedPos;
				sharedOffset = u32(sharedReader.Read7BitEncoded());
			}
			else if (offset < sharedOffset)
			{
				offset = *excludedInputsIt++;
				sharedPos = lastSharedPos;
			}
			else
			{
				lastSharedPos = sharedPos;
				sharedOffset = u32(sharedReader.Read7BitEncoded());
				offset = *excludedInputsIt++;
			}

			if (sharedOffset == offset)
			{
				if (!inExcludeRange)
				{
					if (excludeRangeEnd != lastSharedPos)
						writeRange(excludeRangeEnd, lastSharedPos);
					excludeRangeBegin = sharedPos;
					inExcludeRange = true;
				}
			}
			else
			{
				if (inExcludeRange)
				{
					inExcludeRange = false;
					excludeRangeEnd = sharedPos;
				}
			}
		}

	}

	void CacheEntries::BuildInputs(CacheEntry& entry, const Set<u32>& inputs)
	{
		Vector<u8> temp;
		BuildInputsT(entry, inputs, sharedInputCasKeyOffsets.empty(), temp);
	}

	void CacheEntries::UpdateEntries(Logger& logger, const HashMap2<u32, u32>& oldToNewCasKeyOffset, Vector<u32>& temp, Vector<u8>& temp2, Vector<u8>& temp3)
	{
		if (entries.empty())
			return;

		UnorderedSet<u32> itao(inputsThatAreOutputs.size());
		for (u32 offset : inputsThatAreOutputs)
			if (auto o = oldToNewCasKeyOffset.Find(offset))
				itao.insert(*o);
		inputsThatAreOutputs.swap(itao);


		auto convertOffsets = [&](Vector<u8>& offsets)
			{
				temp.clear();
				u32 newOffsetsSize = 0;
				BinaryReader reader(offsets);
				while (reader.GetLeft())
				{
					u32 newOffset = u32(reader.Read7BitEncoded());
					if (auto o = oldToNewCasKeyOffset.Find(newOffset))
						newOffset = *o;
					temp.push_back(newOffset);
					newOffsetsSize += Get7BitEncodedCount(newOffset);
				}

				std::sort(temp.begin(), temp.end());

				offsets.resize(newOffsetsSize);
				BinaryWriter writer(offsets.data(), 0, newOffsetsSize);
				for (u32 offset : temp)
					writer.Write7BitEncoded(offset);
			};

		for (auto& entry : entries)
			convertOffsets(entry.outputCasKeyOffsets);

		auto writePrimaryRange = [&](CacheEntry& entry, u64 newSize)
			{
				u64 rangeSize = 1 + Get7BitEncodedCount(newSize);
				entry.sharedInputCasKeyOffsetRanges.resize(rangeSize);
				BinaryWriter rangeWriter(entry.sharedInputCasKeyOffsetRanges.data(), 0, rangeSize);
				rangeWriter.Write7BitEncoded(0);
				rangeWriter.Write7BitEncoded(newSize);
				UBA_ASSERT(rangeWriter.GetPosition() == rangeSize);
			};

		// If primary id is not set we use first entry as primaryId and base shared offsets off primary entry
		if (entries.size() == 1 || primaryId == ~0u)
		{
			auto& oldShared = temp2;
			oldShared = sharedInputCasKeyOffsets;

			bool isFirst = true;
			for (auto& entry : entries)
			{
				if (isFirst)
				{
					primaryId = entry.id;

					// Flatten first entry into temp
					Flatten(temp, entry, oldShared);

					// Update temp with new offsets
					u64 newSize = 0;
					for (auto& offset : temp)
					{
						if (auto o = oldToNewCasKeyOffset.Find(offset))
							offset = *o;
						newSize += Get7BitEncodedCount(offset);
					}

					// Sort temp now when it likely is out of order
					std::sort(temp.begin(), temp.end());

					// Write new shared
					sharedInputCasKeyOffsets.resize(newSize);
					BinaryWriter writer(sharedInputCasKeyOffsets.data(), 0, newSize);
					for (auto& offset : temp)
						writer.Write7BitEncoded(offset);

					// Clear extra and set entire shared to range
					entry.extraInputCasKeyOffsets.clear();
					writePrimaryRange(entry, newSize);
					isFirst = false;
				}
				else
				{
					// Flatten using old shared and rebuild it with new shared
					Flatten(temp, entry, oldShared);
					for (auto& offset : temp)
						if (auto o = oldToNewCasKeyOffset.Find(offset))
							offset = *o;

					// Sort temp now when it likely is out of order
					std::sort(temp.begin(), temp.end());

					entry.extraInputCasKeyOffsets.clear();
					entry.sharedInputCasKeyOffsetRanges.clear();
					BuildInputsT(entry, temp, false, temp3);
				}
			}
		}
		else
		{
			// This approach should be faster if there are more than one entry since we expect entries to be very similar to each other
			// It instead tracks removed offsets when calculating the shared offsets and build the ranges from that.

			auto& oldShared = temp2;
			oldShared = sharedInputCasKeyOffsets;
			convertOffsets(sharedInputCasKeyOffsets);


			for (auto& entry : entries)
			{
				// Collect all inputs that are removed from the shared inputs

				auto collectInputs = [&](Vector<u32>& out, u32 rangeBegin, u32 rangeEnd)
					{
						BinaryReader excludedReader(oldShared.data(), rangeBegin, rangeEnd);
						while (excludedReader.GetLeft())
						{
							u32 offset = u32(excludedReader.Read7BitEncoded());
							if (auto o = oldToNewCasKeyOffset.Find(offset))
								offset = *o;
							out.push_back(offset);
						}
					};

				temp.clear();
				auto& excludedOffsets = temp;

				BinaryReader rangeReader(entry.sharedInputCasKeyOffsetRanges);
				u32 lastEnd = 0;
				while (rangeReader.GetLeft())
				{
					u32 begin = u32(rangeReader.Read7BitEncoded());
					collectInputs(excludedOffsets, lastEnd, begin);
					lastEnd = u32(rangeReader.Read7BitEncoded());
				}
				collectInputs(excludedOffsets, lastEnd, u32(oldShared.size()));

				if (excludedOffsets.empty() && entry.extraInputCasKeyOffsets.empty())
				{
					writePrimaryRange(entry, sharedInputCasKeyOffsets.size());
				}
				else
				{
					// Sort excluded inputs..
					std::sort(excludedOffsets.begin(), excludedOffsets.end());

					// Build new ranges based on shared and excluded offsets from shared
					BuildRangesFromExcludedT(entry, excludedOffsets);

					// Create new extras
					convertOffsets(entry.extraInputCasKeyOffsets);
				}
			}
		}
	}

	bool CacheEntries::Validate(Logger& logger)
	{
		Set<u32> sharedOffsets;
		{
			BinaryReader sharedReader(sharedInputCasKeyOffsets);
			while (sharedReader.GetLeft())
			{
				u64 offset;
				if (!sharedReader.TryRead7BitEncoded(offset))
					return false;
				if (!sharedOffsets.insert(u32(offset)).second)
					return false;
			}
		}

		for (auto& entry : entries)
		{
			Set<u32> entryOffsets;

			u32 rangeCount = 0;
			BinaryReader rangeReader(entry.sharedInputCasKeyOffsetRanges);
			while (rangeReader.GetLeft())
			{
				++rangeCount;
				u64 begin = rangeReader.Read7BitEncoded();
				u64 end = rangeReader.Read7BitEncoded();
				BinaryReader sharedReader(sharedInputCasKeyOffsets.data(), begin, end);
				while (sharedReader.GetLeft())
				{
					u64 offset;
					if (!sharedReader.TryRead7BitEncoded(offset))
						return false;
					if (!entryOffsets.insert(u32(offset)).second)
						return false;
				}
			}

			u32 extraCount = 0;
			BinaryReader extraReader(entry.extraInputCasKeyOffsets);
			while (extraReader.GetLeft())
			{
				++extraCount;
				u64 offset;
				if (!extraReader.TryRead7BitEncoded(offset))
					return false;
				if (sharedOffsets.find(u32(offset)) != sharedOffsets.end())
					return false;
				if (!entryOffsets.insert(u32(offset)).second)
					return false;
			}
		}
		return true;
	}

	void CacheEntries::PopulateInputsThatAreOutputs(const Vector<u8>& inputData, StorageImpl& storage, CompactCasKeyTable& table)
	{
		// one entry that is ~0u is equal to test all inputs
		if (inputsThatAreOutputs.size() == 1 && *inputsThatAreOutputs.begin() == ~0u)
			return;

		u32 insertCount = 0;
		BinaryReader inputReader(inputData);
		while (inputReader.GetLeft())
		{
			u32 casKeyOffset = u32(inputReader.Read7BitEncoded());
			CasKey casKey;
			table.GetKey(casKey, casKeyOffset);
			if (storage.HasBeenSeen(casKey))
				if (inputsThatAreOutputs.insert(casKeyOffset).second)
					++insertCount;
		}

		// If there are lots of inputs that are outputs this is most likely a link step and we switch to always check inputs
		if (insertCount > 5)
		{
			inputsThatAreOutputs.clear();
			inputsThatAreOutputs.insert(~0u);
		}
	}

	void CacheEntries::Flatten(Vector<u8>& out, const CacheEntry& entry)
	{
		u64 size = entry.extraInputCasKeyOffsets.size();
		{
			BinaryReader rangeReader(entry.sharedInputCasKeyOffsetRanges);
			while (rangeReader.GetLeft())
			{
				u64 begin = rangeReader.Read7BitEncoded();
				u64 end = rangeReader.Read7BitEncoded();
				size += end - begin;
			}
		}

		out.resize(size);
		BinaryWriter writer(out.data(), 0, out.size());

		BinaryReader extraReader(entry.extraInputCasKeyOffsets);
		u32 nextExtra = ~0u;
		if (extraReader.GetLeft())
			nextExtra = u32(extraReader.Read7BitEncoded());

		auto writeExtra = [&](u32 prevOffset)
			{
				while (nextExtra < prevOffset)
				{
					writer.Write7BitEncoded(nextExtra);
					nextExtra = ~0u;
					if (extraReader.GetLeft())
						nextExtra = u32(extraReader.Read7BitEncoded());
				};
			};

		BinaryReader rangeReader(entry.sharedInputCasKeyOffsetRanges);
		while (rangeReader.GetLeft())
		{
			u64 begin = rangeReader.Read7BitEncoded();
			u64 end = rangeReader.Read7BitEncoded();
			BinaryReader inputReader(sharedInputCasKeyOffsets.data(), begin, end);
			while (inputReader.GetLeft())
			{
				u32 offset = u32(inputReader.Read7BitEncoded());
				writeExtra(offset);
				writer.Write7BitEncoded(offset);
			}
		}

		writeExtra(~0u);
	}

	void CacheEntries::Flatten(Vector<u32>& out, const CacheEntry& entry, const Vector<u8>& sharedOffsets)
	{
		out.clear();

		BinaryReader extraReader(entry.extraInputCasKeyOffsets);
		u32 nextExtra = ~0u;
		if (extraReader.GetLeft())
			nextExtra = u32(extraReader.Read7BitEncoded());

		auto writeExtra = [&](u32 prevOffset)
			{
				while (nextExtra < prevOffset)
				{
					out.push_back(nextExtra);
					nextExtra = ~0u;
					if (extraReader.GetLeft())
						nextExtra = u32(extraReader.Read7BitEncoded());
				};
			};

		BinaryReader rangeReader(entry.sharedInputCasKeyOffsetRanges.data(), 0, entry.sharedInputCasKeyOffsetRanges.size());
		while (rangeReader.GetLeft())
		{
			u64 begin = rangeReader.Read7BitEncoded();
			u64 end = rangeReader.Read7BitEncoded();
			BinaryReader inputReader(sharedOffsets.data(), begin, end);
			while (inputReader.GetLeft())
			{
				u32 offset = u32(inputReader.Read7BitEncoded());
				writeExtra(offset);
				out.push_back(offset);
			}
		}

		writeExtra(~0u);
	}
}
