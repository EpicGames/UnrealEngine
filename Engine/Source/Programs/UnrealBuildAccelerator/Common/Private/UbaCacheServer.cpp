// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaCacheServer.h"
#include "UbaCacheBucket.h"
#include "UbaCompactTables.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaDirectoryIterator.h"
#include "UbaFileAccessor.h"
#include "UbaHashMap.h"
#include "UbaNetworkServer.h"
#include "UbaStorageServer.h"
#include "UbaTrace.h"
#include <algorithm>

#if defined(__clang__) && PLATFORM_WINDOWS && (defined(_M_X64) || defined(_M_IX86))
#include <x86intrin.h>
#endif

//#include <oodle2.h>

// TODO
// - Fix so expiration time is set to oldest if overflowing and decreasing time didn't cause any deletes. That way we can make sure next maintenance will delete entries
// - Sort buckets by last maintenance time to make sure the long ones always get a slot first
// - Change so save happens when bucket is done in same work to minimize latency for the long ones

#define UBA_TRACK_WORK_SCOPE(name) //TrackWorkScope tws(m_server, TCV(name));
#define UBA_CACHE_SERVER_ADD_HINT(name, start) //wc.tracker.AddHint(TCV(name), start);

#if PLATFORM_WINDOWS
#define UBA_FORCEINLINE __forceinline
#else
#define UBA_FORCEINLINE inline __attribute__ ((always_inline))
#endif

namespace uba
{
	static constexpr u32 CacheFileVersion = 9;
	static constexpr u32 CacheFileCompatibilityVersion = 3;

	bool IsCaseInsensitive(u64 id) { return (id & (1ull << 32)) == 0; }

	struct BitArray
	{
		void Init(MemoryBlock& memoryBlock, u32 bitCount, const tchar* hint)
		{
			u32 bytes = AlignUp((bitCount+7) / 8, 8u); // Align up to 64 bits
			data = (u64*)memoryBlock.Allocate(bytes, 8, hint);
			memset(data, 0, bytes);
			count = bytes / 8;
		}

		UBA_FORCEINLINE void Set(u32 bitIndex)
		{
			u32 index = bitIndex / 64;
			UBA_ASSERTF(index < count, TC("Out of bounds (%u/%u). Bit index : %u"), index, count, bitIndex);
			u32 bitOffset = bitIndex - index * 64;
			data[index] |= 1ull << bitOffset;
		}

		UBA_FORCEINLINE bool IsSet(u32 bitIndex)
		{
			u32 index = bitIndex / 64;
			UBA_ASSERTF(index < count, TC("Out of bounds (%u/%u). Bit index : %u"), index, count, bitIndex);
			u32 bitOffset = bitIndex - index * 64;
			return (data[index] & (1ull << bitOffset)) != 0;
		}

		UBA_FORCEINLINE u32 CountSetBits()
		{
			u64 bits = 0;
			for (u64 i=0,e=count; i!=e; ++i)
				bits += CountBits(data[i]);
			return u32(bits);
		}

		template<typename Func>
		void Traverse(const Func& func)
		{
			u32 index = 0;
			for (u64 i=0,e=count; i!=e; ++i)
			{
				u64 v = data[i];
				while (v)
				{
					u64 bitIndex = FindFirstBit(v);
					func(index + u32(bitIndex));
					v &= ~(1ull << bitIndex);
				}
				index += 64;
			}
		}

		static UBA_FORCEINLINE u64 CountBits(u64 bits)
		{
			#if defined(__clang__)
			return __builtin_popcountll(bits);
			#elif PLATFORM_WINDOWS && defined(_M_X64)
			return __popcnt64(bits);
			#else
			// https://en.wikipedia.org/wiki/Hamming_weight
			bits -= (bits >> 1) & 0x5555555555555555ull;
			bits = (bits & 0x3333333333333333ull) + ((bits >> 2) & 0x3333333333333333ull);
			bits = (bits + (bits >> 4)) & 0x0f0f0f0f0f0f0f0full;
			return (bits * 0x0101010101010101) >> 56;
			#endif
		}

		static UBA_FORCEINLINE u64 FindFirstBit(u64 v)
		{
			#if PLATFORM_WINDOWS && (defined(_M_X64) || defined(_M_IX86))
			// Use TZCNT intrinsic on Windows x86/x64
			return _tzcnt_u64(v);
			#elif PLATFORM_WINDOWS && defined(_M_ARM64)
			// Use the ARM64 equivalent
			return _CountTrailingZeros64(v);
			#elif PLATFORM_LINUX && (defined(__x86_64__) || defined(__i386__))
			// Use GCC's built-in TZCNT equivalent for x86/x64
			return __builtin_ia32_tzcnt_u64(v);
			#elif PLATFORM_LINUX && defined(__aarch64__)
			// Use the ARM64 equivalent
			return __builtin_ctzll(v);
			#else
			u64 pos = 0;
			if (v >= 1ull<<32) { v >>= 32; pos += 32; }
			if (v >= 1ull<<16) { v >>= 16; pos += 16; }
			if (v >= 1ull<< 8) { v >>=  8; pos +=  8; }
			if (v >= 1ull<< 4) { v >>=  4; pos +=  4; }
			if (v >= 1ull<< 2) { v >>=  2; pos +=  2; }
			if (v >= 1ull<< 1) {           pos +=  1; }
			return pos;
			#endif
		}

		u64* data = nullptr;
		u32 count = 0;
	};

	struct CacheBucket::MaintenanceContext
	{
		MemoryBlock memoryBlock;
		BitArray deletedOffsets;
		bool isInitialized = false;
		bool shouldTest = false;
	};

	struct CacheServer::ConnectionBucket
	{
		ConnectionBucket(u64 i, u32 version) : pathTable(IsCaseInsensitive(i), 0, 0, version), id(i) {}
		CompactPathTable pathTable;
		CompactCasKeyTable casKeyTable;

		Futex deferredCacheEntryLookupLock;
		UnorderedMap<CasKey, CacheEntry> deferredCacheEntryLookup;
		u64 id;
		u32 index = ~0u;
	};

	struct CacheServer::Connection
	{
		u32 clientVersion;
		UnorderedMap<u64, ConnectionBucket> storeBuckets;
		u64 storeEntryCount = 0;
		UnorderedSet<u32> fetchBuckets;
		u64 fetchEntryCount = 0;
		u64 fetchEntryHitCount = 0;
		u64 connectTime = 0;
	};

	void CacheServerCreateInfo::Apply(Config& config)
	{
	}

	CacheServer::CacheServer(const CacheServerCreateInfo& info)
	:	m_logger(info.logWriter, TC("UbaCacheServer"))
	,	m_server(info.storage.GetServer())
	,	m_storage(info.storage)
	{
		m_checkInputsForDeletedCas = info.checkInputsForDeletedCas;
		m_bootTime = GetTime();

		m_maintenanceReserveSize = info.maintenanceReserveSize;
		m_expirationTimeSeconds = info.expirationTimeSeconds;
		m_bucketCasTableMaxSize = info.bucketCasTableMaxSize;

		m_rootDir.count = GetFullPathNameW(info.rootDir, m_rootDir.capacity, m_rootDir.data, NULL);
		m_rootDir.Replace('/', PathSeparator).EnsureEndsWithSlash();

		m_storage.m_trackedDeletes = &m_trackedDeletes;

		m_server.RegisterService(CacheServiceId,
			[this](const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer)
			{
				return HandleMessage(connectionInfo, messageInfo.type, reader, writer);
			},
			[](u8 messageType)
			{
				return ToString(CacheMessageType(messageType));
			}
		);

		m_server.RegisterOnClientDisconnected(CacheServiceId, [this](const Guid& clientUid, u32 clientId)
			{
				OnDisconnected(clientId);
			});
	}

	CacheServer::~CacheServer()
	{
		m_storage.m_trackedDeletes = nullptr;
		
		m_server.UnregisterOnClientDisconnected(CacheServiceId);
		m_server.UnregisterService(CacheServiceId);
	}

	bool CacheServer::Load(bool validateBuckets)
	{
		u64 startTime = GetTime();

		StringBuffer<> fileName(m_rootDir);
		fileName.EnsureEndsWithSlash().Append(TCV("cachedb"));

		FileAccessor file(m_logger, fileName.data);
		if (!file.OpenMemoryRead(0, false))
		{
			m_logger.Detail(TC("No database found. Starting a new one at %s"), fileName.data);
			m_creationTime = GetSystemTimeAsFileTime();
			m_dbfileDirty = true;
			return true;
		}
		BinaryReader reader(file.GetData(), 0, file.GetSize());

		u32 databaseVersion = reader.ReadU32();
		if (databaseVersion < CacheFileCompatibilityVersion || databaseVersion > CacheFileVersion)
		{
			m_logger.Detail(TC("Can't load database of version %u. Starting a new one at %s"), databaseVersion, fileName.data);
			return true;
		}
		if (databaseVersion == 3)
			m_creationTime = GetSystemTimeAsFileTime() - 1;
		else
			m_creationTime = reader.ReadU64();

		if (databaseVersion != CacheFileVersion)
			m_dbfileDirty = true;

		CacheBucket::LoadStats stats;

		if (databaseVersion == 4)
		{
			u32 bucketCount = reader.ReadU32();
			while (bucketCount--)
			{
				Bucket& bucket = GetBucket(reader.ReadU64(), TC("Loading"));
				bucket.Load(m_logger, reader, databaseVersion, stats, m_storage);
			}
		}
		else
		{
			StringBuffer<MaxPath> bucketsDir(m_rootDir);
			bucketsDir.EnsureEndsWithSlash().Append(TCV("buckets"));
			TraverseDir(m_logger, bucketsDir, [&](const DirectoryEntry& e)
				{
					StringBuffer<128> keyName;
					keyName.Append(e.name, e.nameLen);
					u64 id;
					if (!keyName.Parse(id))
						return;
					GetBucket(id, TC("Loading"), false);
				});

			Futex toDeleteLock;
			Set<u64> toDelete;

			m_server.ParallelFor(GetBucketWorkerCount(), m_buckets, [&](const WorkContext&, auto& it)
				{
					u64 key = it->first;
					Bucket& bucket = it->second;

					StringBuffer<MaxPath> bucketFilename(bucketsDir);
					bucketFilename.EnsureEndsWithSlash().AppendValue(key);
					FileAccessor bucketFile(m_logger, bucketFilename.data);
					if (!bucketFile.OpenMemoryRead(0, false))
					{
						m_logger.Detail(TC("Failed to open bucket file %s"), bucketFilename.data);
						return;
					}
					BinaryReader reader(bucketFile.GetData(), 0, bucketFile.GetSize());
					u32 bucketVersion = reader.ReadU32();

					if (bucket.Load(m_logger, reader, bucketVersion, stats, m_storage))
						if (!validateBuckets || bucket.Validate(m_logger, m_server))
							return;

					bucketFile.Close();

					m_logger.Info(TC("Found invalid bucket %s. Deleting"), bucketFilename.data);
					DeleteFileW(bucketFilename.data);
					SCOPED_FUTEX(toDeleteLock, lock);
					toDelete.insert(key);

				}, TCV("LoadBucket"));

			for (auto key : toDelete)
				m_buckets.erase(key);

		}

		u64 duration = GetTime() - startTime;
		m_logger.Detail(TC("Database loaded from %s (v%u)  in %s (%llu bucket(s) containing %s paths, %s keys, %s cache entries)"), fileName.data, databaseVersion, TimeToText(duration).str, m_buckets.size(), BytesToText(stats.totalPathTableSize).str, BytesToText(stats.totalCasKeyTableSize).str, CountToText(stats.totalCacheEntryCount.load()).str);
		return true;
	}

	CacheBucket::CacheBucket(u64 id, u32 version) : m_pathTable(IsCaseInsensitive(id), 0, 0, version), m_id(id) {}

	bool CacheBucket::Load(Logger& logger, BinaryReader& reader, u32 databaseVersion, LoadStats& outStats, StorageServer& storage)
	{
		if (databaseVersion != CacheFileVersion)
			needsSave = true;

		if (databaseVersion > 8)
			m_pathTable.AddCommonStringSegments();

		u32 pathTableSize = reader.ReadU32();
		if (pathTableSize)
		{
			u64 skipCommonSegments = m_pathTable.GetSize();
			BinaryReader pathTableReader(reader.GetPositionData(), skipCommonSegments, pathTableSize);
			m_pathTable.ReadMem(pathTableReader, true);
			reader.Skip(pathTableSize);
		}
		outStats.totalPathTableSize += pathTableSize;

		u32 casKeyTableSize = reader.ReadU32();
		if (casKeyTableSize)
		{
			BinaryReader casKeyTableReader(reader.GetPositionData(), 0, casKeyTableSize);
			m_casKeyTable.ReadMem(casKeyTableReader, true);
			reader.Skip(casKeyTableSize);

			m_casKeyTable.Debug(m_pathTable);
		}
		outStats.totalCasKeyTableSize += casKeyTableSize;

		u32 entryLookupCount = reader.ReadU32();
		m_cacheEntryLookup.reserve(entryLookupCount);

		while (entryLookupCount--)
		{
			auto insres = m_cacheEntryLookup.try_emplace(reader.ReadCasKey());
			UBA_ASSERT(insres.second);
			auto& cacheEntries = insres.first->second;
			cacheEntries.ReadFromDisk(logger, reader, databaseVersion, storage, m_casKeyTable);
			totalEntryCount += cacheEntries.entries.size();
		}
		outStats.totalCacheEntryCount += totalEntryCount;
		return true;
	}

	bool CacheBucket::Validate(Logger& logger, WorkManager& workManager)
	{
		Atomic<bool> success = true;
		workManager.ParallelFor(16, m_cacheEntryLookup, [&](const WorkContext&, auto& it)
			{
				if (!it->second.Validate(logger))
					success = false;
			}, TCV("ValidateBucket"));
		return success;
	}

	bool CacheServer::Save()
	{
		for (auto& kv : m_buckets)
		{
			Bucket& bucket = kv.second;
			if (bucket.lastSavedTime < bucket.lastUsedTime)
				bucket.needsSave = true;
		}

		return SaveNoLock();
	}

	struct FileWriter
	{
		static constexpr u64 TempBufferSize = 1024*1024;

		FileWriter(Logger& l, const tchar* fn)
		:	logger(l)
		,	fileName(fn)
		,	tempFileName(StringBuffer<MaxPath>(fn).Append(TCV(".tmp")).data)
		,	file(logger, tempFileName.c_str())
		{
			tempBuffer = (u8*)malloc(TempBufferSize);
		}

		~FileWriter()
		{
			free(tempBuffer);
		}

		void WriteBytes(const void* data, u64 size)
		{
			if (size > TempBufferSize)
			{
				if (tempBufferPos)
				{
					written += tempBufferPos;
					success &= file.Write(tempBuffer, tempBufferPos);
					tempBufferPos = 0;
				}
				success &= file.Write(data, size);
				written += size;
				return;
			}

			u8* readPos = (u8*)data;
			u64 left = size;
			while (left)
			{
				if (tempBufferPos != TempBufferSize)
				{
					u64 toWrite = Min(TempBufferSize - tempBufferPos, left);
					memcpy(tempBuffer+tempBufferPos, readPos, toWrite);
					tempBufferPos += toWrite;
					left -= toWrite;
					readPos += toWrite;
				}
				else
				{
					written += tempBufferPos;
					success &= file.Write(tempBuffer, tempBufferPos);
					tempBufferPos = 0;
				}
			}
		}

		template<typename T>
		void Write(const T& v)
		{
			WriteBytes(&v, sizeof(v));
		}

		bool Create() { return file.CreateWrite(); }

		bool Close()
		{
			success &= file.Write(tempBuffer, tempBufferPos);
			written += tempBufferPos;

			if (!success)
				return false;

			if (!file.Close())
				return false;

			if (!MoveFileExW(tempFileName.data(), fileName.data(), MOVEFILE_REPLACE_EXISTING))
				return logger.Error(TC("Can't move file from %s to %s (%s)"), tempFileName.data(), fileName.data(), LastErrorToText().data);

			return true;
		}

		Logger& logger;
		bool success = true;
		u8* tempBuffer = nullptr;
		u64 tempBufferPos = 0;
		u64 written = 0;
		TString fileName;
		TString tempFileName;
		FileAccessor file;
	};

	bool CacheServer::SaveBucket(Bucket& bucket, Vector<u8>& temp)
	{
		u64 saveStart = GetTime();

		StringBuffer<MaxPath> bucketsDir(m_rootDir);
		bucketsDir.EnsureEndsWithSlash().Append(TCV("buckets"));
		if (!m_storage.CreateDirectory(bucketsDir.data))
			return false;
		bucketsDir.EnsureEndsWithSlash();
		StringBuffer<MaxPath> bucketsFile(bucketsDir);
		bucketsFile.AppendValue(bucket.m_id);

		FileWriter file(m_logger, bucketsFile.data);
		
		if (!file.Create())
			return false;

		file.Write(CacheFileVersion);

		u32 pathTableSize = bucket.m_pathTable.GetSize();
		file.Write(pathTableSize);
		file.WriteBytes(bucket.m_pathTable.GetMemory(), pathTableSize);

		u32 casKeyTableSize = bucket.m_casKeyTable.GetSize();
		file.Write(casKeyTableSize);
		file.WriteBytes(bucket.m_casKeyTable.GetMemory(), casKeyTableSize);

		u32 entryLookupCount = u32(bucket.m_cacheEntryLookup.size());
		file.Write(entryLookupCount);

		for (auto& kv2 : bucket.m_cacheEntryLookup)
		{
			file.Write(kv2.first);

			temp.resize(kv2.second.GetTotalSize(CacheNetworkVersion, true));
			BinaryWriter writer(temp.data(), 0, temp.size());
			kv2.second.Write(writer, CacheNetworkVersion, true);
			UBA_ASSERT(writer.GetPosition() == temp.size());
			file.WriteBytes(temp.data(), temp.size());
		}

		if (!file.Close())
			return false;

		bucket.lastSavedTime = GetSystemTimeAsFileTime() - m_creationTime;

		StringBuffer<256> log;
		log.Appendf(TC("    Bucket %u saved"), bucket.index);
		u32 version = bucket.m_pathTable.GetVersion();
		if (version != CacheBucketVersion)
			log.Appendf(TC(" (v%u)"), version);
		log.Appendf(TC(" - %s (%s)"), BytesToText(file.written).str, TimeToText(GetTime() - saveStart).str);
		m_logger.Log(LogEntryType_Detail, log);
		return true;
	}

	bool CacheServer::SaveNoLock()
	{
		if (!SaveDbNoLock())
			return false;

		Atomic<bool> success = true;

		DeleteEmptyBuckets();

		m_server.ParallelFor(GetBucketWorkerCount(), m_buckets, [&, temp = Vector<u8>()](const WorkContext&, auto& it) mutable
			{
				Bucket& bucket = it->second;
				if (!bucket.needsSave)
					return;
				if (SaveBucket(bucket, temp))
					bucket.needsSave = false;
				else
					success = false;
			}, TCV("SaveNoLock"));

		return success;
	}

	bool CacheServer::SaveDbNoLock()
	{
		if (!m_dbfileDirty)
			return true;
		StringBuffer<MaxPath> fileName(m_rootDir);
		fileName.EnsureEndsWithSlash().Append(TCV("cachedb"));

		FileWriter file(m_logger, fileName.data);
		
		if (!file.Create())
			return false;

		file.Write(CacheFileVersion);

		file.Write(m_creationTime);

		if (!file.Close())
			return false;
		m_dbfileDirty = false;
		return true;
	}

	bool CacheServer::DeleteEmptyBuckets()
	{
		for (auto it=m_buckets.begin(); it!=m_buckets.end();)
		{
			Bucket& bucket = it->second;
			if (!bucket.m_cacheEntryLookup.empty())
			{
				++it;
				continue;
			}

			StringBuffer<MaxPath> bucketsFile(m_rootDir);
			bucketsFile.EnsureEndsWithSlash().Append(TCV("buckets")).EnsureEndsWithSlash().AppendValue(it->first);
			DeleteFileW(bucketsFile.data);
			m_logger.Detail(TC("    Bucket %u was empty. Deleted"), bucket.index);
			it = m_buckets.erase(it);
		}
		return true;
	}

	void CacheServer::SetForceFullMaintenance()
	{
		m_forceAllSteps = true;
	}

	void CacheServer::PrintStatusLine(const tchar* additionalInfo)
	{
		SCOPED_READ_LOCK(m_storage.m_casLookupLock, lookupLock);
		u64 totalCasCount = m_storage.m_casLookup.size();
		lookupLock.Leave();

		SCOPED_FUTEX(m_storage.m_accessLock, accessLock);
		u64 totalCasSize = m_storage.m_casTotalBytes;
		u64 totalDeletedCount = m_storage.m_trackedDeletes->size();
		accessLock.Leave();

		SCOPED_FUTEX_READ(m_bucketsLock, bucketsLock);
		u64 bucketCount = m_buckets.size();
		u32 maxPathTable = 0;
		u32 maxKeyTable = 0;
		for (auto& kv : m_buckets)
		{
			maxPathTable = Max(maxPathTable, kv.second.m_pathTable.GetSize());
			maxKeyTable = Max(maxKeyTable, kv.second.m_casKeyTable.GetSize());
		}
		bucketsLock.Leave();

		StringBuffer<> text;
		text.Appendf(TC("CasFiles: %s (%s) Buckets: %llu Entries: %s MaxPathTable: %s MaxKeyTable: %s%s"), CountToText(totalCasCount).str, BytesToText(totalCasSize).str, bucketCount, CountToText(m_totalEntryCount).str, BytesToText(maxPathTable).str, BytesToText(maxKeyTable).str, additionalInfo);
		if (totalDeletedCount)
			text.Appendf(TC(" CasOverflowDeletes: %s"), CountToText(totalDeletedCount).str);

		StringKey statusLineKey = ToStringKeyNoCheck(text.data, text.count);
		if (statusLineKey == m_statusLineKey)
			return;
		m_statusLineKey = statusLineKey;
		m_logger.Log(LogEntryType_Info, text);
	}

	bool CacheServer::RunMaintenance(bool force, bool allowSave, const Function<bool()>& shouldExit)
	{
		if (m_forceMaintenance)
		{
			force = true;
			m_forceMaintenance = false;
		}

		if (m_addsSinceMaintenance == 0 && !force && !m_bucketIsOverflowing)
			return true;

		bool isFirst = true;

		while (true)
		{
			SCOPED_FUTEX(m_connectionsLock, lock2);
			if (!force && !m_connections.empty())
				return true;
			m_isRunningMaintenance = true;

			bool firstLoop = true;
			while (!m_connections.empty())
			{
				if (firstLoop)
					m_logger.Info(TC("Waiting for %llu client(s) to disconnect before starting maintenance"), m_connections.size());
				firstLoop = false;
				lock2.Leave();
				m_server.ShutdownActiveConnections();
				Sleep(200);
				lock2.Enter();
			}

			lock2.Leave();

			PrintContentionSummary(m_logger);

			auto runningMaintenanceGuard = MakeGuard([&]()
				{
					SCOPED_FUTEX(m_connectionsLock, lock3);
					m_isRunningMaintenance = false;
				});

			if (isFirst)
			{
				isFirst = false;

				u32 peakConnectionCount = m_peakConnectionCount;
				m_peakConnectionCount = 0;

				u64 startTime = GetTime();
				auto& storageStats = m_storage.Stats();
				u64 hits = m_cacheKeyHitCount;
				u64 miss = m_cacheKeyFetchCount - hits;
				m_logger.Info(TC("Stats since boot (%s ago)"), TimeToText(startTime - m_bootTime, true).str);
				m_logger.Info(TC("  CacheServer %s hits, %s misses"), CountToText(hits).str, CountToText(miss).str);
				u64 recvCount = storageStats.sendCas.count.load();
				u64 sendCount = storageStats.recvCas.count.load();
				m_logger.Info(TC("  StorageServer cas %s (%s) sent, %s (%s) received"), CountToText(recvCount).str, BytesToText(storageStats.sendCasBytesComp).str, CountToText(sendCount).str, BytesToText(storageStats.recvCasBytesComp).str);
			
				if (m_lastMaintenance)
					m_logger.Info(TC("Stats since last maintenance (%s ago)"), TimeToText(startTime - m_lastMaintenance, true).str);
				m_logger.Info(TC("  Peak connection count: %4u"), peakConnectionCount);
				auto& sentTimer = m_server.GetTotalSentTimer();
				m_logger.Info(TC("  Socket sent %u (%s)"), sentTimer.count.load(), BytesToText(m_server.GetTotalSentBytes()).str);

				//KernelStats& kernelStats = KernelStats::GetGlobal();
				//kernelStats.Print(m_logger, false);
				//kernelStats = {};
				m_server.ResetTotalStats();
			}


			if(!RunMaintenanceInternal(shouldExit, allowSave))
				return false;
			if (!m_bucketIsOverflowing)
				break;
			if (force)
				runningMaintenanceGuard.Cancel();
		}

		return true;
	}

	bool CacheServer::RunMaintenanceInternal(const Function<bool()>& shouldExit, bool allowSave)
	{
		//m_forceAllSteps = true;
		bool forceAllSteps = m_forceAllSteps;
		m_forceAllSteps = false;

		Trace trace(m_logger.m_writer);
		if (forceAllSteps)
		{
			u64 traceReserveSize = 128ull*1024*1024;
			trace.StartWrite(nullptr, traceReserveSize);
			trace.SessionAdded(0, 0, TCV("CacheServer"), {});
			trace.ProcessAdded(0, 0, TCV("Maintenance"), {});

			m_server.SetWorkTracker(&trace);
		}
		auto endTrace = MakeGuard([&]()
			{
				if (!forceAllSteps)
					return;
				m_server.SetWorkTracker(nullptr);
				trace.ProcessExited(0, 0);
				StringBuffer<> traceFile(m_rootDir);
				traceFile.Append(TCV("UbaCacheServer.uba"));
				trace.StopWrite(traceFile.data);
			});

		u32 addsSinceMaintenance = m_addsSinceMaintenance;
		bool entriesAdded = addsSinceMaintenance != 0;
		m_addsSinceMaintenance = 0;

		u64 startTime = GetTime();

		if (m_shouldWipe)
		{
			m_shouldWipe = false;
			m_logger.Info(TC("Obliterating database"));
			m_longestMaintenance = 0;
			m_buckets.clear();
			forceAllSteps = true;
			m_creationTime = GetSystemTimeAsFileTime();
		}
		else
		{
			m_logger.Info(TC("Maintenance starting after %u added cache entries"), addsSinceMaintenance);
		}

		m_lastMaintenance = startTime;
		
		UnorderedSet<CasKey> deletedCasFiles;
		deletedCasFiles.swap(m_trackedDeletes);
		m_storage.HandleOverflow(&deletedCasFiles);
		u64 deletedCasCount = deletedCasFiles.size();

		u64 totalCasSize = 0;

		struct CasFileInfo { CasFileInfo(u32 s = 0) : size(s) {} u32 size; Atomic<bool> isUsed = false; }; // These are compressed cas, should never be over 4gb

		MemoryBlock existingCasMemoryBlock(TC("ExistingCasMemoryBlock"));
		HashMap<CasKey, CasFileInfo> existingCas;


		m_storage.WaitForActiveWork();

		u64 totalCasCount;
		{
			UBA_TRACK_WORK_SCOPE("CollectCas");
			u64 collectCasStartTime = GetTime();

			u32 removedNonExisting = 0;

			// TODO: Make this cleaner... (inside UbaStorage instead)
			SCOPED_WRITE_LOCK(m_storage.m_casLookupLock, lookupLock);
			
			totalCasCount = m_storage.m_casLookup.size();

			// Existing cas entries can be more than 2 million entries.. which uses a lot of memory
			u64 existingCasMemoryReserveSize = existingCas.GetMemoryNeeded(totalCasCount);
			if (!existingCasMemoryBlock.Init(existingCasMemoryReserveSize, nullptr, true))
				existingCasMemoryBlock.Init(existingCasMemoryReserveSize);

			existingCas.Init(existingCasMemoryBlock, totalCasCount, TC("ExistingCas"));

			for (auto i=m_storage.m_casLookup.begin(), e=m_storage.m_casLookup.end(); i!=e;)
			{
				if (i->second.verified && !i->second.exists)
				{
					m_storage.DetachEntry(i->second);
					++removedNonExisting;
					i = m_storage.m_casLookup.erase(i);
					e = m_storage.m_casLookup.end();
					continue;
				}
				totalCasSize += i->second.size;
				UBA_ASSERT(i->second.size < ~0u);
				existingCas.Insert(i->first).size = u32(i->second.size);
				++i;
			}
			lookupLock.Leave();

			if (removedNonExisting)
				m_logger.Detail(TC("  Removed %s cas entries (marked as not existing)"), CountToText(removedNonExisting).str);

			m_logger.Detail(TC("  Found %s (%s) cas files and %s deleted by overflow (%s)"), CountToText(existingCas.Size()).str, BytesToText(totalCasSize).str, CountToText(deletedCasFiles.size()).str, TimeToText(GetTime() - collectCasStartTime).str);
		}

		if (shouldExit && shouldExit())
			return true;

		// Take biggest buckets first
		Vector<Bucket*> sortedBuckets;
		for (auto& kv : m_buckets)
			sortedBuckets.push_back(&kv.second);
		std::sort(sortedBuckets.begin(), sortedBuckets.end(), [](Bucket* a, Bucket* b)
			{
				auto aSize = a->m_casKeyTable.GetSize();
				auto bSize = b->m_casKeyTable.GetSize();
				if (aSize != bSize)
					return aSize > bSize;
				return a->index < b->index;
			});


		Futex globalStatsLock;
		u64 now = GetSystemTimeAsFileTime();
		u64 oldest = 0;
		u64 oldestUsedTime = 0;

		u32 workerCount = m_server.GetWorkerCount();
		u32 workerCountToUse = workerCount > 0 ? workerCount - 1 : 0;
		u32 workerCountToUseForBuckets = Min(workerCountToUse, u32(m_buckets.size()));

		Atomic<u64> totalEntryCount = 0;
		Atomic<u64> deleteEntryCount = 0;
		Atomic<u64> expiredEntryCount = 0;
		Atomic<u64> overflowedEntryCount = 0;
		Atomic<u64> missingOutputEntryCount = 0;
		Atomic<u64> missingInputEntryCount = 0;

		Atomic<u64> activeDropCount = 0;
		auto dropCasGuard = MakeGuard([&]() { while (activeDropCount != 0) Sleep(1); });

		auto EnsureBucketContextInitialized = [&](Bucket& bucket)
			{
				auto& context = *bucket.m_maintenanceContext;
				if (!context.isInitialized)
				{
					UBA_TRACK_WORK_SCOPE("InitContext");
					if (!context.memoryBlock.Init(m_maintenanceReserveSize, nullptr, true)) // Try to use large blocks
						context.memoryBlock.Init(m_maintenanceReserveSize, nullptr, false);
					context.deletedOffsets.Init(context.memoryBlock, bucket.m_casKeyTable.GetSize(), TC("DeletedOffsets"));
					context.isInitialized = true;
				}
			};

		u32 deleteIteration = 0;
		u64 deleteCacheEntriesStartTime = GetTime();
		do
		{
			oldest = 0;
			oldestUsedTime = 0;
			totalEntryCount = 0;

			// Traverse all buckets in parallel
			m_server.ParallelFor(workerCountToUseForBuckets, sortedBuckets, [&](const WorkContext&, auto& it)
			{
				UBA_TRACK_WORK_SCOPE("TraverseBucket");

				Bucket& bucket = **it;
				auto context = bucket.m_maintenanceContext;
				if (!context)
					context = bucket.m_maintenanceContext = new Bucket::MaintenanceContext(TC("MaintenanceContext"));

				// Traverse all deleted cas files and create a bit table which has the bit of caskeytable offset set to true if deleted
				// This is created to make fast lookup further down
				bool foundDeletedCasKey = false;
				for (auto& cas : deletedCasFiles)
					bucket.m_casKeyTable.TraverseOffsets(cas, [&](u32 casKeyOffset) // There can be multiple offsets since caskey can be the same but path different
						{
							EnsureBucketContextInitialized(bucket);
							foundDeletedCasKey = true;
							context->deletedOffsets.Set(casKeyOffset);
						});

				auto& deletedOffsets = context->deletedOffsets;

				Futex bucketLock;
				Vector<CasKey> keysToErase;


				// Check if we need to change expiration time. This is only done first iteration (to prevent very long maintenance times)

				u64 bucketExpirationTimeSeconds = m_expirationTimeSeconds;

				u64 lastUseTimeLimit = 0; // This is the time relative to server startup time

				if (deleteIteration == 0 && m_bucketCasTableMaxSize && bucket.oldestUsedTime)
				{
					// If bucket is larger than max we will take longestUnused and reduce by one hour
					u64 bucketCasTableSize = bucket.m_casKeyTable.GetSize();
					if (bucketCasTableSize >= m_bucketCasTableMaxSize)
					{
						u64 longestUnusedSeconds = GetFileTimeAsSeconds(now - m_creationTime - bucket.oldestUsedTime);

						bucketExpirationTimeSeconds = Min(longestUnusedSeconds, bucketExpirationTimeSeconds);
						if (bucketExpirationTimeSeconds > 60*60)
							bucketExpirationTimeSeconds -= 60*60;
						m_logger.Detail(TC("    Set temporary expiration time for bucket %u to %s to reduce cas table size"), bucket.index, TimeToText(MsToTime(bucketExpirationTimeSeconds*1000), true).str);
					}
				}

				if (bucketExpirationTimeSeconds)
				{
					u64 secondsSinceCreation = GetFileTimeAsSeconds(now - m_creationTime);
					if (secondsSinceCreation > bucketExpirationTimeSeconds)
						lastUseTimeLimit = GetSecondsAsFileTime(secondsSinceCreation - bucketExpirationTimeSeconds);
				}

				u64 bucketOldest = 0;
				bucket.oldestUsedTime = 0;
				bucket.totalEntryCount = 0;
				bucket.totalEntrySize = 0;

				// Loop through all cache entries in parallel
				m_server.ParallelFor<100>(workerCountToUse, bucket.m_cacheEntryLookup, [&, touchedCas = Vector<Atomic<bool>*>()](const WorkContext&, auto& li) mutable
				{
					UBA_TRACK_WORK_SCOPE("TraverseEntry");
					CacheEntries& entries = li->second;

					bool checkInputsForDeletes = false;

					// Check the inputsThatAreOutputs if any of those are deleted, if they are, then we need to checkInputsForDeletes
					// This is an optimization since most build steps only have 1-2 inputs that are outputs
					if (foundDeletedCasKey && m_checkInputsForDeletedCas && !entries.inputsThatAreOutputs.empty())
					{
						checkInputsForDeletes = *entries.inputsThatAreOutputs.begin() == ~0u;
						if (!checkInputsForDeletes)
						{
							for (auto i=entries.inputsThatAreOutputs.begin(), e=entries.inputsThatAreOutputs.end(); i!=e;)
							{
								if (!deletedOffsets.IsSet(*i))
								{
									++i;
									continue;
								}
								i = entries.inputsThatAreOutputs.erase(i);
								e = entries.inputsThatAreOutputs.end();
								checkInputsForDeletes = true;
							}
						}
					}

					// There is currently no idea keeping more than 256kb worth of entries per lookup key (because that is what fetch max returns).. so let's wipe out
					// all the entries that overflow that number
					u64 entriesSize = entries.GetSharedSize();
					u64 capacityLeft = SendMaxSize - 32 - entriesSize;

					// Check if any offset has been deleted in shared offsets..
					bool offsetDeletedInShared = false;
					auto& sharedOffsets = entries.sharedInputCasKeyOffsets;
					if (checkInputsForDeletes)
					{
						BinaryReader reader2(sharedOffsets);
						while (reader2.GetLeft())
						{
							if (!deletedOffsets.IsSet(u32(reader2.Read7BitEncoded())))
								continue;
							offsetDeletedInShared = true;
							break;
						}
					}

					u64 entriesOldestUsed = 0;
					u64 entriesOldest = 0;

					for (auto i=entries.entries.begin(), e=entries.entries.end(); i!=e;)
					{
						auto& entry = *i;
						bool deleteEntry = false;
						u64 neededSize = entries.GetEntrySize(entry, CacheNetworkVersion, false);

						// Check if this entry is outside capacity of network message
						if (neededSize > capacityLeft)
						{
							deleteEntry = true;
							capacityLeft = 0;
							++overflowedEntryCount;
						}

						// Check if entry has expired
						if (!deleteEntry && entry.creationTime < lastUseTimeLimit && entry.lastUsedTime < lastUseTimeLimit)
						{
							deleteEntry = true;
							++expiredEntryCount;
						}

						// This is an attempt at removing entries that has inputs that depends on other entries outputs.
						// and that there is no point keeping them if the other entry is removed
						// Example would be that there is no idea keeping entries that uses a pch if the entry producing the pch is gone
						if (checkInputsForDeletes)
						{
							if (!deleteEntry && offsetDeletedInShared)
							{
								BinaryReader rangeReader(entry.sharedInputCasKeyOffsetRanges);
								while (!deleteEntry && rangeReader.GetLeft())
								{
									u64 begin = rangeReader.Read7BitEncoded();
									u64 end = rangeReader.Read7BitEncoded();
									BinaryReader inputReader(sharedOffsets.data() + begin, 0, end - begin);
									while (inputReader.GetLeft())
									{
										if (!deletedOffsets.IsSet(u32(inputReader.Read7BitEncoded())))
											continue;
										deleteEntry = true;
										++missingInputEntryCount;
										break;
									}
								}
							}

							if (!deleteEntry)
							{
								auto& extraInputs = entry.extraInputCasKeyOffsets;
								BinaryReader extraReader(extraInputs);
								while (extraReader.GetLeft())
								{
									if (!deletedOffsets.IsSet(u32(extraReader.Read7BitEncoded())))
										continue;
									deleteEntry = true;
									++missingInputEntryCount;
									break;
								}
							}
						}

						// We must always traverse this to collect output cas keys.. since if this entry is not deleted we will set cas keys as used
						if (!deleteEntry)
						{
							// Traverse outputs and check if cas files exists for each output, if not, delete entry.
							touchedCas.clear();

							auto& outputs = entry.outputCasKeyOffsets;
							BinaryReader outputsReader(outputs);
							while (outputsReader.GetLeft())
							{
								u64 offset = outputsReader.Read7BitEncoded();
								CasKey casKey;
								bucket.m_casKeyTable.GetKey(casKey, offset);
								UBA_ASSERT(IsCompressed(casKey));

								if (IsExecutable(casKey))
									casKey = AsExecutable(casKey, false);

								if (auto value = existingCas.Find(casKey))
								{
									touchedCas.push_back(&value->isUsed);
									continue;
								}
								deleteEntry = true;
								++missingOutputEntryCount;
								break;
							}
						}

						// Remove entry from entries list and skip increasing ref count of cas files
						if (deleteEntry)
						{
							if (i->id == entries.primaryId)
								entries.primaryId = ~0u;
							bucket.hasDeletedEntries = true;
							++deleteEntryCount;
							i = entries.entries.erase(i);
							e = entries.entries.end();
							continue;
						}

						entriesSize += neededSize;
						capacityLeft -= neededSize;

						u64 lastUsedTime = entry.lastUsedTime;
						if (!lastUsedTime)
							lastUsedTime = entry.creationTime;
						if (!entriesOldestUsed || lastUsedTime < entriesOldestUsed)
							entriesOldestUsed = lastUsedTime;
						if (!entriesOldest || entry.creationTime < entriesOldest)
							entriesOldest = entry.creationTime;

						for (auto v : touchedCas)
							*v = true;

						++i;
					}


					{
						SCOPED_FUTEX(bucketLock, lock2);
						if (!bucket.oldestUsedTime || entriesOldestUsed < bucket.oldestUsedTime)
							bucket.oldestUsedTime = entriesOldestUsed;
						if (!bucketOldest || entriesOldest < bucketOldest)
							bucketOldest = entriesOldest;

						if (entries.entries.empty())
							keysToErase.push_back(li->first);
						else
						{
							bucket.totalEntrySize += entriesSize;
							bucket.totalEntryCount += entries.entries.size();
						}
					}
				}, TCV("ParallelForEntries1"));

				{
					SCOPED_FUTEX(globalStatsLock, l);
					if (!oldestUsedTime || bucketOldest < oldestUsedTime)
						oldestUsedTime = bucket.oldestUsedTime;
					if (!oldest || bucketOldest < oldest)
						oldest = bucketOldest;
				}

				for (auto& key : keysToErase)
					bucket.m_cacheEntryLookup.erase(key);

				totalEntryCount += bucket.totalEntryCount;
			}, TCV("ParallelForBucket1"), WorkPriority_High);

			// Reset deleted cas files and update it again..
			deletedCasFiles.clear();

			for (auto i=existingCas.ValuesBegin(), e=existingCas.ValuesEnd(); i!=e; ++i)
			{
				if (i->isUsed)
				{
					i->isUsed = false;
					continue;
				}
				const CasKey* key = existingCas.GetKey(i);
				if (!key)
					continue;
				deletedCasFiles.insert(*key);
				++deletedCasCount;
				totalCasSize -= i->size;
				existingCas.Erase(*key);
			}

			// Add drop cas as work so it can run in the background
			if (allowSave)
			{
				Vector<CasKey> casKeysBatch;
				auto createBatchWork = [&]()
					{
						activeDropCount += casKeysBatch.size();
						m_server.AddWork([&, ckb = casKeysBatch](const WorkContext&)
							{
								for (auto& key : ckb)
									m_storage.DropCasFile(key, true, TC(""));
								activeDropCount -= ckb.size();
							}, 1, TC("DropCasKeysBatch"));
						casKeysBatch.resize(0);
					};
				for (auto& casKey : deletedCasFiles)
				{
					casKeysBatch.push_back(casKey);
					if (casKeysBatch.size() > 100)
						createBatchWork();
				}
				if (!casKeysBatch.empty())
					createBatchWork();
			}
			++deleteIteration;
		}
		while (!deletedCasFiles.empty()); // if cas files are deleted we need to do another loop and check cache entry inputs to see if files were inputs

		existingCasMemoryBlock.Deinit();

		if (overflowedEntryCount)
			m_logger.Detail(TC("  Found %s overflowed cache entries"), CountToText(overflowedEntryCount.load()).str);
		if (expiredEntryCount)
			m_logger.Detail(TC("  Found %s expired cache entries"), CountToText(expiredEntryCount.load()).str);
		if (missingOutputEntryCount)
			m_logger.Detail(TC("  Found %s cache entries with missing output cas"), CountToText(missingOutputEntryCount.load()).str);
		if (missingInputEntryCount)
			m_logger.Detail(TC("  Found %s cache entries with missing input cas"), CountToText(missingInputEntryCount.load()).str);

		m_logger.Detail(TC("  Deleted %s cas files and %s cache entries over %u buckets (%u iterations in %s)"), CountToText(deletedCasCount).str, CountToText(deleteEntryCount.load()).str, u32(m_buckets.size()), deleteIteration, TimeToText(GetTime() - deleteCacheEntriesStartTime).str);

		m_bucketIsOverflowing = false;

		if (shouldExit && shouldExit())
			return true;


		bool shouldSave = allowSave && (entriesAdded || deletedCasCount || deleteEntryCount || forceAllSteps);

		Event saveCasEvent(EventResetType_Manual);
		Event saveDbEvent(EventResetType_Manual);
		if (shouldSave)
		{
			m_server.AddWork([&](const WorkContext&) { m_storage.SaveCasTable(false, false); saveCasEvent.Set(); }, 1, TC("SaveCas"));
			m_server.AddWork([&](const WorkContext&) { SaveDbNoLock(); saveDbEvent.Set(); }, 1, TC("SaveDb"));
		}

		u64 maxCommittedMemory = 0;

		m_server.ParallelFor(workerCountToUseForBuckets, sortedBuckets, [&](const WorkContext& wc, auto& it)
		{
			u64 bucketStartTime = GetTime();

			Bucket& bucket = **it;

			auto saveGuard = MakeGuard([&]()
				{
					if (shouldSave && bucket.needsSave && !bucket.m_cacheEntryLookup.empty())
					{
						u64 saveStart = GetTime();(void)saveStart;
						Vector<u8> temp;
						if (SaveBucket(bucket, temp))
							bucket.needsSave = false;
						UBA_CACHE_SERVER_ADD_HINT("SaveBucket", saveStart);
					}
				});

			auto deleteContext = MakeGuard([&]() { delete bucket.m_maintenanceContext; bucket.m_maintenanceContext = nullptr; });

			if (!bucket.hasDeletedEntries && !forceAllSteps)
			{
				//m_logger.Detail(TC("    Bucket %u skipped updating. (%s entries)"), bucket.index, CountToText(bucket.totalEntryCount.load()).str);
				return;
			}
			bucket.hasDeletedEntries = false;

			EnsureBucketContextInitialized(bucket);
			MemoryBlock& memoryBlock = bucket.m_maintenanceContext->memoryBlock;

			BitArray usedCasKeyOffsets;
			usedCasKeyOffsets.Init(memoryBlock, bucket.m_casKeyTable.GetSize(), TC("UsedCasKeyOffsets"));

			u64 collectUsedCasKeysStart = GetTime();

			// Collect all caskeys that are used by cache entries.
			for (auto& kv2 : bucket.m_cacheEntryLookup)
			{
				auto collectUsedCasKeyOffsets = [&](const Vector<u8>& offsets)
					{
						BinaryReader reader2(offsets);
						while (reader2.GetLeft())
						{
							u32 offset = u32(reader2.Read7BitEncoded());
							usedCasKeyOffsets.Set(offset);
						}
					};

				collectUsedCasKeyOffsets(kv2.second.sharedInputCasKeyOffsets);
				for (auto& entry : kv2.second.entries)
				{
					collectUsedCasKeyOffsets(entry.extraInputCasKeyOffsets);
					collectUsedCasKeyOffsets(entry.outputCasKeyOffsets);
				}
			}
			u64 usedCasKeyOffsetsCount = usedCasKeyOffsets.CountSetBits();

			UBA_CACHE_SERVER_ADD_HINT("CollectUsedCasKeys", collectUsedCasKeysStart);
			m_logger.Detail(TC("    Bucket %u Collected %s used caskeys. (%s)"), bucket.index, CountToText(usedCasKeyOffsetsCount).str, TimeToText(GetTime() - collectUsedCasKeysStart).str);

			u64 recreatePathTableStart = GetTime();

			// Traverse all caskeys in caskey table and figure out which ones we can delete
			BitArray usedPathOffsets;
			usedPathOffsets.Init(memoryBlock, bucket.m_pathTable.GetSize(), TC("UsedPathOffsets"));

			BinaryReader casKeyTableReader(bucket.m_casKeyTable.GetMemory(), 0, bucket.m_casKeyTable.GetSize());
			usedCasKeyOffsets.Traverse([&](u32 casKeyOffset)
				{
					casKeyTableReader.SetPosition(casKeyOffset);
					u32 pathOffset = u32(casKeyTableReader.Read7BitEncoded());
					usedPathOffsets.Set(pathOffset);
				});

			// Build new path table based on used offsets
			HashMap2<u32, u32> oldToNewPathOffset;
			auto& pathTable = bucket.m_pathTable;
			u32 oldSize = pathTable.GetSize();
			{
				u64 pathReserveCount = pathTable.GetPathCount() + 1000; // Add extra reservation space for growth
				u64 segmentReserveCount = pathTable.GetSegmentCount() + 1000;  // Add extra reservation space for growth
				CompactPathTable newPathTable(pathTable.GetCaseInsensitive(), pathReserveCount, segmentReserveCount, pathTable.GetVersion());
				newPathTable.AddCommonStringSegments();

				oldToNewPathOffset.Init(memoryBlock, usedPathOffsets.CountSetBits(), TC("OldToNewPathOffset"));

				CompactPathTable::AddContext context { pathTable };
				StringBuffer<> temp;
				usedPathOffsets.Traverse([&](u32 pathOffset)
				{
					u32 newPathOffset = newPathTable.AddNoLock(context, pathOffset);

					#if 0
					StringBuffer<> test;
					newPathTable.GetString(test, newPathOffset);
					UBA_ASSERT(test.Equals(temp.data));
					#endif

					if (pathOffset != newPathOffset)
						oldToNewPathOffset.Insert(pathOffset) = newPathOffset;
				});
				pathTable.Swap(newPathTable);
			}
			UBA_CACHE_SERVER_ADD_HINT("RecreatePathTable", recreatePathTableStart);
			m_logger.Detail(TC("    Bucket %u Recreated path table. %s -> %s (%s)"), bucket.index, BytesToText(oldSize).str, BytesToText(pathTable.GetSize()).str, TimeToText(GetTime() - recreatePathTableStart).str);

			// Build new caskey table based on used offsets
			u64 recreateCasKeyTableStart = GetTime();
			HashMap2<u32, u32> oldToNewCasKeyOffset;
			auto& casKeyTable = bucket.m_casKeyTable;
			oldSize = casKeyTable.GetSize();
			{
				oldToNewCasKeyOffset.Init(memoryBlock, usedCasKeyOffsetsCount, TC("OldToNewCasKeyOffset"));
				CompactCasKeyTable newCasKeyTable(usedCasKeyOffsetsCount + Min(usedCasKeyOffsetsCount/2, 10'000ull)); // Add extra reservation space for growth
				BinaryReader reader2(casKeyTable.GetMemory(), 0, oldSize);
				usedCasKeyOffsets.Traverse([&](u32 casKeyOffset)
				{
					reader2.SetPosition(casKeyOffset);
					u32 oldPathOffset = u32(reader2.Read7BitEncoded());
					CasKey casKey = reader2.ReadCasKey();
					u32 newPathOffset = oldPathOffset;
					if (auto value = oldToNewPathOffset.Find(oldPathOffset))
						newPathOffset = *value;
					u32 newCasKeyOffset = newCasKeyTable.AddNoLock(casKey, newPathOffset);
					if (casKeyOffset != newCasKeyOffset)
						oldToNewCasKeyOffset.Insert(casKeyOffset) = newCasKeyOffset;
				});
				casKeyTable.Swap(newCasKeyTable);
			}
			UBA_CACHE_SERVER_ADD_HINT("RecreateCasKeyTable", recreateCasKeyTableStart);
			m_logger.Detail(TC("    Bucket %u Recreated caskey table. %s -> %s (%s)"), bucket.index, BytesToText(oldSize).str, BytesToText(bucket.m_casKeyTable.GetSize()).str, TimeToText(GetTime() - recreateCasKeyTableStart).str);

			if (casKeyTable.GetSize() >= m_bucketCasTableMaxSize) // Still overflowing?
				m_bucketIsOverflowing = true;

			if (oldToNewCasKeyOffset.Size() > 0)
			{
				// Update all casKeyOffsets
				u64 updateEntriesStart = GetTime();

				m_server.ParallelFor<100>(workerCountToUse, bucket.m_cacheEntryLookup, [&, temp = Vector<u32>(), temp2 = Vector<u8>(), temp3 = Vector<u8>()](const WorkContext&, auto& it) mutable
					{
						it->second.UpdateEntries(m_logger, oldToNewCasKeyOffset, temp, temp2, temp3);
					}, TCV("ParallelForEntries2"));

				#if 0
				u8* mem = bucket.m_pathTable.GetMemory();
				u64 memLeft = bucket.m_pathTable.GetSize();
				while (memLeft)
				{
					u8 buffer[256*1024];
					auto compressor = OodleLZ_Compressor_Kraken;
					auto compressionLevel = OodleLZ_CompressionLevel_SuperFast;
					u64 toCompress = Min(memLeft, u64(256*1024 - 128));
					OodleLZ_Compress(compressor, mem, (OO_SINTa)toCompress, buffer, compressionLevel);
					memLeft -= toCompress;
				}
				#endif

				UBA_CACHE_SERVER_ADD_HINT("UpdateCacheEntries", updateEntriesStart);
				m_logger.Detail(TC("    Bucket %u Updated cache entries with new tables (%s)"), bucket.index, TimeToText(GetTime() - updateEntriesStart).str);
			}

			{
				SCOPED_FUTEX(globalStatsLock, l);
				maxCommittedMemory = Max(maxCommittedMemory, memoryBlock.writtenSize);
			}
			bucket.needsSave = true;

			u64 longestUnusedSeconds = 0;
			if (bucket.oldestUsedTime)
				longestUnusedSeconds = GetFileTimeAsSeconds(now - m_creationTime - bucket.oldestUsedTime);
			m_logger.Info(TC("    Bucket %u Done (%s). CacheEntries: %s (%s) PathTable: %s CasTable: %s LongestUnused: %s"), bucket.index, TimeToText(GetTime() - bucketStartTime).str, CountToText(bucket.totalEntryCount).str, BytesToText(bucket.totalEntrySize).str, BytesToText(bucket.m_pathTable.GetSize()).str, BytesToText(bucket.m_casKeyTable.GetSize()).str, TimeToText(MsToTime(longestUnusedSeconds*1000), true).str);
		}, TCV("ParallelForBucket2"), WorkPriority_High);

		// Need to make sure all cas entries are dropped before saving cas table
		u64 dropStartTime = GetTime();
		dropCasGuard.Execute();
		u64 dropCasDuration = GetTime() - dropStartTime;
		if (TimeToMs(dropCasDuration) > 10)
			m_logger.Detail(TC("  Done deleting cas files (%s)"), TimeToText(dropCasDuration).str);

		if (shouldSave)
		{
			DeleteEmptyBuckets();
			saveCasEvent.IsSet();
			saveDbEvent.IsSet();
		}

		m_totalEntryCount = totalEntryCount.load();
		u64 oldestTime = oldest ? GetFileTimeAsTime(now - (m_creationTime + oldest)) : 0;
		u64 longestUnused = oldestUsedTime ? GetFileTimeAsTime(now - (m_creationTime + oldestUsedTime)) : 0;
		u64 duration = GetTime() - startTime;
		m_logger.Info(TC("Maintenance done! (%s) CasFiles: %s (%s) Buckets: %llu Entries: %s Oldest: %s LongestUnused: %s MaxMaintenanceMem: %s/%s"), TimeToText(duration).str, CountToText(existingCas.Size()).str, BytesToText(totalCasSize).str, m_buckets.size(), CountToText(totalEntryCount.load()).str, TimeToText(oldestTime, true).str, TimeToText(longestUnused, true).str, BytesToText(maxCommittedMemory).str, BytesToText(m_maintenanceReserveSize).str);
		
		m_longestMaintenance = Max(m_longestMaintenance, duration);

		return true;
	}

	bool CacheServer::ShouldShutdown()
	{
		if (!m_shutdownRequested)
			return false;
		SCOPED_FUTEX(m_connectionsLock, lock2);
		if (!m_connections.empty() || m_addsSinceMaintenance)
			return false;
		return true;
	}

	void CacheServer::OnDisconnected(u32 clientId)
	{
		StringBuffer<> log;
		log.Appendf(TC("Client %u disconnected"), clientId);

		SCOPED_FUTEX(m_connectionsLock, lock);
		auto it = m_connections.find(clientId);
		if (it != m_connections.end())
		{
			u64 activeCount = m_connections.size() - 1;
			auto& c = it->second;
			log.Appendf(TC(" after %s (%llu active)."), TimeToText(GetTime() - c.connectTime).str, activeCount);
			if (!c.fetchBuckets.empty())
				log.Appendf(TC(" Fetched %llu (%llu hits) entries from bucket "), c.fetchEntryCount, c.fetchEntryHitCount).Join(c.fetchBuckets, [&](auto bucketIndex) { log.AppendValue(bucketIndex); }).Append('.');
			if (!c.storeBuckets.empty())
				log.Appendf(TC(" Stored %llu entries to bucket "), c.storeEntryCount).Join(c.storeBuckets, [&](auto& kv) { log.AppendValue(kv.second.index); });
			m_connections.erase(it);
		}
		lock.Leave();

		m_logger.Log(LogEntryType_Info, log);
	}

	CacheServer::ConnectionBucket& CacheServer::GetConnectionBucket(const ConnectionInfo& connectionInfo, BinaryReader& reader, u32* outClientVersion)
	{
		u64 id = reader.Read7BitEncoded();
		u32 bucketVersion = u32(id >> 35u);
		SCOPED_FUTEX(m_connectionsLock, lock);
		auto& connection = m_connections[connectionInfo.GetId()];
		if (outClientVersion)
		{
			++connection.storeEntryCount; // We know this is the store entry call
			*outClientVersion = connection.clientVersion;
		}
		return connection.storeBuckets.try_emplace(id, id, bucketVersion).first->second;
	}

	CacheServer::Bucket& CacheServer::GetBucket(BinaryReader& reader, const tchar* reason)
	{
		u64 id = reader.Read7BitEncoded();
		return GetBucket(id, reason);
		
	}

	CacheServer::Bucket& CacheServer::GetBucket(u64 id, const tchar* reason, bool addCommon)
	{
		u32 bucketVersion = u32(id >> 35u);
		SCOPED_FUTEX(m_bucketsLock, bucketsLock);
		auto insres = m_buckets.try_emplace(id, id, bucketVersion);
		auto& bucket = insres.first->second;
		if (!insres.second)
			return bucket;
		if (addCommon)
			bucket.m_pathTable.AddCommonStringSegments();
		bucket.index = u32(m_buckets.size() - 1);
		m_logger.Info(TC("Bucket %u created with id %llu and version %u (%s)"), bucket.index, id, bucketVersion, reason);
		return bucket;
	}

	u32 CacheServer::GetBucketWorkerCount()
	{
		u32 workerCount = m_server.GetWorkerCount();
		u32 workerCountToUse = workerCount > 0 ? workerCount - 1 : 0;
		return Min(workerCountToUse, u32(m_buckets.size()));
	}

	bool IsReadOnly(const ConnectionInfo& connectionInfo)
	{
		return connectionInfo.GetCryptoUserData() != 0;
	}

	UBA_NOINLINE bool CacheServer::HandleMessage(const ConnectionInfo& connectionInfo, u8 messageType, BinaryReader& reader, BinaryWriter& writer)
	{
		if (messageType != CacheMessageType_Connect && m_isRunningMaintenance)
			return m_logger.Debug(TC("Can't handle network message %s while running maintenance mode"), ToString(CacheMessageType(messageType)).data).ToFalse();

		switch (messageType)
		{
		case CacheMessageType_Connect:
		{
			u32 clientVersion = reader.ReadU32();
			if (clientVersion < 3 || clientVersion > CacheNetworkVersion)
				return m_logger.Error(TC("Different network versions. Client: %u, Server: %u. Disconnecting"), clientVersion, CacheNetworkVersion);

			TString hint;
			if (reader.GetLeft())
				hint = reader.ReadString();

			u64 startTime = GetTime();
			SCOPED_FUTEX(m_connectionsLock, lock);
			if (m_isRunningMaintenance)
			{
				writer.WriteBool(false);
				writer.WriteString(TCV("Running maintenance..."));
			}
			writer.WriteBool(true);
			auto insres = m_connections.try_emplace(connectionInfo.GetId());
			auto& connection = insres.first->second;
			connection.clientVersion = clientVersion;
			connection.connectTime = GetTime();
			m_peakConnectionCount = Max(m_peakConnectionCount, u32(m_connections.size()));
			lock.Leave();

			u64 endTime = GetTime();
			if (TimeToMs(endTime - startTime) > 60*1000)
				m_logger.Warning(TC("Took %s to connect client"), TimeToText(endTime - startTime).str);

			if (insres.second)
			{
				StringBuffer<> logStr;
				logStr.Appendf(TC("Client %u connected"), connectionInfo.GetId());
				if (clientVersion != CacheNetworkVersion)
					logStr.Appendf(TC(" (v%u)"), clientVersion);
				if (IsReadOnly(connectionInfo))
					logStr.Appendf(TC(" (readonly)"));
				if (!hint.empty())
					logStr.Appendf(TC(" %s"), hint.c_str());
				m_logger.Log(LogEntryType_Info, logStr);
			}

			return true;
		}
		case CacheMessageType_StorePathTable:
		{
			if (IsReadOnly(connectionInfo))
				return false;
			GetConnectionBucket(connectionInfo, reader).pathTable.ReadMem(reader, false);
			return true;
		}
		case CacheMessageType_StoreCasTable:
		{
			if (IsReadOnly(connectionInfo))
				return false;
			GetConnectionBucket(connectionInfo, reader).casKeyTable.ReadMem(reader, false);
			return true;
		}
		case CacheMessageType_StoreEntry:
		{
			if (IsReadOnly(connectionInfo))
				return false;
			u32 clientVersion;
			auto& bucket = GetConnectionBucket(connectionInfo, reader, &clientVersion);
			return HandleStoreEntry(bucket, reader, writer, clientVersion, connectionInfo.GetId());
		}
		case CacheMessageType_StoreEntryDone:
			return HandleStoreEntryDone(connectionInfo, reader);

		case CacheMessageType_FetchPathTable:
			return HandleFetchPathTable(reader, writer);

		case CacheMessageType_FetchPathTable2:
			return HandleFetchPathTable2(reader, writer);

		case CacheMessageType_FetchCasTable:
			return HandleFetchCasTable(reader, writer);

		case CacheMessageType_FetchCasTable2:
			return HandleFetchCasTable2(reader, writer);

		case CacheMessageType_FetchEntries:
			return HandleFetchEntries(reader, writer, connectionInfo.GetId());

		case CacheMessageType_ExecuteCommand:
			return HandleExecuteCommand(reader, writer);

		case CacheMessageType_ReportUsedEntry:
			return HandleReportUsedEntry(reader, writer, connectionInfo.GetId());

		case CacheMessageType_RequestShutdown:
		{
			TString reason = reader.ReadString();
			m_logger.Info(TC("Shutdown requested. Reason: %s"), reason.empty() ? TC("Unknown") : reason.c_str());
			m_shutdownRequested = true;
			writer.WriteBool(true);
			return true;
		}

		default:
			return false;
		}
	}

	bool CacheServer::HandleStoreEntry(ConnectionBucket& connectionBucket, BinaryReader& reader, BinaryWriter& writer, u32 clientVersion, u32 clientId)
	{
		CasKey cmdKey = reader.ReadCasKey();

		u64 inputCount = ~0u;
		if (clientVersion >= 5)
			inputCount = reader.Read7BitEncoded();


		u64 outputCount = reader.Read7BitEncoded();
		u64 index = 0;

		Set<u32> inputs;
		u64 bytesForInput = 0;

		u64 outputStartOffset = reader.GetPosition();
		u64 id = connectionBucket.id;
		Bucket& bucket = GetBucket(id, TC("StoreEntry"));
		connectionBucket.index = bucket.index;

		auto CreateCasKeyOffset = [&](u32& outCasKeyOffset, CasKey& outCasKey, u32 casKeyOffset, Set<u32>& entries, const tchar* direction)
			{
				u32 casKeyTableSize = connectionBucket.casKeyTable.GetSize();
				BinaryReader casKeyReader(connectionBucket.casKeyTable.GetMemory(), casKeyOffset, casKeyTableSize);

				u64 pathOffset;
				if (!casKeyReader.TryRead7BitEncoded(pathOffset))
					return m_logger.Error(TC("Client %u has provided %s offset that is outside caskey table size. Corrupt data? (Offset: %u TableSize: %u ClientVersion: %u PathtableVersion: %u)"), clientId, direction, casKeyOffset, casKeyTableSize, clientVersion, connectionBucket.pathTable.GetVersion());

				if (casKeyReader.GetLeft() < sizeof(CasKey))
					return m_logger.Error(TC("Client %u has provided %s offset that is outside caskey table size. Corrupt data? (Offset: %u TableSize: %u ClientVersion: %u PathtableVersion: %u)"), clientId, direction, casKeyOffset, casKeyTableSize, clientVersion, connectionBucket.pathTable.GetVersion());
				CasKey casKey = casKeyReader.ReadCasKey();
				if (casKey == CasKeyZero)
					return m_logger.Error(TC("Client %u has provided %s cas key that is zero. Corrupt data? (Offset: %u TableSize: %u ClientVersion: %u PathtableVersion: %u)"), clientId, direction, casKeyOffset, casKeyTableSize, clientVersion, connectionBucket.pathTable.GetVersion());

				StringBuffer<> path;
				if (!connectionBucket.pathTable.TryGetString(m_logger, path, pathOffset))
					return m_logger.Error(TC("Client %u has provided corrupt %s key or path table (CasKey: %s Offset: %u TableSize: %u PathOffset: %u PathTableSize: %u ClientVersion: %u PathtableVersion: %u)"), clientId, direction, CasKeyString(casKey).str, casKeyOffset, casKeyTableSize, pathOffset, connectionBucket.pathTable.GetSize(), clientVersion, connectionBucket.pathTable.GetVersion());
			
				if (path.count <= 2)
					return m_logger.Error(TC("Client %u has provided corrupt %s path %s. Corrupt data? (Offset: %u TableSize: %u ClientVersion: %u PathtableVersion: %u)"), clientId, direction, path.data, casKeyOffset, casKeyTableSize, clientVersion, connectionBucket.pathTable.GetVersion());

				for (u32 ci = 0, ce = path.count; ci != ce; ++ci)
					if (path.data[ci] < 0x20)
						return m_logger.Error(TC("Client %u provided corrupt %s path with control char 0x%02x at index %u - rejecting entry (path table desynced). Bucket: %llu CasKey: %s CasKeyOffset: %u PathOffset: %u PathTableSize: %u ClientVersion: %u PathtableVersion: %u"), clientId, direction, u32(path.data[ci]), ci, bucket.m_id, CasKeyString(casKey).str, casKeyOffset, pathOffset, connectionBucket.pathTable.GetSize(), clientVersion, connectionBucket.pathTable.GetVersion());

				pathOffset = bucket.m_pathTable.Add(path.data, path.count);
				if (pathOffset == ~0u)
				{
					m_forceMaintenance = true;
					return m_logger.Warning(TC("Path table in bucket %llu too big to store more keys. Forcing maintenance run (Bump max size of split up in more buckets if this happens frequently)"), bucket.m_id);
				}

				#if 0
				StringBuffer<> test;
				bucket.m_pathTable.GetString(test, pathOffset);
				UBA_ASSERT(test.Equals(path.data));
				#endif

				u32 requiredCasTableSize;
				outCasKeyOffset = bucket.m_casKeyTable.Add(casKey, pathOffset, requiredCasTableSize);
				if (outCasKeyOffset == ~0u)
				{
					m_forceMaintenance = true;
					return m_logger.Warning(TC("CasKey table in bucket %llu too big to store more keys. Forcing maintenance run (Bump max size of split up in more buckets if this happens frequently)"), bucket.m_id);
				}

				auto insres = entries.insert(outCasKeyOffset);
				if (!insres.second)
					return m_logger.Warning(TC("Client %u %s file %s exists more than once in cache entry (PathOffset %u, CasKey: %s CasKeyOffset: %u ClientVersion: %u PathtableVersion: %u)"), clientId, direction, path.data, pathOffset, CasKeyString(casKey).str, outCasKeyOffset, clientVersion, connectionBucket.pathTable.GetVersion());

				outCasKey = casKey;
				return true;
			};


		while (reader.GetLeft())
		{
			bool isInput = index++ >= outputCount;
			if (isInput && !inputCount--) // For client versions under 5 we will hit reader.GetLeft() == false first.
				break;

			u32 offset = u32(reader.Read7BitEncoded());
			if (!isInput)
				continue;

			u32 casKeyOffset = 0;
			CasKey casKey;
			if (!CreateCasKeyOffset(casKeyOffset, casKey, offset, inputs, TC("input")))
				return false;
			bytesForInput += Get7BitEncodedCount(casKeyOffset);

			//m_logger.Info(TC("%s - %s"), path.data, CasKeyString(casKey).str);
		}

		// For client versions 5 and over we have log entries after the inputs
		Vector<u8> logLines;
		if (u64 logLinesSize = reader.GetLeft())
		{
			logLines.resize(logLinesSize);
			reader.ReadBytes(logLines.data(), logLinesSize);
		}

		Vector<u8> inputCasKeyOffsets;
		{
			inputCasKeyOffsets.resize(bytesForInput);
			BinaryWriter w2(inputCasKeyOffsets.data(), 0, inputCasKeyOffsets.size());
			for (u32 input : inputs)
				w2.Write7BitEncoded(input);
		}

		SCOPED_WRITE_LOCK(bucket.m_cacheEntryLookupLock, lock);
		auto insres = bucket.m_cacheEntryLookup.try_emplace(cmdKey);
		auto& cacheEntries = insres.first->second;
		lock.Leave();

		SCOPED_WRITE_LOCK(cacheEntries.lock, lock2);
		

		// Create entry based on existing entry
		CacheEntry newEntry;
		cacheEntries.BuildInputs(newEntry, inputs);

		// Check if there already is an entry with exactly the same inputs
		List<CacheEntry>::iterator matchingEntry = cacheEntries.entries.end();
		for (auto i=cacheEntries.entries.begin(), e=cacheEntries.entries.end(); i!=e; ++i)
		{
			if (i->sharedInputCasKeyOffsetRanges != newEntry.sharedInputCasKeyOffsetRanges || i->extraInputCasKeyOffsets != newEntry.extraInputCasKeyOffsets)
				continue;
			matchingEntry = i;
			break;
		}

		UnorderedSet<CasKey> requestedOutputs;

		// Already exists
		if (matchingEntry != cacheEntries.entries.end())
		{
			bool shouldOverwrite = false;
			Map<TString, CasKey> existing;

			// Collect all outputs from entry with matching inputs
			BinaryReader r2(matchingEntry->outputCasKeyOffsets);
			while (r2.GetLeft())
			{
				u32 existingOffset = u32(r2.Read7BitEncoded());
				CasKey casKey;
				StringBuffer<> path;
				bucket.m_casKeyTable.GetPathAndKey(path, casKey, bucket.m_pathTable, existingOffset);
				if (IsCaseInsensitive(id))
					path.MakeLower();
				existing.try_emplace(path.data, casKey);
			}

			// Traverse outputs and log out the mismatching output, then remove the old entry
			reader.SetPosition(outputStartOffset);
			u64 left = outputCount;
			struct Pair { CasKey key; u32 offset; };
			Vector<Pair> outputKeys;
			while (left--)
			{
				u32 outputOffset = u32(reader.Read7BitEncoded());
				CasKey casKey;
				StringBuffer<> path;
				connectionBucket.casKeyTable.GetPathAndKey(path, casKey, connectionBucket.pathTable, outputOffset);
				if (IsCaseInsensitive(id))
					path.MakeLower();

				auto findIt = existing.find(path.data);
				if (findIt == existing.end())
				{
					m_logger.Warning(TC("Client %u sent cache entry that already exists but does not match output. Output file %s did not exist in existing cache entry. OutputCount Old: %u New: %u"), clientId, path.data, u32(existing.size()), outputCount);
					cacheEntries.entries.erase(matchingEntry);
					shouldOverwrite = true;
					break;
				}
				if (findIt->second != casKey)
				{
					//m_logger.Warning(TC("Existing cache entry matches input but does not match output (%s has different caskey)"), path.data);
					cacheEntries.entries.erase(matchingEntry);
					shouldOverwrite = true;
					break;
				}
				outputKeys.push_back(Pair{casKey, outputOffset});
			}

			if (!shouldOverwrite)
			{
				// Verify so output cas entries exist. Otherwise we want client to resend them
				for (Pair& pair : outputKeys)
				{
					if (!requestedOutputs.insert(pair.key).second)
						continue;
					if (m_storage.EnsureCasFile(pair.key, nullptr))
						continue;
					writer.Write7BitEncoded(pair.offset);
				}
				return true;
			}
		}

		Set<u32> outputs;
		u64 bytesForOutput = 0;

		// Check if all content for output caskeys exist on the server.. We don't want to publish the entry before server has all cas keys
		bool hasAllContent = true;
		reader.SetPosition(outputStartOffset);
		u64 left = outputCount;
		while (left--)
		{
			u32 outputOffset = u32(reader.Read7BitEncoded());

			u32 casKeyOffset = 0;
			CasKey casKey;
			if (!CreateCasKeyOffset(casKeyOffset, casKey, outputOffset, outputs, TC("output")))
				return false;

			bytesForOutput += Get7BitEncodedCount(casKeyOffset);

			if (!requestedOutputs.insert(casKey).second)
				continue;
			if (m_storage.EnsureCasFile(casKey, nullptr))
				continue;
			writer.Write7BitEncoded(outputOffset);
			hasAllContent = false;
		}

		// Write outputs into buffer for entry
		newEntry.outputCasKeyOffsets.resize(bytesForOutput);
		BinaryWriter w2(newEntry.outputCasKeyOffsets.data(), 0, newEntry.outputCasKeyOffsets.size());
		for (u32 output : outputs)
			w2.Write7BitEncoded(output);


		newEntry.creationTime = GetSystemTimeAsFileTime() - m_creationTime;
		newEntry.id = cacheEntries.idCounter++;

		if (logLines.empty())
		{
			newEntry.logLinesType = LogLinesType_Empty;
		}
		else if (cacheEntries.sharedLogLines.empty() && logLines.size() < 150) // If log line is very long it is most likely a warning that will be fixed
		{
			cacheEntries.sharedLogLines = std::move(logLines);
			newEntry.logLinesType = LogLinesType_Shared;
		}
		else
		{
			if (cacheEntries.sharedLogLines == logLines)
			{
				newEntry.logLinesType = LogLinesType_Shared;
			}
			else
			{
				newEntry.logLinesType = LogLinesType_Owned;
				newEntry.logLines = std::move(logLines);
			}
		}

		// Let's check if any inputs are cas files.. this is an optimization for maintenance...
		// We never have to check inputs against deleted cas files if they never existed
		{
			if (cacheEntries.entries.empty())
				cacheEntries.PopulateInputsThatAreOutputs(cacheEntries.sharedInputCasKeyOffsets, m_storage, bucket.m_casKeyTable);
			cacheEntries.PopulateInputsThatAreOutputs(newEntry.extraInputCasKeyOffsets, m_storage, bucket.m_casKeyTable);
		}

		// If cache server has all content we can put the new cache entry directly in the lookup.. otherwise we'll have to wait until client has uploaded content
		if (hasAllContent)
		{
			cacheEntries.entries.emplace_front(std::move(newEntry));
			++m_totalEntryCount;
		}
		else
		{
			SCOPED_FUTEX(connectionBucket.deferredCacheEntryLookupLock, lock3);
			auto& deferredEntry = connectionBucket.deferredCacheEntryLookup[cmdKey];
			if (deferredEntry.id != 0)
				return m_logger.Warning(TC("Client %u %s adding deferred entry that already exists"), clientId);
			deferredEntry = std::move(newEntry);
		}

		//m_logger.Info(TC("Added new cache entry (%u inputs and %u outputs)"), u32(inputs.size()), outputCount);
		bucket.needsSave = true;

		++m_addsSinceMaintenance;

		return true;
	}

	bool CacheServer::HandleStoreEntryDone(const ConnectionInfo& connectionInfo, BinaryReader& reader)
	{
		auto& connectionBucket = GetConnectionBucket(connectionInfo, reader);
		CasKey cmdKey = reader.ReadCasKey();

		bool success = true;
		if (reader.GetLeft())
			success = reader.ReadBool();

		SCOPED_FUTEX(connectionBucket.deferredCacheEntryLookupLock, lock2);
		auto findIt = connectionBucket.deferredCacheEntryLookup.find(cmdKey);
		if (findIt == connectionBucket.deferredCacheEntryLookup.end())
			return true;
		CacheEntry entry(std::move(findIt->second));
		connectionBucket.deferredCacheEntryLookup.erase(findIt);
		lock2.Leave();
		if (!success)
			return true;

		u64 id = connectionBucket.id;
		Bucket& bucket = GetBucket(id, TC("StoreEntryDone"));

		SCOPED_WRITE_LOCK(bucket.m_cacheEntryLookupLock, lock3);
		auto insres = bucket.m_cacheEntryLookup.try_emplace(cmdKey);
		auto& cacheEntries = insres.first->second;
		lock3.Leave();
			
		SCOPED_WRITE_LOCK(cacheEntries.lock, lock4);

		// Check again because another connection might have added the same entry while cas files were transferred
		for (auto i=cacheEntries.entries.begin(), e=cacheEntries.entries.end(); i!=e; ++i)
			if (i->sharedInputCasKeyOffsetRanges == entry.sharedInputCasKeyOffsetRanges && i->extraInputCasKeyOffsets == entry.extraInputCasKeyOffsets)
				return true;

		cacheEntries.entries.emplace_front(std::move(entry));
		++m_totalEntryCount;
		return true;
	}

	bool CacheServer::HandleFetchPathTable(BinaryReader& reader, BinaryWriter& writer)
	{
		Bucket& bucket = GetBucket(reader, TC("FetchPathTable"));
		u32 haveSize = reader.ReadU32();
		u32 size = bucket.m_pathTable.GetSize();
		writer.WriteU32(size);
		u32 toSend = Min(u32(writer.GetCapacityLeft()), size - haveSize);
		writer.WriteBytes(bucket.m_pathTable.GetMemory() + haveSize, toSend);
		return true;
	}

	bool CacheServer::HandleFetchPathTable2(BinaryReader& reader, BinaryWriter& writer)
	{
		Bucket& bucket = GetBucket(reader, TC("FetchPathTable2"));
		u32 haveSize = reader.ReadU32();
		u32 size = bucket.m_pathTable.GetSize();
		u32 toSend = Min(u32(writer.GetCapacityLeft()), size - haveSize);
		writer.WriteBytes(bucket.m_pathTable.GetMemory() + haveSize, toSend);
		return true;
	}

	bool CacheServer::HandleFetchCasTable(BinaryReader& reader, BinaryWriter& writer)
	{
		Bucket& bucket = GetBucket(reader, TC("FetchCasTable"));
		u32 haveSize = reader.ReadU32();
		u32 size = bucket.m_casKeyTable.GetSize();
		writer.WriteU32(size);
		u32 toSend = Min(u32(writer.GetCapacityLeft()), size - haveSize);
		writer.WriteBytes(bucket.m_casKeyTable.GetMemory() + haveSize, toSend);
		return true;
	}

	bool CacheServer::HandleFetchCasTable2(BinaryReader& reader, BinaryWriter& writer)
	{
		Bucket& bucket = GetBucket(reader, TC("FetchCasTable2"));
		u32 haveSize = reader.ReadU32();
		u32 size = bucket.m_casKeyTable.GetSize();
		u32 toSend = Min(u32(writer.GetCapacityLeft()), size - haveSize);
		writer.WriteBytes(bucket.m_casKeyTable.GetMemory() + haveSize, toSend);
		return true;
	}

	bool CacheServer::HandleFetchEntries(BinaryReader& reader, BinaryWriter& writer, u32 clientId)
	{
		Bucket& bucket = GetBucket(reader, TC("FetchEntries"));
		CasKey cmdKey = reader.ReadCasKey();

		u32 clientVersion;
		{
			SCOPED_FUTEX(m_connectionsLock, lock);
			auto& conn = m_connections[clientId];
			conn.fetchBuckets.insert(bucket.index);
			++conn.fetchEntryCount;
			clientVersion = conn.clientVersion;
		}

		++m_cacheKeyFetchCount;

		SCOPED_READ_LOCK(bucket.m_cacheEntryLookupLock, lock);
		auto findIt = bucket.m_cacheEntryLookup.find(cmdKey);
		if (findIt == bucket.m_cacheEntryLookup.end())
		{
			writer.WriteU16(0);
			return true;
		}
		auto& cacheEntries = findIt->second;
		lock.Leave();

		SCOPED_READ_LOCK(cacheEntries.lock, lock2);
		return cacheEntries.Write(writer, clientVersion, false);
	}

	bool CacheServer::HandleReportUsedEntry(BinaryReader& reader, BinaryWriter& writer, u32 clientId)
	{
		Bucket& bucket = GetBucket(reader, TC("ReportUsedEntry"));
		CasKey cmdKey = reader.ReadCasKey();
		u64 entryId = reader.Read7BitEncoded();

		u32 clientVersion;
		{
			SCOPED_FUTEX(m_connectionsLock, lock);
			auto& conn = m_connections[clientId];
			++conn.fetchEntryHitCount;
			clientVersion = conn.clientVersion;
		}

		++m_cacheKeyHitCount;

		SCOPED_READ_LOCK(bucket.m_cacheEntryLookupLock, lock);
		auto findIt = bucket.m_cacheEntryLookup.find(cmdKey);
		if (findIt == bucket.m_cacheEntryLookup.end())
			return true;
		auto& cacheEntries = findIt->second;
		lock.Leave();

		Vector<CasKey> casKeysUsed;
		SCOPED_WRITE_LOCK(cacheEntries.lock, lock2);
		for (auto& entry : cacheEntries.entries)
		{
			if (entryId != entry.id)
				continue;
			u64 fileTime = GetSystemTimeAsFileTime() - m_creationTime;
			entry.lastUsedTime = fileTime;
			bucket.lastUsedTime = fileTime;

			auto& outputs = entry.outputCasKeyOffsets;
			BinaryReader outputsReader(outputs);
			while (outputsReader.GetLeft())
			{
				u64 offset = outputsReader.Read7BitEncoded();
				CasKey casKey;
				bucket.m_casKeyTable.GetKey(casKey, offset);
				if (IsExecutable(casKey))
					casKey = AsExecutable(casKey, false);
				casKeysUsed.push_back(casKey);
			}

			if (clientVersion >= 5 && entry.logLinesType == LogLinesType_Owned)
				if (entry.logLines.size() <= writer.GetCapacityLeft())
					writer.WriteBytes(entry.logLines.data(), entry.logLines.size());
			break;
		}
		lock2.Leave();

		// Set caskeys to accessed even though they might not have been downloaded
		for (CasKey& casKey : casKeysUsed)
			m_storage.CasEntryAccessed(casKey);

		return true;
	}

	bool CacheServer::HandleExecuteCommand(BinaryReader& reader, BinaryWriter& writer)
	{
		StringBuffer<> command;
		reader.ReadString(command);

		StringBuffer<> additionalInfo;
		reader.ReadString(additionalInfo);

		StringBuffer<> tempFile(m_storage.GetTempPath());
		Guid guid;
		CreateGuid(guid);
		tempFile.Append(GuidToString(guid).str);

		FileAccessor file(m_logger, tempFile.data);
		if (!file.CreateWrite())
			return false;

		bool writeSuccess = true;
		auto Write = [&](const void* data, u64 size) { writeSuccess &= file.Write(data, size); };

		u8 bom[] = {0xEF,0xBB,0xBF}; 
		Write(bom, sizeof(bom));

		auto writeLine = [&](const StringView& text)
			{
				u8 buffer[1024];
				BinaryWriter w(buffer, 0, sizeof(buffer));
				w.WriteUtf8String(text.data, text.count);
				w.WriteUtf8String(TC("\n"), 1);
				Write(buffer, w.GetPosition());
			};

		StringBuffer<> line;

		auto writePathFromOffset = [&](Bucket& bucket, u32 offset, u32 index)
			{
				CasKey casKey;
				StringBuffer<> path;
				bucket.m_casKeyTable.GetPathAndKey(path, casKey, bucket.m_pathTable, offset);
				writeLine(line.Clear().Appendf(TC("    %5u %s - %s (%u)"), index, path.data, CasKeyString(casKey).str, offset));
			};

		auto writePathsFromOffsets = [&](Bucket& bucket, const Vector<u8>& offsets)
			{
				u32 index = 0;
				BinaryReader reader2(offsets);
				while (reader2.GetLeft())
					writePathFromOffset(bucket, u32(reader2.Read7BitEncoded()), index++);
			};


		auto writeEntry = [&](Bucket& bucket, CacheEntries& entries)
			{
				#if 0
				HashMap2<u32, u32> oldToNewCasKeyOffset;
				MemoryBlock memory(64*1024);
				oldToNewCasKeyOffset.Init(memory, 1);
				Vector<u32> temp;
				Vector<u8> temp2;
				Vector<u8> temp3;
				entries.UpdateEntries(m_logger, oldToNewCasKeyOffset, temp, temp2, temp3);
				#endif

				SCOPED_READ_LOCK(entries.lock, lock3);
				writeLine(line.Clear().Append(TCV("   SharedInputs:")));
				writePathsFromOffsets(bucket, entries.sharedInputCasKeyOffsets);
				u32 index = 0;
				for (auto& entry : entries.entries)
				{
					writeLine(line.Clear().Appendf(TC("  #%u (%s)"), index, TimeToText(GetFileTimeAsTime(GetSystemTimeAsFileTime() - (m_creationTime + entry.creationTime)), true).str));
					writeLine(line.Clear().Append(TCV("   InputRanges:")));
					BinaryReader sharedReader(entries.sharedInputCasKeyOffsets);
					BinaryReader rangeReader(entry.sharedInputCasKeyOffsetRanges);
					u32 sharedIndex = 0;
					while (rangeReader.GetLeft())
					{
						u64 begin = rangeReader.Read7BitEncoded();
						u64 end = rangeReader.Read7BitEncoded();

						while (begin != sharedReader.GetPosition())
						{
							sharedReader.Read7BitEncoded();
							++sharedIndex;
						}
						u32 startIndex = sharedIndex;
						while (end != sharedReader.GetPosition())
						{
							sharedReader.Read7BitEncoded();
							++sharedIndex;
						}

						writeLine(line.Clear().Appendf(TC("          %llu - %llu   (%llu - %llu)"), startIndex, sharedIndex-1, begin, end));
					}

					writeLine(line.Clear().Append(TCV("   ExtraInputs:")));
					writePathsFromOffsets(bucket, entry.extraInputCasKeyOffsets);
					writeLine(line.Clear().Append(TCV("   Outputs:")));
					writePathsFromOffsets(bucket, entry.outputCasKeyOffsets);
					++index;
				}
			};

		if (command.Equals(TCV("content")))
		{
			writeLine(TCV("UbaCache server summary"));

			StringBufferBase& filterString = additionalInfo;

			u64 now = GetSystemTimeAsFileTime();

			Vector<u8> temp;

			SCOPED_FUTEX(m_bucketsLock, bucketsLock);
			for (auto& kv : m_buckets)
			{
				Bucket& bucket = kv.second;
				SCOPED_READ_LOCK(bucket.m_cacheEntryLookupLock, lock2);

				for (auto& kv2 : bucket.m_cacheEntryLookup)
				{
					CacheEntries& entries = kv2.second;
					SCOPED_READ_LOCK(entries.lock, lock3);

					Set<u32> visibleIndices;
					if (filterString.count)
					{
						u32 index = 0;
						auto findString = [&](const Vector<u8>& offsets)
							{
								BinaryReader reader2(offsets);
								while (reader2.GetLeft())
								{
									u64 offset = reader2.Read7BitEncoded();
									CasKey casKey;
									StringBuffer<> path;
									bucket.m_casKeyTable.GetPathAndKey(path, casKey, bucket.m_pathTable, offset);
									if (path.Contains(filterString.data))
										return true;
									if (Contains(CasKeyString(casKey).str, filterString.data))
										return true;
								}
								return false;
							};

						for (auto& entry : entries.entries)
						{
							entries.Flatten(temp, entry);
							if (findString(temp) || findString(entry.outputCasKeyOffsets))
								visibleIndices.insert(index);
							++index;
						}
						if (visibleIndices.empty())
							continue;
					}

					writeLine(line.Clear().Appendf(TC("Key: %s"), CasKeyString(kv2.first).str));
					if (!entries.inputsThatAreOutputs.empty())
					{
						writeLine(line.Clear().Appendf(TC("  InputsThatAreOutputs:")));
						if (*entries.inputsThatAreOutputs.begin() == ~0u)
							writeLine(line.Clear().Appendf(TC("   WillCheckAll (~0u)")));
						else
						{
							u32 index = 0;
							for (u32 offset : entries.inputsThatAreOutputs)
								writePathFromOffset(bucket, offset, index++);
						}
					}

					u32 index = 0;
					for (auto& entry : entries.entries)
					{
						if (!visibleIndices.empty() && visibleIndices.find(index) == visibleIndices.end())
						{
							++index;
							continue;
						}

						u64 age = GetFileTimeAsTime(now - (m_creationTime + entry.creationTime));
						writeLine(line.Clear().Appendf(TC("  #%u (%s ago)"), index, TimeToText(age, true).str));

						writeLine(line.Clear().Append(TCV("   Inputs:")));
						entries.Flatten(temp, entry);
						writePathsFromOffsets(bucket, temp);
						writeLine(line.Clear().Append(TCV("   Outputs:")));
						writePathsFromOffsets(bucket, entry.outputCasKeyOffsets);
						++index;
					}
				}
			}
		}
		else if (command.Equals(TCV("status")))
		{
			writeLine(TCV("UbaCacheServer status"));
			writeLine(line.Clear().Appendf(TC("  CreationTime: %s ago"), TimeToText(GetFileTimeAsTime(GetSystemTimeAsFileTime() - m_creationTime), true).str));
			writeLine(line.Clear().Appendf(TC("  UpTime: %s"), TimeToText(GetTime() - m_bootTime, true).str));
			writeLine(line.Clear().Appendf(TC("  Longest maintenance: %s"), TimeToText(m_longestMaintenance).str));
			writeLine(line.Clear().Appendf(TC("  Buckets:")));
			u32 index = 0;

			{
				SCOPED_FUTEX(m_bucketsLock, bucketsLock);
				for (auto& kv : m_buckets)
				{
					Bucket& bucket = kv.second;
					SCOPED_READ_LOCK(bucket.m_cacheEntryLookupLock, lock2);
					u64 mostEntries = 0;
					CasKey mostEntriesKey;
					u64 lastUsed = 0;
					u64 totalEntryCount = 0;
					u64 expiredEntryCount = 0;
					u64 totalKeySharedBytes = 0;
					u64 totalCountHasOutputAsInput = 0;
					u64 totalEntryBytes = 0;

					u64 lastUseTimeLimit = 0;
					if (m_expirationTimeSeconds)
					{
						u64 secondsSinceCreation = GetFileTimeAsSeconds(GetSystemTimeAsFileTime() - m_creationTime);
						if (secondsSinceCreation > m_expirationTimeSeconds)
							lastUseTimeLimit = GetSecondsAsFileTime(secondsSinceCreation - m_expirationTimeSeconds);
					}

					for (auto& kv2 : bucket.m_cacheEntryLookup)
					{
						CacheEntries& entries = kv2.second;
						SCOPED_READ_LOCK(entries.lock, lock3);
						totalKeySharedBytes += entries.GetSharedSize();
						totalEntryBytes += entries.GetTotalSize(CacheNetworkVersion, true);
						if (mostEntries < entries.entries.size())
						{
							mostEntries = u64(entries.entries.size());
							mostEntriesKey = kv2.first;
						}
						for (auto& entry : entries.entries)
						{
							lastUsed = Max(lastUsed, entry.lastUsedTime);

							if (entry.creationTime < lastUseTimeLimit && entry.lastUsedTime < lastUseTimeLimit)
								++expiredEntryCount;
						}
						totalEntryCount += entries.entries.size();
						if (!entries.inputsThatAreOutputs.empty())
							++totalCountHasOutputAsInput;
					}
					lock2.Leave();
					u64 lastUsedTime = 0;
					if (lastUsed)
						lastUsedTime = GetFileTimeAsTime(GetSystemTimeAsFileTime() - (m_creationTime + lastUsed));

					writeLine(line.Clear().Appendf(TC("    #%u - %llu (v%u)"), index++, kv.first, bucket.m_pathTable.GetVersion()));
					writeLine(line.Clear().Appendf(TC("      PathTable: %llu (%s)"), bucket.m_pathTable.GetPathCount(), BytesToText(bucket.m_pathTable.GetSize()).str));
					writeLine(line.Clear().Appendf(TC("      CasKeyTable: %llu (%s)"), bucket.m_casKeyTable.GetKeyCount(), BytesToText(bucket.m_casKeyTable.GetSize()).str));
					writeLine(line.Clear().Appendf(TC("      Keys: %llu (%s)"), bucket.m_cacheEntryLookup.size(), BytesToText(totalEntryBytes).str));
					writeLine(line.Clear().Appendf(TC("      KeysWithInputsFromOutputs: %llu"), totalCountHasOutputAsInput));
					writeLine(line.Clear().Appendf(TC("      KeyMostEntries: %llu (%s)"), mostEntries, CasKeyString(mostEntriesKey).str));
					writeLine(line.Clear().Appendf(TC("      TotalEntries: %llu"), totalEntryCount));
					writeLine(line.Clear().Appendf(TC("      TotalKeySharedEntry: %s"), BytesToText(totalKeySharedBytes).str));
					writeLine(line.Clear().Appendf(TC("      LastEntryUsed: %s ago"), TimeToText(lastUsedTime, true).str));
					writeLine(line.Clear().Appendf(TC("      ExpiredEntries: %llu"), expiredEntryCount));
				}
			}
			u64 totalCasSize = 0;
			u64 totalCasCount = 0;
			m_storage.TraverseAllCasFiles([&](const CasKey& casKey, u64 size) { ++totalCasCount; totalCasSize += size; });
			writeLine(line.Clear().Appendf(TC("  CasDb:")));
			writeLine(line.Clear().Appendf(TC("    Count: %llu"), totalCasCount));
			writeLine(line.Clear().Appendf(TC("    Size: %s"), BytesToText(totalCasSize).str));
		}
		else if (command.Equals(TCV("validate")))
		{
			writeLine(line.Clear().Appendf(TC("  Buckets:")));
			{
				u32 index = 0;
				SCOPED_FUTEX(m_bucketsLock, bucketsLock);
				for (auto& kv : m_buckets)
				{
					bool printInvalid = true;
					bool printLargestDiff = false;
					u32 largestDiff = 0;
					CasKey largestDiffKey;
					u64 mostEntries = 0;
					u64 keysWithInvalidSharedInputs = 0;
					u64 invalidExtraInputs = 0;
					u64 totalExtraInputs = 0;
					u64 entriesWithBadExtra = 0;
					u64 entriesWithDuplicatedExtra = 0;
					Bucket& bucket = kv.second;
					SCOPED_READ_LOCK(bucket.m_cacheEntryLookupLock, lock2);
					for (auto& kv2 : bucket.m_cacheEntryLookup)
					{
						CacheEntries& entries = kv2.second;
						SCOPED_READ_LOCK(entries.lock, lock3);

						Set<u32> sharedOffsets;
						UnorderedMap<u32, u32> positionToIndex;
						BinaryReader sharedInputsReader(entries.sharedInputCasKeyOffsets);
						bool hasInvalidSharedInputs = false;
						u32 sharedIndex = 0;
						while (sharedInputsReader.GetLeft())
						{
							positionToIndex[u32(sharedInputsReader.GetPosition())] = sharedIndex++;
							u32 offset = u32(sharedInputsReader.Read7BitEncoded());
							sharedOffsets.insert(offset);
							CasKey casKey;
							StringBuffer<> path;
							bucket.m_casKeyTable.GetPathAndKey(path, casKey, bucket.m_pathTable, offset);
							if (path.count <= 2)
								hasInvalidSharedInputs = true;
						}
						positionToIndex[u32(sharedInputsReader.GetPosition())] = sharedIndex++;
						if (hasInvalidSharedInputs)
							++keysWithInvalidSharedInputs;

						u32 minCount = ~0u;
						u32 maxCount = 0;

						if (mostEntries < entries.entries.size())
							mostEntries = entries.entries.size();

						bool hasInvalidExtraInputs = false;
						bool hasDuplicatedExtraInputs = false;
						bool hasEntriesWithBadExtra = false;
						u32 entryIndex = 0;
						for (auto& entry : entries.entries)
						{
							++totalExtraInputs;

							u32 inputsCount = 0;

							BinaryReader rangesReader(entry.sharedInputCasKeyOffsetRanges);
							while (rangesReader.GetLeft())
							{
								u32 begin = u32(rangesReader.Read7BitEncoded());
								u32 end = u32(rangesReader.Read7BitEncoded());
								auto f1 = positionToIndex.find(begin);
								auto f2 = positionToIndex.find(end);
								UBA_ASSERT(f1 != positionToIndex.end());
								UBA_ASSERT(f2 != positionToIndex.end());
								u32 count = f2->second - f1->second;
								UBA_ASSERT(count);
								inputsCount += count;
							}

							Set<u32> extras;
							BinaryReader extraInputsReader(entry.extraInputCasKeyOffsets);
							while (extraInputsReader.GetLeft())
							{
								u32 offset = u32(extraInputsReader.Read7BitEncoded());
								++inputsCount;

								if (!hasDuplicatedExtraInputs && !extras.insert(offset).second)
									hasDuplicatedExtraInputs = true;

								if (!hasEntriesWithBadExtra && sharedOffsets.find(offset) != sharedOffsets.end())
								{
									#if 0
									u64 fileSize = entries.GetTotalSize(CacheNetworkVersion, true);
									FileAccessor fa(m_logger, TC("e:\\temp\\CacheEntry.bin"));
									fa.CreateMemoryWrite(false, DefaultAttributes(), fileSize);
									BinaryWriter fileWriter(fa.GetData(), 0, fileSize);
									entries.Write(fileWriter, CacheNetworkVersion, true);
									fa.Close();

									HashMap2<u32, u32> oldToNewCasKeyOffset;
									MemoryBlock memory(64*1024);
									oldToNewCasKeyOffset.Init(memory, 1);
									Vector<u32> temp;
									Vector<u8> temp2;
									Vector<u8> temp3;
									entries.UpdateEntries(m_logger, oldToNewCasKeyOffset, temp, temp2, temp3);
									#endif
									
									hasEntriesWithBadExtra = true;
								}
								
								CasKey casKey;
								StringBuffer<> path;
								bucket.m_casKeyTable.GetPathAndKey(path, casKey, bucket.m_pathTable, offset);
								if (path.count <= 2)
									hasInvalidExtraInputs = true;
							}
							if (hasEntriesWithBadExtra)
								++entriesWithBadExtra;
							if (hasInvalidExtraInputs)
								++invalidExtraInputs;
							if (hasDuplicatedExtraInputs)
								++entriesWithDuplicatedExtra;
#if 0
							if (printInvalid && (hasInvalidSharedInputs || hasEntriesWithBadExtra || hasInvalidExtraInputs))
							{
								writeLine(line.Clear().Appendf(TC("EntryIndex: %u InvalidShared: %u BadExtra: %u InvalidExtra: %u"), entryIndex, hasInvalidSharedInputs, hasEntriesWithBadExtra, hasInvalidExtraInputs));
								printInvalid = false;
								writeEntry(bucket, entries);
								break;
							}
#endif
							++entryIndex;

							if (inputsCount < minCount)
								minCount = inputsCount;
							if (inputsCount > maxCount)
								maxCount = inputsCount;
						}
						if (!printInvalid)
							break;

						u32 diff = maxCount - minCount;
						if (diff > largestDiff)
						{
							largestDiff = diff;
							largestDiffKey = kv2.first;
						}
					}
					lock2.Leave();

					writeLine(line.Clear().Appendf(TC("    #%u - %llu (v%u)"), index++, kv.first, bucket.m_pathTable.GetVersion()));
					writeLine(line.Clear().Appendf(TC("      Invalid shared: %llu/%llu"), keysWithInvalidSharedInputs, (bucket.m_cacheEntryLookup.size())));
					writeLine(line.Clear().Appendf(TC("      Invalid extra: %llu/%llu"), invalidExtraInputs, totalExtraInputs));
					writeLine(line.Clear().Appendf(TC("      With bad extra: %llu"), entriesWithBadExtra));
					writeLine(line.Clear().Appendf(TC("      With duplicated extra: %llu"), entriesWithDuplicatedExtra));
					writeLine(line.Clear().Appendf(TC("      LargestDiff: %u"), largestDiff));
					writeLine(line.Clear().Appendf(TC("      MostEntries: %llu"), mostEntries));

					if (largestDiff && printInvalid && printLargestDiff)
					{
						writeEntry(bucket, bucket.m_cacheEntryLookup[largestDiffKey]);
					}
				}
			}
		}
		else if (command.Equals(TCV("updateentries")))
		{
			HashMap2<u32, u32> oldToNewCasKeyOffset;
			MemoryBlock memory(TC("UpdateEntries"), 64*1024);
			oldToNewCasKeyOffset.Init(memory, 1);
			Vector<u32> temp;
			Vector<u8> temp2;
			Vector<u8> temp3;

			SCOPED_FUTEX(m_bucketsLock, bucketsLock);
			for (auto& kv : m_buckets)
			{
				Bucket& bucket = kv.second;
				for (auto& kv2 : bucket.m_cacheEntryLookup)
				{
					CacheEntries& entries = kv2.second;
					SCOPED_READ_LOCK(entries.lock, lock3);
					entries.UpdateEntries(m_logger, oldToNewCasKeyOffset, temp, temp2, temp3);
				}
				bucket.needsSave = true;
			}
		}
		else if (command.Equals(TCV("pathtable")))
		{
			SCOPED_FUTEX(m_bucketsLock, bucketsLock);
			for (auto& kv : m_buckets)
			{
				Bucket& bucket = kv.second;
				SCOPED_READ_LOCK(bucket.m_cacheEntryLookupLock, lock2);

				writeLine(line.Clear().Appendf(TC("Bucket #%u"), bucket.index));

				Set<TString> sortedPaths;

				bucket.m_pathTable.TraversePaths([&](const StringView& path)
					{
						if (!sortedPaths.emplace(path.data).second)
							writeLine(TCV("EEEEERRRRRROOOORRRR!!!!!!"));
					});

				for (auto& path : sortedPaths)
					writeLine(line.Clear().Append(TCV("   ")).Append(path));
			}
		}
		else if (command.Equals(TCV("usedpaths")))
		{
			SCOPED_FUTEX(m_bucketsLock, bucketsLock);
			for (auto& kv : m_buckets)
			{
				Bucket& bucket = kv.second;
				SCOPED_READ_LOCK(bucket.m_cacheEntryLookupLock, lock2);

				writeLine(line.Clear().Appendf(TC("Bucket #%u"), bucket.index));

				Set<TString> sortedPaths;

				BinaryReader tableReader(bucket.m_casKeyTable.GetMemory(), 0, bucket.m_casKeyTable.GetSize());
				while (tableReader.GetLeft())
				{
					u32 stringOffset = (u32)tableReader.Read7BitEncoded();
					tableReader.ReadCasKey();
					StringBuffer<> path;
					bucket.m_pathTable.GetString(path, stringOffset);
					sortedPaths.emplace(path.data);
				}

				for (auto& path : sortedPaths)
					writeLine(line.Clear().Append(TCV("   ")).Append(path));
			}
		}
		else if (command.Equals(TCV("usedsegments")))
		{
			SCOPED_FUTEX(m_bucketsLock, bucketsLock);
			for (auto& kv : m_buckets)
			{
				UnorderedMap<u32, u32> segments;

				Bucket& bucket = kv.second;
				SCOPED_READ_LOCK(bucket.m_cacheEntryLookupLock, lock2);

				writeLine(line.Clear().Appendf(TC("Bucket #%u"), bucket.index));

				BinaryReader tableReader(bucket.m_pathTable.GetMemory(), bucket.m_pathTable.GetCommonSize(), bucket.m_pathTable.GetSize());
				while (tableReader.GetLeft())
				{
					tableReader.Read7BitEncoded(); // parent
					u32 strOffset = u32(tableReader.Read7BitEncoded());
					if (strOffset == 0)
					{
						strOffset = u32(tableReader.GetPosition());
						tableReader.SkipString();
					}
					++segments[strOffset];
				}

				Map<u32, Vector<TString>> sortedSegments;

				for (auto& kv2 : segments)
				{
					tableReader.SetPosition(kv2.first);
					StringBuffer<> segment;
					tableReader.ReadString(segment);
					sortedSegments[1000000 - kv2.second].push_back(segment.data);
				}
				for (auto& kv2 : sortedSegments)
				{
					writeLine(line.Clear().Append(TCV("  ")).AppendValue(1000000 - kv2.first));
					for (auto& str : kv2.second)
						writeLine(line.Clear().Append(TCV("    ")).Append(str));
				}
			}
		}
		else if (command.Equals(TCV("largestsharedinput")))
		{
			CacheEntries* largestEntries = nullptr;
			Bucket* largestEntriesBucket = nullptr;
			u64 largestSize = 0;
			SCOPED_FUTEX(m_bucketsLock, bucketsLock);
			for (auto& kv : m_buckets)
			{
				Bucket& bucket = kv.second;
				for (auto& kv2 : bucket.m_cacheEntryLookup)
				{
					CacheEntries& entries = kv2.second;
					SCOPED_READ_LOCK(entries.lock, lock3);
					if (entries.sharedInputCasKeyOffsets.size() <= largestSize)
						continue;
					largestSize = entries.sharedInputCasKeyOffsets.size();
					largestEntries = &entries;
					largestEntriesBucket = &bucket;
				}
			}
			writeLine(line.Clear().Append(TCV("Largest entry (based on shared inputs)")));
			if (largestEntries)
				writeEntry(*largestEntriesBucket, *largestEntries);
		}
		else if (command.Equals(TCV("largestentry")))
		{
			CacheEntries* largestEntries = nullptr;
			Bucket* largestEntriesBucket = nullptr;
			u64 largestSize = 0;
			SCOPED_FUTEX(m_bucketsLock, bucketsLock);
			for (auto& kv : m_buckets)
			{
				Bucket& bucket = kv.second;
				for (auto& kv2 : bucket.m_cacheEntryLookup)
				{
					CacheEntries& entries = kv2.second;
					u64 totalSize = entries.GetTotalSize(CacheNetworkVersion, true);
					SCOPED_READ_LOCK(entries.lock, lock3);
					if (totalSize <= largestSize)
						continue;
					largestSize = totalSize;
					largestEntries = &entries;
					largestEntriesBucket = &bucket;
				}
			}
			writeLine(line.Clear().Appendf(TC("Largest entry (based on total size of %s)"), BytesToText(largestSize).str));
			if (largestEntries)
				writeEntry(*largestEntriesBucket, *largestEntries);
		}
		else if (command.Equals(TCV("obliterate")))
		{
			m_shouldWipe = true;
			m_addsSinceMaintenance = 1;
			writeLine(line.Clear().Appendf(TC("Cache server database obliteration queued!")));
		}
		else if (command.Equals(TCV("maintenance")))
		{
			m_forceAllSteps = true;
			m_addsSinceMaintenance = 1;
			writeLine(line.Clear().Appendf(TC("Cache server maintenance queued!")));
		}
		else if (command.Equals(TCV("save")))
		{
			Save();
		}
		else
		{
			writeLine(line.Clear().Appendf(TC("Unknown command: %s"), command.data));
		}

		Write("", 1);

		if (!writeSuccess || !file.Close())
			return false;

		CasKey key;
		bool deferCreation = false;
		if (!m_storage.StoreCasFile(key, tempFile.data, CasKeyZero, deferCreation))
			return false;

		writer.WriteCasKey(key);

		DeleteFileW(tempFile.data);
		return true;
	}
}
