// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaCompactTables.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaLogger.h"

namespace uba
{
	static constexpr u32 CasKeyLookupInitialCount = 1024;
	static constexpr u32 CasKeyArrayMaxSize = 16;


	CompactPathTable::CompactPathTable(bool caseInsensitive, u64 reservePathCount, u64 reserveSegmentCount, u32 version)
	:	m_pathTableMem(TC("CompactPathTable"))
	,	m_version(version)
	,	m_caseInsensitive(caseInsensitive)
	{
		m_offsets.Init(reservePathCount);
		m_segmentOffsets.Init(reserveSegmentCount);
	}

	void CompactPathTable::InitMem()
	{
		if (!m_pathTableMem.memory)
			m_pathTableMem.Init(PathTableMaxSize);
		if (!m_pathTableMem.writtenSize)
			m_pathTableMem.AllocateNoLock(1, 1, TC("Init"));
	}

	u32 CompactPathTable::Add(const tchar* str, u64 strLen)
	{
		SCOPED_FUTEX(m_lock, lock);
		u32 res = AddNoLock(str, strLen);
		return res;
	}

	u32 CompactPathTable::Add(const tchar* str, u64 strLen, u32& outRequiredPathTableSize)
	{
		SCOPED_FUTEX(m_lock, lock);
		u32 res = AddNoLock(str, strLen);
		outRequiredPathTableSize = u32(m_pathTableMem.writtenSize);
		return res;
	}

	u32 CompactPathTable::AddNoLock(const tchar* str, u64 strLen)
	{
		InitMem();

		if (m_pathTableMem.writtenSize >= PathTableMaxSize - (strLen + 5))
			return ~0u;

		const tchar* stringKeyString = str;
		StringBuffer<MaxPath> tempStringKeyStr;
		if (m_caseInsensitive)
			stringKeyString = tempStringKeyStr.Append(str).MakeLower().data;

		return InternalAdd(str, stringKeyString, strLen);
	}

	u32 CompactPathTable::AddNoLock(AddContext& context, u32 offset)
	{
		auto& table = context.fromTable;

		if (true)//m_version < 3)
		{
			StringBuffer<> temp;
			table.GetString(temp, offset);

			const tchar* stringKeyString = temp.data;
			StringBuffer<MaxPath> tempStringKeyStr;
			if (m_caseInsensitive)
				stringKeyString = tempStringKeyStr.Append(temp).MakeLower().data;
			return InternalAdd(temp.data, stringKeyString, temp.count);
		}
#if 0
		u32 offsets[MaxSegments];
		bool separators[MaxSegments];

		u32 parentOffset = 0;
		u32 pathOffset = 0;

		BinaryReader reader(table.m_pathTableMem.memory, 0, table.m_pathTableMem.writtenSize);
		u32 offsetCount = 0;

		u32 isChild = 0;

		u32* fromOffsets = context.fromOffsets;
		u32* toOffsets = context.toOffsets;
		u32* pathOffsets = context.pathOffsets;
		u32 searchIndex = context.offsetsCount - 1;

		// Traverse backwards in old tree. Early out if context already have entry
		while (true)
		{
			UBA_ASSERT(offsetCount < sizeof_array(offsets));
			reader.SetPosition(offset);
			u64 parentRelativeOffset = reader.Read7BitEncoded();
			u32 readPos = u32(reader.GetPosition());
			offsets[offsetCount++] = readPos;
			separators[offsetCount] = parentRelativeOffset & 1;
			parentRelativeOffset >>= 1;
			if (parentRelativeOffset == offset || offsetCount == (MaxSegments-1))
				break;
			offset -= u32(parentRelativeOffset);

			while (readPos < fromOffsets[searchIndex])
				--searchIndex;
			if (readPos == fromOffsets[searchIndex])
			{
				parentOffset = toOffsets[searchIndex];
				pathOffset = pathOffsets[searchIndex];
				isChild = 1;
				--offsetCount;
				break;
			}
		}

		if (offsetCount == (MaxSegments-1))
			return false;

		if (!isChild)
			searchIndex = 0;

		u32 pathPositions[MaxSegments];
		pathPositions[offsetCount] = pathOffset;

		auto& path = context.path;
		path.Resize(pathOffset);
		
		u32 oldSegOffsets[MaxSegments];

		// Create full string and store path positions
		for (u32 i=offsetCount;i; --i)
		{
			reader.SetPosition(offsets[i - 1]);
			u32 oldSegOffset = u32(reader.Read7BitEncoded());
			oldSegOffsets[i] = oldSegOffset;
			if (oldSegOffset != 0)
				reader.SetPosition(oldSegOffset);

			if (pathOffset)
				path.Append(separators[i] ? '.' : PathSeparator);

			reader.ReadString(path);

			pathOffset = 1;

			pathPositions[i - 1] = path.count;
		}

		// Go backwards to figure out how many new entries to add
		u32 newOffsetIndices[MaxSegments];
		u32 newOffsetCount = 0;
		for (u32 i=offsetCount;i; --i)
		{
			StringKey key = ToStringKeyNoCheck(path.data, pathPositions[newOffsetCount]);
			bool added = false;
			newOffsetIndices[newOffsetCount] = m_offsets.InsertIndex(key, added); // m_offsets might re-allocate so we need to track index instead
			if (!added)
			{
				parentOffset = m_offsets.GetValueFromIndex(newOffsetIndices[newOffsetCount]);
				isChild = 1;
				break;
			}
			++newOffsetCount;
		}

		// Go forward and add new entries
		for (u32 i=newOffsetCount;i; --i)
		{
			u32 iMinusOne = i - 1;

			offset = u32(m_pathTableMem.writtenSize);
			m_offsets.GetValueFromIndex(newOffsetIndices[iMinusOne]) = offset;

			bool separator = separators[i];

			if (oldSegOffsets[i] && oldSegOffsets[i] < m_commonSize)
			{
				InternalAddWithExistingSegment(offset, parentOffset, separator, oldSegOffsets[i]);
			}
			else
			{
				StringView segment(path.data + pathPositions[i] + isChild, pathPositions[iMinusOne] - pathPositions[i] - isChild);
				StringKey segmentKey = ToStringKeyNoCheck(segment.data, segment.count);
				bool added = false;
				u32& strOffset = m_segmentOffsets.Insert(segmentKey, added);

				if (added)
					strOffset = InternalAddWithNewSegment(offset, parentOffset, separator, segment);
				else
					InternalAddWithExistingSegment(offset, parentOffset, separator, strOffset);
			}

			searchIndex++;
			fromOffsets[searchIndex] = offsets[iMinusOne];
			toOffsets[searchIndex] = offset;
			pathOffsets[searchIndex] = pathPositions[iMinusOne];

			isChild = 1;
			parentOffset = offset;
		}

		context.offsetsCount = searchIndex + 1;

		return offset;
#endif
	}

	void CompactPathTable::AddCommonStringSegments()
	{
		const tchar* commonSegments1[] =
		{
			TC("h"),
			TC("cpp"),
			TC("inl"),
			TC("obj"),
			TC("o"),
			TC("c"),
			TC("lib"),
			TC("rsp"),
			TC("dep"),
			TC("json"),
			TC("sarif"),
			TC("d"),
			TC("gen"),
			TC("generated"),
			TC("init"),
			TC("ispc"),
			// If adding more we want these
		};
		// 68 bytes used

		const tchar* commonSegments2[] =
		{
			TC("H"),
			TC("0"),
			TC("1"),
			TC("2"),
			TC("3"),
			TC("Definitions"),
			TC("Private"),
			TC("Shared"),
			TC("Public"),
			TC("Inc"),
			TC("UHT"),
			TC("x64"),
			TC("res"),
		};
		// 124 bytes used

		InitMem();

		auto addSegments = [this](const tchar** segments, u64 segmentCount)
			{
				for (const tchar** it=segments, **itEnd=it+segmentCount; it!=itEnd; ++it)
				{
					const tchar* seg = *it;
					u32 segLen = TStrlen(seg);
					u64 writtenSize = m_pathTableMem.writtenSize;
					BinaryWriter writer((u8*)m_pathTableMem.AllocateNoLock(segLen+1, 1, TC("AddSegments")), 0, segLen+1);
					StringKey segmentKey = ToStringKeyNoCheck(seg, segLen);
					u32& strOffset = m_segmentOffsets.Insert(segmentKey);
					strOffset = u32(writtenSize);
					writer.WriteString(seg, segLen);
				}
			};

		addSegments(commonSegments1, sizeof_array(commonSegments1));
		if (m_version >= 4)
			addSegments(commonSegments2, sizeof_array(commonSegments2));
		m_commonSize = u32(m_pathTableMem.writtenSize);
	}

	u32 CompactPathTable::InternalAdd(const tchar* str, const tchar* stringKeyString, u64 strLen)
	{
		StringKey key = ToStringKeyNoCheck(stringKeyString, strLen);
		bool added = false;
		u32 offsetIndex = m_offsets.InsertIndex(key, added); // m_offsets might re-allocate so we need to track index instead
		if (!added)
			return m_offsets.GetValueFromIndex(offsetIndex);
			
		const tchar* seg = str;
		u32 parentOffset = 0;
		
		if (m_version >= 3)
		{
			tchar parentSeparator = 0;
			for (const tchar* it = str + strLen - 1; it > str; --it)
			{
				if (*it != PathSeparator && *it != '.')
					continue;
				parentOffset = InternalAdd(str, stringKeyString, it - str);
				parentSeparator = *it;
				seg = it + 1;
				break;
			}

			u64 segLen = strLen - (seg - str);
			u32& offset = m_offsets.GetValueFromIndex(offsetIndex);

			StringKey segmentKey = ToStringKeyNoCheck(seg, segLen);
			added = false;
			u32& strOffset = m_segmentOffsets.Insert(segmentKey, added);

			offset = u32(m_pathTableMem.writtenSize);

			if (added)
				strOffset = InternalAddWithNewSegment(offset, parentOffset, parentSeparator == '.', StringView(seg, u32(segLen)));
			else
				InternalAddWithExistingSegment(offset, parentOffset, parentSeparator == '.', strOffset);

			return offset;
		}

		if (m_version >= 2)
		{
			for (const tchar* it = str + strLen - 1; it > str; --it)
			{
				if (*it != PathSeparator && *it != '.')
					continue;

				parentOffset = InternalAdd(str, stringKeyString, it - str);

				parentOffset = (parentOffset << 1);
				if (*it == '.')
					++parentOffset;


				seg = it + 1;
				break;
			}
		}
		else
		{
			for (const tchar* it = str + strLen - 1; it > str; --it)
			{
				if (*it != PathSeparator)
					continue;
				parentOffset = InternalAdd(str, stringKeyString, it - str);
				seg = it + 1;
				break;
			}
		}

		u64 segLen = strLen - (seg - str);
		u8 bytesForParent = Get7BitEncodedCount(parentOffset);

		u32& offset = m_offsets.GetValueFromIndex(offsetIndex);

		StringKey segmentKey = ToStringKeyNoCheck(seg, segLen);
		added = false;
		u32& strOffset = m_segmentOffsets.Insert(segmentKey, added);
		if (added)
		{
			// Put string directly after current element and set segment offset to 0
			u64 bytesForString = GetStringWriteSize(seg, segLen);
			u64 memSize = bytesForParent + 1 + bytesForString;
			u8* mem = (u8*)m_pathTableMem.AllocateNoLock(memSize, 1, TC("Add1"));
			BinaryWriter writer(mem, 0, memSize);
			writer.Write7BitEncoded(parentOffset);
			writer.Write7BitEncoded(0);
			writer.WriteString(seg, segLen);
			offset = u32(mem - m_pathTableMem.memory);
			strOffset = offset + bytesForParent + 1;
			return offset;
		}

		#if 0
		StringBuffer<> temp;
		BinaryReader reader(m_pathTableMem.memory, insres2.first->second, 1000);
		reader.ReadString(temp);
		UBA_ASSERT(temp.count == segLen && wcsncmp(temp.data, seg, segLen) == 0);
		#endif

		u64 memSize = bytesForParent + Get7BitEncodedCount(strOffset);
		u8* mem = (u8*)m_pathTableMem.AllocateNoLock(memSize, 1, TC("Add2"));
		BinaryWriter writer(mem, 0, memSize);
		writer.Write7BitEncoded(parentOffset);
		writer.Write7BitEncoded(strOffset);
		offset = u32(mem - m_pathTableMem.memory);
		return offset;
	}

	u32 CompactPathTable::InternalAddWithNewSegment(u32 offset, u32 parentOffset, bool separator, const StringView& segment)
	{
		u32 parentRelativeOffset = ((offset - parentOffset) << 1u) + u32(separator);

		// Put string directly after current element and set segment offset to 0
		u64 bytesForString = GetStringWriteSize(segment.data, segment.count);
		u8 bytesForParent = Get7BitEncodedCount(parentRelativeOffset);
		u64 memSize = bytesForParent + 1 + bytesForString;
	
		BinaryWriter writer((u8*)m_pathTableMem.AllocateNoLock(memSize, 1, TC("Add3")), 0, memSize);
		writer.Write7BitEncoded(parentRelativeOffset);
		writer.Write7BitEncoded(0);
		writer.WriteString(segment);
		return offset + bytesForParent + 1;
	}

	void CompactPathTable::InternalAddWithExistingSegment(u32 offset, u32 parentOffset, bool separator, u32 segmentOffset)
	{
		u32 parentRelativeOffset = ((offset - parentOffset) << 1u) + u32(separator);

		u8 bytesForParent = Get7BitEncodedCount(parentRelativeOffset);
		u64 memSize = bytesForParent + Get7BitEncodedCount(segmentOffset);
		BinaryWriter writer((u8*)m_pathTableMem.AllocateNoLock(memSize, 1, TC("Add4")), 0, memSize);
		writer.Write7BitEncoded(parentRelativeOffset);
		writer.Write7BitEncoded(segmentOffset);
	}


	bool CompactPathTable::GetString(StringBufferBase& out, u64 offset) const
	{
		#if UBA_DEBUG
		{
			SCOPED_FUTEX_READ(const_cast<CompactPathTable*>(this)->m_lock, lock)
			UBA_ASSERTF(offset < m_pathTableMem.writtenSize, TC("Reading path key from offset %llu which is out of bounds (Max %llu)"), offset, m_pathTableMem.writtenSize);
		}
		#endif

		u32 offsets[MaxSegments];
		bool separators[MaxSegments];
		u32 offsetCount = 0;

		BinaryReader reader(m_pathTableMem.memory, offset, m_pathTableMem.writtenSize);

		if (m_version >= 3)
		{
			offsetCount = 1;
			while (true)
			{
				UBA_ASSERT(offsetCount < sizeof_array(offsets));
				reader.SetPosition(offset);
				u64 parentRelativeOffset = reader.Read7BitEncoded();
				offsets[offsetCount-1] = u32(reader.GetPosition());
				separators[offsetCount] = parentRelativeOffset & 1;
				parentRelativeOffset >>= 1;
				if (parentRelativeOffset == offset || offsetCount == (MaxSegments-1))
					break;
				offset -= parentRelativeOffset;
				++offsetCount;
			}
		}
		else
		{
			while (offset && offsetCount < (MaxSegments-1))
			{
				++offsetCount;
				UBA_ASSERT(offsetCount < sizeof_array(offsets));
				reader.SetPosition(offset);
				offset = (u32)reader.Read7BitEncoded();
				offsets[offsetCount-1] = u32(reader.GetPosition());
				separators[offsetCount] = 0;
				if (m_version >= 2)
				{
					separators[offsetCount] = offset & 1;
					offset >>= 1;
				}
			}
		}
		if (offsetCount == (MaxSegments-1))
			return false;

		{
			bool isFirst = true;
			for (u32 i=offsetCount;i; --i)
			{
				reader.SetPosition(offsets[i-1]);
				u32 strOffset = u32(reader.Read7BitEncoded());
				if (strOffset != 0)
					reader.SetPosition(strOffset);

				if (!isFirst)
					out.Append(separators[i] ? '.' : PathSeparator);
				isFirst = false;
				reader.ReadString(out);
			}
		}
		return true;
	}

	bool CompactPathTable::TryGetString(Logger& logger, StringBufferBase& out, u64 offset) const
	{
		if (m_version < 3)
			return GetString(out, offset);

		u32 offsets[MaxSegments];
		bool separators[MaxSegments];
		u32 offsetCount = 1;

		u64 memorySize = GetSize();
		BinaryReader reader(m_pathTableMem.memory, offset, memorySize);

		u64 parentRelativeOffset;
		while (true)
		{
			if (offsetCount == sizeof_array(offsets))
				return logger.Error(TC("Too many sections in compressed string"));

			reader.SetPosition(offset);
			if (!reader.TryRead7BitEncoded(parentRelativeOffset))
				return logger.Error(TC("Failed to read parentRelativeOffset from PathTableMemory (%ull/%ull)"), offset, memorySize);
			offsets[offsetCount-1] = u32(reader.GetPosition());
			separators[offsetCount] = parentRelativeOffset & 1;
			parentRelativeOffset >>= 1;
			if (parentRelativeOffset >= offset)
				break;
			offset -= parentRelativeOffset;
			++offsetCount;
		}
		if (parentRelativeOffset > offset)
			return logger.Error(TC("Failed to read from PathTableMemory. Parent offset %llu larger than offset %ull"), parentRelativeOffset, offset);

		bool isFirst = true;
		for (u32 i=offsetCount;i; --i)
		{
			reader.SetPosition(offsets[i-1]);
			u64 strOffset;
			if (!reader.TryRead7BitEncoded(strOffset))
				return logger.Error(TC("Failed to read strOffset from PathTableMemory (%llu/%ull)"), reader.GetPosition(), memorySize);
			if (strOffset != 0)
				reader.SetPosition(strOffset);

			if (!isFirst)
				out.Append(separators[i] ? '.' : PathSeparator);
			isFirst = false;
			if (!reader.TryReadString(out))
				return logger.Error(TC("Failed to read string from PathTableMemory (%llu/%ull)"), strOffset, memorySize);
		}
		return true;
	}

	u8* CompactPathTable::GetMemory()
	{
		return m_pathTableMem.memory;
	}

	u32 CompactPathTable::GetSize() const
	{
		SCOPED_FUTEX_READ(m_lock, lock2)
		return u32(m_pathTableMem.writtenSize);
	}

	u32 CompactPathTable::GetCommonSize()
	{
		return m_commonSize;
	}

	bool CompactPathTable::GetCaseInsensitive()
	{
		return m_caseInsensitive;
	}

	u32 CompactPathTable::GetVersion()
	{
		return m_version;
	}

	bool CompactPathTable::ReadMem(BinaryReader& reader, bool populateLookup)
	{
		if (!m_pathTableMem.memory)
			m_pathTableMem.Init(PathTableMaxSize);

		u64 writtenSize = m_pathTableMem.writtenSize;
		u64 left = reader.GetLeft();
		void* mem = m_pathTableMem.AllocateNoLock(left, 1, TC("ReadMem"));
		reader.ReadBytes(mem, left);

		if (!populateLookup)
			return true;

		BinaryReader reader2(m_pathTableMem.memory, writtenSize, m_pathTableMem.writtenSize);
		if (!writtenSize)
			reader2.Skip(1);

		while (reader2.GetLeft())
		{
			u32 offset = u32(reader2.GetPosition());
			reader2.Read7BitEncoded();
			u64 stringOffset = reader2.Read7BitEncoded();
			if (!stringOffset)
			{
				u32 strOffset = u32(reader2.GetPosition());
				StringBuffer<> seg;
				reader2.ReadString(seg);
				m_segmentOffsets.Insert(ToStringKeyNoCheck(seg.data, seg.count)) = strOffset;
			}
			StringBuffer<> str;
			if (!GetString(str, offset))
				return false;
			if (m_caseInsensitive)
				str.MakeLower();
			m_offsets.Insert(ToStringKeyNoCheck(str.data, str.count)) = offset;
		}
		return true;
	}

	u8* CompactPathTable::BeginCommit(u64 size)
	{
		if (!m_pathTableMem.memory)
			m_pathTableMem.Init(PathTableMaxSize);
		return (u8*)m_pathTableMem.CommitNoLock(size, TC("BeginCommit"));
	}

	void CompactPathTable::EndCommit(u8* data, u64 written)
	{
		m_pathTableMem.AllocateNoLock(written, 1, TC("EndCommit"));
	}

	void CompactPathTable::Swap(CompactPathTable& other)
	{
		m_offsets.Swap(other.m_offsets);
		m_segmentOffsets.Swap(other.m_segmentOffsets);
		m_pathTableMem.Swap(other.m_pathTableMem);
		bool ci = m_caseInsensitive;
		m_caseInsensitive = other.m_caseInsensitive;
		other.m_caseInsensitive = ci;
	}

	CompactCasKeyTable::CompactCasKeyTable(u64 reserveOffsetsCount)
	:	m_casKeyTableMem(TC("CompactCasKeyTable"))
	{
		if (reserveOffsetsCount)
			m_offsets.Init(reserveOffsetsCount);
	}

	CompactCasKeyTable::~CompactCasKeyTable()
	{
		if (m_offsets.IsInitialized())
			for (auto i = m_offsets.ValuesBegin(), e = m_offsets.ValuesEnd(); i!=e; ++i)
			{
				if (i->count > CasKeyArrayMaxSize)
					delete i->stringLookup;
				else if (i->count > 1)
					delete[] i->array;
			}
	}

	u32 CompactCasKeyTable::Add(const CasKey& casKey, u64 stringOffset, u32& outRequiredCasTableSize)
	{
		SCOPED_FUTEX(m_lock, lock2)
		if (!m_casKeyTableMem.memory)
			m_casKeyTableMem.Init(CasKeyTableMaxSize);
		if (!m_offsets.IsInitialized())
			m_offsets.Init(CasKeyLookupInitialCount);

		if (m_casKeyTableMem.writtenSize >= CasKeyTableMaxSize - sizeof(CasKey) + 5)
			return ~0u;

		bool added = false;
		u32* casKeyOffset = InternalAdd(casKey, stringOffset, added);

		if (added)
		{
			u8 bytesForStringOffset = Get7BitEncodedCount(stringOffset);
			u8* mem = (u8*)m_casKeyTableMem.AllocateNoLock(bytesForStringOffset + sizeof(CasKey), 1, TC("Add"));
			BinaryWriter writer(mem, 0, 1000);
			writer.Write7BitEncoded(stringOffset);
			writer.WriteCasKey(casKey);
			*casKeyOffset = u32(mem - m_casKeyTableMem.memory);
			outRequiredCasTableSize = (u32)m_casKeyTableMem.writtenSize;
		}
		else
		{
			BinaryReader reader(m_casKeyTableMem.memory, *casKeyOffset, ~0u);
			reader.Read7BitEncoded();
			outRequiredCasTableSize = Max(outRequiredCasTableSize, u32(reader.GetPosition() + sizeof(CasKey)));
		}
		return *casKeyOffset;
	}

	u32 CompactCasKeyTable::AddNoLock(const CasKey& casKey, u64 stringOffset)
	{
		if (!m_casKeyTableMem.memory)
			m_casKeyTableMem.Init(CasKeyTableMaxSize);
		if (!m_offsets.IsInitialized())
			m_offsets.Init(CasKeyLookupInitialCount);

		bool added = false;
		u32* casKeyOffset = InternalAdd(casKey, stringOffset, added);
		if (added)
		{
			u8 bytesForStringOffset = Get7BitEncodedCount(stringOffset);
			u8* mem = (u8*)m_casKeyTableMem.AllocateNoLock(bytesForStringOffset + sizeof(CasKey), 1, TC("AddNoLock"));
			BinaryWriter writer(mem, 0, 1000);
			writer.Write7BitEncoded(stringOffset);
			writer.WriteCasKey(casKey);
			*casKeyOffset = u32(mem - m_casKeyTableMem.memory);
		}
		return *casKeyOffset;
	}

	u32 NextPow2(u32 v)
	{
		v--;
		v |= v >> 1;
		v |= v >> 2;
		v |= v >> 4;
		v |= v >> 8;
		v |= v >> 16;
		v++;
		return v;
	}


	u32* CompactCasKeyTable::InternalAdd(const CasKey& casKey, u64 stringOffset, bool& outAdded)
	{
		bool added = false;
		Value& value = m_offsets.Insert(casKey, added);
		if (added)
		{
			value.count = 1;
			value.single.stringOffset = u32(stringOffset);
			outAdded = true;
			return &value.single.casKeyOffset;
		}
		
		if (value.count == 1)
		{
			if (value.single.stringOffset == stringOffset)
				return &value.single.casKeyOffset;

			auto newOffsets = new StringAndKey[2];
			newOffsets[0].stringOffset = value.single.stringOffset;
			newOffsets[0].casKeyOffset = value.single.casKeyOffset;
			newOffsets[1].stringOffset = u32(stringOffset);
			value.array = newOffsets;
			value.count = 2;
			outAdded = true;
			return &newOffsets[1].casKeyOffset;
		}

		if (value.count <= CasKeyArrayMaxSize)
		{
			for (u32 i=0, e=value.count; i!=e; ++i)
				if (value.array[i].stringOffset == stringOffset)
					return &value.array[i].casKeyOffset;

			outAdded = true;

			u32 index = value.count;
			u32 newCount = index + 1;

			if (newCount <= CasKeyArrayMaxSize)
			{
				u32 capacity = NextPow2(index);
				u32 newCapacity = NextPow2(newCount);

				if (capacity == newCapacity)
				{
					value.array[index].stringOffset = u32(stringOffset);
					value.count = newCount;
					return &value.array[index].casKeyOffset;
				}

				auto newArray = new StringAndKey[newCapacity];
				memcpy(newArray, value.array, index*sizeof(StringAndKey));

				newArray[index].stringOffset = u32(stringOffset);
				delete[] value.array;
				value.array = newArray;
				value.count = newCount;
				return &newArray[index].casKeyOffset;
			}

			auto& lookup = *new UnorderedMap<u32, u32>();
			for (u32 i=0, e=value.count; i!=e; ++i)
				lookup[value.array[i].stringOffset] = value.array[i].casKeyOffset;
			delete[] value.array;

			value.stringLookup = &lookup;
			value.count = newCount;

			return &lookup[u32(stringOffset)];
		}

		auto& lookup = *value.stringLookup;
		auto insres = lookup.try_emplace(u32(stringOffset));
		outAdded = insres.second;
		return &insres.first->second;
	}

	void CompactCasKeyTable::GetKey(CasKey& outKey, u64 offset) const
	{
		BinaryReader reader(m_casKeyTableMem.memory, offset, ~0u);
		reader.Read7BitEncoded();
		outKey = reader.ReadCasKey();
	}

	bool CompactCasKeyTable::GetPathAndKey(StringBufferBase& outPath, CasKey& outKey, const CompactPathTable& pathTable, u64 offset) const
	{
		#if UBA_DEBUG
		{
			SCOPED_FUTEX_READ(const_cast<CompactCasKeyTable*>(this)->m_lock, lock)
			UBA_ASSERTF(offset + sizeof(CasKey) < m_casKeyTableMem.writtenSize, TC("Reading cas key from offset %llu which is out of bounds (Max %llu)"), offset + sizeof(CasKey), m_casKeyTableMem.writtenSize);
		}
		#endif

		BinaryReader reader(m_casKeyTableMem.memory, offset, ~0u);
		u32 stringOffset = (u32)reader.Read7BitEncoded();
		outKey = reader.ReadCasKey();
		return pathTable.GetString(outPath, stringOffset);
	}

	u8* CompactCasKeyTable::GetMemory()
	{
		return m_casKeyTableMem.memory;
	}

	u32 CompactCasKeyTable::GetSize()
	{
		SCOPED_FUTEX_READ(m_lock, lock2)
		return u32(m_casKeyTableMem.writtenSize);
	}

	void CompactCasKeyTable::ReadMem(BinaryReader& reader, bool populateLookup)
	{
		if (!m_casKeyTableMem.memory)
			m_casKeyTableMem.Init(CasKeyTableMaxSize);
		if (!m_offsets.IsInitialized())
			m_offsets.Init(CasKeyLookupInitialCount);

		u64 writtenSize = m_casKeyTableMem.writtenSize;

		u64 left = reader.GetLeft();
		void* mem = m_casKeyTableMem.AllocateNoLock(left, 1, TC("CompactCasKeyTable"));
		reader.ReadBytes(mem, left);

		if (!populateLookup)
			return;

		BinaryReader reader2(m_casKeyTableMem.memory, writtenSize, m_casKeyTableMem.writtenSize);
		while (reader2.GetLeft())
		{
			u32 offset = u32(reader2.GetPosition());
			u64 stringOffset = reader2.Read7BitEncoded();
			CasKey casKey = reader2.ReadCasKey();
			bool added = false;
			u32* casKeyOffset = InternalAdd(casKey, stringOffset, added);
			UBA_ASSERT(added);
			*casKeyOffset = offset;
		}
	}

	u8* CompactCasKeyTable::BeginCommit(u64 size)
	{
		if (!m_casKeyTableMem.memory)
			m_casKeyTableMem.Init(CasKeyTableMaxSize);
		if (!m_offsets.IsInitialized())
			m_offsets.Init(CasKeyLookupInitialCount);
		return (u8*)m_casKeyTableMem.CommitNoLock(size, TC("CompactCasKeyTable::BeginCommit"));
	}

	void CompactCasKeyTable::EndCommit(u8* data, u64 written)
	{
		m_casKeyTableMem.AllocateNoLock(written, 1, TC("CompactCasKeyTable::EndCommit"));
	}

	void CompactCasKeyTable::Debug(const CompactPathTable& pathTable)
	{
		#if 0
		for (auto vi=m_offsets.ValuesBegin(), ve=m_offsets.ValuesEnd(); vi!=ve; ++vi)
		{
			auto& value = *vi;
			if (value.count < 2)
				continue;

			if (value.count <= CasKeyArrayMaxSize)
			{
				printf("COUNT: %u\n", value.count);
				for (u32 i=0, e=value.count; i!=e; ++i)
				{
					StringBuffer<> str;
					pathTable.GetString(str, value.array[i].stringOffset);
					wprintf(L"   %s\n", str.data);
				}
			}
			else
			{
				printf("COUNT: %u\n", u32(value.stringLookup->size()));
				for (auto& kv : *value.stringLookup)
				{
					StringBuffer<> str;
					pathTable.GetString(str, kv.first);
					wprintf(L"   %s\n", str.data);
				}
			}
		}
		#endif
	}

	void CompactCasKeyTable::Swap(CompactCasKeyTable& other)
	{
		m_offsets.Swap(other.m_offsets);
		m_casKeyTableMem.Swap(other.m_casKeyTableMem);
	}
}
