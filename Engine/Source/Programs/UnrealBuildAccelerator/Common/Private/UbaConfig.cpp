// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaConfig.h"
#include "UbaFileAccessor.h"

namespace uba
{
	bool ConfigTable::GetValueAsString(const tchar*& out, const tchar* key) const
	{
		auto findIt = m_values.find(key);
		if (findIt == m_values.end())
			return m_parent ? m_parent->GetValueAsString(out, key) : false;
		out = findIt->second.string.c_str();
		return true;
	}

	bool ConfigTable::GetValueAsString(TString& out, const tchar* key) const
	{
		const tchar* str;
		if (!GetValueAsString(str, key))
			return false;
		out = str;
		return true;
	}

	bool ConfigTable::GetValueAsU32(u32& out, const tchar* key) const
	{
		auto findIt = m_values.find(key);
		if (findIt == m_values.end())
			return m_parent ? m_parent->GetValueAsU32(out, key) : false;
		StringBuffer<> buf(findIt->second.string);
		return buf.Parse(out);
	}

	bool ConfigTable::GetValueAsU64(u64& out, const tchar* key) const
	{
		auto findIt = m_values.find(key);
		if (findIt == m_values.end())
			return m_parent ? m_parent->GetValueAsU64(out, key) : false;
		StringBuffer<> buf(findIt->second.string);
		return buf.Parse(out);
	}

	bool ConfigTable::GetValueAsBytes(u64& out, const tchar* key) const
	{
		auto findIt = m_values.find(key);
		if (findIt == m_values.end())
			return m_parent ? m_parent->GetValueAsU64(out, key) : false;
		StringView view(findIt->second.string);
		u64 multiplier = 1;
		if (view.EndsWith(TCV("gb")))
		{
			multiplier = 1024ull*1024*1024;
			view = view.RemoveSuffix(2);
		}
		else if (view.EndsWith(TCV("mb")))
		{
			multiplier = 1024ull*1024*1024;
			view = view.RemoveSuffix(2);
		}
		else if (view.EndsWith(TCV("kb")))
		{
			multiplier = 1024ull*1024*1024;
			view = view.RemoveSuffix(2);
		}
		if (!view.Parse(out))
			return false;
		out *= multiplier;
		return true;
	}

	bool ConfigTable::GetValueAsInt(int& out, const tchar* key) const
	{
		auto findIt = m_values.find(key);
		if (findIt == m_values.end())
			return m_parent ? m_parent->GetValueAsInt(out, key) : false;
		#if PLATFORM_WINDOWS
		out = (int)wcstol(findIt->second.string.c_str(), 0, 10);
		#else
		out = atoi(findIt->second.string.c_str());
		#endif
		return true;
	}

	bool ConfigTable::GetValueAsBool(bool& out, const tchar* key) const
	{
		auto findIt = m_values.find(key);
		if (findIt == m_values.end())
			return m_parent ? m_parent->GetValueAsBool(out, key) : false;
		const tchar* value = findIt->second.string.c_str();
		if (Equals(value, TC("true")) || Equals(value, TC("1")))
		{
			out = true;
			return true;
		}
		if (Equals(value, TC("false")) || Equals(value, TC("0")))
		{
			out = false;
			return true;
		}

		return false;
	}

	const ConfigTable* ConfigTable::GetTable(const tchar* name) const
	{
		if (!name || !*name)
			return this;
		auto findIt = m_tables.find(name);
		if (findIt == m_tables.end())
			return nullptr;
		return &findIt->second;
	}

	ConfigTable& ConfigTable::AddTable(const tchar* name)
	{
		if (!name || !*name)
			return *this;
		ConfigTable& table = m_tables.try_emplace(name).first->second;
		table.m_parent = this;
		return table;
	}

	void ConfigTable::AddValue(const tchar* key, int value)
	{
		tchar buf[256];
		TSprintf_s(buf, sizeof_array(buf), TC("%i"), value);
		m_values[key] = Value{ValueType_Value, buf};
	}

	void ConfigTable::AddValue(const tchar* key, u32 value)
	{
		tchar buf[256];
		TSprintf_s(buf, sizeof_array(buf), TC("%u"), value);
		m_values[key] = Value{ValueType_Value, buf};
	}

	void ConfigTable::AddValue(const tchar* key, u64 value)
	{
		tchar buf[256];
		TSprintf_s(buf, sizeof_array(buf), TC("%llu"), value);
		m_values[key] = Value{ValueType_Value, buf};
	}

	void ConfigTable::AddValue(const tchar* key, bool value)
	{
		m_values[key] = Value{ValueType_Value, value ? TC("true") : TC("false")};
	}

	void ConfigTable::AddValue(const tchar* key, const tchar* str)
	{
		m_values[key] = Value{ValueType_String, str};
	}

	bool ConfigTable::SaveToText(Logger& logger, Vector<char>& outText) const
	{
		char line[1024];
		for (auto& kv : m_values)
		{
			const char* quote = kv.second.type == ValueType_String ? "\"" : "";
			#if PLATFORM_WINDOWS
			int written = sprintf_s(line, sizeof_array(line), "%S = %s%S%s\r\n", kv.first.c_str(), quote, kv.second.string.c_str(), quote);
			#else
			int written = snprintf(line, sizeof_array(line), "%s = %s%s%s\r\n", kv.first.c_str(), quote, kv.second.string.c_str(), quote);
			#endif
			
			u64 size = outText.size();
			outText.resize(size + written);
			memcpy(outText.data() + size, line, written);
		}

		for (auto& kv : m_tables)
		{
			#if PLATFORM_WINDOWS
			int written = sprintf_s(line, sizeof_array(line), "[%S]\r\n", kv.first.c_str());
			#else
			int written = snprintf(line, sizeof_array(line), "[%s]\r\n", kv.first.c_str());
			#endif

			u64 size = outText.size();
			outText.resize(size + written);
			memcpy(outText.data() + size, line, written);

			kv.second.SaveToText(logger, outText);
		}

		return true;
	}

	bool Config::LoadFromFile(Logger& logger, const tchar* configFile)
	{
		m_isLoaded = true;

#if PLATFORM_WINDOWS
		StringBuffer<> tempPath;
		if (configFile[1] != ':')
		{
			if (GetCurrentDirectoryW(tempPath) && FileExists(logger, tempPath.EnsureEndsWithSlash().Append(configFile).data))
			{
				configFile = tempPath.data;
			}
			else if (GetDirectoryOfCurrentModule(logger, tempPath.Clear()) && FileExists(logger, tempPath.EnsureEndsWithSlash().Append(configFile).data))
			{
				configFile = tempPath.data;
			}
			else
				return false;
		}
#endif

		FileAccessor fa(logger, configFile);
		if (!fa.OpenMemoryRead(0, false))
			return false;
		logger.Info(TC("  Loading config from %s"), configFile);
		return LoadFromText(logger, (const char*)fa.GetData(), fa.GetSize());
	}

	bool ConfigTable::LoadFromText(Logger& logger, const char* text, u64 textLen)
	{
		const char* i = text;
		const char* e = i + textLen;

		auto consumeEmpty = [&]() -> char
			{
				while (i != e)
				{
					if (*i != ' ' && *i != '\t'&& *i != '\r')
						return *i;
					++i;
				}
				return 0;
			};

		auto consumeIdentifier = [&](StringBufferBase& out) -> char
			{
				const char* first = i;
				while (i != e)
				{
					tchar c = *i;
					bool validChar = (c >= 'a' && c <= 'z') || (c >= 'A' && c <= 'Z') || c == '_' || c == '-';
					if (!validChar)
						if (first == i || c < '0' || c > '9')
							return *i;
					out.Append(*i);
					++i;
				}
				return 0;
			};

		auto consumeLine = [&](StringBufferBase& out, char untilChar) -> char
			{
				bool append = true;
				while (i != e)
				{
					if (*i == '\n')
						return *i;

					append &= (*i != untilChar && *i != '\r');

					if (append)
						out.Append(*i);
					++i;
				}
				return 0;
			};

		ConfigTable* activeTable = this;
		while (true)
		{
			char token = consumeEmpty();
			if (token == 0)
				break;
			if (token == '\n')
			{
				++i;
			}
			else if (token == '#')
			{
				StringBuffer<1024> comment;
				consumeLine(comment, 0);
			}
			else if (token == '[')
			{
				++i;
				StringBuffer<128> tableName;
				token = consumeIdentifier(tableName);
				if (token != ']')
					return logger.Error(TC("No end token after group name %s"), tableName.data);
				++i;
				token = consumeEmpty();
				if (token == 0)
					break;
				if (token != '\n')
					return logger.Error(TC("Unexpected token %c after group %s"), tableName.data);
				++i;
				activeTable = &m_tables.try_emplace(tableName.data).first->second;
				activeTable->m_parent = this;
			}
			else
			{
				StringBuffer<128> key;
				consumeIdentifier(key);
				token = consumeEmpty();
				if (token != '=')
					return logger.Error(TC("Unexpected equals sign after key name %s"), key.data);
				++i;
				token = consumeEmpty();

				ValueType type = ValueType_Value;
				StringBuffer<1024> value;
				if (token == '\"')
				{
					++i;
					token = consumeLine(value, '\"');
					type = ValueType_String;
				}
				else
				{
					token = consumeLine(value, ' ');
				}

				activeTable->m_values[key.data] = Value{type, value.data};
				if (token == 0)
					break;
				++i;
			}
		}

		return true;
	}

	bool Config::IsLoaded() const
	{
		return m_isLoaded;
	}

	bool Config::SaveToFile(Logger& logger, const tchar* configFile) const
	{
		StringBuffer<> dir;
		dir.AppendDir(configFile);
		if (!DirectoryCache().CreateDirectory(logger, dir.data))
			return false;

		FileAccessor fa(logger, configFile);
		if (!fa.CreateWrite())
			return false;

		Vector<char> text;
		if (!SaveToText(logger, text))
			return false;
		if (!fa.Write(text.data(), text.size()))
			return false;

		return fa.Close();
	}
}
