// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaCrypto.h"
#include "UbaLogger.h"
#include "UbaMemory.h"
#include "UbaPlatform.h"
#include "UbaSynchronization.h"

#if PLATFORM_WINDOWS
#define UBA_CRYPTO_TYPE 1
#else
#define UBA_CRYPTO_TYPE 2
#endif

#if UBA_CRYPTO_TYPE == 1
	#include <winternl.h>
	#include <bcrypt.h>
	#pragma comment (lib, "bcrypt.lib")
#elif UBA_CRYPTO_TYPE == 2
	#include "openssl/aes.h"

	struct KeyData
	{
		AES_KEY encryptKey;
		AES_KEY decryptKey;
	};

#endif // UBA_CRYPTO_TYPE

namespace uba
{
	inline constexpr u32 kAesBytes128 = 16;

	CryptoKey Crypto::CreateKey(Logger& logger, const u8* key128)
	{
#if UBA_CRYPTO_TYPE == 1
		BCRYPT_ALG_HANDLE providerHandle = NULL;
		NTSTATUS res = BCryptOpenAlgorithmProvider(&providerHandle, BCRYPT_AES_ALGORITHM, NULL, 0);
		if (!BCRYPT_SUCCESS(res))
		{
			logger.Error(L"ERROR: BCryptOpenAlgorithmProvider - Failed to open aes algorithm (0x%x)", res);
			return InvalidCryptoKey;
		}
		if (!providerHandle)
		{
			logger.Error(L"ERROR: BCryptOpenAlgorithmProvider - Returned null handle");
			return InvalidCryptoKey;
		}
		auto g = MakeGuard([&]() { BCryptCloseAlgorithmProvider(&providerHandle, 0); });

		//ULONG size = 0;
		//u32 len = 0;
		//NTSTATUS ret = BCryptGetProperty(providerHandle, BCRYPT_OBJECT_LENGTH, (UCHAR*)&len, sizeof(len), &size, 0);
		//UBA_ASSERT(BCRYPT_SUCCESS(ret));
		//void* buf = calloc(1, len);
		u32 objectBufferLen = 0;
		u8* objectBuffer = nullptr;


		BCRYPT_KEY_HANDLE keyHandle = NULL;
		res = BCryptGenerateSymmetricKey(providerHandle, &keyHandle, objectBuffer, objectBufferLen, (u8*)key128, kAesBytes128, 0);
		if (!BCRYPT_SUCCESS(res))
		{
			logger.Error(L"ERROR: BCryptGenerateSymmetricKey - Failed to generate symmetric key (0x%x)", res);
			return InvalidCryptoKey;
		}

		return (CryptoKey)(u64)keyHandle;
#elif UBA_CRYPTO_TYPE == 2
		auto data = new KeyData;
		AES_set_encrypt_key(key128, 128, &data->encryptKey);
		AES_set_decrypt_key(key128, 128, &data->decryptKey);
		return (CryptoKey)uintptr_t(data);
#else
		logger.Error(TC("ERROR: Crypto not supported on non-windows platforms"));
		return InvalidCryptoKey;
#endif // UBA_CRYPTO_TYPE
	}

	CryptoKey Crypto::DuplicateKey(Logger& logger, CryptoKey original)
	{
#if UBA_CRYPTO_TYPE == 1
		BCRYPT_KEY_HANDLE newKey;
		u32 objectBufferLen = 0;
		u8* objectBuffer = nullptr;
		NTSTATUS res = BCryptDuplicateKey((BCRYPT_KEY_HANDLE)original, &newKey, objectBuffer, objectBufferLen, 0);
		if (BCRYPT_SUCCESS(res))
			return (CryptoKey)(u64)newKey;
		logger.Error(L"ERROR: BCryptDuplicateKey failed (0x%x)", res);
		return InvalidCryptoKey;
#elif UBA_CRYPTO_TYPE == 2
		auto data = new KeyData(*(KeyData*)uintptr_t(original));
		return (CryptoKey)uintptr_t(data);
#else
		return InvalidCryptoKey;
#endif // UBA_CRYPTO_TYPE
	}

	void Crypto::DestroyKey(CryptoKey key)
	{
#if UBA_CRYPTO_TYPE == 1
		BCryptDestroyKey((BCRYPT_KEY_HANDLE)key);
#elif UBA_CRYPTO_TYPE == 2
		delete (KeyData*)uintptr_t(key);
#endif // UBA_CRYPTO_TYPE
	}

#if UBA_CRYPTO_TYPE == 1
	bool BCryptEncryptDecrypt(Logger& logger, bool encrypt, CryptoKey key, u8* data, u32 size, Guid& inOutInitVector)
	{
		u8 objectBuffer[1024];
		u32 objectBufferLen = sizeof(objectBuffer);

		u32 alignedSize = (size / kAesBytes128) * kAesBytes128;
		u32 overflow = size - alignedSize;

		BCRYPT_KEY_HANDLE newKey;
		NTSTATUS res = BCryptDuplicateKey((BCRYPT_KEY_HANDLE)key, &newKey, objectBuffer, objectBufferLen, 0);
		if (!BCRYPT_SUCCESS(res))
		{
			logger.Error(L"ERROR: BCryptDuplicateKey failed (0x%x)", res);
			return false;
		}
		auto g = MakeGuard([&]() { BCryptDestroyKey(newKey); });

		ULONG cipherTextLength = 0;

		if (encrypt)
		{
			for (u32 i=0; i!=overflow; ++i)
				data[alignedSize + i] ^= data[i]; 
			res = BCryptEncrypt(newKey, data, alignedSize, NULL, (u8*)&inOutInitVector, kAesBytes128, data, alignedSize, &cipherTextLength, 0);
		}
		else
		{
			res = BCryptDecrypt(newKey, data, alignedSize, NULL, (u8*)&inOutInitVector, kAesBytes128, data, alignedSize, &cipherTextLength, 0);
			for (u32 i=0; i!=overflow; ++i)
				data[alignedSize + i] ^= data[i]; 
		}

		if (!BCRYPT_SUCCESS(res))
		{
			logger.Error(L"ERROR: %s failed (0x%x)", (encrypt ? L"BCryptEncrypt" : L"BCryptDecrypt"), res);
			return false;
		}
		if (cipherTextLength != alignedSize)
		{
			logger.Error(L"ERROR: %s cipher text length does not match aligned size", (encrypt ? L"BCryptEncrypt" : L"BCryptDecrypt"));
			return false;
		}
		return true;
	}
#elif UBA_CRYPTO_TYPE == 2
	bool OpenSslEncryptDecrypt(CryptoKey key, u8* data, u32 size, bool enc, Guid& inOutInitVector)
	{
		auto& keyData = *(KeyData*)uintptr_t(key);
 
		u32 alignedSize = (size / kAesBytes128) * kAesBytes128;
		u32 overflow = size - alignedSize;

		if (enc)
		{
			for (u32 i=0; i!=overflow; ++i)
				data[alignedSize + i] ^= data[i]; 
			AES_cbc_encrypt(data, data, alignedSize, &keyData.encryptKey, (u8*)&inOutInitVector, AES_ENCRYPT);
		}
		else
		{
			AES_cbc_encrypt(data, data, alignedSize, &keyData.decryptKey, (u8*)&inOutInitVector, AES_DECRYPT);
			for (u32 i=0; i!=overflow; ++i)
				data[alignedSize + i] ^= data[i]; 
		}
		return true;
	}
#endif

	bool EncryptDecrypt(Logger& logger, CryptoKey key, u8* data, u32 size, Guid& inOutInitVector, bool enc)
	{
		// We don't fully encrypt data smaller than 16 bytes. There are a few messages that are harmless that does not need to be encrypted since they contain nothing of value
		if (size < kAesBytes128)
		{
			// This is just here to make unit tests happy.. no special security. Using init vector here will probably reduce security since it would be possible to reverse engineer part of init vector
			for (u32 i=0; i!=size; ++i)
				data[i] ^= 5381;//initVector.data1;
			return true;
		}

#if UBA_CRYPTO_TYPE == 1
		return BCryptEncryptDecrypt(logger, enc, key, data, size, inOutInitVector);
#elif UBA_CRYPTO_TYPE == 2
		return OpenSslEncryptDecrypt(key, data, size, enc, inOutInitVector);
#else
		return false;
#endif
	}

	bool Crypto::Encrypt(Logger& logger, CryptoKey key, u8* data, u32 size, Guid& inOutInitVector)
	{
		return EncryptDecrypt(logger, key, data, size, inOutInitVector, true);
	}

	bool Crypto::Decrypt(Logger& logger, CryptoKey key, u8* data, u32 size, Guid& inOutInitVector)
	{
		return EncryptDecrypt(logger, key, data, size, inOutInitVector, false);
	}
	
	bool CryptoFromString(u8* out, u32 outSize, const tchar* str)
	{
		if (!str || !*str || outSize != 16 || TStrlen(str) != 32)
			return false;
		((u64*)out)[0] = StringToValue(str, 16);
		((u64*)out)[1] = StringToValue(str + 16, 16);
		return true;
	}
}