// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaDirectoryTableHolder.h"
#include "UbaDirectoryIterator.h"
#include "UbaEnvironment.h"
#include "UbaFileAccessor.h"
#include "UbaFileMapping.h"
#include "UbaProtocol.h"

#define UBA_DEBUG_TRACK_DIR 0 // UBA_DEBUG_LOGGER

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool GetDirKey(StringKey& outDirKey, StringBufferBase& outDirPath, const tchar*& outLastSlash, StringView filePath)
	{
		StringView dirPath = filePath.GetPath();
		outLastSlash = dirPath.data + dirPath.count;
		UBA_ASSERTF(dirPath.count, TC("Can't get dir key for path %s"), filePath.data);
		if (!dirPath.count)
			return false;
		outDirPath.Append(dirPath);
		outDirKey = CaseInsensitiveFs ? ToStringKeyLower(outDirPath) : ToStringKey(outDirPath);
		return true;
	}

	StringKey GetKeyAndFixedName2(StringBuffer<>& outFixedFilePath, StringKeyHasher& outHasher, StringView filePath, StringKey* outDirKey, StringView* outFileName)
	{
		StringBuffer<> workingDir;
		if (!IsAbsolutePath(filePath))
		{
			GetCurrentDirectoryW(workingDir);
			workingDir.EnsureEndsWithSlash();
		}
		FixPath(outFixedFilePath, filePath, workingDir);

		StringKey dirKey;
		StringBuffer<> dirNameForHash;
		const tchar* lastSlash = nullptr;
		GetDirKey(dirKey, dirNameForHash, lastSlash, outFixedFilePath);
		if (outDirKey)
			*outDirKey = dirKey;
		if (outFileName)
			*outFileName = StringView(lastSlash + 1, outFixedFilePath.count - u32(lastSlash - outFixedFilePath.data) - 1);

		if (CaseInsensitiveFs)
			dirNameForHash.MakeLower();

		outHasher.Update(dirNameForHash);

		if (lastSlash)
		{
			StringBuffer<256> baseFileNameForHash;
			baseFileNameForHash.Append(lastSlash);
			if (CaseInsensitiveFs)
				baseFileNameForHash.MakeLower();

			outHasher.Update(baseFileNameForHash);
		}

		StringKey result = ToStringKey(outHasher);

#if UBA_DEBUG
		StringBuffer<> testPath(outFixedFilePath);
		if (CaseInsensitiveFs)
			testPath.MakeLower();
		StringKey testKey = ToStringKey(testPath);
		UBA_ASSERTF(testKey == result, TC("Key mismatch for %s"), outFixedFilePath.data);
#endif

		return result;
	}

	StringKey GetKeyAndFixedName(StringBuffer<>& outFixedFilePath, const tchar* filePath)
	{
		StringKeyHasher hasher;
		return GetKeyAndFixedName2(outFixedFilePath, hasher, ToView(filePath), nullptr, nullptr);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	OverlayEditScope::OverlayEditScope(DirectoryTableOverlay& o)
	:	overlay(o)
	{
		o.lock.Enter();
	}

	OverlayEditScope::~OverlayEditScope()
	{
		overlay.lock.Leave();
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	DirectoryTableHolder::DirectoryTableHolder(LogWriter& logWriter, const tchar* logPrefix, bool runningRemote)
	:	m_logger(logWriter, logPrefix)
	,	m_runningRemote(runningRemote)
	,	m_directoryTableMemory(TC("SessionDirectoryTable"))
	,	m_directoryTable(m_directoryTableMemory)
	{
		m_directoryTableHandle = FileMapping_Create(m_logger, PAGE_READWRITE|SEC_RESERVE, DirTableMaxSize, nullptr, TC("DirMappings"));
		UBA_ASSERT(m_directoryTableHandle.IsValid());
		m_directoryTableMem = FileMapping_Map(m_logger, m_directoryTableHandle, FILE_MAP_WRITE, 0, DirTableMaxSize);
		UBA_ASSERT(m_directoryTableMem);

		m_directoryTable.m_memory = m_directoryTableMem;
		m_directoryTable.m_lookup.reserve(30000);
	}

	DirectoryTableHolder::~DirectoryTableHolder()
	{
		FileMapping_Unmap(m_logger, m_directoryTableMem, DirTableMaxSize, TC("DirectoryTable"), false); // Can't discard since only parts might be mapped
		FileMapping_Close(m_logger, m_directoryTableHandle, TC("DirectoryTable"));

		#if UBA_DEBUG_LOGGER
		m_debugLogger = StopDebugLogger(m_debugLogger);
		#endif
	}

	void DirectoryTableHolder::SetTreatTempDirAsEmpty()
	{
#if PLATFORM_WINDOWS
		wchar_t systemTemp[256];
		GetEnvironmentVariableW(L"TEMP", systemTemp, 256);
		StringBuffer<> temp;
		FixPath(systemTemp, nullptr, 0, temp);
		temp.MakeLower();
		m_directoryForcedEmpty = ToStringKey(temp);
#endif
	}

	bool DirectoryTableHolder::RefreshDirectory(const tchar* dirPath, bool forceRegister)
	{
		#if UBA_SUPPORT_SYMLINKS
		char real[512];
		char* r = realpath(dirPath, real);
		if (!r)
			return false;
		dirPath = r;
		#endif
		
		UBA_ASSERT(!m_runningRemote);

		StringBuffer<> fixedDirPath;
		StringKeyHasher hasher;
		GetKeyAndFixedName2(fixedDirPath, hasher, ToView(dirPath), nullptr, nullptr);

		StringKey dirKey = ToStringKey(hasher);

		auto& dirTable = m_directoryTable;
		SCOPED_READ_LOCK(dirTable.m_lookupLock, lookupLock);
		auto res = dirTable.m_lookup.find(dirKey);
		if (res == dirTable.m_lookup.end())
		{
			lookupLock.Leave();
			if (forceRegister)
				WriteDirectoryEntries(dirKey, fixedDirPath);
			return true;
		}
		DirectoryTable::Directory& dir = res->second;
		lookupLock.Leave();
		SCOPED_WRITE_LOCK(dir.lock, dirLock);

		while (dir.populatedOffset == 0)
		{
			dirLock.Leave();
			Sleep(1);
			dirLock.Enter();
		}

		// WriteDirectoryEntriesInternal's isRefresh branch writes [prev=snapshotLatestOffset][itemCount]
		// with NO root preamble. If we entered with populatedOffset!=1, dir.latestOffset is 0, and the
		// resulting entry has prev_offset=0 which aliases onto the parser's root-entry grammar in
		// PopulateDirectoryRecursive — producing a bogus itemCount and walking off the page.
		if (dir.populatedOffset != 1)
			return m_logger.Error(TC("Refusing to refresh dir '%.*s': dir not populated. dirKey=%s populatedOffset=%u latestOffset=%u dirFilesCached=%zu forceRegister=%u"), fixedDirPath.count, fixedDirPath.data, KeyToString(dirKey).data, dir.populatedOffset, dir.latestOffset, dir.files.size(), forceRegister);

		DirTableOffset tableOffset;
		return WriteDirectoryEntriesInternal(tableOffset, dir, dirKey, fixedDirPath, true);
	}

	bool DirectoryTableHolder::RegisterNewFile(const tchar* filePath)
	{
		#if UBA_SUPPORT_SYMLINKS
		char real[512];
		char* r = realpath(filePath, real);
		if (!r)
			return false;
		filePath = r;
		#endif

		UBA_ASSERT(!m_runningRemote);
		StringBuffer<> fixedFilePath;
		auto key = GetKeyAndFixedName(fixedFilePath, filePath);
		return RegisterCreateFileForWrite(key, fixedFilePath, 0, 0, DefaultAttributes(), true, true);
	}

	void DirectoryTableHolder::RegisterDeleteFile(const tchar* filePath)
	{
		#if UBA_SUPPORT_SYMLINKS
		char real[512];
		char* r = realpath(filePath, real);
		if (!r)
			return;
		filePath = r;
		#endif

		UBA_ASSERT(!m_runningRemote);
		StringBuffer<> fixedFilePath;
		auto key = GetKeyAndFixedName(fixedFilePath, filePath);
		RegisterDeleteFile(key, fixedFilePath);
	}

	bool DirectoryTableHolder::RegisterNewDirectory(const tchar* directoryPath, bool ensureAncestors)
	{
		#if UBA_SUPPORT_SYMLINKS
		char real[512];
		char* r = realpath(directoryPath, real);
		if (!r)
			return false;
		directoryPath = r;
		#endif

		UBA_ASSERT(!m_runningRemote);
		StringBuffer<> fixedDirPath;
		auto dirKey = GetKeyAndFixedName(fixedDirPath, directoryPath);
		if (ensureAncestors && !EnsurePath(fixedDirPath))
			return false;
		if (!RegisterCreateFileForWrite(dirKey, fixedDirPath, 0, 0, DefaultDirAttributes(), true, true))
			return false;
		WriteDirectoryEntries(dirKey, fixedDirPath);
		return true;
	}

	bool DirectoryTableHolder::EnsurePath(StringView path)
	{
		UBA_ASSERT(!m_runningRemote);
		StringView parentPath = path.GetPath();
		if (!parentPath.count)
			return true; // No parent above (we've reached the root)

		StringBuffer<> parentForHash;
		parentForHash.Append(parentPath);
		if (CaseInsensitiveFs)
			parentForHash.MakeLower();
		StringKey parentKey = ToStringKey(parentForHash);

		auto& dirTable = m_directoryTable;
		{
			SCOPED_READ_LOCK(dirTable.m_lookupLock, lookupLock);
			auto findIt = dirTable.m_lookup.find(parentKey);
			if (findIt != dirTable.m_lookup.end())
			{
				DirectoryTable::Directory& dir = findIt->second;
				lookupLock.Leave();
				SCOPED_READ_LOCK(dir.lock, dirLock);
				while (dir.populatedOffset == 0)
				{
					dirLock.Leave();
					Sleep(1);
					dirLock.Enter();
				}
				if (dir.populatedOffset == 1)
					return true;
			}
		}

		// Parent is at populatedOffset==2 or missing. To fix it, the grandparent has to be at 1.
		if (!EnsurePath(parentPath))
			return false;

		FileInformation parentInfo;
		if (!GetFileInformation(parentInfo, m_logger, parentPath))
			return m_logger.Error(TC("EnsurePath: no on-disk entry for '%.*s' (%s)"), parentPath.count, parentPath.data, LastErrorToText().data);
		if (!IsDirectory(parentInfo.attributes))
			return m_logger.Error(TC("EnsurePath: '%.*s' is not a directory"), parentPath.count, parentPath.data);

		StringView grandParentPath = parentPath.GetPath();

		if (!grandParentPath.count)
		{
			WriteDirectoryEntries(parentKey, parentPath);
			SCOPED_READ_LOCK(dirTable.m_lookupLock, lookupLock);
			DirectoryTable::Directory& parentDir = dirTable.m_lookup.find(parentKey)->second;
			lookupLock.Leave();
			SCOPED_READ_LOCK(parentDir.lock, parentLock);
			if (parentDir.populatedOffset != 1)
				return m_logger.Error(TC("EnsurePath: root-level parent '%.*s' did not populate (populatedOffset=%u)"), parentPath.count, parentPath.data, parentDir.populatedOffset);
			return true;
		}

		StringBuffer<> grandParentForHash;
		grandParentForHash.Append(grandParentPath);
		if (CaseInsensitiveFs)
			grandParentForHash.MakeLower();
		StringKey grandParentKey = ToStringKey(grandParentForHash);

		StringView parentName = parentPath.GetFileName();

		SCOPED_WRITE_LOCK(dirTable.m_lookupLock, lookupWriteLock);
		auto grandRes = dirTable.m_lookup.try_emplace(grandParentKey, dirTable.m_memoryBlock);
		DirectoryTable::Directory& grandParentDir = grandRes.first->second;
		auto parentRes = dirTable.m_lookup.try_emplace(parentKey, dirTable.m_memoryBlock);
		DirectoryTable::Directory& parentDir = parentRes.first->second;
		lookupWriteLock.Leave();

		// Lock order: grandparent first, then parent.
		SCOPED_WRITE_LOCK(grandParentDir.lock, grandLock);
		SCOPED_WRITE_LOCK(parentDir.lock, parentLock);

		// Race re-check after acquiring locks.
		if (parentDir.populatedOffset == 1)
			return true;

		UBA_ASSERTF(grandParentDir.populatedOffset == 1, TC("EnsurePath: grandparent '%.*s' not populatedOffset==1 after recursion (populatedOffset=%u)"), grandParentPath.count, grandParentPath.data, grandParentDir.populatedOffset);

		u32 parentVolIndex = m_volumeCache.GetSerialIndex(parentInfo.volumeSerialNumber);

		// Inject parent into grandparent.files. Grandparent is already populatedOffset==1 and we
		// don't re-traverse it, so a parent created after grandparent was enumerated would
		// otherwise be missing from grandparent's file list.
		if (grandParentDir.files.find(parentKey) == grandParentDir.files.end())
		{
			u8 injectTemp[MaxPath + 256];
			BinaryWriter writer(injectTemp, 0, sizeof(injectTemp));
			writer.WriteStringKey(grandParentKey);
			writer.Write7BitEncoded(grandParentDir.latestOffset); // chain into grandparent's existing journal
			writer.Write7BitEncoded(1); // single child
			u64 injectEntryPos = writer.GetPosition();
			writer.WriteString(parentName);
			writer.WriteFileAttributes(parentInfo.attributes);
			writer.WriteVolumeSerial(parentVolIndex);
			writer.WriteFileIndex(parentInfo.index);
			// Parent is a directory — no lastWriteTime/fileSize
			u64 injectWritten = writer.GetPosition();

			SCOPED_WRITE_LOCK(dirTable.m_memoryLock, memoryLock);
			u32 writePos = dirTable.m_memorySize;
			EnsureDirectoryTableMemory(writePos + 8 + injectWritten);
			BinaryWriter memWriter(m_directoryTableMem + writePos);
			memWriter.Write7BitEncoded(injectWritten);
			u32 sizeEncodedBytes = u32(memWriter.GetPosition());
			u32 parentEntryAbs = writePos + sizeEncodedBytes + u32(injectEntryPos);
			u32 grandRecordPrevAbs = writePos + sizeEncodedBytes + sizeof(StringKey);
			memWriter.WriteBytes(injectTemp, injectWritten);
			dirTable.m_memorySize += u32(memWriter.GetPosition());

			grandParentDir.files.try_emplace(parentKey, DirTableOffset{parentEntryAbs});
			grandParentDir.latestOffset = grandRecordPrevAbs;
		}

		// Fully populate the parent from disk. The previous implementation wrote an empty
		// first-entry here (itemCount=0) and marked populatedOffset==1 — which made the parent
		// look fully populated while dropping every file that already existed on disk. Any later
		// WriteDirectoryEntries/RefreshDirectory then early-outs on populatedOffset==1 and the
		// files stay invisible. Traverse for real instead: WriteDirectoryEntriesInternal writes the
		// proper first record (prev=0, real itemCount) and sets parentDir.latestOffset. We hold
		// parentDir.lock and have released the memoryLock above, matching its expectations.
		grandLock.Leave(); // grandparent linkage done; don't hold it across parent's disk traversal
		DirTableOffset parentTableOffset;
		if (!WriteDirectoryEntriesInternal(parentTableOffset, parentDir, parentKey, parentPath, false))
		{
			parentDir.populatedOffset = 2;
			return m_logger.Error(TC("EnsurePath: failed to populate parent '%.*s'"), parentPath.count, parentPath.data);
		}

		parentDir.populatedOffset = 1;
		return true;
	}

	const u8* DirectoryTableHolder::GetDirectoryTableMemory()
	{
		return m_directoryTableMem;
	}

	DirTableSize DirectoryTableHolder::GetDirectoryTableSize(DirectoryTableOverlay* overlay)
	{
		u32 overlaySize = 0u;
		if (overlay)
		{
			SCOPED_READ_LOCK(overlay->lock, lock);
			overlaySize = overlay->size;
		}
		SCOPED_READ_LOCK(m_directoryTable.m_memoryLock, lock);
		return {m_directoryTable.m_memorySize, overlaySize};
	}

	void DirectoryTableHolder::CreateOverlay(DirectoryTableOverlay& overlay)
	{
		UBA_ASSERT(!overlay.handle.IsValid());
		SCOPED_WRITE_LOCK(overlay.lock, lock);
		overlay.handle = FileMapping_Create(m_logger, PAGE_READWRITE|SEC_RESERVE, OverlayTableMaxSize, nullptr, TC("Overlay"));
		UBA_ASSERTF(overlay.handle.IsValid(), TC("Failed to create overlay handle (size %u)"), OverlayTableMaxSize);
		overlay.memory = FileMapping_Map(m_logger, overlay.handle, FILE_MAP_WRITE, 0, OverlayTableMaxSize);
		UBA_ASSERT(overlay.memory);
	}

	void DirectoryTableHolder::ResetOverlay(DirectoryTableOverlay& overlay, void* data, u64 dataSize)
	{
		overlay.size = 0;
		overlay.lookup.clear();
		overlay.roots.clear();
		if (!dataSize)
			return;
		EnsureOverlayMemory(overlay, dataSize);
		memcpy(overlay.memory, data, dataSize);
		overlay.size = u32(dataSize);
	}

	bool DirectoryTableHolder::EnsureOverlayMemory(DirectoryTableOverlay& overlay, u64 neededSize)
	{
		u64 totalSize = overlay.size + neededSize;
		if (totalSize <= overlay.committed)
			return true;
		u64 newSize = AlignUp(totalSize, 32ull*1024);
		if (newSize > OverlayTableMaxSize)
		{
			static bool called = m_logger.Error(TC("Overlay table overflow. OverlayTableMaxSize need to be increased (Size: %llu)"), overlay.size);
			return false;
		}

		u64 toCommit = newSize - overlay.committed;
		u8* address = overlay.memory + overlay.committed;
		if (!FileMapping_Commit(m_logger, address, toCommit, false))
			m_logger.Error(TC("Failed to commit memory for directory table (Committed: %llu, ToCommit: %llu) (%s)"), m_directoryTableMemCommitted, toCommit, LastErrorToText().data);
		overlay.committed += u32(toCommit);
		return true;
	}

	void DirectoryTableHolder::DestroyOverlay(DirectoryTableOverlay& overlay)
	{
		if (!overlay.handle.IsValid())
			return;
		SCOPED_WRITE_LOCK(overlay.lock, lock);
		FileMapping_Unmap(m_logger, overlay.memory, OverlayTableMaxSize, TC("Overlay"), false); // Can't discard since only parts might be mapped
		overlay.memory = nullptr;
		FileMapping_Close(m_logger, overlay.handle, TC("Overlay"));
		overlay.handle = {};
		overlay.committed = 0;
		overlay.size = 0;
		overlay.lookup.clear();
	}

	void DirectoryTableHolder::PrintOverlay(DirectoryTableOverlay& overlay)
	{
		StringBuffer<> path;
		for (auto& root : overlay.roots)
		{
			auto findIt = overlay.lookup.find(root);
			UBA_ASSERT(findIt != overlay.lookup.end());
			PrintOverlay(overlay, findIt->second, path);
		}
		m_logger.Info(TC("OVERLAY MEMORY SIZE: %llu"), overlay.size);
	}

	void DirectoryTableHolder::PrintOverlay(DirectoryTableOverlay& overlay, DirectoryTableOverlay::Dir& dir, StringBufferBase& path)
	{
		for (auto& file : dir.files)
		{
			u32 pathLen = path.count;
			BinaryReader reader(overlay.memory, file.second);
			reader.ReadString(path);

			auto findIt = overlay.lookup.find(file.first);
			if (findIt != overlay.lookup.end())
			{
				path.Append(PathSeparator);
				PrintOverlay(overlay, findIt->second, path);
			}
			else
			{
				//if (reader.ReadFileAttributes())
					m_logger.Info(TC("OVERLAY FILE: %s"), path.data);
			}
			path.Resize(pathLen);
		}
	}

	bool DirectoryTableHolder::RegisterOverlayCreateOrWrite(OverlayEditScope& scope, StringKey filePathKey, StringView filePath, u64 fileSize, u64 lastWriteTime, u32 attributes, StringView symlinkContent)
	{
		UBA_ASSERTF(attributes != 0, TC("Registering file %.*s with 0 attributes"), filePath.count, filePath.data);
		FileInformation info;
		info.attributes = attributes;
		info.index = ++m_fileIndexCounter;
		info.volumeSerialNumber = 1;
		info.size = fileSize;
		info.lastWriteTime = lastWriteTime;
		return WriteOverlayEntry(scope, filePath, info, symlinkContent);
	}

	bool DirectoryTableHolder::RegisterOverlayDelete(OverlayEditScope& scope, StringKey filePathKey, StringView filePath)
	{
		StringKey dirKey;
		const tchar* lastSlash;
		StringBuffer<> dirName;
		if (!GetDirKey(dirKey, dirName, lastSlash, filePath))
			return false;
		FileInformation fileInfo;
		return WriteOverlayEntry(scope, dirKey, filePath, ToView(lastSlash+1), fileInfo, !m_runningRemote);
	}

	bool DirectoryTableHolder::RegisterNonRootedDir(OverlayEditScope& scope, StringView dirPath)
	{
		UBA_ASSERT(dirPath.Last() != PathSeparator);
		if (!WriteOverlayDirInLookup(scope, dirPath.GetPath())) // Create parent dir without any roots
			return false;
		FileInformation fileInfo;
		fileInfo.attributes = DefaultDirAttributes();
		fileInfo.index = ++m_fileIndexCounter;
		fileInfo.volumeSerialNumber = 1;
		return WriteOverlayEntry(scope, dirPath, fileInfo); // Add non rooted dir
	}

	bool DirectoryTableHolder::PopulateOverlayFile(DirectoryTableOverlay& overlay, StringView filePath, const GetOverlayFileInformation& getFileInformation)
	{
		StringKey dirKey;
		StringView fileName;
		StringBuffer<> fixedFilePath;
		StringKeyHasher hasher;
		StringKey filePathKey = GetKeyAndFixedName2(fixedFilePath, hasher, filePath, &dirKey, &fileName);
		FileInformation fileInfo;
		if (getFileInformation)
		{
			if (!getFileInformation(fileInfo, filePathKey, filePath))
				return false;
		}
		else
		{
			if (!GetFileInformation(fileInfo, m_logger, filePath))
				return false;
		}
		OverlayEditScope scope(overlay);
		return WriteOverlayEntry(scope, dirKey, fixedFilePath, fileName, fileInfo, !m_runningRemote);
	}

	bool DirectoryTableHolder::AddOverlayFile(DirTableOffset& outOffset, OverlayEditScope& scope, StringView filePath)
	{
		FileInformation fi;
		fi.size = InvalidValue; // TODO: This is very wrong.. but can be used to know if file is actually in open state
		fi.attributes = DefaultAttributes();
		fi.index = ++m_fileIndexCounter;
		fi.volumeSerialNumber = 1;
		return WriteOverlayEntry(scope, filePath, fi, {}, &outOffset);
	}

	bool DirectoryTableHolder::WriteOverlayEntry(OverlayEditScope& scope, StringView filePath, const FileInformation& fileInfo, StringView symlinkContent, DirTableOffset* outOffset)
	{
		StringKey dirKey;
		StringView fileName;
		StringBuffer<> fixedFilePath;
		StringKeyHasher hasher;
		StringKey filePathKey = GetKeyAndFixedName2(fixedFilePath, hasher, filePath, &dirKey, &fileName);
		if (!WriteOverlayEntry(scope, dirKey, fixedFilePath, fileName, fileInfo, !m_runningRemote, symlinkContent, outOffset))
			return false;
		if (IsDirectory(fileInfo.attributes))
			return WriteOverlayDirInLookup(scope, filePath);
		return true;
	}

	bool DirectoryTableHolder::WriteOverlayEntry(OverlayEditScope& scope, StringKey dirKey, StringView filePath, StringView fileName, const FileInformation& fileInfo, bool populateMainTableParentDir, StringView symlinkContent, DirTableOffset* outOffset)
	{
		auto dirOffsetIns = scope.overlay.lookup.try_emplace(dirKey);
		auto& dir = dirOffsetIns.first->second;

		if (!dir.offset)
		{
			StringView dirPath(filePath.data, u32(fileName.data - filePath.data) - 1);

			// I have a feeling there is a bug here somehow but can't create it in unit tests.
			// Essentially the theory is that if a remote helper runs in a directory tree and gets the attribute of a overlay file (prepopulated)
			// That will create the entire hierarchy to the overlay file.
			// It then queries a sibling directory for a file. Now the dir hierarchy exists but sibling directory will not exist since the directory table
			// does not know that it has not queried the directory for the main table.
			if (populateMainTableParentDir)
			{
				UBA_ASSERT(!m_runningRemote);
				WriteDirectoryEntries(dirKey, dirPath);
			}

			WriteOverlayDirectoryEntriesRecursive(scope, dirKey, dirPath);
		}

		auto& fileOffset = dir.files[CaseInsensitiveFs ? ToStringKeyLower(fileName) : ToStringKey(fileName)];
		bool res = WriteOverlayEntry(scope, dirKey, dir, fileName, fileInfo, fileOffset, symlinkContent);
		if (outOffset)
			outOffset->internal = fileOffset;
		return res;
	}

	bool DirectoryTableHolder::WriteOverlayEntry(OverlayEditScope& scope, StringKey dirKey, DirectoryTableOverlay::Dir& dir, StringView fileName, const FileInformation& fileInfo, u32& outFileOffset, StringView symlinkContent)
	{
		u8 temp[MaxPath * 2 + 256]; // filename + symlink content + entry headers
		BinaryWriter writer(temp, 0, sizeof(temp));
		writer.WriteStringKey(dirKey);
		writer.Write7BitEncoded(dir.offset); // Previous entry for same directory
		if (!dir.offset)
		{
			writer.WriteFileAttributes(DefaultDirAttributes());
			writer.WriteVolumeSerial(0);
			writer.WriteFileIndex(++m_fileIndexCounter);
		}
		writer.Write7BitEncoded(1); // Count
		u32 relativePosition = u32(writer.GetPosition());
		writer.WriteString(fileName);
		writer.WriteFileAttributes(fileInfo.attributes);
		writer.WriteVolumeSerial(fileInfo.volumeSerialNumber);
		writer.WriteFileIndex(fileInfo.index);
		
		#if UBA_SUPPORT_SYMLINKS
		if (symlinkContent.count)
		{
			UBA_ASSERT(IsSymlink(fileInfo.attributes));
			writer.WriteString(symlinkContent);
			UBA_ASSERT(symlinkContent.count == fileInfo.size);
		}
		#endif

		writer.WriteFileTime(fileInfo.lastWriteTime);
		writer.WriteFileSize(fileInfo.size);
		
		auto& overlay = scope.overlay;
		u64 written = writer.GetPosition();
		if (!EnsureOverlayMemory(overlay, written + 8))
			return false;
		BinaryWriter memWriter(overlay.memory, overlay.size, overlay.committed);
		memWriter.Write7BitEncoded(written); // Storage size
		dir.offset = u32(memWriter.GetPosition()) + sizeof(StringKey);
		outFileOffset = u32(memWriter.GetPosition()) + relativePosition;

		memWriter.WriteBytes(writer.GetData(), written);
		overlay.size = u32(memWriter.GetPosition());
		return true;
	}

	bool DirectoryTableHolder::WriteOverlayDirInLookup(OverlayEditScope& scope, StringView dirPath)
	{
		auto& overlay = scope.overlay;
		UBA_ASSERT(dirPath.Last() != PathSeparator);
		StringKey dirKey = CaseInsensitiveFs ? ToStringKeyLower(dirPath) : ToStringKey(dirPath);
		auto insres = overlay.lookup.try_emplace(dirKey);
		if (!insres.second)
			return true;
		auto& dir = insres.first->second;
		u8 temp[256];
		BinaryWriter writer(temp, 0, sizeof(temp));
		writer.WriteStringKey(dirKey);
		writer.Write7BitEncoded(0); // Previous entry for same directory
		writer.WriteFileAttributes(DefaultDirAttributes());
		writer.WriteVolumeSerial(0);
		writer.WriteFileIndex(++m_fileIndexCounter);
		writer.Write7BitEncoded(0); // File count
		u64 written = writer.GetPosition();
		if (!EnsureOverlayMemory(overlay, written + 8))
			return false;
		BinaryWriter memWriter(overlay.memory, overlay.size, overlay.committed);
		memWriter.Write7BitEncoded(writer.GetPosition()); // Storage size
		dir.offset = u32(memWriter.GetPosition()) + sizeof(StringKey);
		memWriter.WriteBytes(writer.GetData(), written);
		overlay.size = u32(memWriter.GetPosition());
		return true;
	}

	bool DirectoryTableHolder::WriteOverlayDirectoryEntriesRecursive(OverlayEditScope& scope, StringKey filePathKey, StringView filePath)
	{
		auto& overlay = scope.overlay;
		StringView dirPath = filePath.GetPath();
		if (!dirPath.count)
		{
			overlay.roots.push_back(filePathKey);
			return true;
		}
		StringKey dirKey = CaseInsensitiveFs ? ToStringKeyLower(dirPath) : ToStringKey(dirPath);
		auto& dir = overlay.lookup.try_emplace(dirKey).first->second;

		u8 temp[256];
		BinaryWriter writer(temp, 0, sizeof(temp));

		if (!dir.offset)
			if (!WriteOverlayDirectoryEntriesRecursive(scope, dirKey, dirPath))
				return false;

		auto res = dir.files.try_emplace(filePathKey);
		if (!res.second)
			return true;

		// If the main table already has this ancestor (with real on-disk st_dev/st_ino),
		// don't override it with a synthesized overlay entry — GNU rm's fts_read aborts
		// when (st_dev, st_ino) differs between fstatat and a later fstat for the same path.
		{
			auto& mainTable = m_directoryTable;
			SCOPED_READ_LOCK(mainTable.m_lookupLock, mainLookupLock);
			auto parentIt = mainTable.m_lookup.find(dirKey);
			if (parentIt != mainTable.m_lookup.end())
			{
				DirectoryTable::Directory& mainParent = parentIt->second;
				SCOPED_READ_LOCK(mainParent.lock, parentLock);
				if (mainParent.files.find(filePathKey) != mainParent.files.end())
					return true;
			}
		}

		FileInformation fi;
		fi.attributes = DefaultDirAttributes();
		fi.index = ++m_fileIndexCounter;
		fi.volumeSerialNumber = 1;
		return WriteOverlayEntry(scope, dirKey, dir, filePath.GetFileName(), fi, res.first->second);
	}

	bool DirectoryTableHolder::RegisterCreateFileForWrite(StringKey filePathKey, StringView filePath, u64 fileSize, u64 lastWriteTime, u32 attributes, bool registerRealFile, bool invalidateStorage)
	{
		auto& dirTable = m_directoryTable;

		StringKey dirKey;
		const tchar* lastSlash;
		StringBuffer<> dirName;
		if (!GetDirKey(dirKey, dirName, lastSlash, filePath))
			return true;

		// Remote is not updating its own directory table
		UBA_ASSERT(!m_runningRemote);
		UBA_ASSERTF(!IsInTemp(filePath), TC("Registering temp files to main table not allowed (%.*s)"), filePath.count, filePath.data);

		#if 0//_DEBUG  // Bring this back, turned off right now because a few lines above the call to this method we add a mapping
		{
			SCOPED_FUTEX(m_fileMappingTableLookupLock, lookupLock);
			auto findIt = m_fileMappingTableLookup.find(filePathKey);
			if (findIt != m_fileMappingTableLookup.end())
			{
				FileMappingEntry& entry = findIt->second;
				lookupLock.Leave();
				SCOPED_FUTEX(entry.lock, entryCs);
				UBA_ASSERT(!entry.mapping);
			}
		}
		#endif

		bool shouldWriteToDisk = registerRealFile && ShouldWriteToDisk(filePath);

		// When not writing to disk we need to populate lookup before adding non-written files.. otherwise they will be lost once lookup is actually populated
		if (!shouldWriteToDisk)
		{
			UBA_FOR_ASSERT(u32 res =) WriteDirectoryEntries(dirKey, dirName);
			UBA_ASSERTF(res, TC("Failed to write directory entries for %.*s"), dirName.count, dirName.data);
		}

		SCOPED_READ_LOCK(dirTable.m_lookupLock, lookupCs);
		auto findIt = dirTable.m_lookup.find(dirKey);
		if (findIt == dirTable.m_lookup.end())
			return true;

		DirectoryTable::Directory& dir = findIt->second;
		lookupCs.Leave();

		SCOPED_WRITE_LOCK(dir.lock, dirLock);

		// To prevent race where code creating dir manage to add to lookup but then got here later than this thread.
		while (dir.populatedOffset == 0)
		{
			dirLock.Leave();
			Sleep(1);
			dirLock.Enter();
		}

		// TODO: This should be removed. We don't want to retry here. This should be handled on the outside and never happen.
		// This will save one directory up. If the grand parent dir saw this dir but it didn't see itself somehow then this will save the situation.. but that is very unlikely
		if (dir.populatedOffset == 2)
		{
			dirLock.Leave();
			UBA_FOR_ASSERT(u32 res =) WriteDirectoryEntries(dirKey, dirName);
			UBA_ASSERT(res);
			dirLock.Enter();
		}

		// Refuse to register a file when the parent dir is not in populatedOffset==1.
		// populatedOffset==2 means TraverseDir failed (or parent's cached enumeration
		// didn't list this dir, e.g. dir was created after parent was enumerated). Writing
		// a non-root entry with prev_offset=dir.latestOffset==0 aliases onto the parser's
		// root-entry grammar in PopulateDirectoryRecursive and corrupts the journal.
		if (dir.populatedOffset != 1)
			return m_logger.Warning(TC("Refusing to register create-for-write '%.*s': dir not populated. dir='%.*s' dirKey=%s populatedOffset=%u latestOffset=%u shouldWriteToDisk=%u attributes=0x%x fileSize=%llu lastWriteTime=%llu dirFilesCached=%zu"), filePath.count, filePath.data, dirName.count, dirName.data, KeyToString(dirKey).data, dir.populatedOffset, dir.latestOffset, u32(shouldWriteToDisk), attributes, fileSize, lastWriteTime, dir.files.size());

		if (filePathKey == StringKeyZero)
		{
			StringBuffer<> forKey;
			forKey.Append(filePath);
			if (CaseInsensitiveFs)
				forKey.MakeLower();
			filePathKey = ToStringKey(forKey);
		}
		auto insres = dir.files.try_emplace(filePathKey, ~u32(0));

		u64 fileIndex = InvalidValue;
		u32 volumeSerial = 0;
		bool isDirectory = false;

		if (shouldWriteToDisk)
		{
			FileInformation info;
			if (!GetFileInformation(info, m_logger, filePath))
			{
				if (insres.second)
					dir.files.erase(insres.first);
				return m_logger.Error(TC("Failed to get file information for %.*s while checking file added for write. This should not happen! (%s) [size=%llu lastWriteTime=%llu attributes=0x%x newEntry=%i]"), filePath.count, filePath.data, LastErrorToText().data, fileSize, lastWriteTime, attributes, insres.second ? 1 : 0);
			}

			attributes = info.attributes;
			volumeSerial = info.volumeSerialNumber;
			lastWriteTime = info.lastWriteTime;
			isDirectory = IsDirectory(attributes);
			if (isDirectory)
				fileSize = 0;
			else
				fileSize = info.size;
			fileIndex = info.index;
		}
		else
		{
			// TODO: Do we need more code here?
			UBA_ASSERTF(attributes != 0, TC("Registering file %.*s with no attributes set (Needs to be set from the outside when not writing to disk)"), filePath.count, filePath.data);
			volumeSerial = 1;
			fileIndex = ++m_fileIndexCounter;
		}

		// Check if new write is actually a write. The file might just have been open with write permissions and then actually never written to.
		// We check this by using lastWriteTime. If it hasn't change, directory table is already up-to-date
		if (!insres.second && IsValidEntry(insres.first->second))
		{
			BinaryReader reader(m_directoryTableMem + insres.first->second.internal);
			reader.SkipString();

			u32 oldAttr = reader.ReadFileAttributes();

			if (IsDirectory(oldAttr))  // Ignore updating directories.. they should always be the same regardless
			{
				UBA_ASSERT(isDirectory || !shouldWriteToDisk);
				return true;
			}
			reader.ReadVolumeSerial();
			u64 oldFileIndex = reader.ReadFileIndex();(void)oldFileIndex;

			#if UBA_SUPPORT_SYMLINKS
			// Old entry was a symlink: skip the matching-lastWriteTime
			// fast-path (its semantics differ from regular files) and fall
			// through to re-register as a regular file. Was UBA_ASSERT,
			// but AOSP/Horde edges hit this whenever a path the host-side
			// dirtable previously cached as a symlink gets reported back
			// as a regular file from an edge running on an AL2025 helper.
			const bool oldWasSymlink = IsSymlink(oldAttr);
			#else
			constexpr bool oldWasSymlink = false;
			#endif

			u64 oldLastWriteTime = reader.ReadFileTime();
			if (!oldWasSymlink && lastWriteTime == oldLastWriteTime)
			{
#if !PLATFORM_WINDOWS
				if (oldFileIndex == fileIndex)
				//UBA_ASSERTF(oldFileIndex == fileIndex, TC("Same write time but Not same file? %llu vs %llu (%.*s)"), oldFileIndex, fileIndex, filePath.count, filePath.data); // Checking so it is really the same file
#endif
				{
					u64 oldSize = reader.ReadFileSize();
					if (oldSize == fileSize && oldAttr == attributes) // Only attributes could change from a chmod
						return true;
					// TODO: Somehow this can happen and I have no idea how. last written time should be set on close file so it shouldnt be possible.
					//else
					//	m_logger.Warning(TC("Somehow file %s has same last written time at two points in time but different size (old %llu new %llu)"), filePath.data, oldSize, fileSize);
				}
			}
		}

		// There are directory crawlers happening in parallel so we need to really make sure to invalidate this one since a crawler can actually
		// hit this file with information from a query before it was written.. and then it will turn it back to "verified" using old info
		if (registerRealFile && invalidateStorage)
			InvalidateCachedFileInfo(filePathKey);

		FileEntryAdded(filePathKey, lastWriteTime, fileSize);

		u32 volumeIndex = m_volumeCache.GetSerialIndex(volumeSerial);
		u8 temp[MaxPath * 2 + 256]; // filename + symlink content + entry headers
		u64 written;
		u64 entryPos;
		{
			BinaryWriter writer(temp, 0, sizeof(temp));
			writer.WriteStringKey(dirKey);
			writer.Write7BitEncoded(dir.latestOffset); // Previous entry for same directory
			writer.Write7BitEncoded(1); // Count
			entryPos = writer.GetPosition();
			writer.WriteString(lastSlash + 1);
			writer.WriteFileAttributes(attributes);
			writer.WriteVolumeSerial(volumeIndex);
			writer.WriteFileIndex(fileIndex);

			#if UBA_SUPPORT_SYMLINKS
			if (IsSymlink(attributes))
			{
				UBA_ASSERT(!isDirectory);
				UBA_ASSERT(shouldWriteToDisk);
				char link[MaxPath];
				int res = readlink(filePath.CheckTerminated(), link, sizeof(link));
				UBA_ASSERTF(res != -1, TC("readlink %s failed (errno=%d)"), filePath.data, errno);
				UBA_ASSERTF(res < (int)sizeof(link), TC("readlink %s target exceeds MaxPath (%zu)"), filePath.data, sizeof(link));
				writer.WriteString(StringView(link, res));
				fileSize = res;
			}
			#endif

			if (!isDirectory)
			{
				writer.WriteFileTime(lastWriteTime);
				writer.WriteFileSize(fileSize);
			}

			written = writer.GetPosition();
		}

#if UBA_DEBUG_TRACK_DIR
		m_debugLogger->Info(TC("TRACKADD    %s (Size: %llu, Attr: %u, Key: %s, Id: %llu)"), filePath.data, fileSize, attributes, KeyToString(filePathKey).data, fileIndex);
#endif

		SCOPED_WRITE_LOCK(dirTable.m_memoryLock, memoryLock);
		u32 writePos = dirTable.m_memorySize;
		EnsureDirectoryTableMemory(writePos + 8 + written);
		BinaryWriter writer(m_directoryTableMem + writePos);
		writer.Write7BitEncoded(written); // Storage size
		insres.first->second = {dirTable.m_memorySize + u32(writer.GetPosition() + entryPos)}; // Storing position to last write time
		u32 tableOffset = u32(writer.GetPosition()) + sizeof(StringKey);
		writer.WriteBytes(temp, written);
		dir.latestOffset = dirTable.m_memorySize + tableOffset;
		dirTable.m_memorySize += u32(writer.GetPosition());
		return true;
	}

	bool DirectoryTableHolder::RegisterDeleteFile(StringKey filePathKey, StringView filePath)
	{
		StringKey dirKey;
		const tchar* lastSlash;
		StringBuffer<> dirName;
		if (!GetDirKey(dirKey, dirName, lastSlash, filePath))
			return false;

		UBA_ASSERT(!m_runningRemote);
		UBA_ASSERT(!IsInTemp(filePath));

		auto& dirTable = m_directoryTable;
		SCOPED_READ_LOCK(dirTable.m_lookupLock, lookupCs);
		auto res = dirTable.m_lookup.find(dirKey);
		if (res == dirTable.m_lookup.end())
			return true;
		DirectoryTable::Directory& dir = res->second;
		lookupCs.Leave();
		SCOPED_WRITE_LOCK(dir.lock, dirLock);

		while (dir.populatedOffset == 0)
		{
			dirLock.Leave();
			Sleep(1);
			dirLock.Enter();
		}

		if (dir.populatedOffset == 2) // Invalid directory, don't need to register this
			return true;

		if (dir.populatedOffset != 1)
			return m_logger.Error(TC("Refusing to register delete '%.*s': dir not populated. dir='%.*s' dirKey=%s populatedOffset=%u latestOffset=%u dirFilesCached=%zu"), filePath.count, filePath.data, dirName.count, dirName.data, KeyToString(dirKey).data, dir.populatedOffset, dir.latestOffset, dir.files.size());

		if (filePathKey == StringKeyZero)
		{
			StringBuffer<> forKey;
			forKey.Append(filePath);
			if (CaseInsensitiveFs)
				forKey.MakeLower();
			filePathKey = ToStringKey(forKey);
		}

		// Does not exist, no need adding to file table
		if (dir.files.erase(filePathKey) == 0)
			return true;

		u8 temp[1024];
		u64 written;
		{
			BinaryWriter writer(temp, 0, sizeof(temp));
			writer.WriteStringKey(dirKey);
			writer.Write7BitEncoded(dir.latestOffset); // Previous entry for same directory
			writer.Write7BitEncoded(1); // Count
			writer.WriteString(lastSlash + 1);
			writer.WriteFileAttributes(0);
			writer.WriteVolumeSerial(0);
			writer.WriteFileIndex(0);
			if (true) // !IsDirectory()
			{
				writer.WriteFileTime(0);
				writer.WriteFileSize(0);
			}
			written = writer.GetPosition();
		}

#if UBA_DEBUG_TRACK_DIR
		m_debugLogger->Info(TC("TRACKDEL    %s (Key: %s)"), filePath.data, KeyToString(filePathKey).data);
#endif

		{
			SCOPED_WRITE_LOCK(dirTable.m_memoryLock, memoryLock);
			u32 writePos = dirTable.m_memorySize;
			EnsureDirectoryTableMemory(writePos + 8 + written);
			BinaryWriter writer(m_directoryTableMem + writePos);
			writer.Write7BitEncoded(written); // Storage size
			u32 tableOffset = u32(writer.GetPosition()) + sizeof(StringKey);
			writer.WriteBytes(temp, written);
			dir.latestOffset = dirTable.m_memorySize + tableOffset;
			dirTable.m_memorySize += u32(writer.GetPosition());
		}
		return true;
	}

	u32 DirectoryTableHolder::WriteDirectoryEntries(StringKey dirKey, StringView dirPath, DirTableOffset* outTableOffset)
	{
		UBA_ASSERT(!m_runningRemote);
		auto& dirTable = m_directoryTable;
		DirTableOffset temp;
		if (!outTableOffset)
			outTableOffset = &temp;
		WriteDirectoryEntriesRecursive(*outTableOffset, dirKey, dirPath, {});
		SCOPED_READ_LOCK(dirTable.m_memoryLock, memoryLock);
		return dirTable.m_memorySize;
	}

	void DirectoryTableHolder::WriteDirectoryEntriesRecursive(DirTableOffset& outTableOffset, StringKey dirKey, StringView dirPath, StringKey tableOffsetIsFile)
	{
		auto& dirTable = m_directoryTable;

		SCOPED_WRITE_LOCK(dirTable.m_lookupLock, lookupLock);
		auto res = dirTable.m_lookup.try_emplace(dirKey, dirTable.m_memoryBlock);
		DirectoryTable::Directory& dir = res.first->second;
		lookupLock.Leave();

		// Compute parent path / key (might not exist for root). No locks held — this only
		// reads the input dirPath.
		u64 dirlen = dirPath.count;
		StringView parentDirPath;
		StringKey parentKey = StringKeyZero;
		bool hasParent = false;
		if (dirlen)
		{
			StringView parentDirPath2 = dirPath.GetPath();


			const tchar* rit = (tchar*)dirPath.data + dirlen - 2;
			while (rit > dirPath.data)
			{
				if (*rit != PathSeparator) { --rit; continue; }
				break;
			}
			if (rit > dirPath.data)
			{
				parentDirPath = StringView(dirPath.data, u32(rit - dirPath.data));
				StringBuffer<> parentDirForHash;
				parentDirForHash.Append(parentDirPath);
				if (CaseInsensitiveFs)
					parentDirForHash.MakeLower();
				parentKey = ToStringKey(parentDirForHash);
				hasParent = true;
			}
#if !PLATFORM_WINDOWS
			// Direct child of the POSIX root: parent is the root, represented as the empty
			// dirPath in this codebase (children of root are keyed as hash("/" + "/name"),
			// so root's own hash is the empty hasher). Without this branch the symlink-prefix
			// safety net below is unreachable for /sbin, /lib, /var, … so the realpath check
			// in WriteDirectoryEntriesInternal asserts whenever any of those is a usrmerge
			// symlink (e.g. /sbin -> usr/sbin on modern Debian/Ubuntu).
			else if (rit == dirPath.data && *dirPath.data == PathSeparator)
			{
				parentDirPath = StringView(dirPath.data, 0);
				parentKey = ToStringKey(parentDirPath);
				hasParent = true;
			}
#endif

			UBA_ASSERTF(parentDirPath2.Equals(parentDirPath), TC("Paths should be the same (%.*s vs %.*s)"), parentDirPath2.count, parentDirPath2.data, parentDirPath.count, parentDirPath.data);
		}

		DirTableOffset parentOffset;
		if (hasParent)
			WriteDirectoryEntriesRecursive(parentOffset, parentKey, parentDirPath, dirKey);

		auto UpdateOffsetToFile = [&]()
			{
				if (tableOffsetIsFile == StringKeyZero || dirKey == m_directoryForcedEmpty)
					return;
				outTableOffset = {};
				auto findIt = dir.files.find(tableOffsetIsFile);
				if (findIt == dir.files.end())
					return;
				outTableOffset = findIt->second;
				UBA_ASSERT(IsValidEntry(outTableOffset));

				#if UBA_SUPPORT_SYMLINKS
				BinaryReader reader(dirTable.GetReader(outTableOffset.internal));
				reader.SkipString();
				u32 attr = reader.ReadFileAttributes();
				if (IsSymlink(attr))
					outTableOffset = {};
				#endif
			};

		SCOPED_WRITE_LOCK(dir.lock, dirLock);

		if (dir.populatedOffset == 1)
		{
			outTableOffset = {dir.latestOffset};
			UpdateOffsetToFile();
			return;
		}

		if (hasParent && !IsValidEntry(parentOffset))
		{
			outTableOffset = {};
			dir.populatedOffset = 2;
			return;
		}

		if (WriteDirectoryEntriesInternal(outTableOffset, dir, dirKey, dirPath, false))
		{
			dir.populatedOffset = 1;
			UpdateOffsetToFile();
		}
		else
		{
			outTableOffset = {};
			dir.populatedOffset = 2;
		}

		dirLock.Leave();

		if (hasParent && parentKey == m_directoryForcedEmpty)
			RegisterCreateFileForWrite(parentKey, dirPath, 0, 0, DefaultDirAttributes(), true, false);
	}

	bool DirectoryTableHolder::WriteDirectoryEntriesInternal(DirTableOffset& outTableOffset, DirectoryTable::Directory& dir, const StringKey& dirKey, StringView dirPath, bool isRefresh)
	{
		UBA_ASSERT(dir.populatedOffset != 3);

		if (dir.latestOffset != InvalidTableOffset && !isRefresh)
			isRefresh = true;

		auto& dirTable = m_directoryTable;

		u32 volumeSerial = 0;
		u32 volumeSerialIndex = 0;
		u32 dirAttributes = 0;
		u64 fileIndex = 0;

		u64 written = 0;
		u32 itemCount = 0;

		StringKeyHasher hasher;
		if (dirPath.count)
		{
			StringBuffer<> forHash;
			forHash.Append(dirPath);
			if (CaseInsensitiveFs)
				forHash.MakeLower();
			hasher.Update(forHash.data, forHash.count);
		}

#if UBA_DEBUG_TRACK_DIR
		m_debugLogger->BeginScope();
		auto dg = MakeGuard([&]() { m_debugLogger->EndScope(); });
		StringBuffer<> str;
		str.Append(TCV("TRACKDIR "));
		if (isRefresh)
			str.Append(TCV("(Refresh) "));
		str.Append(dirPath).Append('\n');
		m_debugLogger->Log(LogEntryType_Info, str);
#endif


		Vector<u8> memoryBlock;
		memoryBlock.resize(4096);
		BinaryWriter memoryWriter(memoryBlock.data(), 0, memoryBlock.size());

		enum : u32 { NewFileFlag = 0x40000000 };

		if (dirKey != m_directoryForcedEmpty)
		{
			StringBuffer<4> realPath;
			if constexpr (IsWindows)
			{
				if (dirPath.count == 2)
					dirPath = realPath.Append(dirPath).Append(PathSeparator);
			}
			else
			{
				if (!dirPath.count)
					dirPath = realPath.Append(PathSeparator);
			}

			bool res = TraverseDir(m_logger, dirPath,
				[&](const DirectoryEntry& e)
				{
					StringBuffer<256> fileNameForHash;
					fileNameForHash.Append(PathSeparator).Append(e.name, e.nameLen);
					if (CaseInsensitiveFs)
						fileNameForHash.MakeLower();

					StringKey fileKey = ToStringKey(hasher, fileNameForHash.data, fileNameForHash.count);
					auto res = dir.files.try_emplace(fileKey, ~0u);
					if (!res.second)
						return;
					UBA_ASSERT(e.attributes);
					UBA_ASSERT(e.nameLen < 512);

					res.first->second = {u32(memoryWriter.GetPosition()) | NewFileFlag }; // Temporary offset that will be used further down to calculate the real offset

					memoryWriter.WriteString(e.name, e.nameLen);

					#if UBA_DEBUG_TRACK_DIR
					m_debugLogger->Info(TC("    %s (Size: %llu, Attr: %u, Key: %s, Id: %llu)"), e.name, e.size, e.attributes, KeyToString(fileKey).data, e.id);
					#endif

					u64 id = e.id;
					if (id == 0xffffffffffffffffllu) // When using projfs we might not have the file yet and in that case we need to make this up.
						id = ++m_fileIndexCounter;

					memoryWriter.WriteFileAttributes(e.attributes);
					memoryWriter.WriteVolumeSerial(e.volumeSerial == volumeSerial ? volumeSerialIndex : m_volumeCache.GetSerialIndex(e.volumeSerial));
					memoryWriter.WriteFileIndex(id);

					#if UBA_SUPPORT_SYMLINKS
					if (IsSymlink(e.attributes))
					{
						fileNameForHash.Clear().Append(dirPath).EnsureEndsWithSlash().Append(e.name, e.nameLen);
						UBA_ASSERTF(!IsDirectory(e.attributes), TC("%s can't be both symlink and directory"), fileNameForHash.data);
						char link[MaxPath]; // /proc magic symlinks report lstat size=0; trust readlink result.
						int res2 = readlink(fileNameForHash.data, link, sizeof(link));
						UBA_ASSERTF(res2 != -1, TC("readlink %s failed (errno=%d)"), fileNameForHash.data, errno);
						UBA_ASSERTF(res2 < (int)sizeof(link), TC("readlink %s target exceeds MaxPath (%zu)"), fileNameForHash.data, sizeof(link));
						memoryWriter.WriteString(StringView(link, res2));
					}
					#endif

					if (!IsDirectory(e.attributes))
					{
						memoryWriter.WriteFileTime(e.lastWritten);
						memoryWriter.WriteFileSize(e.size);
					}

					FileEntryAdded(res.first->first, e.lastWritten, e.size);


					++itemCount;
					if (memoryWriter.GetPosition() > memoryBlock.size() - MaxPath)
					{
						memoryBlock.resize(memoryBlock.size() * 2);
						memoryWriter.ChangeData(memoryBlock.data(), memoryBlock.size());
					}
				}, true,
				[&](const DirectoryInfo& e)
					{
						volumeSerial = e.volumeSerial;
						volumeSerialIndex = m_volumeCache.GetSerialIndex(volumeSerial);
						dirAttributes = e.attributes;
						fileIndex = e.id;
					});

			if (!res)
			{
				UBA_ASSERT(!itemCount);
				#if UBA_DEBUG_TRACK_DIR
				m_debugLogger->Info(TC("    FAILED (not existing?)"));
				#endif
				return false;
			}

			// With symlinks we must only populate real paths — a swapped symlink would make a whole subtree wrong.
			#if UBA_SUPPORT_SYMLINKS
			if constexpr (IsDebug)
			{
				char real[512];
				StringBuffer<> terminated(dirPath);
				char* r = realpath(terminated.data, real);
				UBA_ASSERT(r);
				UBA_ASSERTF(terminated.Equals(real), TC("Traversing non-real dir path (path: %s  real: %s"), terminated.data, real);
			}
			#endif
		}
		else
		{
			#if UBA_DEBUG_TRACK_DIR
			m_debugLogger->Info(TC("    FORCED EMPTY"));
			#endif

			dirAttributes = DefaultDirAttributes();
		}

		written = memoryWriter.GetPosition();

		u64 storageSizeBase = sizeof(StringKey) + Get7BitEncodedCount(itemCount) + written;

		u32 tableOffset;

		SCOPED_WRITE_LOCK(dirTable.m_memoryLock, memoryLock);
		u32 snapshotLatestOffset = dir.latestOffset;
		u64 storageSize = storageSizeBase + Get7BitEncodedCount(snapshotLatestOffset);
		u32 writePos = dirTable.m_memorySize;
		EnsureDirectoryTableMemory(writePos + 128 + storageSize);
		BinaryWriter tableWriter(m_directoryTableMem + dirTable.m_memorySize);

		if (isRefresh)
		{
			tableWriter.Write7BitEncoded(storageSize);
			tableWriter.WriteStringKey(dirKey);
			tableOffset = writePos + u32(tableWriter.GetPosition());
			tableWriter.Write7BitEncoded(snapshotLatestOffset);
		}
		else
		{
			storageSize += Get7BitEncodedCount(dirAttributes) + Get7BitEncodedCount(volumeSerialIndex) + sizeof(fileIndex);
			tableWriter.Write7BitEncoded(storageSize);
			tableWriter.WriteStringKey(dirKey);
			tableOffset = writePos + u32(tableWriter.GetPosition());
			tableWriter.Write7BitEncoded(snapshotLatestOffset);
			tableWriter.WriteFileAttributes(dirAttributes);
			tableWriter.WriteVolumeSerial(volumeSerialIndex);
			tableWriter.WriteFileIndex(fileIndex);
		}

		tableWriter.Write7BitEncoded(itemCount);
		u32 filesOffset = writePos + u32(tableWriter.GetPosition());
		tableWriter.WriteBytes(memoryBlock.data(), written);
		dirTable.m_memorySize += u32(tableWriter.GetPosition());
		dir.latestOffset = tableOffset;

		memoryLock.Leave();

		// Update offsets to be relative to full memory
		for (auto& kv : dir.files)
		{
			u32& offset = kv.second.internal;
			UBA_ASSERT(offset != ~0u);
			if (offset & NewFileFlag)
				kv.second.internal = (offset & ~NewFileFlag) + filesOffset;
		}

		outTableOffset = {tableOffset};
		return true;
	}

	void DirectoryTableHolder::EnsureDirectoryTableMemory(u64 neededSize)
	{
		auto& dirTable = m_directoryTable;
		if (neededSize <= m_directoryTableMemCommitted)
			return;

		u64 newSize = AlignUp(neededSize, u64(1024*1024));
		if (newSize > DirTableMaxSize)
		{
			static bool called = m_logger.Error(TC("Directory table overflow. DirTableMaxSize need to be increased (Size: %llu)"), dirTable.m_memorySize);
		}

		u64 toCommit = newSize - m_directoryTableMemCommitted;
		u8* address = m_directoryTableMem + m_directoryTableMemCommitted;
		if (!FileMapping_Commit(m_logger, address, toCommit, false))
			m_logger.Error(TC("Failed to commit memory for directory table (Committed: %llu, ToCommit: %llu) (%s)"), m_directoryTableMemCommitted, toCommit, LastErrorToText().data);
		m_directoryTableMemCommitted += toCommit;
		LockMemory(m_logger, address, toCommit, TC("DirectoryTable"));
	}

	bool DirectoryTableHolder::IsInTemp(StringView path)
	{
		return false;
	}

	void DirectoryTableHolder::FileEntryAdded(StringKey filePathKey, u64 lastWritten, u64 size)
	{
	}

	bool DirectoryTableHolder::ShouldWriteToDisk(StringView fileName)
	{
		return true;
	}

	void DirectoryTableHolder::InvalidateCachedFileInfo(const StringKey& filePathKey)
	{
	}

	bool DirectoryTableHolder::WriteDirectoryTableToFile(StringView filePath)
	{
		FileAccessor fa(m_logger, filePath.CheckTerminated());
		if (!fa.CreateWrite())
			return false;

		MemoryBlock memoryBlock(TC("WriteDirTableToFile"));
		DirectoryTable dirTable(memoryBlock);
		dirTable.Init(m_directoryTableMem, u32(m_directoryTable.m_lookup.size()), u32(m_directoryTable.m_memorySize));
		dirTable.TraverseAllFilesNoLock([&](const DirectoryTable::EntryInformation& info, StringView fileName, DirTableOffset fileOffset)
			{
				char buf[512];
				u32 len = fileName.Parse(buf, sizeof(buf));
				UBA_ASSERT(len);
				buf[len-1] = '\n';
				fa.Write(buf, len);
			});
		return fa.Close();
	}

	#if UBA_DEBUG_LOGGER
	void DirectoryTableHolder::CreateDebugLogger(const tchar* outputFile)
	{
		m_debugLogger = StartDebugLogger(m_logger, outputFile);
	}
	#endif

	bool CreateDirectoryTableSnapshot(DirTableSnapshot& out, u64 mappingSize,
		const u8* prevSnapshot, u64 prevSnapshotSize,
		const u8* mainMem, u32 mainSize,
		const u8* overlayMem, u32 overlaySize,
		const tchar* hint, Logger& logger)
	{
		(void)prevSnapshotSize;

		auto AlignUpU32 = [](u32 v, u32 a) { return (v + a - 1) & ~(a - 1); };
		u32 memoryBlockOffset = AlignUpU32(u32(sizeof(DirectoryTable)), 64);
		u32 allocPoolOffset   = AlignUpU32(memoryBlockOffset + u32(sizeof(MemoryBlock)), 64);

		if (mappingSize < u64(allocPoolOffset) + 64*1024)
			return logger.Error(TC("CreateDirectoryTableSnapshot: mappingSize %llu too small (need %u)"), mappingSize, allocPoolOffset + 64*1024);
		if (mappingSize > u64(~u32(0)))
			return logger.Error(TC("CreateDirectoryTableSnapshot: mappingSize %llu exceeds 4 GB"), mappingSize);

		// Prev's m_lookup holds raw Node* pointers — caller must keep prev mapped at the same VA.
		auto* prevTable = reinterpret_cast<const DirectoryTable*>(prevSnapshot);

		FileMappingHandle handle = FileMapping_Create(logger, PAGE_READWRITE, mappingSize, nullptr, hint ? hint : TC("DirSnapshot"));
		if (!handle.IsValid())
			return logger.Error(TC("CreateDirectoryTableSnapshot: FileMapping_Create failed (size=%llu)"), mappingSize);

		u8* base = FileMapping_Map(logger, handle, FILE_MAP_WRITE, 0, mappingSize);
		if (!base)
		{
			FileMapping_Close(logger, handle, TC("DirSnapshot"));
			return logger.Error(TC("CreateDirectoryTableSnapshot: FileMapping_Map failed (size=%llu)"), mappingSize);
		}

		// Must never be destructed — Deinit would unmap the FileMapping.
		MemoryBlock* mb = new (base + memoryBlockOffset) MemoryBlock(TC("DirSnapshotMem"), base + allocPoolOffset);
		mb->reserveSize    = mappingSize - allocPoolOffset;
		mb->committedSize  = mappingSize - allocPoolOffset;
		mb->writtenSize    = 0;

		DirectoryTable* table = new (base) DirectoryTable(*mb);

		// Heap-backed during build because MemoryBlock::Free is a no-op for mapping-backed memory;
		// compact_buckets() at the end snaps the final exact-size array into the arena.
		table->m_lookup.set_bucket_alloc(DirectoryTable::BucketAllocator{});

		u32 mainStart = 0;
		u32 overlayStart = 0;
		if (prevSnapshot)
		{
			for (auto& kv : prevTable->m_lookup)
			{
				auto ins = table->m_lookup.try_emplace(kv.first, *mb);
				DirectoryTable::Directory& d = ins.first->second;
				d.latestOffset        = kv.second.latestOffset;
				d.latestOverlayOffset = kv.second.latestOverlayOffset;
			}
			mainStart    = prevTable->m_memorySize;
			overlayStart = prevTable->m_overlaySize;
			if (mainStart > mainSize) { logger.Warning(TC("CreateDirectoryTableSnapshot: prev m_memorySize %u > mainSize %u; clamping"), mainStart, mainSize); mainStart = mainSize; }
			if (overlayStart > overlaySize) { logger.Warning(TC("CreateDirectoryTableSnapshot: prev m_overlaySize %u > overlaySize %u; clamping"), overlayStart, overlaySize); overlayStart = overlaySize; }
		}

		auto CountFirstRecordsInRange = [](const u8* mem, u32 from, u32 to) -> u32
		{
			if (!mem || from >= to) return 0;
			u32 count = 0;
			BinaryReader reader(mem, from, to);
			while (reader.GetPosition() < to)
			{
				u64 storageSize = reader.Read7BitEncoded();
				u64 dataStart = reader.GetPosition();
				reader.ReadStringKey();
				if (reader.Read7BitEncoded() == InvalidTableOffset)
					++count;
				reader.SetPosition(dataStart + storageSize);
			}
			return count;
		};

		u32 lookupReserve = 0;
		if (prevTable) lookupReserve += u32(prevTable->m_lookup.size());
		lookupReserve += CountFirstRecordsInRange(mainMem,    mainStart,    mainSize);
		lookupReserve += CountFirstRecordsInRange(overlayMem, overlayStart, overlaySize);
		table->m_lookup.reserve(lookupReserve);

		if (mainMem && mainSize > mainStart)
			table->ParseDirectoryTableNoLock(mainMem, mainStart, mainSize, 0);
		if (overlayMem && overlaySize > overlayStart)
			table->ParseDirectoryTableNoLock(overlayMem, overlayStart, overlaySize, OverlayTableFlag);

		constexpr u32 IsDirBit = 0x40000000u;
		static_assert(!(IsDirBit & OverlayTableFlag), "IsDirBit must not collide with OverlayTableFlag");
		static_assert(!(IsDirBit & DirectoryEntryFlag), "IsDirBit must not collide with DirectoryEntryFlag");

		struct PopulateContext
		{
			DirectoryTable* table;
			MemoryBlock* mb;
			const u8* mainMem;
			const u8* overlayMem;
			Vector<u32> recOffsets;
			Vector<u32> subdirOffsets;
			StringBuffer<MaxPath> pathBuf;
			UnorderedSet<StringKey> killed; // heap-backed; arena Deallocate is a no-op so deleted keys go here, not into files

			u64 ChainItemCount(u32 latestOffset, u32 isOverlayFlag)
			{
				const u8* mem = isOverlayFlag ? overlayMem : mainMem;
				if (latestOffset == InvalidTableOffset || !mem)
					return 0;
				u64 sum = 0;
				u32 cur = latestOffset & ~isOverlayFlag;
				while (true)
				{
					BinaryReader r(mem, cur);
					u32 prev = u32(r.Read7BitEncoded());
					if (prev == InvalidTableOffset)
					{
						u32 dirAttr = r.ReadFileAttributes();
						if (dirAttr)
						{
							r.ReadVolumeSerial();
							r.ReadFileIndex();
							sum += r.Read7BitEncoded();
						}
						break;
					}
					sum += r.Read7BitEncoded();
					cur = prev;
				}
				return sum;
			}

			// Newest->oldest, first-seen-wins. Older records for keys already in `files` or
			// `killed` are skipped — that's what keeps doomed keys from ever allocating a Node.
			void WalkChain(u32 latestOffset, u32 isOverlayFlag, DirectoryTable::EntryLookup& files)
			{
				const u8* mem = isOverlayFlag ? overlayMem : mainMem;
				if (latestOffset == InvalidTableOffset || !mem)
					return;

				recOffsets.clear();
				u32 cur = latestOffset & ~isOverlayFlag;
				while (true)
				{
					BinaryReader r(mem, cur);
					u32 prev = u32(r.Read7BitEncoded());
					recOffsets.push_back(u32(r.GetPosition()));
					if (prev == InvalidTableOffset)
						break;
					cur = prev;
				}

				u32 prefixLen = pathBuf.count;
				u32 recCount = u32(recOffsets.size());
				for (u32 i = 0; i < recCount; ++i)
				{
					bool isRoot = (i + 1 == recCount);
					BinaryReader r(mem, recOffsets[i]);
					if (isRoot)
					{
						u32 dirAttr = r.ReadFileAttributes();
						if (!dirAttr)
							continue;
						r.ReadVolumeSerial();
						r.ReadFileIndex();
					}
					u64 itemCount = r.Read7BitEncoded();
					while (itemCount--)
					{
						u32 itemOffset = u32(r.GetPosition());

						pathBuf.Resize(prefixLen);
						pathBuf.Append(PathSeparator);
						u32 nameStart = pathBuf.count;
						r.ReadString(pathBuf);
						if (CaseInsensitiveFs)
							for (u32 j = nameStart; j < pathBuf.count; ++j)
								pathBuf.data[j] = ToLower(pathBuf.data[j]);
						u32 attr = r.ReadFileAttributes();
						r.ReadVolumeSerial();
						r.ReadFileIndex();
						#if UBA_SUPPORT_SYMLINKS
						if (IsSymlink(attr))
							r.SkipString();
						#endif
						if (!IsDirectory(attr))
						{
							r.ReadFileTime();
							r.ReadFileSize();
						}

						StringKey key = ToStringKey(pathBuf.data, pathBuf.count);

						if (files.find(key) != files.end())
							continue;
						if (killed.find(key) != killed.end())
							continue;

						if (attr == 0)
						{
							killed.insert(key);
							continue;
						}

						u32 v = itemOffset | isOverlayFlag;
						if (IsDirectory(attr))
							v |= IsDirBit;
						files.try_emplace(key, DirTableOffset{v});
					}
				}

				pathBuf.Resize(prefixLen);
			}

			void Populate()
			{
				StringKeyHasher pathHasher;
				if (pathBuf.count)
					pathHasher.Update(pathBuf.data, pathBuf.count);
				StringKey dirKey = ToStringKey(pathHasher);

				auto findIt = table->m_lookup.find(dirKey);
				if (findIt == table->m_lookup.end())
					return;
				DirectoryTable::Directory& dir = findIt->second;
				if (dir.populatedOffset == dir.latestOffset && dir.populatedOverlayOffset == dir.latestOverlayOffset)
					return;

				dir.files.set_bucket_alloc(DirectoryTable::BucketAllocator{});

				u64 itemUpperBound = ChainItemCount(dir.latestOffset, 0) + ChainItemCount(dir.latestOverlayOffset, OverlayTableFlag);
				if (itemUpperBound > 0)
					dir.files.reserve(u32(dir.files.size() + itemUpperBound));

				// Overlay first so its newer add/delete decisions are seen before main's older records.
				killed.clear();
				WalkChain(dir.latestOverlayOffset, OverlayTableFlag, dir.files);
				WalkChain(dir.latestOffset,        0,                dir.files);

				dir.files.compact_buckets(DirectoryTable::BucketAllocator{*mb});

				u32 subdirsStart = u32(subdirOffsets.size());
				for (auto& kv : dir.files)
				{
					if (kv.second.internal & IsDirBit)
					{
						subdirOffsets.push_back(kv.second.internal & ~IsDirBit);
						kv.second.internal &= ~IsDirBit;
					}
				}

				dir.populatedOffset        = dir.latestOffset;
				dir.populatedOverlayOffset = dir.latestOverlayOffset;

				u32 prePushLen = pathBuf.count;
				u32 subdirsEnd = u32(subdirOffsets.size());
				for (u32 i = subdirsStart; i < subdirsEnd; ++i)
				{
					u32 readerOffset = subdirOffsets[i];
					BinaryReader r = table->GetReader(readerOffset);
					if (prePushLen == 0 || pathBuf.data[prePushLen - 1] != PathSeparator)
						pathBuf.Append(PathSeparator);
					u32 nameStart = pathBuf.count;
					r.ReadString(pathBuf);
					if (CaseInsensitiveFs)
						for (u32 j = nameStart; j < pathBuf.count; ++j)
							pathBuf.data[j] = ToLower(pathBuf.data[j]);
					Populate();
					pathBuf.Resize(prePushLen);
				}
				subdirOffsets.resize(subdirsStart);
			}
		};

		table->m_memory        = mainMem;
		table->m_memorySize    = mainSize;
		table->m_overlayMemory = overlayMem;
		table->m_overlaySize   = overlaySize;

		PopulateContext ctx;
		ctx.table      = table;
		ctx.mb         = mb;
		ctx.mainMem    = mainMem;
		ctx.overlayMem = overlayMem;
		ctx.recOffsets.reserve(64);

		#if PLATFORM_WINDOWS
		for (tchar l = 'a'; l != 'z'; ++l)
		{
			ctx.pathBuf.Clear();
			ctx.pathBuf.Append(l).Append(':');
			ctx.Populate();
		}
		#else
		ctx.pathBuf.Clear();
		ctx.Populate();
		#endif

		// +256 headroom: consumer-side dirs added at runtime should fit without a rehash that
		// would leak the old bucket array into the mapped arena.
		table->m_lookup.compact_buckets(DirectoryTable::BucketAllocator{*mb}, u32(table->m_lookup.size() + 256));

		out.handle       = handle;
		out.mappedMemory = base;
		out.mappedSize   = u32(allocPoolOffset + mb->writtenSize);
		return true;
	}
}