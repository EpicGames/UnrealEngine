// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaEnvironment.h"
#include "UbaFile.h"
#include "UbaPlatform.h"
#include "UbaStringBuffer.h"

#if PLATFORM_WINDOWS
#include <powerbase.h>
#pragma comment(lib, "Powrprof.lib")
#include <winternl.h>
#elif PLATFORM_LINUX
#include <sys/prctl.h>
#include <sys/stat.h>
#include <sys/types.h>
#include <poll.h>
#include <fcntl.h>
#include <unistd.h>
#include <stdlib.h>
#include <signal.h>
#include <dirent.h>
#include <string.h>
#ifndef __NR_pidfd_open
#define __NR_pidfd_open 434
#endif
#elif PLATFORM_MAC
#include <mach/vm_statistics.h>
#include <mach/mach_types.h>
#include <mach/mach_init.h>
#include <mach/mach_host.h>
#include <mach/mach.h>
#include <sys/sysctl.h>
#include <sys/stat.h>
#include <sys/types.h>
#include <fcntl.h>
#include <unistd.h>
#include <signal.h>
#include <stdlib.h>
#include <string.h>
#include <libproc.h>
#endif

#if PLATFORM_WINDOWS
typedef struct _PROCESSOR_POWER_INFORMATION {
    ULONG  Number;
    ULONG  MaxMhz;
    ULONG  CurrentMhz;
    ULONG  MhzLimit;
    ULONG  MaxIdleState;
    ULONG  CurrentIdleState;
} PROCESSOR_POWER_INFORMATION, *PPROCESSOR_POWER_INFORMATION;
#else
extern char** environ;
#endif

namespace uba
{
	MutexHandle CreateMutexW(bool bInitialOwner, const tchar* lpName)
	{
		#if PLATFORM_WINDOWS
		return (MutexHandle)(u64)::CreateMutexW(NULL, bInitialOwner, lpName);
		#else
		// TODO: This is used to check for exclusivity and also for trace streams (only created by host and read by visualizer)
		SetLastError(ERROR_SUCCESS);
		return ((MutexHandle)(u64)1337); // Just some random value
		#endif
	}

	void ReleaseMutex(MutexHandle mutex)
	{
		if (mutex == InvalidMutexHandle)
			return;
		#if PLATFORM_WINDOWS
		::ReleaseMutex((HANDLE)mutex);
		#else
		#endif
	}

	void CloseMutex(MutexHandle mutex)
	{
		if (mutex == InvalidMutexHandle)
			return;
		#if PLATFORM_WINDOWS
		::CloseHandle((HANDLE)mutex);
		#else
		#endif
	}

	u32 GetEnvironmentVariableW(const tchar* name, tchar* buffer, u32 nSize)
	{
		#if PLATFORM_WINDOWS
		return ::GetEnvironmentVariableW(name, buffer, nSize);
		#else
		const char* env = getenv(name);
		if (!env)
		{
			SetLastError(203); // ERROR_ENVVAR_NOT_FOUND
			return 0;
		}

		auto envLen = strlen(env);
		if (nSize <= envLen)
			return envLen + 1;
		memcpy(buffer, env, envLen + 1);
		return envLen;
		#endif
	}

	bool SetEnvironmentVariableW(const tchar* name, const tchar* value)
	{
		#if PLATFORM_WINDOWS
		return ::SetEnvironmentVariableW(name, value);
		#else
		return setenv(name, value, 1) == 0;
		#endif
	}

	u32 ExpandEnvironmentStringsW(const tchar* lpSrc, tchar* lpDst, u32 nSize)
	{
		#if PLATFORM_WINDOWS
		return ::ExpandEnvironmentStringsW(lpSrc, lpDst, nSize);
		#else
		UBA_ASSERTF(false, TC("ExpandEnvironmentStringsW not implemented (%s)"), lpSrc);
		return 0;
		#endif
	}

	ProcHandle GetCurrentProcessHandle()
	{
		#if PLATFORM_WINDOWS
		return (ProcHandle)(u64)GetCurrentProcess();
		#else
		UBA_ASSERTF(false, TC("GetCurrentProcessHandle not implemented"));
		return (ProcHandle)0;
		#endif
	}

	u32 GetLogicalProcessorCount()
	{
		#if PLATFORM_WINDOWS
		return ::GetActiveProcessorCount(ALL_PROCESSOR_GROUPS);
		#else
		return (u32)sysconf(_SC_NPROCESSORS_ONLN);
		#endif
	}

	u32 GetProcessorGroupCount()
	{
		#if PLATFORM_WINDOWS
		static u32 s_processorGroupCount = u32(GetActiveProcessorGroupCount());
		if (s_processorGroupCount)
			return s_processorGroupCount;
		#endif
		return 1u;
	}

	void ElevateCurrentThreadPriority()
	{
		#if PLATFORM_WINDOWS
		//SetThreadPriority(GetCurrentThread(), THREAD_PRIORITY_HIGHEST);
		#elif PLATFORM_MAC
		pthread_set_qos_class_self_np(QOS_CLASS_USER_INITIATED, 0);
		#endif
	}

#if PLATFORM_WINDOWS
	u64 GetMaxPageFileSize(Logger& logger, MEMORYSTATUSEX& memStatus)
	{
		struct SYSTEM_PAGEFILE_INFORMATION
		{
			ULONG NextEntryOffset;
			ULONG TotalSize;      // in pages
			ULONG TotalInUse;     // in pages
			ULONG PeakUsage;      // in pages
			UNICODE_STRING PageFileName;
		};

		Vector<u8> queryBuffer(64*1024);
		ULONG returnedSize;
		constexpr auto systemPageFileInformation = (SYSTEM_INFORMATION_CLASS)0x12;
		if (NtQuerySystemInformation(systemPageFileInformation, queryBuffer.data(), (u32)queryBuffer.size(), &returnedSize) == STATUS_SUCCESS)
		{
			u64 totalSize = 0;
			u8* entryIt = queryBuffer.data();
			while (true)
			{
				auto& info = *reinterpret_cast<SYSTEM_PAGEFILE_INFORMATION*>(entryIt);
				totalSize += u64(info.TotalSize)*4096;
				if (info.NextEntryOffset == 0)
					break;
				entryIt += info.NextEntryOffset;
			}
			return totalSize;
		}

		// Old fallback path.. doesn't really work well with system managed page files because they take into account lots of things
		// There is some binary blob inside the registry that contains some sort of history etc.

		tchar systemDrive = 'c';
		{
			tchar temp[32];
			if (GetEnvironmentVariableW(TC("SystemDrive"), temp, 32))
				systemDrive = ToLower(temp[0]);
		}

		u64 maxPageSize = 0;
		wchar_t str[1024];
		DWORD strBytes = sizeof(str);
		LSTATUS res = RegGetValueW(HKEY_LOCAL_MACHINE, TC("SYSTEM\\CurrentControlSet\\Control\\Session Manager\\Memory Management"), TC("PagingFiles"), RRF_RT_REG_MULTI_SZ, NULL, str, &strBytes);
		if (res == ERROR_SUCCESS)
		{
			DWORD pagefileOnOsVolume = 0;
			DWORD pagefileOnOsVolumeSize = 4;
			RegGetValueW(HKEY_LOCAL_MACHINE, TC("SYSTEM\\CurrentControlSet\\Control\\Session Manager\\Memory Management"), TC("PagefileOnOsVolume"), RRF_RT_DWORD, NULL, &pagefileOnOsVolume, &pagefileOnOsVolumeSize);

			wchar_t* line = str;
			for (; size_t lineLen = wcslen(line); line += lineLen + 1)
			{
				if (lineLen < 3)
					continue;

				u64 maxSizeMb = 0;

				StringBuffer<8> drive;
				drive.Append(line, 3); // Get drive root path

				if (drive[0] == '?') // Drive '?' can exist when "Automatically manage paging file size for all drives".. 
				{
					// We can use ExistingPageFiles registry key to figure out which drive...
					// This key can contain multiple page files normally.. have no idea if it can contain multiple when drive is '?'.. but for now, just use the first
					wchar_t str2[1024];
					DWORD str2Bytes = sizeof(str2);
					res = RegGetValueW(HKEY_LOCAL_MACHINE, TC("SYSTEM\\CurrentControlSet\\Control\\Session Manager\\Memory Management"), TC("ExistingPageFiles"), RRF_RT_REG_MULTI_SZ, NULL, str2, &str2Bytes);
					if (res != ERROR_SUCCESS)
						continue;

					auto colon = wcschr(str2, ':'); // Path is something like \??\C:\pagefile.sys or similar.. let's search for : and use character in front of it.
					if (colon == nullptr || colon == str2)
						continue;

					drive[0] = colon[-1];

					if (pagefileOnOsVolume && ToLower(drive[0]) != systemDrive)
						continue;
				}
				else if (!pagefileOnOsVolume || ToLower(drive[0]) == systemDrive)
				{
					const wchar_t* maxSizeStr = wcsrchr(line, ' ');
						
					if (!maxSizeStr || !StringBuffer<32>(maxSizeStr + 1).Parse(maxSizeMb))
					{
						logger.Warning(TC("Unrecognized page file information format (please report): %s"), line);
						continue;
					}

					if (maxSizeMb) // Custom set page file size
					{
						maxPageSize += maxSizeMb * 1024 * 1024;
						continue;
					}
				}
				else
				{
					logger.Warning(TC("Page file is set on drive %c: but registry key value PagefileOnOsVolume is set to 1. Fix registry"), drive[0]);
				}

				// Max possible system-managed page file
				u64 maxSize = Max(u64(memStatus.ullTotalPhys) * 3, 4ull * 1024 * 1024 * 1024);

				// There seems to be something inside windows that limits the system managed pagefile..
				// .. machines with 256gb ram and 2tb os drives still hit oom at ~326gb
				maxSize = Min(maxSize, 64ull * 1024 * 1024 * 1024);

				// Check if disk is limiting factor of system-managed page file
				// Page file can be max 1/8 of volume size and ofc not more than free space
				ULARGE_INTEGER totalNumberOfBytes;
				ULARGE_INTEGER totalNumberOfFreeBytes;
				if (!GetDiskFreeSpaceExW(drive.data, NULL, &totalNumberOfBytes, &totalNumberOfFreeBytes))
					return logger.Error(TC("GetDiskFreeSpaceExW failed to get information about %s (%s)"), drive.data, LastErrorToText().data);

				u64 maxDiskPageFileSize = Min(totalNumberOfBytes.QuadPart / 8, totalNumberOfFreeBytes.QuadPart);
				maxPageSize += Min(maxDiskPageFileSize, maxSize);
			}
		}
		return maxPageSize;
	}
#endif

	void GetMachineId(StringBufferBase& out)
	{
		bool success = false;
		tchar str[256];

#if PLATFORM_WINDOWS
		DWORD strBytes = sizeof(str);
		success = RegGetValueW(HKEY_LOCAL_MACHINE, TC("SOFTWARE\\Microsoft\\Cryptography"), TC("MachineGuid"), RRF_RT_REG_SZ, NULL, str, &strBytes) == ERROR_SUCCESS;
		if (success)
			out.Append(str);
#elif PLATFORM_LINUX
		if (FILE* id = fopen("/etc/machine-id", "r"))
		{
			success = fgets(str, sizeof_array(str), id) != nullptr;
			fclose(id);
			if (success)
				out.Append(str);
		}
#else
		(void)str;
		success = GetComputerNameW(out); // TODO: Revisit below
		//if (FILE* id = popen("ioreg -rd1 -c IOPlatformExpertDevice | awk -F\\\" '/IOPlatformUUID/ { print $4 }'", "r"))
		//{
		//	success = fgets(str, sizeof(str), id) != nullptr;
		//	fclose(id);
		//	if (success)
		//		out.Append(str, strcspn(str, "\n"));
		//}
#endif
		if (!success)
			out.Append(TCV("NoMachineId"));
	}

	bool GetMemoryInfo(Logger& logger, u64& outAvailable, u64& outTotal, u64* outMaxPageFileSize)
	{
		if (outMaxPageFileSize)
			*outMaxPageFileSize = 0;

#if PLATFORM_WINDOWS
		MEMORYSTATUSEX memStatus = { sizeof(memStatus) };
		if (!GlobalMemoryStatusEx(&memStatus))
		{
			outAvailable = 0;
			outTotal = 0;
			return logger.Error(TC("Failed to get global memory status (%s)"), LastErrorToText().data);
		}

		// Page file can grow and we want to use the absolute max size to figure out when we need to wait to start new processes
		static u64 maxPageSize = GetMaxPageFileSize(logger, memStatus);

		if (outMaxPageFileSize)
			*outMaxPageFileSize = maxPageSize;

		u64 currentPageSize = memStatus.ullTotalPageFile - memStatus.ullTotalPhys;
		if (currentPageSize < maxPageSize)
		{
			outTotal = memStatus.ullTotalPhys + maxPageSize;
			outAvailable = memStatus.ullAvailPageFile + (maxPageSize - currentPageSize);
		}
		else
		{
			outTotal = memStatus.ullTotalPageFile;
			outAvailable = memStatus.ullAvailPageFile;
		}
#elif PLATFORM_MAC
		u64 memKb;
		GetPhysicallyInstalledSystemMemory(memKb);
		outTotal = memKb*1024*1024;

		mach_msg_type_number_t count = HOST_VM_INFO64_COUNT;
		vm_statistics64_data_t vm_stat;
		if (host_statistics64(mach_host_self(), HOST_VM_INFO64, (host_info64_t)&vm_stat, &count) == KERN_SUCCESS)
		{
			vm_size_t page_size;
			host_page_size(mach_host_self(), &page_size);
			uint64_t active      = vm_stat.active_count * page_size;
			uint64_t wired       = vm_stat.wire_count * page_size;
			uint64_t compressed  = vm_stat.compressor_page_count * page_size;
			uint64_t used = active + wired + compressed;
			outAvailable = outTotal - used;
		}
		else
		{
			outAvailable = outTotal;
		}
#else
		u64 memKb;
		GetPhysicallyInstalledSystemMemory(memKb);
		outTotal = memKb*1024*1024;
		outAvailable = outTotal;
#endif
		return true;
	}

	void GetSystemInfo(Logger& logger, StringBufferBase& out)
	{
		u32 cpuCount = GetLogicalProcessorCount();
		u32 cpuGroupCount = GetProcessorGroupCount();

		StringBuffer<128> cpuStr(TC("CPU"));
		if (IsRunningArm())
			cpuStr.Append(TCV("[Arm]"));
		cpuStr.Append(':');
		if (cpuGroupCount != 1)
			cpuStr.AppendValue(cpuGroupCount).Append('x');
		cpuStr.AppendValue(cpuCount/cpuGroupCount);

		u64 totalMemoryInKilobytes = 0;

#if PLATFORM_WINDOWS
		GetPhysicallyInstalledSystemMemory(&totalMemoryInKilobytes);

		{
			u32 maxMHz = 0;
			DWORD valueSize = 4;
			const tchar* key = TC("HARDWARE\\DESCRIPTION\\System\\CentralProcessor\\0");
			LSTATUS res = RegGetValueW(HKEY_LOCAL_MACHINE, key, TC("~MHz"), RRF_RT_REG_DWORD, NULL, &maxMHz, &valueSize);
			if (res != ERROR_SUCCESS)
			{
				// This will not always be the same and since we use the system info as part of key for client uniqueness it is annoying to get multiple sessions for same instance
				Vector<PROCESSOR_POWER_INFORMATION> procInfos;
				procInfos.resize(cpuCount);
				if (CallNtPowerInformation(ProcessorInformation, NULL, 0, procInfos.data(), cpuCount*sizeof(PROCESSOR_POWER_INFORMATION)) == STATUS_SUCCESS)
					maxMHz = procInfos[0].MaxMhz;
			}
			cpuStr.Appendf(TC(" @ %.1fGHz"), float(maxMHz) / 1000.0f);
		}

#else
		u64 throwAway;
		GetMemoryInfo(logger, throwAway, totalMemoryInKilobytes);
		totalMemoryInKilobytes /= 1024;
		double processorMhz = 0.0;
	#if PLATFORM_LINUX
		if (FILE *fp = fopen("/proc/cpuinfo", "r"))
		{
			char line[256];
			while (processorMhz == 0.0 && fgets(line, sizeof(line), fp))
				if (strncmp(line, "cpu MHz", 7) == 0)
					sscanf(line, "cpu MHz : %lf", &processorMhz);
			fclose(fp);
			cpuStr.Appendf(" @ %.1fGHz", processorMhz/1000.0);
		}
	#else
		char brand[128];
		size_t size = sizeof(brand);
		if (sysctlbyname("machdep.cpu.brand_string", brand, &size, NULL, 0) == 0)
			cpuStr.Clear().Append(brand).Appendf(" CPU:%u", cpuCount);
	#endif
#endif

		out.Appendf(TC("%s Mem:%ugb"), cpuStr.data, u32(totalMemoryInKilobytes/(1024*1024)));

		#if PLATFORM_WINDOWS
		if (!IsRunningWine())
		{
			DWORD value = 0;
			DWORD valueSize = 4;
			const tchar* fsKey = TC("SYSTEM\\CurrentControlSet\\Control\\FileSystem");
			LSTATUS res = RegGetValueW(HKEY_LOCAL_MACHINE, fsKey, TC("NtfsDisableLastAccessUpdate"), RRF_RT_REG_DWORD, NULL, &value, &valueSize);
			if (res != ERROR_SUCCESS)
			{
				logger.Detail(TC("Failed to retreive ntfs registry key (%i)"), res);
			}
			else
			{
				u32 lastAccessSettingsValue = value & 0xf;
				if (lastAccessSettingsValue == 0 || lastAccessSettingsValue == 2)
					out.Append(TCV(" NtfsLastAccessEnabled"));
			}
			value = 0;
			res = RegGetValueW(HKEY_LOCAL_MACHINE, fsKey, TC("NtfsDisable8dot3NameCreation"), RRF_RT_REG_DWORD, NULL, &value, &valueSize);
			if (res == ERROR_SUCCESS)
				if (value == 0)
					out.Append(TCV(" NtfsShortNamesEnabled"));
		}
		else
		{
			//StringBuffer<> testDir;
			//testDir.Append(m_rootDir).Append(TCV("UbaTestShortNames"));
			//::RemoveDirectory(testDir.data);
			//wchar_t shortName[1024];
			//if (::CreateDirectoryW(testDir.data, NULL))
			//	if (GetShortPathName(testDir.data, shortName, 1024) != 0 && !Contains(shortName, TC("UbaTestShortNames")))
			//		out.Append(TCV(" NtfsShortNamesEnabled"));
		}
		#endif
	}

	bool GetCpuTime(u64& outTotalTime, u64& outIdleTime)
	{
		outTotalTime = 0;
		outIdleTime = 0;
#if PLATFORM_WINDOWS

		GROUP_AFFINITY originalAffinity {};
		GROUP_AFFINITY newAffinity{};
		static u16 groupCount = GetActiveProcessorGroupCount();
		if (groupCount <= 1)
		{
			u64 idleTime, kernelTime, userTime;
			if (!GetSystemTimes((FILETIME*)&idleTime, (FILETIME*)&kernelTime, (FILETIME*)&userTime))
				return false;
			outIdleTime += idleTime;
			outTotalTime += kernelTime + userTime;
		}
		else
		{
			for (u16 group=0; group!=groupCount; ++group)
			{
				newAffinity.Mask = ~0ull;
				newAffinity.Group = group;
				if (!SetThreadGroupAffinity(GetCurrentThread(), &newAffinity, group == 0 ? &originalAffinity : NULL))
					return false;
				u64 idleTime, kernelTime, userTime;
				if (!GetSystemTimes((FILETIME*)&idleTime, (FILETIME*)&kernelTime, (FILETIME*)&userTime))
					return false;
				outIdleTime += idleTime;
				outTotalTime += kernelTime + userTime;
			}
			if (!SetThreadGroupAffinity(GetCurrentThread(), &originalAffinity, nullptr))
				return false;
		}

#elif PLATFORM_LINUX
		int fd = open("/proc/stat", O_RDONLY | O_CLOEXEC);
		if (fd == -1)
			return false;
		char buffer[512];
		int size = read(fd, buffer, sizeof_array(buffer) - 1);
		close(fd);
		if (size == -1)
			return false;
		buffer[size] = 0;
		char* endl = strchr(buffer, '\n');
		if (!endl)
			return false;
		u64 values[16] = { 0 };
		u32 valueCount = 0;
		*endl = 0;
		char* parsePos = buffer;

		// cpu
		parsePos = strchr(parsePos, ' ');
		if (!parsePos)
			return false;

		while (true)
		{
			// remove spaces
			while (*parsePos && (*parsePos < '0' || *parsePos > '9'))
				++parsePos;
			if (!*parsePos)
				break;
			char* numberStart = parsePos;
			while (*parsePos && *parsePos >= '0' && *parsePos <= '9')
				++parsePos;
			bool isLast = *parsePos == 0;
			*parsePos = 0;
			++parsePos;
			values[valueCount++] = strtoull(numberStart, nullptr, 10);
			if (isLast)
				break;
		}

		// user: normal processes executing in user mode
		// nice: niced processes executing in user mode
		// system: processes executing in kernel mode
		// idle: twiddling thumbs
		// iowait: waiting for I/O to complete
		// irq: servicing interrupts
		// softirq: servicing softirqs
		// steal
		if (valueCount <= 6)
			return false;
		u64 work = values[0] + values[1] + values[2];
		outIdleTime = values[3] + values[4] + values[5] + values[6] + values[7];
		outTotalTime = work + outIdleTime;

#else // PLATFORM_MAC
		mach_msg_type_number_t  CpuMsgCount = 0;
		processor_flavor_t CpuInfoType = PROCESSOR_CPU_LOAD_INFO;;
		natural_t CpuCount = 0;
		processor_cpu_load_info_t CpuData;
		host_t host = mach_host_self();
		u64 work = 0;
		int res = host_processor_info(host, CpuInfoType, &CpuCount, (processor_info_array_t *)&CpuData, &CpuMsgCount);
		if(res != KERN_SUCCESS)
			return false;//m_logger.Error(TC("Kernel error: %s"), mach_error_string(res));
		for(int i = 0; i < (int)CpuCount; i++)
		{
			work += CpuData[i].cpu_ticks[CPU_STATE_SYSTEM];
			work += CpuData[i].cpu_ticks[CPU_STATE_USER];
			work += CpuData[i].cpu_ticks[CPU_STATE_NICE];
			outIdleTime += CpuData[i].cpu_ticks[CPU_STATE_IDLE];
		}
		outTotalTime = work + outIdleTime;
#endif
		return true;
	}

	bool g_memoryLockEnabled;

	void SetMemoryWorkingSet(Logger& logger, u64 minWorkingSet)
	{
		if (!SetMemoryWorkingSet(minWorkingSet))
		{
			LogEntryType type = LogEntryType_Detail;
			#if UBA_DEBUG
			type = LogEntryType_Error;
			#endif
			logger.Logf(type, TC("Failed to set memory working set (%s)"), LastErrorToText().data);
			return;
		}

		logger.Detail(TC("MemoryWorkingSet min set to %s"), BytesToText(minWorkingSet).str);
		g_memoryLockEnabled = true;
	}

	void LockMemory(Logger& logger, void* dest, u64 size, const tchar* hint)
	{
		if (!g_memoryLockEnabled)
			return;

		#if PLATFORM_WINDOWS
		if (!VirtualLock(dest, size))
		{
			g_memoryLockEnabled = false;
			LogEntryType type = LogEntryType_Detail;
			#if UBA_DEBUG
			type = LogEntryType_Error;
			#endif
			logger.Logf(type, TC("LockMemory of %s failed for %s (%s)"), BytesToText(size).str, hint, LastErrorToText().data);
		}
		#endif
	}

	#if PLATFORM_LINUX
	static char g_orphanReaperCgroup[512];

	const char* GetOrphanReaperCgroup() { return g_orphanReaperCgroup; }

	static bool TaskHasSessionTag(pid_t p, const char* tag, size_t tagLen)
	{
		char path[64];
		int n = snprintf(path, sizeof(path), "/proc/%d/environ", (int)p);
		if (n <= 0 || (u32)n >= sizeof(path)) return false;
		int fd = ::open(path, O_RDONLY | O_CLOEXEC);
		if (fd < 0) return false;
		char buf[8192];
		ssize_t r = ::read(fd, buf, sizeof(buf) - 1);
		::close(fd);
		if (r <= 0) return false;
		for (ssize_t i = 0; i < r; )
		{
			const char* entry = buf + i;
			ssize_t entryLen = 0;
			while (i + entryLen < r && buf[i + entryLen]) ++entryLen;
			if (entryLen > 20 && strncmp(entry, "UBA_SESSION_PROCESS=", 20) == 0)
				return entryLen == (ssize_t)(20 + tagLen) && strncmp(entry + 20, tag, tagLen) == 0;
			i += entryLen + 1;
		}
		return false;
	}

	// Two passes catch fork()s racing with the walk. Caller handles the leader.
	static void KillDescendantsViaProc(pid_t rootPid)
	{
		char hostTag[16];
		int hostTagLenI = snprintf(hostTag, sizeof(hostTag), "%d", (int)::getpid());
		size_t hostTagLen = (hostTagLenI > 0) ? (size_t)hostTagLenI : 0;

		pid_t stack[1024];
		u32 stackSize = 0;
		stack[stackSize++] = rootPid;

		for (int pass = 0; pass < 2; ++pass)
		{
			u32 cursor = 0;
			if (pass) stack[stackSize++] = rootPid;

			while (cursor < stackSize && stackSize < sizeof(stack)/sizeof(stack[0]) - 64)
			{
				pid_t p = stack[cursor++];

				char taskDir[64];
				int n = snprintf(taskDir, sizeof(taskDir), "/proc/%d/task", (int)p);
				if (n <= 0 || (u32)n >= sizeof(taskDir)) continue;
				DIR* d = ::opendir(taskDir);
				if (!d) continue;
				while (struct dirent* e = ::readdir(d))
				{
					if (e->d_name[0] < '0' || e->d_name[0] > '9') continue;
					char childrenPath[128];
					n = snprintf(childrenPath, sizeof(childrenPath), "%s/%s/children", taskDir, e->d_name);
					if (n <= 0 || (u32)n >= sizeof(childrenPath)) continue;
					int fd = ::open(childrenPath, O_RDONLY | O_CLOEXEC);
					if (fd < 0) continue;
					char buf[4096];
					ssize_t r = ::read(fd, buf, sizeof(buf) - 1);
					::close(fd);
					if (r <= 0) continue;
					buf[r] = 0;
					char* tok = buf;
					while (*tok)
					{
						while (*tok && (*tok < '0' || *tok > '9')) ++tok;
						if (!*tok) break;
						pid_t cp = 0;
						while (*tok >= '0' && *tok <= '9') { cp = cp * 10 + (*tok - '0'); ++tok; }
						if (cp <= 1 || cp == rootPid) continue;
						if (hostTagLen && !TaskHasSessionTag(cp, hostTag, hostTagLen)) continue;
						::kill(cp, SIGKILL);
						if (stackSize < sizeof(stack)/sizeof(stack[0]))
							stack[stackSize++] = cp;
					}
				}
				::closedir(d);
			}
		}
	}
	#endif

	#if PLATFORM_MAC
	// No environ-via-/proc on macOS; can't filter by UBA_SESSION_PROCESS like the Linux variant.
	static void KillDescendantsViaProc(pid_t rootPid)
	{
		pid_t stack[1024];
		u32 stackSize = 0;
		stack[stackSize++] = rootPid;

		for (int pass = 0; pass < 2; ++pass)
		{
			u32 cursor = 0;
			if (pass) stack[stackSize++] = rootPid;

			while (cursor < stackSize && stackSize < sizeof(stack)/sizeof(stack[0]) - 64)
			{
				pid_t p = stack[cursor++];
				pid_t kids[256];
				int n = proc_listpids(PROC_PPID_ONLY, (uint32_t)p, kids, (int)sizeof(kids));
				if (n <= 0) continue;
				int count = n / (int)sizeof(pid_t);
				for (int i = 0; i < count; ++i)
				{
					pid_t cp = kids[i];
					if (cp <= 1 || cp == rootPid) continue;
					::kill(cp, SIGKILL);
					if (stackSize < sizeof(stack)/sizeof(stack[0]))
						stack[stackSize++] = cp;
				}
			}
		}
	}
	#endif

	#if PLATFORM_LINUX || PLATFORM_MAC
	void CancelProcessTree(int leaderPid)
	{
		if (leaderPid <= 1) return;
		::kill(-leaderPid, SIGKILL);
		KillDescendantsViaProc((pid_t)leaderPid);
		::kill((pid_t)leaderPid, SIGKILL);
	}
	#endif

	#if PLATFORM_LINUX

	// Build /sys/fs/cgroup<rel>/uba-<pid>/ from /proc/self/cgroup's v2 line `0::<rel>` and mkdir.
	// Failure (EACCES/EROFS/no cgroupv2/no delegation) leaves the path empty; watchdog falls back
	// to /proc-walk by UBA_SESSION_PROCESS env-var match — same correctness, just not kernel-atomic.
	static void TryCreateSessionCgroup()
	{
		int fd = ::open("/proc/self/cgroup", O_RDONLY | O_CLOEXEC);
		if (fd >= 0)
		{
			char buf[1024];
			ssize_t n = ::read(fd, buf, sizeof(buf) - 1);
			::close(fd);
			if (n > 0)
			{
				buf[n] = 0;
				const char* line = buf;
				while (line && *line)
				{
					if (line[0] == '0' && line[1] == ':' && line[2] == ':')
					{
						const char* path = line + 3;
						const char* eol = path;
						while (*eol && *eol != '\n') ++eol;
						const char base[] = "/sys/fs/cgroup";
						constexpr u32 baseLen = sizeof(base) - 1;
						u32 rel = u32(eol - path);
						if (baseLen + rel + 32 < sizeof(g_orphanReaperCgroup))
						{
							char tmp[sizeof(g_orphanReaperCgroup)];
							int len = snprintf(tmp, sizeof(tmp), "%s%.*s/uba-%d", base, (int)rel, path, (int)::getpid());
							if (len > 0 && ::mkdir(tmp, 0755) == 0)
								memcpy(g_orphanReaperCgroup, tmp, size_t(len) + 1);
						}
						break;
					}
					while (*line && *line != '\n') ++line;
					if (*line == '\n') ++line;
				}
			}
		}

		// UBA_VERBOSE_ORPHAN_REAPER=1 prints how to enable cgroup v2 delegation (Delegate=yes in user@.service.d).
		if (!g_orphanReaperCgroup[0])
		{
			const char* v = getenv("UBA_VERBOSE_ORPHAN_REAPER");
			if (v && *v && *v != '0')
			{
				LoggerWithWriter logger(g_consoleLogWriter);
				logger.Info(TC("Orphan reaper using /proc-walk fallback (cgroup v2 delegation unavailable). "
					"For kernel-atomic cgroup.kill: "
					"sudo mkdir -p /etc/systemd/system/user@.service.d && "
					"echo -e '[Service]\\nDelegate=yes' | sudo tee /etc/systemd/system/user@.service.d/delegate.conf && "
					"sudo systemctl daemon-reexec, then log out and back in."));
			}
		}
	}

	// Args: $1=host_pid  $2=cgroup_path (empty if unavailable)  $3=host_pgid.
	// /proc walk catches setsid/nsjail descendants that kill(-pgid) misses; three passes catch grandchildren.
	static const char kWatchdogScript[] =
		"while kill -0 \"$1\" 2>/dev/null; do sleep 0.2; done; "
		"if [ -n \"$2\" ] && [ -d \"$2\" ]; then "
		"  printf 1 > \"$2/cgroup.kill\" 2>/dev/null; "
		"  rmdir \"$2\" 2>/dev/null; "
		"else "
		"  if [ -n \"$3\" ]; then kill -KILL -- -\"$3\" 2>/dev/null; fi; "
		"  self=$$; pass=0; "
		"  while [ $pass -lt 3 ]; do "
		"    killed=0; "
		"    for d in /proc/[0-9]*; do "
		"      p=${d#/proc/}; "
		"      [ \"$p\" = \"$self\" ] && continue; "
		// Wrap pipe in (...) 2>/dev/null so the shell's own redirect-failure message is suppressed too.
		"      if ( tr '\\0' '\\n' < \"$d/environ\" | grep -q \"^UBA_SESSION_PROCESS=$1$\" ) 2>/dev/null; then "
		"        kill -KILL \"$p\" 2>/dev/null && killed=1; "
		"      fi; "
		"    done; "
		"    [ $killed -eq 0 ] && break; "
		"    sleep 0.1; pass=$((pass+1)); "
		"  done; "
		"fi";

	void SetupOrphanReaper()
	{
		if (setpgid(0, 0) != 0 && errno != EPERM) {}

		TryCreateSessionCgroup();

		pid_t parentPid = getpid();
		pid_t parentPgid = getpgrp();

		pid_t watcher = fork();
		if (watcher < 0)
			return;
		if (watcher == 0)
		{
			setpgid(0, 0); // leave host's pgid so kill(-pgid) below can't hit us
			prctl(PR_SET_NAME, (unsigned long)"uba-orphankill", 0, 0, 0);

			char hostStr[16], pgidStr[16];
			snprintf(hostStr, sizeof(hostStr), "%d", (int)parentPid);
			snprintf(pgidStr, sizeof(pgidStr), "%d", (int)parentPgid);
			const char* cgroupArg = g_orphanReaperCgroup[0] ? g_orphanReaperCgroup : "";
			char* argv[] = { (char*)"uba-orphankill", (char*)"-c", (char*)kWatchdogScript, (char*)"uba-orphankill", hostStr, (char*)cgroupArg, pgidStr, nullptr };
			execve("/bin/sh", argv, ::environ);
			_exit(127);
		}
	}
	#endif // PLATFORM_LINUX (SetupOrphanReaper)

	#if PLATFORM_MAC
	// `ps -E -ww -ax` prints env after command; awk match is anywhere on the line.
	static const char kMacWatchdogScript[] =
		"while kill -0 \"$1\" 2>/dev/null; do sleep 0.2; done; "
		"self=$$; pass=0; "
		"while [ $pass -lt 3 ]; do "
		"  killed=0; "
		"  for p in $(ps -E -ww -ax -o pid=,command= 2>/dev/null | "
		"             awk -v t=\"UBA_SESSION_PROCESS=$1\" -v me=\"$self\" "
		"                 '$0 ~ t && $1 != me { print $1 }'); do "
		"    kill -KILL \"$p\" 2>/dev/null && killed=1; "
		"  done; "
		"  [ $killed -eq 0 ] && break; "
		"  sleep 0.1; pass=$((pass+1)); "
		"done";

	void SetupOrphanReaper()
	{
		pid_t parentPid = getpid();

		pid_t watcher = fork();
		if (watcher < 0)
			return;
		if (watcher == 0)
		{
			setsid(); // detach so host's session-end signals don't kill us

			char hostStr[16];
			snprintf(hostStr, sizeof(hostStr), "%d", (int)parentPid);
			char* argv[] = { (char*)"uba-orphankill", (char*)"-c", (char*)kMacWatchdogScript, (char*)"uba-orphankill", hostStr, nullptr };
			execve("/bin/sh", argv, ::environ);
			_exit(127);
		}
	}
	#endif // PLATFORM_MAC

	#if PLATFORM_WINDOWS
	StringView GetUbaWineModulePath()
	{
		static StringView sv = []()
			{
				static StringBuffer<> dllPath;
				LoggerWithWriter logger(g_consoleLogWriter);
				if (GetDirectoryOfCurrentModule(logger, dllPath))
					dllPath.EnsureEndsWithSlash();
				dllPath.Append(TCV("UbaWine.dll.so"));
				return StringView(dllPath);
			}();
		return sv;
	}
	void* GetUbaWineModule()
	{
		static HMODULE ubaWineModule = []()
			{
				StringView sv = GetUbaWineModulePath();
				HMODULE res = LoadLibraryW(sv.data);
				if (!res)
					LoggerWithWriter(g_consoleLogWriter).Info(TC("Failed to load UbaWine module from %s"), sv.data);
				return res;
			}();
		return ubaWineModule;
	}
	#endif
}
