// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaFile.h"
#include "UbaDirectoryIterator.h"
#include "UbaEnvironment.h"
#include "UbaEvent.h"
#include "UbaPathUtils.h"
#include "UbaProcessStats.h"
#include <algorithm>

#if PLATFORM_WINDOWS
#include <io.h>
#include <psapi.h>
#include <WinCon.h>
#include <RestartManager.h>
#pragma comment(lib, "Rstrtmgr.lib")
extern "C" NTSTATUS NTAPI NtQueryInformationFile(HANDLE FileHandle, PIO_STATUS_BLOCK IoStatusBlock, PVOID FileInformation, ULONG Length, FILE_INFORMATION_CLASS FileInformationClass);
extern "C" NTSTATUS NTAPI NtQueryFullAttributesFile(const OBJECT_ATTRIBUTES *attr, FILE_NETWORK_OPEN_INFORMATION *info);
#else
#include <dlfcn.h>
#include <execinfo.h>
#include <sys/time.h>
#endif

#if PLATFORM_MAC
#include <mach-o/dyld.h>
#include <sys/clonefile.h>
#include <copyfile.h>
#endif

#define UBA_WSL_WORKAROUND 0 // (PLATFORM_LINUX && UBA_DEBUG)  // WSL IS BROKEN, don't know how to work around this. Somehow a directory can be created successfully but then when creating child dir it does not exist yet

namespace uba
{
#if PLATFORM_WINDOWS
	void GetProcessHoldingFile(StringBufferBase& out, const tchar* fileName)
	{
		DWORD dwSession;
		WCHAR szSessionKey[CCH_RM_SESSION_KEY + 1] = { 0 };
		DWORD dwError = RmStartSession(&dwSession, 0, szSessionKey);
		if (dwError != ERROR_SUCCESS)
			return;
		auto sg = MakeGuard([dwSession]() { RmEndSession(dwSession); });
		dwError = RmRegisterResources(dwSession, 1, &fileName, 0, NULL, 0, NULL);
		if (dwError != ERROR_SUCCESS)
			return;

		DWORD dwReason;
		UINT nProcInfoNeeded;
		UINT nProcInfo = 10;
		RM_PROCESS_INFO rgpi[10] = {};
		dwError = RmGetList(dwSession, &nProcInfoNeeded, &nProcInfo, rgpi, &dwReason);
		if (dwError != ERROR_SUCCESS)
			return;

		for (u32 i = 0; i < nProcInfo; i++)
		{
			HANDLE hProcess = OpenProcess(PROCESS_QUERY_LIMITED_INFORMATION, FALSE, rgpi[i].Process.dwProcessId);
			if (!hProcess)
				continue;
			auto pg = MakeGuard([hProcess]() { CloseHandle(hProcess); });

			FILETIME ftCreate, ftExit, ftKernel, ftUser;
			if (!GetProcessTimes(hProcess, &ftCreate, &ftExit, &ftKernel, &ftUser))
				continue;
			if (CompareFileTime(&rgpi[i].Process.ProcessStartTime, &ftCreate) != 0)
				continue;
			WCHAR sz[MaxPath];
			DWORD cch = MaxPath;
			if (!QueryFullProcessImageNameW(hProcess, 0, sz, &cch))
				continue;
			if (cch <= MaxPath)
				out.Appendf(TC(" - %s"), sz); // Using Appendf to have capacity check
		}
	}

	template<typename T>
	void PrefixFileName(T& fileName, StringView prefix, StringBufferBase& temp)
	{
		StringView sv(ToView(fileName));
		if (IsAbsolutePath(sv) && !IsUncPath(sv))
		{
			UBA_ASSERT(sv.count < MaxPath);
			temp.Append(prefix);
			FixPath(temp, sv, {});
			if constexpr (std::is_same_v<T, StringView>)
				fileName = temp;
			else
				fileName = temp.data;
		}
		else if constexpr (std::is_same_v<T, StringView>)
		{
			if (fileName.data[fileName.count] != 0) // must be null terminated
			{
				temp.Append(fileName);
				fileName = temp;
			}
		}
		
	}

	#define PREFIX_FILENAME(fileName, prefix) \
		StringBuffer<MaxPath> STRING_JOIN(temp, __LINE__); \
		PrefixFileName(fileName, prefix, STRING_JOIN(temp, __LINE__));
	#define MAKE_LONG_FILENAME(fileName) PREFIX_FILENAME(fileName, TCV("\\\\?\\"))
	#define MAKE_NT_FILENAME(fileName)  PREFIX_FILENAME(fileName, TCV("\\??\\"))
#else
	#define ENSURE_TERMINATED(fileName) \
		StringBuffer<MaxPath> STRING_JOIN(temp, __LINE__); \
		if (fileName.data[fileName.count] != 0) \
		{ \
			STRING_JOIN(temp, __LINE__).Append(fileName); \
			fileName = STRING_JOIN(temp, __LINE__); \
		}
#endif


	bool ReadFile(Logger& logger, const tchar* fileName, FileHandle fileHandle, void* b, u64 bufferLen)
	{
		auto& stats = KernelStats::GetCurrent();
		ExtendedTimerScope ts(stats.readFile);
		u8* buffer = (u8*)b;
		u64 readLeft = bufferLen;
		u64 firstZeroReadTime = 0;

		while (readLeft)
		{
			u32 toRead = u32(Min(readLeft, u64(~u32(0)) - 1));
#if PLATFORM_WINDOWS
			DWORD wasRead = 0;
			if (!::ReadFile(AsHANDLE(fileHandle), buffer, toRead, &wasRead, NULL))
				if (GetLastError() != ERROR_IO_PENDING)
					return logger.Error(TC("ERROR reading %llu bytes from file %s with handle %llu (error: %s)"), toRead, fileName, AsHANDLE(fileHandle), LastErrorToText().data);
#else
			ssize_t wasRead = read(AsFileDescriptor(fileHandle), buffer, toRead);
			if (wasRead == -1)
			{
				UBA_ASSERTF(false, TC("ERROR ReadFile error handling not implemented (Trying to read %u bytes from fd %llu)"), toRead, fileHandle);
				return false;
			}
#endif
			if (wasRead == 0)
			{
				if (firstZeroReadTime == 0)
					firstZeroReadTime = GetTime();
				else if (TimeToMs(GetTime() - firstZeroReadTime) > 3*1000)
					return logger.Error(TC("ERROR reading file %s trying to read %u bytes from offset %llu but ReadFile returns 0 bytes read.. Is the file big enough?"), fileName, toRead, bufferLen - readLeft);
			}

			readLeft -= wasRead;
			buffer += wasRead;
		}

		stats.readFile.bytes += bufferLen;
		return true;
	}

	bool WriteFile(Logger& logger, const tchar* fileName, FileHandle fileHandle, const void* b, u64 bufferLen)
	{
		auto& stats = KernelStats::GetCurrent();
		ExtendedTimerScope ts(stats.writeFile);
		const u8* buffer = (const u8*)b;
		u64 writeLeft = bufferLen;
		while (writeLeft)
		{
#if PLATFORM_WINDOWS
			DWORD written = 0;
			if (!::WriteFile(AsHANDLE(fileHandle), buffer, (DWORD)writeLeft, &written, NULL))
				return logger.Error(TC("ERROR WriteFile failed writing to file %s (%s)"), fileName, LastErrorToText().data);
#else
			auto written = write(AsFileDescriptor(fileHandle), buffer, writeLeft);
			if (written == -1)
			{
				if (errno == EINTR)
					continue; // signal-interrupted; retry from same position
				return false;
			}
#endif
			buffer += written;
			writeLeft -= written;
		}
		return true;
	}

	UBA_NOINLINE bool ReportOpenFileError(Logger& logger, const tchar* fileName, u32 lastError) // To prevent stack usage
	{
		StringBuffer<4096> additionalInfo;
		#if PLATFORM_WINDOWS
		if (lastError == ERROR_SHARING_VIOLATION)
			GetProcessHoldingFile(additionalInfo, fileName);
		#endif
		return logger.Error(TC("ERROR opening file %s for read (%s%s)"), fileName, LastErrorToText(lastError).data, additionalInfo.data);
	}

	bool OpenFileSequentialRead(Logger& logger, const tchar* fileName, FileHandle& outHandle, bool fileNotFoundIsError, bool overlapped, bool showErrors)
	{
		u32 dwFlagsAndAttributes = DefaultAttributes();
		#if PLATFORM_WINDOWS
		dwFlagsAndAttributes |= (overlapped ? FILE_FLAG_OVERLAPPED : FILE_FLAG_SEQUENTIAL_SCAN);
		MAKE_LONG_FILENAME(fileName);
		#endif

		outHandle = uba::CreateFileW(fileName, GENERIC_READ, FILE_SHARE_READ|FILE_SHARE_WRITE, OPEN_EXISTING, dwFlagsAndAttributes);
		if (outHandle != InvalidFileHandle)
			return true;
		u32 lastError = GetLastError();
		if (lastError == ERROR_FILE_NOT_FOUND || lastError == ERROR_PATH_NOT_FOUND)
			return !fileNotFoundIsError;
		if (lastError == ERROR_ACCESS_DENIED)
		{
			// Remotes can ask for files via storage (SessionServer::HandleMessage->StoreCasFile->StorageImpl::StoreCasFile->this) and sometimes they exist as folders (but should be treated as not existing)
			u32 attr = GetFileAttributesW(fileName);
			if (attr != INVALID_FILE_ATTRIBUTES)
				if (IsDirectory(attr))
					return false;
		}

		if (!showErrors)
			return false;

		return ReportOpenFileError(logger, fileName, lastError);
	}

	bool GetFileBasicInformationByHandle(FileBasicInformation& out, Logger& logger, const tchar* fileName, FileHandle hFile, bool errorOnFail)
	{
#if PLATFORM_WINDOWS
		ExtendedTimerScope ts(KernelStats::GetCurrent().getFileInfo);

		if (IsRunningWine())
			return GetFileBasicInformation(out, logger, fileName, errorOnFail);
		IO_STATUS_BLOCK b;
		FILE_NETWORK_OPEN_INFORMATION info;
		NTSTATUS res = NtQueryInformationFile(AsHANDLE(hFile), &b, &info, sizeof(info), FileNetworkOpenInformation);
		if (res != STATUS_SUCCESS)
			return errorOnFail ? logger.Error(TC("GetFileBasicInformationByHandle (NtQueryInformationFile) failed on %s (0x%x)"), fileName, res) : false;
		out.attributes = info.FileAttributes;
		out.lastWriteTime = info.LastWriteTime.QuadPart;
		out.size = info.EndOfFile.QuadPart;
		return true;
#else
		FileInformation info;
		if (!GetFileInformationByHandle(info, logger, fileName, hFile))
			return false;
		out.attributes = info.attributes;
		out.lastWriteTime = info.lastWriteTime;
		out.size = info.size;
		return true;
#endif
	}

	bool GetFileBasicInformation(FileBasicInformation& out, Logger& logger, const tchar* fileName, bool errorOnFail)
	{
#if PLATFORM_WINDOWS
		ExtendedTimerScope ts(KernelStats::GetCurrent().getFileInfo);
		FILE_NETWORK_OPEN_INFORMATION info;
		MAKE_NT_FILENAME(fileName);
		UNICODE_STRING us;
		us.Length = USHORT(TStrlen(fileName)*sizeof(tchar));
		us.MaximumLength = us.Length;
		us.Buffer = (tchar*)fileName;
		OBJECT_ATTRIBUTES oa;
		InitializeObjectAttributes(&oa, &us, OBJ_CASE_INSENSITIVE, NULL, NULL);
		NTSTATUS res = NtQueryFullAttributesFile(&oa, &info);
		if (res != STATUS_SUCCESS)
			return errorOnFail ? logger.Error(TC("GetFileBasicInformationByHandle (NtQueryFullAttributesFile) failed on %s (0x%x)"), fileName, res) : false;
		out.attributes = info.FileAttributes;
		out.lastWriteTime = info.LastWriteTime.QuadPart;
		out.size = info.EndOfFile.QuadPart;
		return true;
#else
		FileInformation info;
		if (!GetFileInformation(info, logger, ToView(fileName)))
			return false;
		out.attributes = info.attributes;
		out.lastWriteTime = info.lastWriteTime;
		out.size = info.size;
		return true;
#endif
	}

	bool GetFileInformationByHandle(FileInformation& out, Logger& logger, const tchar* fileName, FileHandle hFile)
	{
		ExtendedTimerScope ts(KernelStats::GetCurrent().getFileInfo);
#if PLATFORM_WINDOWS
		BY_HANDLE_FILE_INFORMATION info;
		if (!::GetFileInformationByHandle(AsHANDLE(hFile), &info))
			return logger.Error(TC("GetFileInformationByHandle failed on %s (%s)"), fileName, LastErrorToText().data);
		out.attributes = info.dwFileAttributes;
		out.volumeSerialNumber = info.dwVolumeSerialNumber;
		out.lastWriteTime = (u64&)info.ftLastWriteTime;
		out.size = u64(ToLargeInteger(info.nFileSizeHigh, info.nFileSizeLow).QuadPart);
		out.index = u64(ToLargeInteger(info.nFileIndexHigh, info.nFileIndexLow).QuadPart);
		return true;
#else
		struct stat attr;
		int res = fstat(AsFileDescriptor(hFile), &attr);
		if (res != 0)
			return logger.Error(TC("GetFileInformationByHandle (fstat) failed on %s (%s)"), fileName, strerror(errno));
		out.lastWriteTime = FromTimeSpec(attr.st_mtimespec);
		out.attributes = attr.st_mode;
		out.volumeSerialNumber = attr.st_dev;
		out.index = attr.st_ino;
		out.size = attr.st_size;
		return true;
#endif
	}

	bool GetFileInformation(FileInformation& out, Logger& logger, StringView fileName)
	{
#if PLATFORM_WINDOWS
		MAKE_LONG_FILENAME(fileName);
		FileHandle h = uba::CreateFileW(fileName.data, 0, 0, OPEN_EXISTING, FILE_FLAG_BACKUP_SEMANTICS);
		if (h == InvalidFileHandle)
			return false;// logger.Error(TC("GetFileInformation: CreateFile failed for file %s (%s)"), fileName, LastErrorToText().data);
		auto handleGuard = MakeGuard([&]() { uba::CloseFile(fileName.data, h); });
		if (!GetFileInformationByHandle(out, logger, fileName.data, h))
			return false;// logger.Error(TC("Failed to get file information for %s while checking file added for write. This should not happen! (%s)"), fileName, LastErrorToText().data);
		return true;
#else
		ENSURE_TERMINATED(fileName);
		struct stat attr;
		// With symlink support, lstat preserves S_IFLNK so the dirtable can record symlinks
		// distinctly from their targets. Without it, fall back to stat (target-resolving).
	#if UBA_SUPPORT_SYMLINKS
		int res = lstat(fileName.data, &attr);
	#else
		int res = stat(fileName.data, &attr);
	#endif
		if (res != 0)
		{
			if (errno != ENOENT)
				logger.Warning(TC("GetFileInformation: stat failed for file %s and this error handling not implemented (%s)"), fileName.data, strerror(errno));
			SetLastError(ERROR_FILE_NOT_FOUND);
			return false;
		}
		out.lastWriteTime = FromTimeSpec(attr.st_mtimespec);
		out.attributes = attr.st_mode;
		out.volumeSerialNumber = attr.st_dev;
		out.index = attr.st_ino;
		out.size = attr.st_size;
		return true;
#endif
	}

	bool FileExists(Logger& logger, const tchar* fileName, u64* outSize, u32* outAttributes, u64* lastWriteTime)
	{
		ExtendedTimerScope ts(KernelStats::GetCurrent().getFileInfo);
#if PLATFORM_WINDOWS
		MAKE_LONG_FILENAME(fileName);
		WIN32_FILE_ATTRIBUTE_DATA data;
		if (!::GetFileAttributesExW(fileName, GetFileExInfoStandard, &data))
		{
			DWORD lastError = GetLastError();
			if (lastError != ERROR_FILE_NOT_FOUND && lastError != ERROR_PATH_NOT_FOUND)
				logger.Error(TC("GetFileAttributesW failed on %s (%s)"), fileName, LastErrorToText(lastError).data);
			return false;
		}

		if (outSize)
		{
			LARGE_INTEGER li;
			li.HighPart = (LONG)data.nFileSizeHigh;
			li.LowPart = data.nFileSizeLow;
			*outSize = u64(li.QuadPart);
		}
		if (outAttributes)
			*outAttributes = data.dwFileAttributes;

		if (lastWriteTime)
			*lastWriteTime = (u64&)data.ftLastWriteTime;

		return true;
#else
		struct stat attr;
		if (stat(fileName, &attr) == -1)
		{
			if (errno == ENOENT)
			{
				SetLastError(ERROR_FILE_NOT_FOUND);
				return false;
			}
			if (errno == ENOTDIR)
			{
				SetLastError(ERROR_PATH_NOT_FOUND);
				return false;
			}
			UBA_ASSERTF(false, TC("FileExists error handling for %u is not implemented (%s)"), errno, strerror(errno));
			return false;
		}

		if (outSize)
			*outSize = attr.st_size;
		if (outAttributes)
			*outAttributes = attr.st_mode;
		if (lastWriteTime)
			*lastWriteTime = FromTimeSpec(attr.st_mtimespec);
		return true;
#endif
	}

	bool SetFilePointer(Logger& logger, const tchar* fileName, FileHandle handle, u64 position)
	{
#if PLATFORM_WINDOWS
		if (!::SetFilePointerEx(AsHANDLE(handle), ToLargeInteger(position), NULL, FILE_BEGIN))
			return logger.Error(TC("SetFilePointerEx failed on %s (%s)"), fileName, LastErrorToText().data);
		return true;
#else
		if (lseek(AsFileDescriptor(handle), position, SEEK_SET) != position)
			return logger.Error("lseek to %llu failed for %s: %s", position, fileName, strerror(errno));
		return true;
#endif
	}

	bool SetEndOfFile(Logger& logger, const tchar* fileName, FileHandle handle, u64 size)
	{
		ExtendedTimerScope ts(KernelStats::GetCurrent().setFileInfo);
#if PLATFORM_WINDOWS
		FILE_END_OF_FILE_INFO info;
		info.EndOfFile = ToLargeInteger(size);
		if (!::SetFileInformationByHandle(AsHANDLE(handle), FileEndOfFileInfo, &info, sizeof(info)))
			return logger.Error(TC("SetFileInformationByHandle failed on %s (%s)"), fileName, LastErrorToText().data);
		return true;
#else
		if (ftruncate(AsFileDescriptor(handle), size) == -1)
			return logger.Error(TC("ftruncate failed on %s (%s)"), fileName, LastErrorToText().data);
		return false;
#endif
	}

	bool MakeSparse(Logger& logger, const tchar* fileName, FileHandle h, u64 size)
	{
#if PLATFORM_WINDOWS
		HANDLE handle = AsHANDLE(h);
		DWORD dwTemp;
		if (!::DeviceIoControl(handle, FSCTL_SET_SPARSE, NULL, 0, NULL, 0, &dwTemp, NULL))
			return logger.Error(TC("Failed to make file %s sparse (%s)"), fileName, LastErrorToText().data);
		if (!SetEndOfFile(logger, fileName, h, size))
			return false;
#endif
		return true;
	}

	bool GetDirectoryOfCurrentModule(Logger& logger, StringBufferBase& out)
	{
		#if PLATFORM_WINDOWS
			HMODULE hm = NULL;
			if (!GetModuleHandleEx(GET_MODULE_HANDLE_EX_FLAG_FROM_ADDRESS | GET_MODULE_HANDLE_EX_FLAG_UNCHANGED_REFCOUNT, (LPCWSTR)&GetDirectoryOfCurrentModule, &hm))
				return logger.Error(TC("GetModuleHandleEx failed (%s)"), LastErrorToText().data);
			u32 len = GetModuleFileNameW(hm, out.data + out.count, out.capacity - out.count);
			if (!len)
				return logger.Error(TC("GetModuleFileNameW failed (%s)"), LastErrorToText().data);
			out.count += len;
			UBA_ASSERTF(GetLastError() == ERROR_SUCCESS, TC("GetModuleFileNameW failed (%s)"), LastErrorToText().data);
			const tchar* lastSlash = out.Last('\\');
			out.Resize(lastSlash - out.data);
			return true;

		#else
			// Can be a shared library, not only executable.. so can't use /proc/self/exe
			Dl_info info;
			if (!dladdr((void*)&GetDirectoryOfCurrentModule, &info))
				return logger.Error("dladdr failed to get info for address to GetDirectoryOfCurrentModule");
			out.count = GetFullPathNameW(info.dli_fname, out.capacity, out.data, nullptr);
			if (!out.count)
				return logger.Error("GetFullPathNameW failed to return full name for %s", info.dli_fname);
			char* lastSlash = strrchr(out.data, '/');
			out.Resize(lastSlash - out.data);
			return true;
		#endif
	}

	bool DeleteAllFiles(Logger& logger, const tchar* dir, bool deleteDir, u32* count)
	{
		bool success = true;
		bool traverseRes = TraverseDir(logger, ToView(dir),
			[&](const DirectoryEntry& e)
			{
				StringBuffer<MaxPath> fullPath(dir);
				fullPath.EnsureEndsWithSlash().Append(e.name);

				if (IsReadOnly(e.attributes))
				{
					#if PLATFORM_WINDOWS
					SetFileAttributesW(fullPath.data, e.attributes & ~FILE_ATTRIBUTE_READONLY);
					#else
					UBA_ASSERT(false);
					#endif
				}

				if (IsDirectory(e.attributes))
				{
					if (!DeleteAllFiles(logger, fullPath.data, true, count))
						success = false;
				}
				else
				{
					if (!DeleteFileW(fullPath.data))
					{
						logger.Warning(TC("Failed to delete file %s (%s)"), fullPath.data, LastErrorToText().data);
						success = false;
					}
					else if (count)
						++(*count);
				}
			});

		if (!traverseRes || !success)
			return false;

		if (!deleteDir)
			return true;

		if (RemoveDirectoryW(dir))
			return true;

		u32 lastError = GetLastError();

		if (lastError == ERROR_FILE_NOT_FOUND || lastError == ERROR_PATH_NOT_FOUND)
			return true;

		logger.Warning(TC("Failed to delete directory %s (%s)"), dir, LastErrorToText(lastError).data);
		return false;
	}

	// 1=found, 0=not found, -1=hard error
	static int InternalSearchPathForFile(Logger& logger, StringBufferBase& out, StringView file, StringView workingDir, StringView applicationDir)
	{
		UBA_ASSERT(!IsAbsolutePath(file));

		StringBuffer<MaxPath> fullPath;

		StringView ext;

		#if PLATFORM_WINDOWS
		if (file.count < 4 || file[file.count - 4] != '.')
			ext = TCV(".exe");
		#endif

		auto TestFileExists = [&](const tchar* extraInfo)
			{
				fullPath.EnsureEndsWithSlash().Append(file).Append(ext);

				if (GetFileAttributesW(fullPath.data) != INVALID_FILE_ATTRIBUTES)
				{
					FixPath(fullPath.data, nullptr, 0, out);
					return true;
				}
				u32 lastError = GetLastError();
				if (lastError != ERROR_FILE_NOT_FOUND && lastError != ERROR_PATH_NOT_FOUND)
					logger.Warning(TC("SearchPathForFile tried to find the file %s%s but got error when getting attributes (%s)"), fullPath.data, extraInfo, LastErrorToText(lastError).data);
				return false;
			};

		if (applicationDir.count)
		{
			fullPath.Append(applicationDir);
			if (TestFileExists(TC("")))
				return 1;
		}

		if (workingDir.count)
		{
			fullPath.Clear().Append(workingDir);
			if (TestFileExists(TC("")))
				return 1;
		}

		char varSeparator = IsWindows ? ';' : ':';
		tchar buff[32 * 1024] = { 0 };
		u32 len = GetEnvironmentVariableW(TC("PATH"), buff, sizeof_array(buff));
		if (!len || len == sizeof_array(buff))
			{ logger.Error(TC("Failed to get PATH environment variable")); return -1; }
		if (len >= sizeof_array(buff))
			{ logger.Error(TC("Failed to get PATH variable, buffer too small (need %u)"), len); return -1; }

		tchar* lastStart = buff;
		tchar* it = buff;
		bool end = false;
		while (!end)
		{
			end = *it == 0;
			if (*it != varSeparator && !end)
			{
				++it;
				continue;
			}
			if (u32 varLen = u32(it - lastStart))
			{
				fullPath.Clear().Append(lastStart, varLen);
				if (TestFileExists(TC(" using PATH environment variable")))
					return 1;
			}
			lastStart = ++it;
		}
		return 0;
	}

	bool SearchPathForFile(Logger& logger, StringBufferBase& out, StringView file, StringView workingDir, StringView applicationDir)
	{
		return InternalSearchPathForFile(logger, out, file, workingDir, applicationDir) == 1;
	}

	bool SearchPathCache::SearchPathForFile(Logger& logger, StringBufferBase& out, StringView file, StringView workingDir, StringView applicationDir)
	{
		UBA_ASSERT(!IsAbsolutePath(file));

		TString cacheKey;
		cacheKey.reserve(file.count + workingDir.count + applicationDir.count + 4);
		cacheKey.append(file.data, file.count);
		cacheKey.push_back(TC('\x1f'));
		cacheKey.append(workingDir.data, workingDir.count);
		cacheKey.push_back(TC('\x1f'));
		cacheKey.append(applicationDir.data, applicationDir.count);

		SCOPED_FUTEX_READ(m_lock, readLock);
		auto it = m_map.find(cacheKey);
		if (it != m_map.end())
		{
			if (it->second.empty())
				return false;
			out.Append(it->second.c_str(), u32(it->second.size()));
			return true;
		}
		readLock.Leave();

		int result = InternalSearchPathForFile(logger, out, file, workingDir, applicationDir);
		if (result != 1)
			return false; // Cache only cache hits - Files could be created later and we don't want cache to say they don't exist. Ideally maybe SearchPathCache should be used in very specific scenarios

		SCOPED_FUTEX(m_lock, writeLock);
		if (m_map.find(cacheKey) == m_map.end())
			m_map.emplace(std::move(cacheKey), out.ToString());
		return true;
	}

	FileHandle CreateFileW(const tchar* fileName, u32 desiredAccess, u32 shareMode, u32 createDisp, u32 flagsAndAttributes)
	{
		ExtendedTimerScope ts(KernelStats::GetCurrent().createFile);
	#if PLATFORM_WINDOWS
		MAKE_LONG_FILENAME(fileName);
		return (FileHandle)(u64)::CreateFileW(fileName, desiredAccess, shareMode, NULL, createDisp, flagsAndAttributes, NULL);
	#else
		int flags = O_CLOEXEC;
		if (createDisp == CREATE_ALWAYS)
			flags |= O_CREAT | O_TRUNC;
		else if (createDisp == OPEN_EXISTING)
			;
		else if (createDisp == CREATE_NEW)
			flags |= O_CREAT | O_EXCL;
		else
			UBA_ASSERTF(false, TC("CreateFileW create disposition %u not supported"), createDisp);

		if ((desiredAccess & (GENERIC_WRITE | GENERIC_READ)) == (GENERIC_WRITE | GENERIC_READ))
			flags |= O_RDWR;
		else if (desiredAccess & GENERIC_WRITE || desiredAccess == FILE_WRITE_ATTRIBUTES)
			flags |= O_WRONLY;
		else if (desiredAccess & GENERIC_READ)
			flags |= O_RDONLY;
		else if (!desiredAccess)
			flags |= O_RDONLY;
		else
			UBA_ASSERTF(false, TC("CreateFileW desired access %u not supported"), desiredAccess);
		int mode = flagsAndAttributes;
		int fd = open(fileName, flags, mode);

		if (fd != -1) // We want to give ERROR_ACCESS_DENIED on directories. writing fails automatically but read can succeed
		{
			if ((flags & O_ACCMODE) != O_RDONLY)
				return (FileHandle)fd;

			struct stat attr;
			int res = fstat(fd, &attr);
			if (res != 0)
			{
				UBA_ASSERTF(false, TC("CreateFileW (fstat) error handling for %u (%s) not implemented"), errno, strerror(errno));
			}
			if (S_ISREG(attr.st_mode))
				return (FileHandle)fd;
			close(fd);
			SetLastError(ERROR_ACCESS_DENIED);
			return InvalidFileHandle;
		}

		if (errno == EEXIST)
		{
			SetLastError(ERROR_FILE_EXISTS);
			return InvalidFileHandle;
		}

		if (errno == ENOENT)
		{
			SetLastError(ERROR_FILE_NOT_FOUND);
			return InvalidFileHandle;
		}

		if (errno == ENOTDIR)
		{
			SetLastError(ERROR_PATH_NOT_FOUND);
			return InvalidFileHandle;
		}

		if (errno == EACCES || errno == EISDIR)
		{
			SetLastError(ERROR_ACCESS_DENIED);
			return InvalidFileHandle;
		}

		UBA_ASSERTF(false, TC("CreateFileW failed for %s - Error handling for %u (%s) not implemented"), fileName, errno, strerror(errno));
		return InvalidFileHandle;
	#endif
	}

	bool CloseFile(const tchar* fileName, FileHandle h)
	{
		ExtendedTimerScope ts(KernelStats::GetCurrent().closeFile);
		if (h == InvalidFileHandle)
			return true;

#if PLATFORM_WINDOWS
		return ::CloseHandle(AsHANDLE(h));
#else
		if (close(AsFileDescriptor(h)) == 0)
			return true;
		UBA_ASSERTF(false, TC("CloseFile error handling not implemented while failing to close %s with handle %i (%s)"), fileName, AsFileDescriptor(h), strerror(errno));
		return false;
#endif
	}

	bool CreateDirectoryW(const tchar* pathName)
	{
#if PLATFORM_WINDOWS
		MAKE_LONG_FILENAME(pathName);
		return ::CreateDirectoryW(pathName, NULL);
#else
		if (mkdir(pathName, 0777) == 0)
		{
			SetLastError(ERROR_SUCCESS);
			return true;
		}
		if (errno == EEXIST)
		{
			SetLastError(ERROR_ALREADY_EXISTS);
			return false;
		}
		if (errno == ENOENT || errno == ENOTDIR)
		{
			SetLastError(ERROR_PATH_NOT_FOUND);
			return false;
		}
		if (errno == EACCES)
		{
			SetLastError(ERROR_ACCESS_DENIED);
			return InvalidFileHandle;
		}
		UBA_ASSERTF(false, TC("CreateDirectoryW failed creating %s - error handling %i not handled (%s)"), pathName, errno, strerror(errno));
		return false;
#endif
	}

	bool RemoveDirectoryW(const tchar* pathName)
	{
#if PLATFORM_WINDOWS
		MAKE_LONG_FILENAME(pathName);
		if (::RemoveDirectoryW(pathName))
			return true;
		return false;
#else
		int res = rmdir(pathName);
		if (res == 0)
			return true;
		if (errno == ENOENT)
		{
			SetLastError(ERROR_FILE_NOT_FOUND);
			return false;
		}
		if (errno == ENOTEMPTY)
		{
			SetLastError(ERROR_DIR_NOT_EMPTY);
			return false;
		}
		if (errno == ENOTDIR)
		{
			SetLastError(ERROR_ACCESS_DENIED);
			return false;
		}
		if (errno == EPERM || errno == EACCES)
		{
			SetLastError(ERROR_ACCESS_DENIED);
			return false;
		}
		UBA_ASSERTF(false, TC("RemoveDirectoryW error handling not implemented (%s): %s"), strerror(errno), pathName);
		return false;
#endif
	}

	bool DeleteFileW(const tchar* fileName)
	{
#if PLATFORM_WINDOWS
		MAKE_LONG_FILENAME(fileName);
		if (::DeleteFileW(fileName))
			return true;
		return false;
#else
		int res = remove(fileName);
		if (res == 0)
			return true;
		if (errno == ENOENT)
		{
			SetLastError(ERROR_FILE_NOT_FOUND);
			return false;
		}
		if (errno == EPERM)
		{
			SetLastError(ERROR_ACCESS_DENIED);
			return false;
		}
		if (errno == ENOTEMPTY || errno == EEXIST)
		{
			SetLastError(ERROR_DIR_NOT_EMPTY);
			return false;
		}
		if (errno == EISDIR)
		{
			SetLastError(ERROR_ACCESS_DENIED);
			return false;
		}

		UBA_ASSERTF(false, TC("DeleteFileW failed on %s - Error handling not implemented (%s)"), fileName, strerror(errno));
		return false;
#endif
	}

	bool CopyFileW(const tchar* existingFileName, const tchar* newFileName, bool bFailIfExists, u8* scratch, u64 scratchSize)
	{
		ExtendedTimerScope ts(KernelStats::GetCurrent().copyFile);
#if PLATFORM_WINDOWS
		MAKE_LONG_FILENAME(existingFileName);
		MAKE_LONG_FILENAME(newFileName);
		return ::CopyFileW(existingFileName, newFileName, bFailIfExists);
#elif PLATFORM_MAC
		if (clonefile(existingFileName, newFileName, 0) == 0)
			return true;

		if (scratch)
		{
			int sourceFd = open(existingFileName, O_RDONLY | O_CLOEXEC, 0);
			if (sourceFd == -1)
				return false;
			auto closeSourceFile = MakeGuard([&]() { close(sourceFd); });
			struct stat sourceStat;
			if (fstat(sourceFd, &sourceStat) != 0)
				return false;
			if (!S_ISREG(sourceStat.st_mode))
				return false;
			u64 fileSize = sourceStat.st_size;
			int targetFd = open(newFileName, O_CREAT | O_TRUNC | O_WRONLY | O_CLOEXEC, 0600);
			auto closeTargetFile = MakeGuard([&]() { close(targetFd); });
			if (targetFd == -1)
				return false;//m_logger.Error(TC("Failed to create file %s for move from temporary file %s (%s)"), m_fileName, realFileName, strerror(errno));
			u64 left = fileSize;
			while (left)
			{
				u64 toCopy = Min(left, scratchSize);
				auto bytesRead = read(sourceFd, scratch, toCopy);
				if (bytesRead <= 0)
					return false;//m_logger.Error(TC("Failed to do read from file %s (%s)"), realFileName, strerror(errno));
				u8* readPos = scratch;
				u64 toWrite = bytesRead;
				while (toWrite)
				{
					auto written = write(targetFd, readPos, toWrite);
					if (written == -1)
						return false;// m_logger.Error(TC("Failed to write to file %s (%s)"), m_fileName, strerror(errno));
					toWrite -= written;
					readPos += written;
				}
				left -= bytesRead;
			}
			closeSourceFile.Execute();
			if (fchmod(targetFd, sourceStat.st_mode & 07777) != 0)
				return false;
			fchown(targetFd, sourceStat.st_uid, sourceStat.st_gid);

			return true;
		}
		else
		{
			if (copyfile(existingFileName, newFileName, 0, COPYFILE_ALL) == 0)
				return true;
		}
		UBA_ASSERTF(false, TC("CopyFileW failed on %s - Error handling not implemented (%s)"), existingFileName, strerror(errno));
		return false;
#else

		UBA_ASSERTF(false, TC("CopyFileW not implemented (From %s to %s)"), existingFileName, newFileName);
		return false;
#endif
	}

	u32 GetLongPathNameW(const tchar* lpszShortPath, tchar* lpszLongPath, u32 cchBuffer)
	{
#if PLATFORM_WINDOWS
		return ::GetLongPathNameW(lpszShortPath, lpszLongPath, cchBuffer);
#else
		UBA_ASSERTF(false, TC("GetLongPathNameW not implemented"));
		return false;
#endif
	}

	bool GetFileLastWriteTime(u64& outTime, FileHandle fileHandle)
	{
		ExtendedTimerScope ts(KernelStats::GetCurrent().getFileTime);
#if PLATFORM_WINDOWS
		FILETIME lastWriteTime;
		auto res = ::GetFileTime(AsHANDLE(fileHandle), NULL, NULL, &lastWriteTime);
		outTime = (u64&)lastWriteTime;
		return res;
#else
		struct stat attr;
		int res = fstat(AsFileDescriptor(fileHandle), &attr);
		if (res != 0)
		{
			UBA_ASSERTF(errno == ENOENT, TC("GetFileLastWriteTime (fstat) error handling not implemented: %s"), strerror(errno));
			SetLastError(ERROR_FILE_NOT_FOUND);
			return false;
		}
		outTime = FromTimeSpec(attr.st_mtimespec);
		return true;
#endif
	}

	bool SetFileLastWriteTime(FileHandle fileHandle, u64 writeTime)
	{
#if PLATFORM_WINDOWS
		return SetFileTime(AsHANDLE(fileHandle), (FILETIME*)&writeTime, NULL, (FILETIME*)&writeTime);
#else
		struct timespec times[2];
		times[0] = times[1] = ToTimeSpec(writeTime);
		if (futimens(AsFileDescriptor(fileHandle), times) == 0)
			return true;
		UBA_ASSERTF(false, TC("SetFileLastWriteTime (futimens) error handling not implemented: %s"), strerror(errno));
		return false;
#endif
	}

	bool MoveFileExW(const tchar* existingFileName, const tchar* newFileName, u32 dwFlags)
	{
#if PLATFORM_WINDOWS
		MAKE_LONG_FILENAME(existingFileName);
		MAKE_LONG_FILENAME(newFileName);
		return ::MoveFileExW(existingFileName, newFileName, dwFlags);
#else
		int res = rename(existingFileName, newFileName);
		if (res == 0)
		{
			SetLastError(ERROR_SUCCESS);
			return true;
		}
		UBA_ASSERTF(false, TC("MoveFileExW error handling not implemented (%s to %s): %s"), existingFileName, newFileName, strerror(errno));
		return false;
#endif
	}

	bool GetFileSizeEx(u64& outFileSize, FileHandle hFile)
	{
		ExtendedTimerScope ts(KernelStats::GetCurrent().getFileInfo);
#if PLATFORM_WINDOWS
		LARGE_INTEGER lpFileSize;
		if (!::GetFileSizeEx(AsHANDLE(hFile), &lpFileSize))
			return false;
		outFileSize = u64(lpFileSize.QuadPart);
		return true;
#else
		struct stat attr;
		int res = fstat(AsFileDescriptor(hFile), &attr);
		if (res == 0)
		{
			outFileSize = attr.st_size;
			return true;
		}

		UBA_ASSERTF(false, TC("GetFileSizeEx error handling not implemented"));
		return false;
#endif
	}

	u32 GetFileAttributesW(const tchar* fileName)
	{
		ExtendedTimerScope ts(KernelStats::GetCurrent().getFileInfo);
#if PLATFORM_WINDOWS
		MAKE_LONG_FILENAME(fileName);
		return ::GetFileAttributesW(fileName);
#else
		struct stat attr;
		if (stat(fileName, &attr) == -1)
		{
			if (errno == ENOENT)
			{
				SetLastError(ERROR_FILE_NOT_FOUND);
				return INVALID_FILE_ATTRIBUTES;
			}
			if (errno == ENOTDIR)
			{
				SetLastError(ERROR_DIRECTORY);
				return INVALID_FILE_ATTRIBUTES;
			}
			UBA_ASSERTF(false, TC("GetFileAttributesW error handling not implemented %s (%s)"), fileName, strerror(errno));
			return INVALID_FILE_ATTRIBUTES;
		}
		return attr.st_mode;
#endif
	}

	bool IsReadOnly(u32 attributes)
	{
#if PLATFORM_WINDOWS
		return (attributes & FILE_ATTRIBUTE_READONLY) != 0;
#else
		return false;
#endif
	}

	u32 DefaultAttributes(bool execute)
	{
#if PLATFORM_WINDOWS
		return FILE_ATTRIBUTE_NORMAL;
#else
		return S_IFREG | S_IRUSR | S_IWUSR | (execute ? S_IXUSR : 0) | S_IRGRP | S_IROTH;
#endif
	}

	u32 DefaultDirAttributes()
	{
#if PLATFORM_WINDOWS
		return FILE_ATTRIBUTE_DIRECTORY;
#else
		return S_IFDIR | S_IRUSR | S_IWUSR | S_IXUSR | S_IRGRP | S_IXGRP | S_IROTH | S_IXOTH;
#endif
	}


	bool CreateHardLinkW(const tchar* newFileName, const tchar* existingFileName)
	{
#if PLATFORM_WINDOWS
		MAKE_LONG_FILENAME(newFileName);
		MAKE_LONG_FILENAME(existingFileName);
		return ::CreateHardLinkW(newFileName, existingFileName, NULL);
#else
		int res = link(existingFileName, newFileName); // We need to use links in order for explicit dynamic library dependencies  to be found at the same path.
		//int res = symlink(existingFileName, newFileName);
		if (res == 0)
			return true;

		#if PLATFORM_MAC
		if (errno == EPERM) // Because of System Integrity Protection we might not be allowed to link this file, fallback to copy
			return false;
		#endif
		if (errno == EEXIST)
		{
			SetLastError(ERROR_ALREADY_EXISTS);
			return false;
		}
		UBA_ASSERTF(false, TC("CreateHardLinkW %s to %s error handling not implemented (%s)"), existingFileName, newFileName, strerror(errno));
		return false;
#endif
	}

	u32 GetFullPathNameW(const tchar* fileName, u32 nBufferLength, tchar* lpBuffer, tchar** lpFilePart)
	{
#if PLATFORM_WINDOWS
		return ::GetFullPathNameW(fileName, nBufferLength, lpBuffer, lpFilePart);
#else
		char source[1024];
		size_t sourceStart = 0;
		size_t fileNameStart = 0;

		if (fileName[0] == '~' && fileName[1] == '/')
		{
			strcpy(source, getenv("HOME"));
			sourceStart = strlen(source);
			source[sourceStart++] = '/';
			fileNameStart = 2;
			TSprintf_s(source + sourceStart, 1024 - sourceStart, "%s", fileName + fileNameStart);
			return strlen(strcpy(lpBuffer, source));
		}

#if 0
		char fullPath[1024];
		if (!realpath(fileName, fullPath))
		{
			UBA_ASSERTF(false, TC("realpath error handling not implemented for path %s (%s)"), fileName, strerror(errno));
			return 0;
		}
		u32 len = TStrlen(fullPath);
		UBA_ASSERT(len < sizeof_array(fullPath));
#elif 0
		const char* fullPath = canonicalize_file_name(fileName);
		u32 len = TStrlen(fullPath);
#else
		char cwd[PATH_MAX];
		if (!getcwd(cwd, PATH_MAX))
		{
			UBA_ASSERT(false);
			return 0;
		}
		u32 cwdlen = u32(strlen(cwd));
		cwd[cwdlen++] = '/';
		cwd[cwdlen] = 0;

		char fullPath[1024];
		u32 len;
		if (!FixPath2(fileName, ~0u, cwd, cwdlen, fullPath, sizeof(fullPath), &len))
			return 0;
#endif
		UBA_ASSERT(len < nBufferLength);
		memcpy(lpBuffer, fullPath, len + 1);
		return len;
#endif
	}

	bool SearchPathW(const tchar* a, const tchar* b, const tchar* c, u32 d, tchar* e, tchar** f)
	{
#if PLATFORM_WINDOWS
		return ::SearchPathW(a, b, c, d, e, f) != 0;
#else
		UBA_ASSERTF(false, TC("SearchPathW not implemented"));
		return false;
#endif
	}

	u64 GetSystemTimeAsFileTime()
	{
#if PLATFORM_WINDOWS
		FILETIME temp;
		::GetSystemTimeAsFileTime(&temp); // Contains a 64-bit value representing the number of 100-nanosecond intervals since January 1, 1601 (UTC).
		return u64(ToLargeInteger(temp.dwHighDateTime, temp.dwLowDateTime).QuadPart);
#else
		timeval tv;
		gettimeofday(&tv, NULL);
		return u64(tv.tv_sec) * 10'000'000ull + u64(tv.tv_usec)*10ull;
#endif
	}

	u64 GetFileTimeAsSeconds(u64 fileTime)
	{
#if PLATFORM_WINDOWS
		return TimeToMs(fileTime)/1000;
#else
		return fileTime / 10'000'000ull;
#endif
	}

	u64 GetFileTimeAsTime(u64 fileTime)
	{
		return MsToTime(GetFileTimeAsSeconds(fileTime)*1000);
	}

	u64 GetSecondsAsFileTime(u64 seconds)
	{
#if PLATFORM_WINDOWS
		return MsToTime(seconds*1000);
#else
		return seconds * 10'000'000ull;
#endif
	}

	bool GetCurrentDirectoryW(StringBufferBase& out)
	{
#if PLATFORM_WINDOWS
		u32 res = ::GetCurrentDirectoryW(out.capacity, out.data);
		if (!res || res > out.capacity)
		{
			UBA_ASSERT(false);
			return false;
		}
		out.count = res;
		return true;
#else
		if (!getcwd(out.data, out.capacity))
		{
			UBA_ASSERT(false);
			return false;
		}
		out.count = strlen(out.data);
		return true;
#endif
	}

	bool DirectoryCache::CreateDirectory(Logger& logger, const tchar* dir, bool* outAlreadyExists)
	{
		bool throwaway;
		bool& alreadyExists = outAlreadyExists ? *outAlreadyExists : throwaway;

		u64 dirLen = TStrlen(dir);
		if (dir[dirLen - 1] == PathSeparator || (IsWindows && (dir[dirLen - 1] == NonPathSeparator)))
			--dirLen;
		TString key(dir, dir + dirLen);
		dir = key.c_str();

		SCOPED_FUTEX(m_createdDirsLock, lock);
		CreatedDir& cd = m_createdDirs.try_emplace(key).first->second;
		lock.Leave();

		if (cd.handled)
		{
			alreadyExists = true;
			return true;
		}

		SCOPED_FUTEX(cd.lock, dirLock);

		if (cd.handled)
		{
			alreadyExists = true;
			return true;
		}
		auto g = MakeGuard([&]() { cd.handled = true; });

		if (uba::CreateDirectoryW(dir))
		{
			#if UBA_WSL_WORKAROUND
			while (!IsDirectory(GetFileAttributesW(dir)))
				Sleep(1);
			#endif
			alreadyExists = false;
			return true;
		}
		u32 lastError = GetLastError();
		if (lastError == ERROR_ALREADY_EXISTS)
		{
			alreadyExists = true;
			return true;
		}
		if (lastError != ERROR_PATH_NOT_FOUND)
			return logger.Error(TC("Failed to create directory %s (%s)"), dir, LastErrorToText(lastError).data);

		tchar temp[512];
		const tchar* lastSep = nullptr;
		for (const tchar* p = dir; *p; ++p)
			if (*p == PathSeparator || *p == NonPathSeparator)
				lastSep = p;
		UBA_ASSERT(lastSep);
		u64 pos = u64(lastSep - dir);
		memcpy(temp, dir, pos * sizeof(tchar));
		temp[pos] = 0;
		if (pos == 2 && temp[1] == ':')
			return false;
		if (!CreateDirectory(logger, temp))
			return false;
		if (uba::CreateDirectoryW(dir))
		{
			#if UBA_WSL_WORKAROUND
			while (!IsDirectory(GetFileAttributesW(dir)))
				Sleep(1);
			#endif
			alreadyExists = false;
			return true;
		}
		lastError = GetLastError();
		if (lastError == ERROR_ALREADY_EXISTS)
		{
			alreadyExists = true;
			return true;
		}
		return logger.Error(TC("Failed to create directory %s (%s)"), dir, LastErrorToText(lastError).data);
	}

	void DirectoryCache::Clear()
	{
		SCOPED_FUTEX(m_createdDirsLock, lock);
		m_createdDirs.clear();
	}

	bool GetAlternativeUbaPath(Logger& logger, StringBufferBase& out, StringView firstPath, bool isWindowsArm)
	{
		out.Append(firstPath);
		const tchar* engineDir = IsWindows ? TC("\\Engine\\") : TC("/Engine/");
		const tchar* engineDirPos;
		if (!out.Contains(engineDir, true, &engineDirPos))
			return false;//logger.Error(TC("Failed to find Engine dir in %s"), out.data);
		#if PLATFORM_WINDOWS
		const tchar* platformStr = TC("Win64");
		#elif PLATFORM_LINUX
		const tchar* platformStr = TC("Linux");
		#else
		const tchar* platformStr = TC("Mac");
		#endif
		out.Resize(engineDirPos - out.data + 8).Append(TCV("Binaries")).Append(PathSeparator).Append(platformStr).Append(PathSeparator).Append("UnrealBuildAccelerator").Append(PathSeparator);
		if constexpr (IsWindows)
			out.Append(isWindowsArm ? TC("arm64") : TC("x64")).Append(PathSeparator);
		return true;
	};

	Function<void(u32)> g_systemErrorCallback;

	void ReportSystemError(u32 error)
	{
		if (g_systemErrorCallback)
			g_systemErrorCallback(error);
	}

	void SetSystemErrorCallback(const Function<void(u32)>& callback)
	{
		g_systemErrorCallback = callback;
	}

	VolumeCache::~VolumeCache()
	{
		#if PLATFORM_WINDOWS
		for (auto& volume : volumes)
			CloseHandle(AsHANDLE(volume.handle));
		#endif
	}

	bool VolumeCache::Init(Logger& logger)
	{
		#if PLATFORM_WINDOWS
		tchar volumeName[MAX_PATH] = {0};
		HANDLE hFind = FindFirstVolumeW(volumeName, sizeof_array(volumeName));

		if (hFind == INVALID_HANDLE_VALUE)
			return logger.Error(TC("FindFirstVolume failed (%s)"), LastErrorToText().data);

		do
		{
			DWORD serialNumber = 0;
			if (!GetVolumeInformationW(volumeName, nullptr, 0, &serialNumber, nullptr, nullptr, nullptr, 0))
				continue;
			Volume& volume = volumes.emplace_back();
			volume.serialNumber = serialNumber;


			tchar driveLetters[MAX_PATH] = {0};
			DWORD cchReturnLength = 0;
			if (GetVolumePathNamesForVolumeNameW(volumeName, driveLetters, sizeof_array(driveLetters), &cchReturnLength))
			{
				StringBuffer<128> drives;
				for (tchar* p = driveLetters; *p; p += TStrlen(p) + 1)
					drives.Append(p);
				volume.drives = drives.data;
			}

			u32 volumeNameLen = TStrlen(volumeName);
			if (volumeName[volumeNameLen - 1] == PathSeparator)
				volumeName[volumeNameLen - 1] = 0;

			volume.handle = (FileHandle)(u64)::CreateFileW(volumeName, 0, FILE_SHARE_READ|FILE_SHARE_WRITE, NULL, OPEN_EXISTING, 0, NULL);
		}
		while (FindNextVolumeW(hFind, volumeName, sizeof_array(volumeName)));

		FindVolumeClose(hFind);

		std::sort(volumes.begin(), volumes.end(), [](const Volume& a, const Volume& b)
			{
				bool aempt = a.drives.empty();
				bool bempt = b.drives.empty();
				if (aempt != bempt)
					return bempt;
				if (!aempt)
					return a.drives < b.drives;
				return a.serialNumber < b.serialNumber;
			});
		#endif
		return true;
	}

	u32 VolumeCache::GetSerialIndex(u32 volumeSerial)
	{
		#if PLATFORM_WINDOWS
		u32 index = 1;
		for (auto& volume : volumes)
		{
			if (volume.serialNumber == volumeSerial)
				return index;
			++index;
		}
		if (!volumeSerial)
			return index;
		#endif
		return volumeSerial;
	}

	void VolumeCache::Write(BinaryWriter& writer)
	{
		writer.WriteU16(u16(volumes.size()));
		for (auto& volume : volumes)
			writer.WriteU32(volume.serialNumber);
	}

	void VolumeCache::Read(BinaryReader& reader)
	{
		u32 count = reader.ReadU16();
		volumes.resize(count);
		for (u32 i=0; i!=count;++i)
			volumes[i].serialNumber = reader.ReadU32();
	}

	bool VolumeCache::Volume::UpdateStats(u8& outBusyPercent, u32& outReadCount, u64& outReadBytes, u32& outWriteCount, u64& outWriteBytes)
	{
		outBusyPercent = 0;
		outReadCount = 0;
		outReadBytes = 0;
		outWriteCount = 0;
		outWriteBytes = 0;

		#if PLATFORM_WINDOWS
		DISK_PERFORMANCE perf = {};
		DWORD bytesReturned;
		if (!::DeviceIoControl(AsHANDLE(handle), IOCTL_DISK_PERFORMANCE, NULL, 0, &perf, sizeof(perf), &bytesReturned, NULL))
		{
			::CloseHandle(AsHANDLE(handle));
			handle = InvalidFileHandle;
			return false;
		}

		u64 queryTime = perf.QueryTime.QuadPart;
		u64 idleTime = perf.IdleTime.QuadPart;
		u32 readCount = perf.ReadCount;
		u32 writeCount = perf.WriteCount;
		u64 readBytes = perf.BytesRead.QuadPart;
		u64 writeBytes = perf.BytesWritten.QuadPart;

		if (prevQueryTime)
		{
			double busyPercent = 100.0 - 100.0*(double(idleTime - prevIdleTime) / double(queryTime - prevQueryTime));
			busyPercent = Max(0.0, Min(100.0, busyPercent));
			outBusyPercent = (u8)busyPercent;
			outReadCount = readCount - prevReadCount;
			outReadBytes = readBytes - prevReadBytes;
			outWriteCount = writeCount - prevWriteCount;
			outWriteBytes = writeBytes - prevWriteBytes;
		}
		prevQueryTime = queryTime;
		prevIdleTime = idleTime;

		prevReadCount = readCount;
		prevReadBytes = readBytes;
		prevWriteCount = writeCount;
		prevWriteBytes = writeBytes;
		#endif
		return true;
	}
}
