// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaFileAccessor.h"
#include "UbaDefinitions.h"
#include "UbaEvent.h"
#include "UbaFile.h"
#include "UbaLogger.h"
#include "UbaStats.h"

#if PLATFORM_LINUX
#include <sys/sendfile.h>
#elif PLATFORM_MAC
#include <copyfile.h>
#endif

#define UBA_USE_SPARSE_FILE 0

namespace uba
{
	constexpr u64 WriteUnit = 4096;

	#if PLATFORM_WINDOWS
	void GetProcessHoldingFile(StringBufferBase& out, const tchar* fileName);
	#else
	Atomic<u32> g_tempFileCounter = 0;
	#endif
		
#if PLATFORM_WINDOWS
	bool SetDeleteOnClose(Logger& logger, const tchar* fileName, FileHandle& handle, bool value)
	{
		ExtendedTimerScope ts(KernelStats::GetCurrent().setFileInfo);
		FILE_DISPOSITION_INFO info;
		info.DeleteFile = value;
		if (!::SetFileInformationByHandle(AsHANDLE(handle), FileDispositionInfo, &info, sizeof(info)))
			return logger.Error(TC("SetFileInformationByHandle (FileDispositionInfo) failed on %llu %s (%s)"), uintptr_t(handle), fileName, LastErrorToText().data);
		return true;
	}

	u64 GetFilePointer(Logger& logger, FileHandle& handle)
	{
		LARGE_INTEGER pos;
		if (SetFilePointerEx(AsHANDLE(handle), ToLargeInteger(0), &pos, FILE_CURRENT))
			return pos.QuadPart;
		logger.Error(TC("SetFilePointerEx failed (%s)"), LastErrorToText().data);
		return ~0ull;
	}
#endif

	FileAccessor::FileAccessor(Logger& logger, const tchar* fileName)
	:	m_logger(logger)
	,	m_fileName(fileName)
	{
	}

	FileAccessor::~FileAccessor()
	{
		InternalClose(false, nullptr);
	}

	bool FileAccessor::CreateWrite(bool allowRead, u32 flagsAndAttributes, u64 fileSize, const tchar* tempPath)
	{
		return InternalCreateWrite(allowRead, flagsAndAttributes, fileSize, tempPath, false);
	}

	bool FileAccessor::InternalCreateWrite(bool allowRead, u32 flagsAndAttributes, u64 fileSize, const tchar* tempPath, bool isMemoryMap)
	{
		#if PLATFORM_WINDOWS
		UBA_ASSERT((flagsAndAttributes & FILE_FLAG_OVERLAPPED) == 0 || EventIsNative);
		#endif


		UBA_ASSERT(flagsAndAttributes != 0);
		m_size = fileSize;
		m_flagsAndAttributes = flagsAndAttributes;

		const tchar* realFileName = m_fileName;

		#if !PLATFORM_WINDOWS
		m_tempPath = tempPath;
		StringBuffer<> tempFile;
		if (tempPath)
		{
			#if PLATFORM_LINUX
			allowRead = true; // Needed if rename does not work
			#endif
			m_tempFileIndex = g_tempFileCounter++;
			realFileName = tempFile.Append(tempPath).Append("Temp_").AppendValue(m_tempFileIndex).data;
		}
		#endif

		u32 createDisp = CREATE_ALWAYS;
		u32 dwDesiredAccess = GENERIC_WRITE | DELETE;
		if (allowRead)
			dwDesiredAccess |= GENERIC_READ;
		u32 dwShareMode = 0;// FILE_SHARE_READ | FILE_SHARE_WRITE;
		u32 retryCount = 0;
		StringBuffer<256> additionalInfo;
		while (true)
		{
			m_fileHandle = uba::CreateFileW(realFileName, dwDesiredAccess, dwShareMode, createDisp, flagsAndAttributes);
			if (m_fileHandle != InvalidFileHandle)
			{
				if (retryCount)
				{
					LogEntryType logType = retryCount > 10 ? LogEntryType_Warning : LogEntryType_Info;
					m_logger.Logf(logType, TC("Had to retry for %u seconds to open file %s for write (because it was being used%s)"), retryCount/2, realFileName, additionalInfo.data);
				}
				break;
			}
			u32 lastError = GetLastError();
			#if PLATFORM_WINDOWS
			constexpr u32 ERROR_INSUFFICIENT_RESOURCES = 1450;
			if (lastError == ERROR_SHARING_VIOLATION || lastError == ERROR_USER_MAPPED_FILE || lastError == ERROR_ACCESS_DENIED || lastError == ERROR_INSUFFICIENT_RESOURCES)
			{
				// Since we have unmap of files in jobs it might be that the file is not unmapped yet, therefor we should retry
				if (retryCount == 1)
					GetProcessHoldingFile(additionalInfo, m_fileName);

				// File can also be read-only for whatever reason
				FileBasicInformation info;
				if (lastError == ERROR_ACCESS_DENIED && !additionalInfo.count)
					if (GetFileBasicInformation(info, m_logger, realFileName, false))
						if (IsReadOnly(info.attributes))
							additionalInfo.Append(TCV(" - File has read-only attribute set"));

				if (retryCount < 40)
				{
					Sleep(500);
					++retryCount;
					continue;
				}
			}
			#endif
			const tchar* retryText = retryCount ? TC(" after retrying for 20 seconds") : TC("");
			return m_logger.Warning(TC("ERROR opening file %s for write%s (%s%s)"), realFileName, retryText, LastErrorToText(lastError).data, additionalInfo.data);
		}

#if PLATFORM_WINDOWS
		if (!SetDeleteOnClose(m_logger, m_fileName, m_fileHandle, true))
			return false;

		if (fileSize != 0)
		{
			u64 allocSize = m_size;
			if ((m_flagsAndAttributes & FILE_FLAG_NO_BUFFERING) != 0)
				allocSize = AlignUp(fileSize, 4*1024);

			if ((m_flagsAndAttributes & FILE_FLAG_OVERLAPPED) != 0)
				(u64&)m_fileHandle |= OverlappedIoFlag;

			#if UBA_USE_SPARSE_FILE
			{
				HANDLE handle = AsHANDLE(m_fileHandle);
				DWORD dwTemp;
				if (!::DeviceIoControl(handle, FSCTL_SET_SPARSE, NULL, 0, NULL, 0, &dwTemp, NULL))
				{
					DWORD lastError = GetLastError();
					if (lastError != ERROR_INVALID_FUNCTION) // Some file systems don't support this
						return m_logger.Error(TC("Failed to make file %s sparse (%s)"), realFileName, LastErrorToText(lastError).data);
				}

				if (!SetEndOfFile(m_logger, realFileName, m_fileHandle, allocSize))
					return false;
			}
			#else
			{
				FILE_ALLOCATION_INFO info;
				info.AllocationSize = ToLargeInteger(allocSize);
				if (!::SetFileInformationByHandle(AsHANDLE(m_fileHandle), FileAllocationInfo, &info, sizeof(info)))
					if (!IsRunningWine())
						return m_logger.Error(TC("SetFileInformationByHandle (FileDispositionInfo) failed on %llu %s (%s)"), uintptr_t(m_fileHandle), realFileName, LastErrorToText().data);
			}
			#endif
		}
#elif PLATFORM_LINUX
		if (isMemoryMap && fileSize != 0) // We want to make sure we allocate memory on the file system to catch out-of-diskspace
		{
			if (fallocate(AsFileDescriptor(m_fileHandle), 0, 0, fileSize) == -1)
				if (errno != ENOSYS && errno != EOPNOTSUPP)
					return m_logger.Error(TC("fallocate failed on %i %s (%s)"), AsFileDescriptor(m_fileHandle), realFileName, strerror(errno));
		}
#endif
		m_isWrite = true;
		return true;
	}

	bool FileAccessor::CreateMemoryWrite(bool allowRead, u32 flagsAndAttributes, u64 size, const tchar* tempPath)
	{
		allowRead = true; // It is not possible to have write only access to file mappings it seems

		UBA_ASSERT(flagsAndAttributes != 0);
		if (!InternalCreateWrite(allowRead, flagsAndAttributes, size, tempPath, true))
			return false;

		const tchar* realFileName = m_fileName;

		#if !PLATFORM_WINDOWS
		StringBuffer<> tempFile;
		if (m_tempPath)
			realFileName = tempFile.Append(m_tempPath).Append("Temp_").AppendValue(m_tempFileIndex).data;
		#endif

		// Size 0 can never be written to so just return a pointer to some memory
		if (size == 0)
		{
			m_data = (u8*)"";
			return true;
		}

		m_mappingHandle = FileMapping_CreateFromFile(m_logger, m_fileHandle, PAGE_READWRITE, size, realFileName);
		if (!m_mappingHandle.IsValid())
			return m_logger.Error(TC("Failed to create memory map %s with size %llu (%s)"), realFileName, size, LastErrorToText().data);

		m_data = FileMapping_Map(m_logger, m_mappingHandle, FILE_MAP_WRITE, 0, size);
		if (!m_data)
			return m_logger.Error(TC("Failed to map view of file %s with size %llu, for write (%s)"), realFileName, size, LastErrorToText().data);

		return true;
	}

	bool FileAccessor::Close(u64* lastWriteTime)
	{
		return InternalClose(true, lastWriteTime);
	}

	bool FileAccessor::Write(const void* data, u64 dataLen, u64 offset, bool lastWrite)
	{
		auto& stats = KernelStats::GetCurrent();
		ExtendedTimerScope ts(stats.writeFile);
		if (!m_isWrite)
			return m_logger.Error(TC("File %s is not opened for write"), m_fileName);
		stats.writeFile.bytes += dataLen;
		return InternalWrite(data, dataLen, offset, lastWrite);
	}

	bool FileAccessor::InternalWrite(const void* data, u64 dataLen, u64 offset, bool lastWrite)
	{
#if PLATFORM_WINDOWS
		bool noBuffering = (m_flagsAndAttributes & FILE_FLAG_NO_BUFFERING) != 0;
		if (noBuffering)
		{
			if (m_writeThroughBufferSize)
			{
				if (offset != m_writeThroughBufferPos)
					return m_logger.Error(TC("NoBuffering require file to be written sequentially or at 4k sizes (%s)"), m_fileName);

				UBA_ASSERT(!((u64)m_fileHandle & OverlappedIoFlag) || !offset);
				u64 toWrite = Min(WriteUnit - m_writeThroughBufferSize, dataLen);
				memcpy(m_writeThroughBuffer + m_writeThroughBufferSize, data, toWrite);
				m_writeThroughBufferSize += u32(toWrite);
				dataLen -= toWrite;
				data = (const u8*)data + toWrite;
				if (m_writeThroughBufferSize < WriteUnit)
					return true;
				m_writeThroughBufferSize = 0;
				if (!InternalWrite(m_writeThroughBuffer, WriteUnit, 0, lastWrite))
					return false;
			}
		}

		if ((u64)m_fileHandle & OverlappedIoFlag)
		{
			constexpr u64 BlockSize = 1024 * 1024;
			constexpr u64 BlockCount = 32;
			OVERLAPPED ol[BlockCount];
			Event ev[BlockCount];
			u64 writeOffset = offset;
			u64 writeLeft = dataLen;
			u8* pos = (u8*)data;
			u64 i = 0;
			u8 tailBuffer[WriteUnit];

			auto WaitAndCheckError = [&](u64 index)
				{
					if (!ev[index].IsSet())
						return m_logger.Error(L"Overlapped I/O WriteFile FAILED on waiting for event!");
					u32 error = u32(ol[index].Internal);
					if (error != ERROR_SUCCESS)
						return m_logger.Error(L"Overlapped I/O WriteFile FAILED!: %s", LastErrorToText(error).data);
					return true;
				};

			auto eg = MakeGuard([&]()
				{
					for (u64 j=0, e=Min(i, BlockCount); j!=e; ++j)
						if (!WaitAndCheckError(j))
							return false;
					return true;
				});


			#if 0//UBA_USE_SPARSE_FILE
			u8 experiment[WriteUnit];
			if (m_writeThroughBufferPos == 0 && lastWrite && dataLen > WriteUnit*4)
			{
				u64 lastOffset = (dataLen / WriteUnit) * WriteUnit - WriteUnit;
				if (lastOffset)
				{
					ev[0].Create(true);
					ol[0] = {};
					ol[0].hEvent = ev[0].GetHandle();
					ol[0].Offset = ToLow(lastOffset);
					ol[0].OffsetHigh = ToHigh(lastOffset);
					if (!::WriteFile(AsHANDLE(m_fileHandle), experiment, WriteUnit, NULL, ol))
					{
						u32 lastError = GetLastError();
						if (lastError != ERROR_IO_PENDING)
							return m_logger.Error(L"FAILED!: %s", LastErrorToText(lastError).data);
					}
					if (!WaitAndCheckError(0))
						return false;
				}
			}
			#endif

			while (writeLeft)
			{
				u64 index = i % BlockCount;

				if (i < BlockCount)
					ev[i].Create(EventResetType_Manual);
				else
				{
					if (!WaitAndCheckError(index))
						return false;
				}

				u64 toWrite = Min(writeLeft, BlockSize);
				u64 toActuallyWrite = toWrite;

				if (noBuffering && toWrite < BlockSize)
				{
					toActuallyWrite = (toWrite / WriteUnit) * WriteUnit;
					if (!toActuallyWrite)
					{
						if (lastWrite)
						{
							memcpy(tailBuffer, pos, toWrite);
							pos = tailBuffer;
							toActuallyWrite = WriteUnit;
							m_writeThroughBufferSize = 0;
						}
						else
						{
							if (!m_writeThroughBuffer)
								m_writeThroughBuffer = new u8[WriteUnit];
							memcpy(m_writeThroughBuffer, pos, toWrite);
							m_writeThroughBufferSize = u32(toWrite);
							break;
						}
					}
					else
						toWrite = toActuallyWrite;
				}

				ol[index] = {};
				ol[index].hEvent = ev[index].GetHandle();
				ol[index].Offset = ToLow(writeOffset);
				ol[index].OffsetHigh = ToHigh(writeOffset);
				ev[index].Reset();

				if (!::WriteFile(AsHANDLE(m_fileHandle), pos, u32(toActuallyWrite), NULL, ol + index))
				{
					u32 lastError = GetLastError();
					if (lastError != ERROR_IO_PENDING)
						return m_logger.Error(L"FAILED!: %s", LastErrorToText(lastError).data);
				}
				++i;
				writeOffset += toActuallyWrite;
				pos += toWrite;
				writeLeft -= toWrite;
			}

			if (!eg.Execute())
				return false;

			m_writeThroughBufferPos = u64(pos - (const u8*)data);
			return true;
		}
#endif

		u32 retryIndex = 0;
		u64 writeLeft = dataLen;
		u8* pos = (u8*)data;
		while (writeLeft)
		{
			u32 toWrite = u32(Min(writeLeft, 256llu * 1024 * 1024));
			u32 toActuallyWrite = toWrite;

#if PLATFORM_WINDOWS
			DWORD written;
			if (!::WriteFile(AsHANDLE(m_fileHandle), pos, toActuallyWrite, &written, NULL))
			{
				DWORD lastError = GetLastError();
				if ((lastError == ERROR_WORKING_SET_QUOTA || lastError == ERROR_PAGEFILE_QUOTA) && retryIndex < 5)
				{
					if (retryIndex == 0)
					{
						const tchar* quotaType = lastError == ERROR_WORKING_SET_QUOTA ? TC("Workingset") : TC("Pagefile");
						m_logger.Info(TC("Insufficient %s quota while writing %s. Will retry..."), quotaType, m_fileName);
					}
					++retryIndex;
					Sleep(retryIndex*1000);
					continue;
				}

				m_logger.Error(TC("ERROR writing file %s writing %u bytes (%llu bytes written out of %llu. FilePos: %llu) (%s)"), m_fileName, toActuallyWrite, (dataLen - writeLeft), dataLen, GetFilePointer(m_logger, m_fileHandle), LastErrorToText(lastError).data);

				if (lastError == ERROR_DISK_FULL)
					ExitProcess(ERROR_DISK_FULL);

				return false;
			}
			if (written > toWrite)
				written = toWrite;
#else
			ssize_t written = write(AsFileDescriptor(m_fileHandle), pos, toActuallyWrite);
			if (written == -1)
				return m_logger.Error(TC("ERROR writing file %s writing %u bytes (%llu bytes written out of %llu) (%s)"), m_fileName, toActuallyWrite, (dataLen - writeLeft), dataLen, strerror(errno));
#endif
			writeLeft -= written;
			pos += written;
		}

		return true;
	}

	bool FileAccessor::OpenRead()
	{
		UBA_ASSERT(false);
		return false;
	}

	bool FileAccessor::OpenMemoryRead(u64 offset, bool errorOnFail)
	{
		if (!OpenFileSequentialRead(m_logger, m_fileName, m_fileHandle, true, false, false))
			return errorOnFail ? m_logger.Error(TC("Failed to open file %s for read (%s)"), m_fileName, LastErrorToText().data) : false;

		FileBasicInformation info;
		if (!GetFileBasicInformationByHandle(info))
			return errorOnFail ? m_logger.Error(TC("GetFileInformationByHandle failed on %s"), m_fileName) : false;

		m_size = info.size;
#if PLATFORM_WINDOWS
		if (m_size)
			m_mappingHandle = FileMapping_CreateFromFile(m_logger, m_fileHandle, PAGE_READONLY, m_size, m_fileName);
		else
			m_mappingHandle = FileMapping_CreateFromFile(m_logger, InvalidFileHandle, PAGE_READONLY, 1, m_fileName);

		if (!m_mappingHandle.IsValid())
			return errorOnFail ? m_logger.Error(TC("Failed to create mapping handle for %s with size %llu (%s)"), m_fileName, m_size, LastErrorToText().data) : false;
#else
		m_mappingHandle.internal = ~0ull;
		m_mappingHandle.shmFd = AsFileDescriptor(m_fileHandle);
		if (offset == m_size)
			return true;
#endif
		m_data = FileMapping_Map(m_logger, m_mappingHandle, FILE_MAP_READ, offset, m_size);
		if (!m_data)
			return errorOnFail ? m_logger.Error(TC("%s - MapViewOfFile failed (%s)"), m_fileName, LastErrorToText().data) : false;

		return true;
	}

	bool FileAccessor::GetFileBasicInformationByHandle(FileBasicInformation& out)
	{
		return uba::GetFileBasicInformationByHandle(out, m_logger, m_fileName, m_fileHandle);
	}

	bool FileAccessor::InternalClose(bool success, u64* lastWriteTime)
	{
		if (m_data)
		{
			if (m_size && !FileMapping_Unmap(m_logger, m_data, m_size, m_fileName))
				return m_logger.Error(TC("Failed to unmap memory for %s (%s)"), m_fileName, LastErrorToText().data);
			m_data = nullptr;
		}

		if (m_mappingHandle.IsValid())
		{
			if (!FileMapping_Close(m_logger, m_mappingHandle, m_fileName))
				return m_logger.Error(TC("Failed to close file mapping for %s (%s)"), m_fileName, LastErrorToText().data);
			m_mappingHandle = {};
		}

		if (m_fileHandle == InvalidFileHandle)
			return true;
		const tchar* realFileName = m_fileName;
		StringBuffer<> tempFile;

		auto closeFile = MakeGuard([&]()
			{
				if (!CloseFile(realFileName, m_fileHandle))
					return m_logger.Error(TC("Failed to close file %s (%s)"), realFileName, LastErrorToText().data);
				m_fileHandle = InvalidFileHandle;
				return true;
			});


		if (!m_isWrite)
			return closeFile.Execute();

		if (m_writeThroughBufferSize)
		{
			m_writeThroughBufferSize = 0;
			if (!Write(m_writeThroughBuffer, WriteUnit, m_writeThroughBufferPos))
				return false;
		}

		if ((m_flagsAndAttributes & FILE_FLAG_NO_BUFFERING) != 0)
			if (!SetEndOfFile(m_logger, m_fileName, m_fileHandle, m_size))
				return false;

		if (m_writeThroughBuffer)
		{
			delete[] m_writeThroughBuffer;
			m_writeThroughBuffer = nullptr;
		}

		#if !PLATFORM_WINDOWS
		if (m_tempPath)
			realFileName = tempFile.Append(m_tempPath).Append("Temp_").AppendValue(m_tempFileIndex).data;
		#endif

		if (success)
		{
			#if PLATFORM_WINDOWS
			if (!SetDeleteOnClose(m_logger, realFileName, m_fileHandle, false))
				return m_logger.Error(TC("Failed to remove delete on close for file %s (%s)"), realFileName, LastErrorToText().data);
			#else
			if (m_tempPath)
			{
				int res;
				{
					ExtendedTimerScope ts(KernelStats::GetCurrent().renameFile);
					res = rename(realFileName, m_fileName);
				}
				if (res == -1)
				{
					if (errno == ENOSPC)
						ReportSystemError(errno);

					if (errno != EXDEV)
						return m_logger.Error(TC("Failed to rename temporary file %s to %s (%s)"), realFileName, m_fileName, strerror(errno));

					ExtendedTimerScope ts(KernelStats::GetCurrent().renameFileFallback);

					// Need to copy, can't rename over devices
					int targetFd = open(m_fileName, O_CREAT | O_TRUNC | O_WRONLY | O_CLOEXEC, m_flagsAndAttributes);
					auto closeTargetFile = MakeGuard([&]() { close(targetFd); });
					if (targetFd == -1)
						return m_logger.Error(TC("Failed to create file %s for move from temporary file %s (%s)"), m_fileName, realFileName, strerror(errno));
						
					int sourceFd = AsFileDescriptor(m_fileHandle);
					#if PLATFORM_MAC
					if (fcopyfile(sourceFd, targetFd, 0, COPYFILE_ALL) == -1)
						return m_logger.Error(TC("Failed to do fcopyfile from temporary %s to file %s (%s)"), realFileName, m_fileName, strerror(errno));
					#else
					if (lseek(sourceFd, 0, SEEK_SET) == -1)
						return m_logger.Error(TC("Failed to do lseek to beginning for sendfile (%s)"), strerror(errno));
					u64 left = m_size;
					while (left)
					{
						auto written = copy_file_range(sourceFd, NULL, targetFd, NULL, left, 0);
						if (written == -1)
							break;
						left -= written;
					}
					if (left)
					{
						left = m_size;
						if (lseek(sourceFd, 0, SEEK_SET) == -1)
							return m_logger.Error(TC("Failed to do lseek to beginning of sourceFd (%s)"), strerror(errno));
						if (lseek(targetFd, 0, SEEK_SET) == -1)
							return m_logger.Error(TC("Failed to do lseek to beginning of targetFd (%s)"), strerror(errno));
						while (left)
						{
							u8 buffer[256*1024];
							u64 toCopy = Min(left, u64(sizeof_array(buffer)));
							auto bytesRead = read(sourceFd, buffer, toCopy);
							if (bytesRead <= 0)
								return m_logger.Error(TC("Failed to do read from file %s (%s)"), realFileName, strerror(errno));
							u8* readPos = buffer;
							u64 toWrite = bytesRead;
							while (toWrite)
							{
								auto written = write(targetFd, readPos, toWrite);
								if (written == -1)
									return m_logger.Error(TC("Failed to write to file %s (%s)"), m_fileName, strerror(errno));
								toWrite -= written;
								readPos += written;
							}
							left -= bytesRead;
						}
					}
					#endif

					remove(realFileName); // Remove real file now when we have copied it over
				}
				
				if (lastWriteTime) // Have to reopen it seems
				{
					if (!CloseFile(m_fileName, m_fileHandle))
						return m_logger.Error(TC("Failed to close file %s (%s)"), m_fileName, LastErrorToText().data);
					if (!OpenFileSequentialRead(m_logger, m_fileName, m_fileHandle))
						return m_logger.Error(TC("Failed to re-open file %s (%s)"), m_fileName, LastErrorToText().data);
				}
			}
			#endif

			if (lastWriteTime)
			{
				*lastWriteTime = 0;
				if (!GetFileLastWriteTime(*lastWriteTime, m_fileHandle))
					m_logger.Warning(TC("Failed to get file time for %s %llu (%s)"), m_fileName, m_fileHandle, LastErrorToText().data);
			}
		}
		else
		{
			#if !PLATFORM_WINDOWS
			if (m_tempPath && remove(realFileName) == -1)
				return m_logger.Error(TC("Failed to remove temporary file %s (%s)"), realFileName, strerror(errno));
			#endif
		}
		return closeFile.Execute();
	}
}
