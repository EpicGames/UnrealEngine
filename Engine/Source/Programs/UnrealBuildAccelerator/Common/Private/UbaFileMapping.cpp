// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaFileMapping.h"
#include "UbaBottleneck.h"
#include "UbaDefinitions.h"
#include "UbaFileHandle.h"
#include "UbaLogger.h"
#include "UbaProcessStats.h"

#if !PLATFORM_WINDOWS
#include <dirent.h>
#include <sys/file.h>
#endif
#if PLATFORM_LINUX
#include "UbaLinuxRawSyscall.h" // RawSyscallDirect for UBA_USE_MEMFD path in UbaFileMapping.inl
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace uba
{
	#if UBA_DEBUG_FILE_MAPPING
	Futex g_fileMappingsLock;
	struct DebugFileMapping { TString hint; Atomic<u64> viewCount = 0; };
	UnorderedMap<HANDLE, DebugFileMapping> g_fileMappings;
	UnorderedMap<const void*, HANDLE> g_viewMappings;
	#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#if PLATFORM_WINDOWS
#include "UbaFileMappingWin.inl"
#else
namespace uba
{
#include "UbaFileMapping.inl"
}
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace uba
{
	void FileMapping_CopyMem(void* dest, const void* source, u64 size)
	{
		auto& stats = KernelStats::GetCurrent();
		ExtendedTimerScope ts(stats.memoryCopy);
		stats.memoryCopy.bytes += size;
		memcpy(dest, source, size);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

}

