// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaFileMappingBuffer.h"
#include "UbaSharedMemoryAllocator.h"
#include "UbaTrace.h"

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	FileMappingBuffer::FileMappingBuffer(Logger& logger, SharedMemoryAllocator& allocator)
		: m_logger(logger)
		, m_allocator(allocator)
	{
	}

	FileMappingBuffer::~FileMappingBuffer()
	{
		if (!m_memoryHandle.IsValid())
			return;
		m_allocator.DecommitMemory(m_memoryHandle, 0, m_committed);
		m_allocator.UnreserveMemory(m_memoryHandle);
		m_allocator.CloseHandle(m_logger, m_memoryHandle, m_name.c_str());
	}

	void FileMappingBuffer::TraceStatus(Trace& trace, u32 startRow, const tchar* hint, u32 traceLevel)
	{
		if (!traceLevel)
			return;

		trace.StatusUpdate(startRow, 1, m_name, LogEntryType_Info);
		StringBuffer<> str;

		str.Appendf(TC("Mem: %s/%s"), BytesToText(m_used).str, BytesToText(m_committed).str);
		str.Appendf(TC(", Independents: %s (%llu)"), BytesToText(m_independentSize).str, m_independentMappingCount.load());

		trace.StatusUpdate(startRow, 9, str, LogEntryType_Info);
	}

	bool FileMappingBuffer::Init(const tchar* name, u64 capacity, u64 commitStepSize)
	{
		m_name = name;
		m_commitStepSize = commitStepSize;
		m_capacity = capacity;
		m_memoryHandle = m_allocator.CreateHandle(m_logger, m_name.c_str(), true);
		if (!m_memoryHandle.IsValid())
			return false;
		m_memory = m_allocator.ReserveMemory(m_memoryHandle, m_capacity); // We manually commit to memory
		return m_memory != nullptr;
	}

	MappedView FileMappingBuffer::AllocAndMapView(u64 size, u64 alignment, const tchar* hint, bool isIndependent)
	{
		if (isIndependent)
		{
			MappedView res;
			// TODO: This can come from network with tiny allocations.. kind of sucks to take a 64k causing fragmentation.
			// Should categorize in small and big blocks.
			SharedMemoryHandle handle = m_allocator.CreateHandle(m_logger, hint);
			m_allocator.ExtendMemory(handle, size, hint, false);

			m_independentSize += size;

			u8* memory = m_allocator.MapView(handle, hint, SharedMemoryMapType_ReadWrite);
			res.handle = handle;

			res.memory = memory;
			res.size = size;

			++m_independentMappingCount;
			return res;
		}

		MappedView res;

		SCOPED_FUTEX(m_usedLock, lock);
		u64 offset = AlignUp(m_used, alignment);
		u64 requiredCommit = AlignUp(offset + size, PageSize);
		if (requiredCommit > m_committed)
		{
			u64 commitSize = requiredCommit - m_committed;
			commitSize = AlignUp(commitSize, m_commitStepSize);
			commitSize = Min(commitSize, m_capacity - m_committed);
			m_allocator.CommitMemory(m_memoryHandle, m_committed, commitSize);
			m_committed += commitSize;
		}
		m_used = offset + size;
		lock.Leave();

		res.handle = m_memoryHandle;
		res.offset = offset;
		res.size = size;
		res.memory = m_memory + offset;
		return res;
	}

	MappedView FileMappingBuffer::MapView(SharedMemoryHandle handle, u64 offset, u64 size, const tchar* hint)
	{
		return InternalMapView(handle, offset, size, hint);
	}

	MappedView FileMappingBuffer::MapView(StringView handleName, u64 size, const tchar* hint)
	{
		const tchar* handleStr = handleName.data + 1;
		const tchar* handleStrEnd = TStrchr(handleStr, '-');
		u64 mappingOffset = 0;
		if (handleStrEnd)
		{
			const tchar* mappingOffsetStr = handleStrEnd + 1;
			mappingOffset = StringToValueBase62(mappingOffsetStr, TStrlen(mappingOffsetStr));
		}
		else
			handleStrEnd = handleStr + TStrlen(handleStr);
		SharedMemoryHandle h = SharedMemoryHandle::FromU64(StringToValueBase62(handleStr, handleStrEnd - handleStr));
		return InternalMapView(h, mappingOffset, size, hint);
	}

	MappedView FileMappingBuffer::InternalMapView(SharedMemoryHandle handle, u64 offset, u64 size, const tchar* hint)
	{
		UBA_ASSERT(handle.IsValid());

		MappedView res;

		if (handle == m_memoryHandle)
		{
			res.handle = handle;
			res.offset = offset;
			res.size = size;
			res.memory = m_memory + offset;
			return res;
		}

		u8* data = m_allocator.MapView(handle, hint, SharedMemoryMapType_ReadOnly);
		if (!data)
			return {};
		UBA_ASSERT(offset == 0);
		res.handle = handle;
		res.offset = offset;
		res.size = size;
		res.memory = data;
		++m_independentMappingCount;
		return res;
	}

	void FileMappingBuffer::UnmapView(MappedView view, const tchar* hint_, u64 newSize)
	{
		if (!view.handle.IsValid())
			return;
		if (view.handle == m_memoryHandle)
			return;
		m_allocator.UnmapView(view.handle, view.memory);
		--m_independentMappingCount;
	}

	bool FileMappingBuffer::IsIndependent(SharedMemoryHandle handle)
	{
		return handle != m_memoryHandle;
	}

	void FileMappingBuffer::CloseIndependent(SharedMemoryHandle handle, u64 size, const tchar* hint)
	{
		m_independentSize -= size;
		m_allocator.CloseHandle(m_logger, handle, hint);
	}

	SharedMemoryHandle FileMappingBuffer::GetMemoryHandle()
	{
		return m_memoryHandle;
	}

	u64 FileMappingBuffer::GetUsed()
	{
		return m_used;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
