// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaFixedSizeFileMappingAllocator.h"
#include "UbaEnvironment.h"
#include "UbaFileMapping.h"
#include "UbaLogger.h"
#include "UbaPlatform.h"
#include "UbaStringBuffer.h"

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	FixedSizeFileMappingAllocator::FixedSizeFileMappingAllocator(Logger& logger, const tchar* name)
		:	m_logger(logger)
		,	m_name(name)
	{
	}

	FixedSizeFileMappingAllocator::~FixedSizeFileMappingAllocator()
	{
		if (m_mappingHandle.IsValid())
			FileMapping_Close(m_logger, m_mappingHandle, TC("FixedSizeFileMappingAllocator"));
	}

	bool FixedSizeFileMappingAllocator::Init(u64 blockSize, u64 capacity)
	{
		m_mappingHandle = FileMapping_Create(m_logger, PAGE_READWRITE|SEC_RESERVE, capacity, nullptr, TC("FixedSizeFileMappingAllocator"));
		if (!m_mappingHandle.IsValid())
			return m_logger.Error(TC("%s - Failed to create memory map with capacity %llu (%s)"), m_name, capacity, LastErrorToText().data);

		m_blockSize = blockSize;
		m_capacity = capacity;
		return true;
	}

	FileMappingAllocation FixedSizeFileMappingAllocator::Alloc(const tchar* hint)
	{
		SCOPED_FUTEX(m_mappingLock, lock);

		u64 index = m_mappingCount;
		bool needCommit = false;
		if (!m_availableBlocks.empty())
		{
			auto it = m_availableBlocks.begin();
			index = *it;
			m_availableBlocks.erase(it);
		}
		else
		{
			if ((m_mappingCount + 1) * m_blockSize > m_capacity)
			{
				m_logger.Error(TC("%s - Out of capacity (%llu bytes / %llu slots of %llu) for %s — bump m_capacity in the Init() call"), m_name, m_capacity, m_capacity / m_blockSize, m_blockSize, hint);
				return { {}, 0, 0 };
			}
			++m_mappingCount;
			needCommit = true;
		}
		lock.Leave();

		u64 offset = index*m_blockSize;
		u8* data = FileMapping_Map(m_logger, m_mappingHandle, FILE_MAP_READ|FILE_MAP_WRITE, offset, m_blockSize, false);
		if (!data)
		{
			m_logger.Error(TC("%s - Alloc failed to map view of file for %s (%s)"), m_name, hint, LastErrorToText().data);
			return { {}, 0, 0 };
		}

		if (needCommit)
		{
			if (!FileMapping_Commit(m_logger, data, m_blockSize, true))
			{
				m_logger.Error(TC("%s - Failed to allocate memory for %s (%s)"), m_name, hint, LastErrorToText().data);
				return { {}, 0, 0 };
			}
		}
		LockMemory(m_logger, data, m_blockSize, hint);

		return {m_mappingHandle, offset, data};
	}

	void FixedSizeFileMappingAllocator::Free(FileMappingAllocation allocation)
	{
		UBA_ASSERT(allocation.handle == m_mappingHandle);
		if (!FileMapping_Unmap(m_logger, allocation.memory, m_blockSize, m_name, true))
			m_logger.Error(TC("%s - Failed to unmap view of file (%s)"), m_name, LastErrorToText().data);
		u64 index = allocation.offset / m_blockSize;
		SCOPED_FUTEX(m_mappingLock, lock);
		m_availableBlocks.insert(index);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
