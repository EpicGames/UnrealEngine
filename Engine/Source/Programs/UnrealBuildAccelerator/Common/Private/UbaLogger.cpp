// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaLogger.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaFileAccessor.h"
#include "UbaParkingLot.h"
#include "UbaPlatform.h"
#include "UbaStringBuffer.h"
#include "UbaThread.h"
#include "UbaTimer.h"

#if PLATFORM_WINDOWS
#include <io.h>
#include <dbghelp.h>
#else
#include <unistd.h>
#define _vsnwprintf_s(buffer,capacity,count,format,args) vsnprintf(buffer, capacity, format, args) // TODO: This will overflow
#define _vscwprintf(format, args) vsnprintf(0, 0, format, args)
#endif


namespace uba
{
	CustomAssertHandler* g_assertHandler;
	void* g_assertHandlerUserData;

	void ParseCallstack(StringBufferBase& out, BinaryReader& reader)
	{
		StringView searchPaths[3];
		StringBuffer<512> currentModuleDir;
		LoggerWithWriter logger(g_nullLogWriter);
		GetDirectoryOfCurrentModule(logger, currentModuleDir);
		StringBuffer<512> alternativePath;
		u32 searchPathIndex = 0;
		if (GetAlternativeUbaPath(logger, alternativePath, currentModuleDir, IsWindows && IsArmBinary))
			searchPaths[searchPathIndex++] = alternativePath;
		searchPaths[searchPathIndex] = currentModuleDir;

		tchar executable[512] = { 0 };
		#if !PLATFORM_WINDOWS
		readlink("/proc/self/exe", executable, sizeof_array(executable));
		#endif

		ParseCallstackInfo(out, reader, executable, searchPaths);
	}

	ANALYSIS_NORETURN void UbaAssert(const tchar* text, const char* file, u32 line, const char* expr, bool allowTerminate, u32 terminateCode, void* context, u32 skipCallstackCount)
	{
		#if UBA_USE_PARKINGLOT
		static ParkingLot::WordMutex assertMutex;
		SCOPED_CRITICAL_SECTION(assertMutex, scopedLock);
		#else
		static ReaderWriterLock& assertLock = *new ReaderWriterLock(); // Leak to prevent asan annoyances during shutdown when asserts happen
		SCOPED_WRITE_LOCK(assertLock, lock);
		#endif

		static u8* writerMem = new u8[4096];
		BinaryWriter writer(writerMem, 0, 4096);
		WriteCallstackInfo(writer, 2, context);

		static auto& sb = *new StringBuffer<16*1024>();
		WriteAssertInfo(sb.Clear(), text, file, line, expr, context);

		BinaryReader reader(writerMem, 0, writer.GetPosition());
		ParseCallstack(sb, reader);

#if PLATFORM_WINDOWS
	#if UBA_ASSERT_MESSAGEBOX
		int ret = MessageBoxW(GetConsoleWindow(), sb.data, TC("Assert"), MB_ABORTRETRYIGNORE);
		if (ret != IDABORT)
		{
			if (ret == IDRETRY)
				DebugBreak();
			return;
		}

		SetFocus(GetConsoleWindow());
		SetActiveWindow(GetConsoleWindow());
	#else
		if (IsDebuggerPresent())
			DebugBreak();
	#endif
#endif

		if (g_assertHandler)
		{
			g_assertHandler(sb.data, sb.count, g_assertHandlerUserData);
			return;
		}

		TFputs(sb.data, stdout);
		TFputs(TC("\n"), stdout);
		fflush(stdout);

		if (!allowTerminate)
			return;
#if PLATFORM_WINDOWS
		// *(int*)nullptr = 42; // Use this to force a crashdump instead of exiting
		ExitProcess(terminateCode);
#else
		_Exit(-1);
#endif
	}

	void SetCustomAssertHandler(CustomAssertHandler* handler, void* userData)
	{
		g_assertHandler = handler;
		g_assertHandlerUserData = userData;
	}

	ANALYSIS_NORETURN void FatalError(u32 code, const tchar* format, ...)
	{
		va_list arg;
		va_start(arg, format);
		StringBuffer<2048> buffer;
		buffer.Append(format, arg);
		va_end(arg);
#if PLATFORM_WINDOWS
		wprintf(TC("UBA FATAL ERROR %u: %s\n"), code, buffer.data);
		fflush(stdout);
		if (IsDebuggerPresent())
			DebugBreak();
		ExitProcess(code);
#else
		printf(TC("UBA FATAL ERROR %u: %s\n"), code, buffer.data);
		fflush(stdout);
		kill(getpid(), SIGKILL);
#endif
	}

	Logger& Logger::Info(const tchar* format, ...)
	{
		va_list arg;
		va_start(arg, format);
		LogArg(LogEntryType_Info, format, arg);
		va_end(arg);
		return *this;
	}

	Logger& Logger::Detail(const tchar* format, ...)
	{
		va_list arg;
		va_start(arg, format);
		LogArg(LogEntryType_Detail, format, arg);
		va_end(arg);
		return *this;
	}

	Logger& Logger::Debug(const tchar* format, ...)
	{
		va_list arg;
		va_start(arg, format);
		LogArg(LogEntryType_Debug, format, arg);
		va_end(arg);
		return *this;
	}

	bool Logger::Warning(const tchar* format, ...)
	{
		va_list arg;
		va_start(arg, format);
		LogArg(LogEntryType_Warning, format, arg);
		va_end(arg);
		return false;
	}

	bool Logger::Error(const tchar* format, ...)
	{
		va_list arg;
		va_start(arg, format);
		LogArg(LogEntryType_Error, format, arg);
		va_end(arg);
		return false;
	}

	Logger& Logger::Logf(LogEntryType type, const tchar* format, ...)
	{
		va_list arg;
		va_start(arg, format);
		LogArg(type, format, arg);
		va_end(arg);
		return *this;
	}

	void Logger::LogArg(LogEntryType type, const tchar* format, va_list& args)
	{
		#if !PLATFORM_WINDOWS
		constexpr size_t _TRUNCATE = (size_t)-1;
		va_list args2;
		va_copy(args2, args);
		auto g = MakeGuard([&]() { va_end(args2); });
		#endif

		tchar buffer[2048];
		int len = _vsnwprintf_s(buffer, sizeof_array(buffer), _TRUNCATE, format, args);
		if (len >= 0 && len < sizeof_array(buffer))
		{
			Log(type, buffer, u32(len));
			return;
		}

		#if PLATFORM_WINDOWS
		len = _vscwprintf(format, args);
		if (len < 0)
		{
			Error(TC("LogArg failed. bad format"));
			return;
		}
		va_list& newArgs = args;
		#else
		va_list& newArgs = args2;
		#endif

		Vector<tchar> buf;
		buf.resize(len + 1);
		len = _vsnwprintf_s(buf.data(), buf.size(), buf.size(), format, newArgs);
		if (len < 0)
		{
			Error(TC("LogArg failed. bad format"));
			return;
		}
		Log(type, buf.data(), u32(len));
	}

	void Logger::Log(LogEntryType type, const StringView& str)
	{
		Log(type, str.data, str.count);
	}

	LoggerWithWriter::LoggerWithWriter(LogWriter& writer, const tchar* prefix)
	:	m_writer(writer)
	,	m_prefix(prefix)
	,	m_prefixLen(prefix ? u32(TStrlen(prefix)) : 0)
	{
	}

	void FilteredLogWriter::Log(LogEntryType type, const tchar* str, u32 strLen, const tchar* prefix, u32 prefixLen)
	{
		if (type > m_level)
			return;
		m_writer.Log(type, str, strLen, prefix, prefixLen);
	}

	class ConsoleLogWriter final : public LogWriter
	{
	public:
		ConsoleLogWriter();
		virtual void BeginScope() override;
		virtual void EndScope() override;
		virtual void Log(LogEntryType type, const tchar* str, u32 strLen, const tchar* prefix = nullptr, u32 prefixLen = 0) override;
		void LogNoLock(LogEntryType type, const tchar* str, u32 strLen, const tchar* prefix, u32 prefixLen, bool addEndOfLine);
		Futex m_lock;
#if PLATFORM_WINDOWS
		HANDLE m_stdout = 0;
		u32 m_defaultAttributes = 0;
#else
		bool m_isTty = false;
#endif
	};
	UBA_API LogWriter& g_consoleLogWriter = *new ConsoleLogWriter(); // Leak to prevent asan annoyances during shutdown when asserts happen
	thread_local u32 t_consoleLogScopeCount = 0;
	Atomic<u64> g_consoleLogLineCount{0};
	u64 GetConsoleLogLineCount() { return g_consoleLogLineCount.load(); }

	class NullLogWriter final : public LogWriter
	{
	public:
		virtual void BeginScope() override {}
		virtual void EndScope() override {}
		virtual void Log(LogEntryType type, const tchar* str, u32 strLen, const tchar* prefix = nullptr, u32 prefixLen = 0) override {}
	} g_nullLogWriterImpl;
	UBA_API LogWriter& g_nullLogWriter = g_nullLogWriterImpl;


	ConsoleLogWriter::ConsoleLogWriter()
	{
#if PLATFORM_WINDOWS
		if (_isatty(_fileno(stdout)))
		{
			m_stdout = GetStdHandle(STD_OUTPUT_HANDLE);
			CONSOLE_SCREEN_BUFFER_INFO csbi;
			GetConsoleScreenBufferInfo(m_stdout, &csbi);
			m_defaultAttributes = csbi.wAttributes;
		}
#else
		m_isTty = isatty(fileno(stdout)) != 0;
#endif
	}

	void ConsoleLogWriter::BeginScope()
	{
		if (!t_consoleLogScopeCount++)
			m_lock.Enter();
	}

	void ConsoleLogWriter::EndScope()
	{
		if (--t_consoleLogScopeCount)
			return;
#if PLATFORM_WINDOWS
		if (!m_stdout)
#endif
			fflush(stdout);
		m_lock.Leave();
	}

	void ConsoleLogWriter::Log(LogEntryType type, const tchar* str, u32 strLen, const tchar* prefix, u32 prefixLen)
	{
		if (t_consoleLogScopeCount)
			return LogNoLock(type, str, strLen, prefix, prefixLen, true);
		SCOPED_FUTEX(m_lock, lock);
		LogNoLock(type, str, strLen, prefix, prefixLen, true);
#if PLATFORM_WINDOWS
		if (!m_stdout)
#endif
			fflush(stdout);
	}

	void ConsoleLogWriter::LogNoLock(LogEntryType type, const tchar* str, u32 strLen, const tchar* prefix, u32 prefixLen, bool addEndOfLine)
	{
		if (addEndOfLine)
			g_consoleLogLineCount.fetch_add(1, std::memory_order_relaxed);
#if PLATFORM_WINDOWS
		if (!m_stdout)
		{
			if (prefixLen)
			{
				TFputs(prefix, stdout);
				TFputs(TC(" - "), stdout);
			}
			_putws(str);
		}
		else
		{
			if (prefixLen)
			{
				WriteConsoleW(m_stdout, prefix, prefixLen, NULL, NULL);
				WriteConsoleW(m_stdout, TC(" - "), 3, NULL, NULL);
			}
			switch (type)
			{
			case LogEntryType_Warning:
				SetConsoleTextAttribute(m_stdout, FOREGROUND_GREEN | FOREGROUND_RED | FOREGROUND_INTENSITY);
				WriteConsoleW(m_stdout, str, strLen, NULL, NULL);
				SetConsoleTextAttribute(m_stdout, (WORD)m_defaultAttributes);
				break;
			case LogEntryType_Error:
				SetConsoleTextAttribute(m_stdout, FOREGROUND_RED | FOREGROUND_INTENSITY);
				WriteConsoleW(m_stdout, str, strLen, NULL, NULL);
				SetConsoleTextAttribute(m_stdout, (WORD)m_defaultAttributes);
				break;
			default:
				WriteConsoleW(m_stdout, str, strLen, NULL, NULL);
				break;
			}
			if (addEndOfLine)
				WriteConsoleW(m_stdout, TC("\r\n"), 2, NULL, NULL);
		}
#else
		if (prefixLen)
		{
			TFputs(prefix, stdout);
			TFputs(TC(" - "), stdout);
		}
		// ANSI colour for warnings (bright yellow) and errors (bright red). Only emit when
		// stdout is a TTY — otherwise the escapes would be garbage in log files / pipes.
		const tchar* colorPrefix = nullptr;
		if (m_isTty)
		{
			if (type == LogEntryType_Warning)
				colorPrefix = TC("\033[93m");
			else if (type == LogEntryType_Error)
				colorPrefix = TC("\033[91m");
		}
		if (colorPrefix)
		{
			TFputs(colorPrefix, stdout);
			TFputs(str, stdout);
			TFputs(TC("\033[0m"), stdout);
		}
		else
		{
			TFputs(str, stdout);
		}
		if (addEndOfLine)
			TFputs(TC("\n"), stdout);
#endif
	}

	void LogToConsoleNoLineFeed(LogEntryType type, const tchar* str, u32 strLen)
	{
		auto& logWriter = (ConsoleLogWriter&)g_consoleLogWriter;
		logWriter.BeginScope();
		logWriter.LogNoLock(type, str, strLen, nullptr, 0, false);
		logWriter.EndScope();
	}

	LastErrorToText::LastErrorToText(u32 lastError)
	{
#if PLATFORM_WINDOWS
		size_t size = ::FormatMessageW(FORMAT_MESSAGE_FROM_SYSTEM | FORMAT_MESSAGE_IGNORE_INSERTS, NULL, lastError, MAKELANGID(LANG_ENGLISH, SUBLANG_ENGLISH_US), data, capacity, NULL);
		if (!size)
			Append(TCV("Unknown win32 error ")).AppendValue(lastError);
		else
			Resize(size - 2);
#else
		Append(strerror(int(lastError)));
#endif
	}

	u32 UToA(tchar* out, u32 val)
	{
		tchar tmp[16];
		tchar* last = tmp + sizeof_array(tmp) - 1;
		tchar* p = last;
		*p = 0;
		do
		{
			*--p = '0' + (val % 10);
			val /= 10;
		}
		while (val);
		u32 len = u32(last - p);
		memcpy(out, p, (len+1)*sizeof(tchar));
		return len;
	}

	void Format(tchar* out, u32 value, const tchar* suffix)
	{
		out += UToA(out, value);
		do { *out = *suffix; if (!*suffix) break; ++out; ++suffix; } while (true);
	}

	void Format1dp(tchar* out, double value, const tchar* suffix)
	{
		u32 scaled = (u32)(value * 10 + 0.5); // round to 1 decimal
		out += UToA(out, scaled / 10);
		*out++ = '.';
		*out++ = '0' + scaled % 10;
		do { *out = *suffix; if (!*suffix) break; ++out; ++suffix; } while (true);
	}

	BytesToText::BytesToText(u64 bytes, bool base2)
	{
		u64 v = base2 ? 1024 : 1000;

		if (bytes < v)
			Format(str, u32(bytes), TC("b"));
		else if (bytes < v * v)
			Format1dp(str, double(bytes)/double(v), TC("kb"));
		else if (bytes < v * v * v)
			Format1dp(str, double(bytes)/double(v * v), TC("mb"));
		else if (bytes < v * v * v * v)
			Format1dp(str, double(bytes)/double(v * v * v), TC("gb"));
		else
			Format1dp(str, double(bytes)/double(v * v * v * v), TC("tb"));
	}

	BytesToText::operator StringView () const { return ToView(str); }

	CountToText::CountToText(u64 count)
	{
		if (count < 1000)
			UToA(str, u32(count));
		else if (count < 1000 * 1000)
			Format1dp(str, double(count) / (1000ull), TC("k"));
		else if (count < 1000ull * 1000 * 1000)
			Format1dp(str, double(count) / (1000ull * 1000), TC("m"));
		else if (count < 1000ull * 1000 * 1000 * 1000)
			Format1dp(str, double(count) / (1000ull * 1000 * 1000), TC("g"));
		else
			Format1dp(str, double(count) / (1000ull * 1000 * 1000 * 1000), TC("t"));
	}

	CountToText::operator StringView () const { return ToView(str); }

	TimeToText::TimeToText(u64 time, bool allowMinutes) : TimeToText(time, allowMinutes, GetFrequency()) {}
	TimeToText::TimeToText(u64 time, bool allowMinutes, u64 frequency)
	{
		u64 ms = TimeToMs(time, frequency);
		if (ms == 0 && time != 0)
			TStrcpy_s(str, 32, TC("<1ms"));
		else if (ms < 1000)
			Format(str, u32(ms), TC("ms"));
		else if (ms < 60 * 1000 || !allowMinutes)
			Format1dp(str, double(ms) / 1000, TC("s"));
		else
		{
			u32 totalSec = u32(float(ms) / 1000);
			u32 totalMin = totalSec / 60;
			u32 min = totalMin % 60;
			u32 sec = totalSec % 60;
			u32 hour = totalMin / 60;
			u32 days = hour / 24;
			hour -= days*24;
			if (days)
				TSprintf_s(str, 32, TC("%ud%uh%um"), (unsigned int)days, (unsigned int)hour, (unsigned int)min);
			else if (hour)
				TSprintf_s(str, 32, TC("%uh%um%us"), (unsigned int)hour, (unsigned int)min, (unsigned int)sec);
			else
				TSprintf_s(str, 32, TC("%um%us"), (unsigned int)min, (unsigned int)sec);
		}
	}

	TimeToText::operator StringView () const { return ToView(str); }

	#if PLATFORM_WINDOWS
	#define TStrFtime ::wcsftime
	#define TLocalTime(a, b) ::localtime_s(a, b) == 0
	#else
	#define TStrFtime ::strftime
	#define TLocalTime(a, b) ::localtime_s(a, b) != 0
	#endif

	DateToText::DateToText(float seconds)
	{
		auto rawTime = (time_t)seconds;
		struct tm timeinfo;
		if (TLocalTime(&timeinfo, &rawTime))
			TStrFtime(str, sizeof_array(str), TC("%Y-%m-%d %H:%M:%S"), &timeinfo);
		else
			str[0] = 0;
	}

	DateToText::DateToText(u64 fileTime)
	{
		#if PLATFORM_WINDOWS
		u64 unixNs = (fileTime - 116444736'000'000'000ull) * 100ull;
		#else
		u64 unixNs = fileTime * 100;
		#endif

		time_t sec = (time_t)(unixNs / 1'000'000'000ull);
		struct tm timeinfo;
		if (TLocalTime(&timeinfo, &sec))
			TStrFtime(str, sizeof_array(str), TC("%Y-%m-%d %H:%M:%S"), &timeinfo);
		else
			TStrcpy_s(str, sizeof_array(str), TC("BADTIME"));
	}

#if UBA_DEBUG_LOGGER || UBA_DEBUG
	thread_local u32 t_debugLogScopeCount = 0;

	class DebugLogWriter final : public LogWriter
	{
	public:
		virtual void BeginScope() override
		{
			if (!m_file)
				return;
			if (!t_debugLogScopeCount++)
				m_logLock.Enter();
		}

		virtual void EndScope() override
		{
			if (!m_file)
				return;
			if (--t_debugLogScopeCount)
				return;
			//

			m_logLock.Leave();
		}

		virtual void Log(LogEntryType type, const tchar* str, u32 strLen, const tchar* prefix = nullptr, u32 prefixLen = 0) override
		{
			if (!m_file)
				return;
			if (t_debugLogScopeCount)
				return LogNoLock(type, str, strLen, prefix, prefixLen);
			SCOPED_FUTEX(m_logLock, lock);
			LogNoLock(type, str, strLen, prefix, prefixLen);
		}

		void LogNoLock(LogEntryType type, const tchar* str, u32 strLen, const tchar* prefix, u32 prefixLen)
		{
			#if PLATFORM_WINDOWS
			char buffer[512];
			size_t destLen;
			if (wcstombs_s(&destLen, buffer, sizeof(buffer), str, sizeof(buffer)-2) != 0)
			{
				strcpy_s(buffer, sizeof(buffer), "BAD_STRING\n");
				destLen = 11;
			}
			else
			{
				buffer[destLen-1] = '\r'; // replace null terminator with \r
				buffer[destLen++] = '\n';
			}
			m_file->Write(buffer, destLen);
			#else
			m_file->Write(str, strLen);
			#endif
		}

		TString m_fileName;
		FileAccessor* m_file = nullptr;
		Futex m_logLock;

	};

	Logger* StartDebugLogger(Logger& outerLogger, const tchar* fileName)
	{
		auto debugWriter = new DebugLogWriter();
		debugWriter->m_fileName = fileName;
		auto fa = new FileAccessor(outerLogger, debugWriter->m_fileName.c_str());
		if (!fa->CreateWrite())
		{
			delete fa;
			return new LoggerWithWriter(g_nullLogWriter);
		}

		#if PLATFORM_WINDOWS
		unsigned char utf8BOM[] = { 0xef,0xbb,0xbf }; 
		fa->Write(utf8BOM, sizeof(utf8BOM));
		#endif

		debugWriter->m_file = fa;
		return new LoggerWithWriter(*debugWriter);
	}

	Logger* StopDebugLogger(Logger* logger)
	{
		auto debugLogger = (LoggerWithWriter*)logger;
		if (&debugLogger->m_writer != &g_nullLogWriter)
		{
			auto debugWriter = (DebugLogWriter*)&debugLogger->m_writer;
			if (debugWriter->m_file)
			{
				debugWriter->m_file->Close();
				delete debugWriter->m_file;
				debugWriter->m_file = nullptr;
			}
			delete debugWriter;
		}
		delete debugLogger;
		return nullptr;
	}

#endif

	#if UBA_TRACK_CONTENTION
	List<ContentionTracker>& GetContentionTrackerList();
	#endif

	void ParseCallstackInfo(StringBufferBase& out, BinaryReader& reader, const tchar* executable, const StringView* searchPaths, bool writeHeader)
	{
		#if PLATFORM_WINDOWS
		bool isRunningWine = reader.ReadBool();
		#else
		bool isRunningWine = false;
		#endif

		struct CallstackEntry { u64 moduleIndex; u64 memoryOffset; };
		u64 callstackCount = reader.Read7BitEncoded();
		Vector<CallstackEntry> entries(callstackCount);
		for (auto& entry : entries)
		{
			entry.moduleIndex = reader.Read7BitEncoded();
			entry.memoryOffset = reader.Read7BitEncoded();
		}

		struct ModuleEntry { u64 start; u64 size; TString name; bool handled; UnorderedMap<u64, TString> symbols; };
		u64 moduleCount = reader.Read7BitEncoded();
		Vector<ModuleEntry> modules(moduleCount);
		for (auto& mod : modules)
		{
			mod.start = reader.Read7BitEncoded();
			mod.size = reader.Read7BitEncoded();
			mod.name = reader.ReadString();
			mod.handled = false;
		}

		if (writeHeader)
			out.Append(TCV("\n CALLSTACK")).Append(isRunningWine ? TCV(" (Wine)") : TCV("")).Append(':');

		if (entries.empty())
		{
			out.Append(TCV("\n   <No entries available>"));
			return;
		}

#if PLATFORM_WINDOWS
		static u64 processHandleCounter = 45234523;
		auto processHandle = (HANDLE)processHandleCounter++; // TODO: Reuse handle based on a bunch of criterias?

		StringBuffer<> searchPathString;
		for (auto it=searchPaths; it->count; ++it)
		{
			if (it != searchPaths)
				searchPathString.Append(';');
			searchPathString.Append(*it);
		}

		if (SymInitializeW(processHandle, searchPathString.data, FALSE))
		{
			SymSetOptions(SYMOPT_LOAD_LINES);

			for (auto& mod : modules)
			{
				if (mod.name.empty())
				{
					mod.name = TC("<Unknown>");
					continue;
				}

				auto res = SymLoadModuleExW(processHandle, NULL, mod.name.c_str(), NULL, mod.start, (DWORD)mod.size, NULL, 0);
				if (res != 0 || GetLastError() == ERROR_SUCCESS)
				{
					// This is very annoying but it can actually succeed in loading symbols even though pdb is not available.
					// .. and use internal symbols or something? those symbols are just wrong and messed up and we rather
					// have have the relative addresses printed out!
					IMAGEHLP_MODULEW64 moduleInfo = {0};
					moduleInfo.SizeOfStruct = sizeof(moduleInfo);
					if (SymGetModuleInfoW64(processHandle, res, &moduleInfo))
						if (*moduleInfo.LoadedPdbName)
							mod.handled = true;
					continue;
				}

				// Try something more?
			}
		}
		auto symCleanup = MakeGuard([processHandle]() { SymCleanup(processHandle); });

		for (auto& entry : entries)
		{
			if (writeHeader)
				out.Append(TCV("\n   "));
			if (entry.moduleIndex == ~0u)
			{
				out.Append(TCV("<Unknown>"));
				continue;
			}
			auto& mod = modules[entry.moduleIndex];
			if (mod.handled)
			{
				u8 buffer[1024];

				auto& info = *(SYMBOL_INFOW*)buffer;
				memset(&info, 0, sizeof(info));
				info.SizeOfStruct = sizeof(info);
				info.MaxNameLen = (sizeof(buffer) - sizeof(info))/sizeof(tchar);
				DWORD64 displacement2 = 0;
				bool gotSymbol = false;
				if (SymFromAddrW(processHandle, mod.start + entry.memoryOffset, &displacement2, &info))
				{
					out.Appendf(TC("%s"), info.Name);
					gotSymbol = true;
				}

				auto& line = *(IMAGEHLP_LINEW64*)buffer;
				line.SizeOfStruct = sizeof(line);
				DWORD displacement = 0;
				bool gotLine = false;
				if (SymGetLineFromAddrW64(processHandle, mod.start + entry.memoryOffset, &displacement, &line))
				{
					auto fileName = line.FileName;
					if (auto lastSlash = TStrrchr(fileName, '\\'))
						fileName = lastSlash + 1;
					if (gotSymbol)
						out.Append(TCV(" ("));
					out.Appendf(TC("%s:%u"), fileName, line.LineNumber);
					if (gotSymbol)
						out.Append(TCV(")"));
					gotLine = true;
				}
				if (gotSymbol || gotLine)
					continue;
			}
			out.Appendf(TC("%s: +0x%llx"), mod.name.c_str(), entry.memoryOffset);
		}
#else
		for (auto& mod : modules)
		{
			if (mod.handled)
				continue;
			if (mod.name.empty())
				mod.name = executable;
			StringView name = StringView(mod.name).GetFileName();
			for (auto it=searchPaths; it->count; ++it)
			{
				struct stat attr;
				StringBuffer<512> buf;
				buf.Clear().Append(*it).EnsureEndsWithSlash().Append(name);
				if (stat(buf.data, &attr) == -1)
					continue;
				mod.name = buf.data;
				break;
			}
			mod.handled  = true;
		}

		for (u64 i=0; i!=modules.size(); ++i)
		{
			auto& mod = modules[i];
			StringBuffer<1024> cmd;
			#if PLATFORM_LINUX
			cmd.Append("addr2line");
			#else
			cmd.Appendf("atos -o %s --offset", mod.name.c_str());
			#endif
			Vector<u64> memoryOffsets;
			for (auto& entry : entries)
				if (entry.moduleIndex == i)
				{
					cmd.Appendf(" 0x%x", entry.memoryOffset);
					memoryOffsets.push_back(entry.memoryOffset);
				}
			#if PLATFORM_LINUX
			cmd.Append(" -f -C -p -e ").Append(mod.name);
			#endif
			cmd.Append(" 2>/dev/null");

			fflush(stdout);

			u32 memoryOffsetIndex = 0;
			if (FILE* fp = popen(cmd.data, "r"))
			{
				auto cg = MakeGuard([&]() { pclose(fp); });

				u32 index = 0;
				u32 countBeforeCallstack = out.count;
				errno = 0;
				StringBuffer<1024> str;
				while (true)
				{
					if (!fgets(str.data, str.capacity, fp))
						break;
					str.count = TStrlen(str.data);
					if (str.EndsWith("\n"))
						str.Resize(str.count - 1);
					mod.symbols.try_emplace(memoryOffsets[memoryOffsetIndex++], str.ToString());
				}
			}
		}

		u32 skipIndex = 0; // remove signal handler logic
		for (u64 i=0; i!=entries.size() && !skipIndex; ++i)
			if (entries[i].moduleIndex != ~0u)
				if (modules[entries[i].moduleIndex].symbols[entries[i].memoryOffset].find("__restore_rt") != -1)
					skipIndex = i + 1;

		for (auto& entry : entries)
		{
			if (skipIndex > 0)
			{
				--skipIndex;
				continue;
			}
			if (entry.moduleIndex == ~0u)
			{
				out.Appendf(TC("\n   <Unknown>: 0x%llx"), entry.memoryOffset);
				continue;
			}
			auto& mod = modules[entry.moduleIndex];
		
			auto pg = MakeGuard([&]() { out.Appendf(TC("\n   %s: 0x%llx"), mod.name.c_str(), entry.memoryOffset); });

			StringBuffer<1024> str(mod.symbols[entry.memoryOffset]);
			if (!str.count || str[0] == ':' || str[0] == '?')
				continue;
			char* fileName = str.data;
			pg.Cancel();
			if (out.capacity - out.count < strlen(fileName) + 5)
				break;
			out.Appendf(TC("\n   %s"), fileName);
		}
#endif
	}

	void PrintContentionSummary(Logger& logger)
	{
	#if UBA_TRACK_CONTENTION
		logger.Info(TC("  ------- Contention summary -------"));
		List<ContentionTracker*> list;
		for (auto& ct : GetContentionTrackerList())
			if (TimeToMs(ct.time) > 1)
				list.push_back(&ct);
		list.sort([](const ContentionTracker* a, const ContentionTracker* b)
			{
				if (a->time != b->time)
					return a->time > b->time;
				return a < b;
			});

		for (auto& ct : list)
		{
			StringBuffer<512> fn;
			fn.Append(ct->file);
			StringBuffer<256> s;
			s.Append(TCV("  ")).AppendFileName(fn.data).Append(':').AppendValue(ct->line).Append(TCV(" - ")).AppendValue(ct->count).Append(TCV(" (")).Append(TimeToText(ct->time).str).Append(')');
			logger.Info(s.data);
			ct->count = 0;
			ct->time = 0;
		}
	#endif
	}

	void TraverseAllCallstacks(const Function<void(const CallstackInfo&)>& func, const TraverseThreadErrorFunc& errorFunc)
	{
		UnorderedMap<CasKey, CallstackInfo> callstacks;

		TraverseAllThreads([&](u32 tid, void** callstack, u32 callstackCount, const tchar* desc)
			{
				StackBinaryWriter<4096> stackWriter;
				u32* stackSize = (u32*)stackWriter.AllocWrite(sizeof(u32));
				WriteCallstackInfo(stackWriter, callstack, callstackCount);
				*stackSize = u32(stackWriter.GetPosition() - 4);

				CasKeyHasher hasher;
				hasher.Update(stackWriter.GetData(), stackWriter.GetPosition());

				auto insres = callstacks.try_emplace(ToCasKey(hasher, false));
				CallstackInfo& cs = insres.first->second;
				cs.threadIds.push_back(tid);
				if (!insres.second)
					return;
				if (desc)
					cs.desc = desc;
				cs.data.resize(stackWriter.GetPosition());
				memcpy(cs.data.data(), stackWriter.GetData(), cs.data.size());
			}, errorFunc);
		for (auto& kv : callstacks)
		{
			CallstackInfo& cs = kv.second;
			StringBuffer<1024> temp;
			if (!cs.desc.empty())
				temp.Append(cs.desc).Append(TCV(" - "));
			temp.Append(TCV("Thread Ids: "));
			for (u32 tid : cs.threadIds)
			{
				if (temp.count >= temp.capacity - 10)
				{
					temp.Append(TCV("..."));
					break;
				}
				temp.AppendValue(tid).Append(' ');
			}
			cs.desc = temp.data;
			func(cs);
		}
	}

	void PrintAllCallstacks(Logger& logger)
	{
		TraverseAllCallstacks([&](const CallstackInfo& cs)
			{
				BinaryReader stackReader(cs.data.data(), 0, cs.data.size());
				stackReader.Skip(4); // Skip the size
				static auto& sb = *new StringBuffer<16*1024>();
				ParseCallstack(sb.Clear(), stackReader);
				logger.Info(TC("%s%s"), cs.desc.c_str(), sb.data);
			},
			[&](const StringView& error)
			{
				logger.Info(error.data);
			});
	}

	LogStallScope::LogStallScope(Logger& l, LogEntryType t, u64 ts, const tchar* m) : logger(l), type(t), timeSeconds(ts), timeStart(GetTime()), messageFormat(m) {}
	LogStallScope::~LogStallScope() { Leave(); }
	void LogStallScope::Leave()
	{
		if (!timeStart)
			return;
		u64 delta = GetTime() - timeStart;
		if (delta > MsToTime(timeSeconds*1000))
			logger.Logf(type, messageFormat, TimeToText(delta).str);
		timeStart = 0;
	}
}
