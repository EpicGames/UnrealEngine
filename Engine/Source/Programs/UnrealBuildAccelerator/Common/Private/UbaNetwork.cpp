// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaNetwork.h"

namespace uba
{
	StringView ToString(SystemMessageType type)
	{
		switch (type)
		{
			#define UBA_SYSTEM_MESSAGE(x) case SystemMessageType_##x: return AsView(TC("System_")#x);
			UBA_SYSTEM_MESSAGES
			#undef UBA_SYSTEM_MESSAGE
		default:
			return ToView(TC("UnknownSystemMessage")); // Should never happen
		}
	}

	StringView ToString(StorageMessageType type)
	{
		switch (type)
		{
			#define UBA_STORAGE_MESSAGE(x) case StorageMessageType_##x: return AsView(TC("Storage_")#x);
			UBA_STORAGE_MESSAGES
			#undef UBA_STORAGE_MESSAGE
		default:
			return ToView(TC("UnknownStorageMessage")); // Should never happen
		}
	}

	StringView ToString(SessionMessageType type)
	{
		switch (type)
		{
			#define UBA_SESSION_MESSAGE(x) case SessionMessageType_##x: return AsView(TC("Session_")#x);
			UBA_SESSION_MESSAGES
			#undef UBA_SESSION_MESSAGE
		default:
			return ToView(TC("UnknownSessionMessage")); // Should never happen
		}
	}

	StringView ToString(CacheMessageType type)
	{
		switch (type)
		{
			#define UBA_CACHE_MESSAGE(x) case CacheMessageType_##x: return AsView(TC("Cache_")#x);
			UBA_CACHE_MESSAGES
			#undef UBA_CACHE_MESSAGE
		default:
			return ToView(TC("UnknownCacheMessage")); // Should never happen
		}
	}

	StringView MessageToString(u8 serviceId, u8 messageType)
	{
		switch (serviceId)
		{
		case SystemServiceId:
			return ToString(SystemMessageType(messageType));
		case StorageServiceId:
			return ToString(StorageMessageType(messageType));
		case SessionServiceId:
			return ToString(SessionMessageType(messageType));
		case CacheServiceId:
			return ToString(CacheMessageType(messageType));
		default:
			return ToView(TC("UnknownServiceId")); // Should never happen
		}
	}
}