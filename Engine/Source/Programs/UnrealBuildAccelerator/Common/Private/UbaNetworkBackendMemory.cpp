// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaNetworkBackendMemory.h"
#include "UbaEvent.h"
#include "UbaHash.h"
#include "UbaPlatform.h"
#include "UbaStringBuffer.h"
#include "UbaTimer.h"

#define UBA_VALIDATE_MULTIPLE_SEND 0

namespace uba
{
	struct NetworkBackendMemory::Connection
	{
		struct Peer
		{
			RecvHeaderCallback* headerCallback = nullptr;
			RecvBodyCallback* bodyCallback = nullptr;
			void* context = nullptr;
			u32 headerSize = 0;
			const tchar* recvHint = nullptr;
			
			DisconnectCallback* disconnectCallback = nullptr;
			void* disconnectContext = nullptr;

			#if UBA_VALIDATE_MULTIPLE_SEND
			Atomic<u32> sendCount = 0;
			#endif

			DataSentCallback* dataSentCallback = nullptr;
			void* dataSentContext = nullptr;
			DataReceivedCallback* dataRecvCallback = nullptr;
			void* dataRecvContext = nullptr;
		};

		Peer peer[2];
		Guid uid;
		Atomic<bool> connected = false;
	};

	NetworkBackendMemory::NetworkBackendMemory(LogWriter& writer, const tchar* prefix)
	{
	}

	NetworkBackendMemory::~NetworkBackendMemory()
	{
		delete m_connection;
	}

	void NetworkBackendMemory::Shutdown(void* connection)
	{
		u64 from = ((uintptr_t)connection)-1;
		u64 to = from == 0 ? 1 : 0;
		auto& rc = m_connection->peer[to];

		m_connection->connected = false;

		if (auto cb = rc.disconnectCallback)
			cb(rc.disconnectContext, m_connection->uid, connection);
	}

	bool NetworkBackendMemory::Send(Logger& logger, void* connection, const void* data, u32 dataSize, SendContext& sendContext, const tchar* sendHint)
	{
		UBA_ASSERT(m_connection);
		sendContext.isUsed = true;
		sendContext.isFinished = true;

		u64 from = ((uintptr_t)connection)-1;
		u64 to = from == 0 ? 1 : 0;

		auto& peer = m_connection->peer[to];

		#if UBA_VALIDATE_MULTIPLE_SEND
		++peer.sendCount;
		auto sg = MakeGuard([&]() { --peer.sendCount; });
		#endif

		u8* dataPos = (u8*)data;
		u32 dataLeft = dataSize;
		while (dataLeft)
		{
			if (!m_connection->connected)
				return false;

			void* bodyContext = nullptr;
			u8* bodyData = nullptr;
			u32 bodySize = 0;

			u32 headerSize = peer.headerSize;

			if (!peer.headerCallback(peer.context, m_connection->uid, dataPos, bodyContext, bodyData, bodySize))
				return false;

			UBA_ASSERT(dataLeft >= headerSize);
			dataPos += headerSize;
			dataLeft -= headerSize;

			if (!bodySize)
				continue;

			memcpy(bodyData, dataPos, bodySize);
			auto bc = peer.bodyCallback;
			if (!bc)
				return logger.Error(TC("Connection body callback not set"));

			if (!bc(peer.context, false, dataPos, bodyContext, bodyData, bodySize))
				return false;

			UBA_ASSERT(dataLeft >= bodySize);
			dataPos += bodySize;
			dataLeft -= bodySize;
		}

		if (auto c = m_connection->peer[from].dataSentCallback)
			c(m_connection->peer[from].dataSentContext, dataSize);
		if (auto c = m_connection->peer[to].dataRecvCallback)
			c(m_connection->peer[to].dataRecvContext, dataSize);

		return true;
	}

	void NetworkBackendMemory::SetDataSentCallback(void* connection, void* context, DataSentCallback* callback)
	{
		UBA_ASSERT(m_connection);
		u64 from = ((uintptr_t)connection) - 1;
		m_connection->peer[from].dataSentCallback = callback;
		m_connection->peer[from].dataSentContext = context;
	}

	void NetworkBackendMemory::SetDataReceivedCallback(void* connection, void* context, DataReceivedCallback* callback)
	{
		UBA_ASSERT(m_connection);
		u64 from = ((uintptr_t)connection) - 1;
		m_connection->peer[from].dataRecvCallback = callback;
		m_connection->peer[from].dataRecvContext = context;
	}

	void NetworkBackendMemory::SetRecvCallbacks(void* connection, void* context, u32 headerSize, RecvHeaderCallback* h, RecvBodyCallback* b, const tchar* recvHint)
	{
		UBA_ASSERT(m_connection);
		auto& peer = m_connection->peer[((uintptr_t)connection)-1];

		#if UBA_VALIDATE_MULTIPLE_SEND
		UBA_ASSERT(peer.sendCount <= 1);
		#endif

		peer.headerCallback = h;
		peer.bodyCallback = b;
		peer.context = context;
		peer.headerSize = headerSize;
		peer.recvHint = recvHint;
	}

	void NetworkBackendMemory::SetRecvTimeout(void* connection, u32 timeoutMs, void* context, RecvTimeoutCallback* callback)
	{
	}

	void NetworkBackendMemory::SetThreadedSend(void* connection)
	{
	}

	void NetworkBackendMemory::SetDisconnectCallback(void* connection, void* context, DisconnectCallback* callback)
	{
		UBA_ASSERT(m_connection);
		auto& peer = m_connection->peer[((uintptr_t)connection)-1];
		peer.disconnectCallback = callback;
		peer.disconnectContext = context;
	}

	void NetworkBackendMemory::SetAllowLessThanBodySize(void* connection, bool allow)
	{
		// Not implemented
	}

	void NetworkBackendMemory::SetPriority(void* connection)
	{
	}

	bool NetworkBackendMemory::StartListen(Logger& logger, u16 port, const tchar* ip, const ListenConnectedFunc& connectedFunc, bool linger)
	{
		SCOPED_WRITE_LOCK(m_connectedFuncLock, l);
		m_connectedFunc = connectedFunc;
		return true;
	}

	void NetworkBackendMemory::StopListen()
	{
	}

	bool NetworkBackendMemory::Connect(Logger& logger, const tchar* ip, const ConnectedFunc& connectedFunc, u16 port, bool* timedOut)
	{
		u64 start = GetTime();
		SCOPED_WRITE_LOCK(m_connectedFuncLock, l);
		while (!m_connectedFunc)
		{
			if (TimeToMs(GetTime() - start) > 1000*2)
				return false;
			l.Leave();
			Sleep(10);
			l.Enter();
		}

		if (m_connection) // Ignore multiple connects
			return true;

		m_connection = new Connection();
		m_connection->connected = true;
		if (!m_connectedFunc((void*)1, {}))
			return false;
		l.Leave();
		return connectedFunc((void*)2, {}, timedOut);
	}

	bool NetworkBackendMemory::Connect(Logger& logger, const sockaddr& remoteSocketAddr, const ConnectedFunc& connectedFunc, bool* timedOut, const tchar* nameHint)
	{
		return false;
	}

	void NetworkBackendMemory::DeleteConnection(void* connection)
	{
		//delete m_connection;
	}

	void NetworkBackendMemory::SetSendBufferSize(void* connection, u32 bytes)
	{
	}

	void NetworkBackendMemory::SetRecvBufferSize(void* connection, u32 bytes)
	{
	}

	void NetworkBackendMemory::GetTotalSendAndRecv(u64& outSend, u64& outRecv)
	{
		outSend = 0;
		outRecv = 0;
	}

	void NetworkBackendMemory::Validate(Logger& logger, const Vector<void*>& connections, bool full)
	{
		logger.Info(TC("  NetworkBackendMemory"));
		logger.Info(TC("     NoInfo"));
	}
}