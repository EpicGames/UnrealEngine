// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaNetworkBackendQuic.h"
#include "UbaPlatform.h"

#if UBA_USE_QUIC
#include "msquic.h"
#pragma comment (lib, "Crypt32.lib")
#pragma comment (lib, "ncrypt.lib")
#pragma comment (lib, "msquic.lib")

namespace uba
{
	Atomic<u32> connectionCount = 0;

	NetworkBackendQuic::Connection::Connection(NetworkBackendQuic& b, HQUIC ch)
	:	backend(b)
	,	connectionHandle(ch)
	,	refCount(1)
	{
		u32 count = ++connectionCount;
		wprintf(L"CONNECTION CREATED (Count: %u)\n", count);
	}

	void NetworkBackendQuic::Connection::AddRef()
	{
		++refCount;
	}

	void NetworkBackendQuic::Connection::Release()
	{
		if (--refCount)
			return;
		if (HQUIC handle = connectionHandle)
			backend.m_msQuic->ConnectionClose(handle);
		delete this;
		u32 count = --connectionCount;
		wprintf(L"CONNECTION Destroyed (Count: %u)\n", count);
	}

	NetworkBackendQuic::NetworkBackendQuic(LogWriter& writer, const wchar_t* prefix)
	:	m_logger(writer, prefix)
	{

	}

	void NetworkBackendQuic::Shutdown(void* connection)
	{
		auto& conn = *(Connection*)connection;
		if (!conn.connected)
			return;
		if (conn.connectionHandle)
			m_msQuic->ConnectionShutdown(conn.connectionHandle, QUIC_CONNECTION_SHUTDOWN_FLAG_NONE, 0);
		conn.connected = false;
	}

	void NetworkBackendQuic::Close(void* connection)
	{
		auto& conn = *(Connection*)connection;
		SCOPED_WRITE_LOCK(m_connectionsLock, lock);
		bool erased = m_connections.erase(conn.connectionHandle);
		lock.Leave();
		if (erased)
			conn.Release();
	}

	bool NetworkBackendQuic::Send(Logger& logger, void* connection, const void* data, u32 dataSize, SendContext& sendContext, const tchar* sendHint)
	{
		SendContext* sc = &sendContext;
		UBA_ASSERT(!sc->isUsed);
		sc->isUsed = true;

		auto& conn = *(Connection*)connection;
		if (!conn.connected)
		{
			sc->isFinished = true;
			return false;
		}

		QUIC_BUFFER temp;
		QUIC_BUFFER* buffer = nullptr;

		Event* completeEvent = nullptr;
		auto eg = MakeGuard([&]() { if (completeEvent) completeEvent->~Event(); });

		if (sendContext.flags & SendFlags_Async)
		{
			u8* newMem = (u8*)malloc(sizeof(SendContext) + dataSize);
			sc = new (newMem) SendContext(sendContext.flags);
			void* newData = newMem + sizeof(SendContext);
			memcpy(newData, data, dataSize);
			data = newData;
			buffer = (QUIC_BUFFER*)sc->data;
			sendContext.isFinished = true;
		}
		else if (sendContext.flags & SendFlags_ExternalWait)
		{
			buffer = (QUIC_BUFFER*)sc->data;
		}
		else
		{
			buffer = &temp;
			completeEvent = (Event*)sc->data;
			completeEvent->Create(true);
		}

		sc->size = dataSize;

		buffer->Buffer = (u8*)data;
		buffer->Length = dataSize;

		QUIC_STATUS Status = QUIC_STATUS_SUCCESS;

		static auto StaticStreamCallback = [](HQUIC Stream, void* Context, QUIC_STREAM_EVENT* Event) { return ((NetworkBackendQuic&)((Connection*)Context)->backend).StreamCallback(*(Connection*)Context, Stream, Event); };
		HQUIC streamHandle;
		QUIC_STREAM_OPEN_FLAGS openFlags = QUIC_STREAM_OPEN_FLAG_UNIDIRECTIONAL; //QUIC_STREAM_OPEN_FLAG_NONE
		if (QUIC_FAILED(Status = m_msQuic->StreamOpen(conn.connectionHandle, openFlags, StaticStreamCallback, connection, &streamHandle)))
		{
			sc->isFinished = true;
			if (sc != &sendContext)
				delete sc;
			return m_logger.Error(L"StreamSend failed, 0x%x!\n", Status);
		}

		conn.AddRef();

		QUIC_SEND_FLAGS flags = QUIC_SEND_FLAG_START | QUIC_SEND_FLAG_FIN;
		if (QUIC_FAILED(Status = m_msQuic->StreamSend(streamHandle, buffer, 1, flags, sc)))
		{
			m_msQuic->StreamClose(streamHandle);

			sc->isFinished = true;
			if (sc != &sendContext)
				delete sc;
			conn.Release();

			return m_logger.Error(L"StreamSend failed, 0x%x!\n", Status);
		}

		m_msQuic->StreamSend(streamHandle, NULL, 0, QUIC_SEND_FLAG_NONE, NULL);

		if (completeEvent)
			completeEvent->IsSet();

		return true;
	}

	void NetworkBackendQuic::SetDataSentCallback(void* connection, void* context, DataSentCallback* callback)
	{
		auto& conn = *(Connection*)connection;
		conn.dataSentCallback = callback;
	}

	void NetworkBackendQuic::SetRecvCallbacks(void* connection, void* context, u32 headerSize, RecvHeaderCallback* h, RecvBodyCallback* b, const wchar_t* recvHint)
	{
		auto& conn = *(Connection*)connection;
		UBA_ASSERT(headerSize <= 16);
		conn.context = context;
		conn.headerCallback = h;
		conn.bodyCallback = b;
		conn.headerSize = headerSize;
	}

	void NetworkBackendQuic::SetRecvTimeout(void* connection, u32 timeoutMs)
	{
	}

	void NetworkBackendQuic::SetThreadedSend(void* connection)
	{
	}

	void NetworkBackendQuic::SetDisconnectCallback(void* connection, void* context, DisconnectCallback* callback)
	{
		auto& conn = *(Connection*)connection;
		conn.disconnectCallback = callback;
		conn.disconnectContext = context;
	}

	QUIC_STATUS NetworkBackendQuic::StreamCallback(Connection& conn, HQUIC streamHandle, QUIC_STREAM_EVENT* ev)
	{
		switch (ev->Type) {
		case QUIC_STREAM_EVENT_SEND_COMPLETE:
		{
			if (!ev->SEND_COMPLETE.ClientContext)
				break; // Flush

			auto& context = *(SendContext*)ev->SEND_COMPLETE.ClientContext;
			if (auto c = conn.dataSentCallback)
				c(conn.context, context.size);

			context.isFinished = true;
			if (context.flags & SendFlags_Async)
			{
				context.~SendContext();
				free(&context);
			}
			else if (context.flags & SendFlags_ExternalWait)
				;
			else
				((Event*)context.data)->Set();
			break;
		}
		case QUIC_STREAM_EVENT_RECEIVE:
		{
			Stream* streamPtr = nullptr;

			auto insres = conn.recvStreams.try_emplace(streamHandle);
			if (insres.second)
				conn.AddRef();
			streamPtr = &insres.first->second;
			Stream& stream = *streamPtr;

			UBA_ASSERT(conn.headerSize);
			auto bufferCount = ev->RECEIVE.BufferCount;
			auto buffers = ev->RECEIVE.Buffers;
			for (u32 i = 0; i != bufferCount; ++i)
			{
				u32 avail = buffers[i].Length;
				u8* data = buffers[i].Buffer;

				while (avail)
				{
					if (stream.headerRead < conn.headerSize)
					{
						u32 headerLeft = conn.headerSize - stream.headerRead;
						if (avail < headerLeft)
						{
							memcpy(stream.headerData + stream.headerRead, data, avail);
							stream.headerRead += u8(avail);
							avail = 0;
							break;
						}

						memcpy(stream.headerData + stream.headerRead, data, headerLeft);
						stream.headerRead += u8(headerLeft);
						avail -= headerLeft;
						data += headerLeft;

						if (!conn.headerCallback(conn.context, stream.headerData, stream.bodyContext, stream.bodyData, stream.bodySize))
						{
							CloseConnection(conn);
							return QUIC_STATUS_SUCCESS;
						}
					}

					bool success = true;
					if (stream.bodySize)
					{
						u32 bodyLeft = stream.bodySize - stream.bodyRead;
						if (avail < bodyLeft)
						{
							memcpy(stream.bodyData + stream.bodyRead, data, avail);
							stream.bodyRead += avail;
							avail = 0;
							break;
						}

						memcpy(stream.bodyData + stream.bodyRead, data, bodyLeft);
						stream.bodyRead += bodyLeft;
						avail -= bodyLeft;
						data += bodyLeft;
						success = conn.bodyCallback(conn.context, false, stream.headerData, stream.bodyContext, stream.bodyData);
					}

					stream.headerRead = 0;
					stream.bodyContext = nullptr;
					stream.bodyData = nullptr;
					stream.bodySize = 0;
					stream.bodyRead = 0;

					if (!success)
					{
						UBA_ASSERT(false);
						return QUIC_STATUS_SUCCESS;
					}
				}
			}
			//m_logger.Info(L"[strm][%p] Data received", conn.stream); // Data was received from the peer on the stream.
			break;
		}
		case QUIC_STREAM_EVENT_PEER_RECEIVE_ABORTED:
		{
			break;
		}
		case QUIC_STREAM_EVENT_PEER_SEND_SHUTDOWN:
		{
			StreamShutdown(conn, streamHandle);
			break;
		}
		case QUIC_STREAM_EVENT_PEER_SEND_ABORTED:
		{
			break;
		}
		case QUIC_STREAM_EVENT_SHUTDOWN_COMPLETE:
		{
			StreamShutdown(conn, streamHandle);
			break;
		}
		case QUIC_STREAM_EVENT_START_COMPLETE:
			break;

		case QUIC_STREAM_EVENT_SEND_SHUTDOWN_COMPLETE:
		{
			break;
		}
		case QUIC_STREAM_EVENT_IDEAL_SEND_BUFFER_SIZE:
			break;

		default:
			printf("");
			break;
		}
		return QUIC_STATUS_SUCCESS;
	}

	const QUIC_BUFFER Alpn = { sizeof("Uba") - 1, (uint8_t*)"Uba" };

	uint8_t DecodeHexChar(_In_ char c)
	{
		if (c >= '0' && c <= '9') return c - '0';
		if (c >= 'A' && c <= 'F') return 10 + c - 'A';
		if (c >= 'a' && c <= 'f') return 10 + c - 'a';
		return 0;
	}

	uint32_t DecodeHexBuffer(_In_z_ const char* HexBuffer, _In_ uint32_t OutBufferLen, _Out_writes_to_(OutBufferLen, return) uint8_t* OutBuffer)
	{
		uint32_t HexBufferLen = (uint32_t)strlen(HexBuffer) / 2;
		if (HexBufferLen > OutBufferLen) {
			return 0;
		}

		for (uint32_t i = 0; i < HexBufferLen; i++) {
			OutBuffer[i] =
				(DecodeHexChar(HexBuffer[i * 2]) << 4) |
				DecodeHexChar(HexBuffer[i * 2 + 1]);
		}

		return HexBufferLen;
	}

	bool NetworkBackendQuic::Init(const char* app)
	{
		if (m_msQuic)
			return true;

		QUIC_STATUS Status = QUIC_STATUS_SUCCESS;
		if (QUIC_FAILED(Status = MsQuicOpen2(&m_msQuic)))
			return m_logger.Error(L"MsQuicOpen2 failed, 0x%x!", Status);

		const QUIC_REGISTRATION_CONFIG RegConfig = { app, QUIC_EXECUTION_PROFILE_LOW_LATENCY };
		if (QUIC_FAILED(Status = m_msQuic->RegistrationOpen(&RegConfig, &m_registration)))
			return m_logger.Error(L"RegistrationOpen failed, 0x%x!", Status);

		return true;
	}

	NetworkBackendQuic::~NetworkBackendQuic()
	{
		SCOPED_WRITE_LOCK(m_connectionsLock, lock);
		for (auto& kv : m_connections)
			kv.second->Release();
		m_connections.clear();
		lock.Leave();

		if (m_configuration)
			m_msQuic->ConfigurationClose(m_configuration);
		if (m_registration)
			m_msQuic->RegistrationClose(m_registration);
		if (m_msQuic)
			MsQuicClose(m_msQuic);
	}


	bool NetworkBackendQuic::StartListen(Logger& logger, u16 port, const wchar_t* ip, const ListenConnectedFunc& connectedFunc, bool linger)
	{
		if (!Init("UbaServer"))
			return false;

		QUIC_ADDR Address = { 0 }; // Configures the address used for the listener to listen on all IP addresses and the given UDP port.
		QuicAddrSetFamily(&Address, QUIC_ADDRESS_FAMILY_UNSPEC);
		//if (ip && *ip)
		//{
		//	char target[256];
		//	sprintf_s(target, sizeof_array(target), "%ls", ip);
		//	QuicAddrFromString(target, u16(port), &Address);
		//}
		//else
		{
			QuicAddrSetPort(&Address, port);
		}

		QUIC_SETTINGS Settings = { 0 };
		Settings.IdleTimeoutMs = 100000;
		Settings.IsSet.IdleTimeoutMs = TRUE;
		Settings.ServerResumptionLevel = QUIC_SERVER_RESUME_AND_ZERORTT; // Configures the server's resumption level to allow for resumption and 0-RTT.
		Settings.IsSet.ServerResumptionLevel = TRUE;

		InitNetworkSettings(Settings, false);

		auto StaticListenerCallback = [](HQUIC Listener, void* Context, QUIC_LISTENER_EVENT* Event)
			{ return ((NetworkBackendQuic*)Context)->ListenerCallback(Listener, Event); };

		QUIC_STATUS Status = QUIC_STATUS_SUCCESS;
		if (QUIC_FAILED(Status = m_msQuic->ConfigurationOpen(m_registration, &Alpn, 1, &Settings, sizeof(Settings), NULL, &m_configuration)))
			return m_logger.Error(L"ConfigurationOpen failed, 0x%x!", Status);
		if (!LoadCredentials(false))
			return false;

		if (QUIC_FAILED(Status = m_msQuic->ListenerOpen(m_registration, StaticListenerCallback, this, &m_listener)))
			return m_logger.Error(L"ListenerOpen failed, 0x%x!", Status);
		if (QUIC_FAILED(Status = m_msQuic->ListenerStart(m_listener, &Alpn, 1, &Address)))
			return m_logger.Error(L"ListenerStart failed, 0x%x!", Status);

		m_logger.Info(L"Listening on all 0.0.0.0:%u", port);
		m_listenConnectedFunc = connectedFunc;

		m_logger.Info(L"[conn][%p] LISTENING", m_listener);

		return true;
	}

	void NetworkBackendQuic::StopListen()
	{
		if (m_listener != NULL)
			m_msQuic->ListenerClose(m_listener);
		m_listener = NULL;
	}

	QUIC_STATUS NetworkBackendQuic::ListenerCallback(HQUIC Listener, QUIC_LISTENER_EVENT* Event)
	{
		QUIC_STATUS Status = QUIC_STATUS_NOT_SUPPORTED;
		switch (Event->Type) {
		case QUIC_LISTENER_EVENT_NEW_CONNECTION:
		{
			auto& nc = Event->NEW_CONNECTION;
			nc.Info->RemoteAddress->Ipv4;

			auto connPtr = new Connection{ *this, nc.Connection };
			SCOPED_WRITE_LOCK(m_connectionsLock, lock);
			auto insres = m_connections.try_emplace(nc.Connection, connPtr);
			UBA_ASSERT(insres.second);
			Connection& conn = *connPtr;
			lock.Leave();

			sockaddr remoteSockAddr = *(sockaddr*)&nc.Info->RemoteAddress->Ipv4;
			conn.connected = true;
			if (!m_listenConnectedFunc(&conn, remoteSockAddr))
			{
				CloseConnection(nc.Connection);
				return QUIC_FAILED(QUIC_STATUS_USER_CANCELED);
			}

			// A new connection is being attempted by a client. For the handshake to proceed, the server must provide a configuration for QUIC to use. The app MUST set the callback handler before returning.
			static auto StaticConnectionCallback = [](HQUIC Connection, void* Context, QUIC_CONNECTION_EVENT* Event) { return ((NetworkBackendQuic*)Context)->ListenerConnectionCallback(Connection, Event); };
			m_msQuic->SetCallbackHandler(nc.Connection, (void*)StaticConnectionCallback, this);
			return m_msQuic->ConnectionSetConfiguration(nc.Connection, m_configuration);
		}
		default:
			break;
		}
		return Status;
	}

	QUIC_STATUS NetworkBackendQuic::ListenerConnectionCallback(HQUIC connection, QUIC_CONNECTION_EVENT* Event)
	{
		switch (Event->Type) {
		case QUIC_CONNECTION_EVENT_CONNECTED:
		{
			m_logger.Info(L"[conn][%p] Connected", connection); // The handshake has completed for the connection.
			m_msQuic->ConnectionSendResumptionTicket(connection, QUIC_SEND_RESUMPTION_FLAG_NONE, 0, NULL);
			break;
		}
		case QUIC_CONNECTION_EVENT_SHUTDOWN_INITIATED_BY_TRANSPORT:
		{
			// The connection has been shut down by the transport. Generally, this is the expected way for the connection to shut down with this protocol, since we let idle timeout kill the connection.
			if (Event->SHUTDOWN_INITIATED_BY_TRANSPORT.Status == QUIC_STATUS_CONNECTION_IDLE)
				m_logger.Info(L"[conn][%p] Successfully shut down on idle.", connection);
			else
				m_logger.Info(L"[conn][%p] Shut down by transport, 0x%x", connection, Event->SHUTDOWN_INITIATED_BY_TRANSPORT.Status);
			CloseConnection(connection);
			break;
		}
		case QUIC_CONNECTION_EVENT_SHUTDOWN_INITIATED_BY_PEER:
		{
			m_logger.Info(L"[conn][%p] Shut down by peer, 0x%llu", connection, (unsigned long long)Event->SHUTDOWN_INITIATED_BY_PEER.ErrorCode); // The connection was explicitly shut down by the peer.
			CloseConnection(connection);
			break;
		}
		case QUIC_CONNECTION_EVENT_SHUTDOWN_COMPLETE:
			//m_logger.Info(L"[conn][%p] All done", connection); // The connection has completed the shutdown process and is ready to be safely cleaned up.
			//m_msQuic->ConnectionClose(connection);
			CloseConnection(connection);
			break;

		case QUIC_CONNECTION_EVENT_PEER_STREAM_STARTED:
		{
			StreamCreated(connection, Event->PEER_STREAM_STARTED.Stream);
			break;
		}
		case QUIC_CONNECTION_EVENT_RESUMED:
			m_logger.Info(L"[conn][%p] Connection resumed!", connection); // The connection succeeded in doing a TLS resumption of a previous connection's session.
			break;

		case QUIC_CONNECTION_EVENT_IDEAL_PROCESSOR_CHANGED:
			break;

		case QUIC_CONNECTION_EVENT_STREAMS_AVAILABLE:
			break;

		case QUIC_CONNECTION_EVENT_PEER_NEEDS_STREAMS:
			UBA_ASSERT(false);
			break;

		default:
			printf("");
			break;
		}
		return QUIC_STATUS_SUCCESS;
	}

	bool NetworkBackendQuic::Connect(Logger& logger, const wchar_t* ip, const ConnectedFunc& connectedFunc, u16 port, bool* timedOut)
	{
		if (!NetworkBackendQuic::Init("UbaClient"))
			return false;

		QUIC_STATUS Status = QUIC_STATUS_SUCCESS;

		if (!m_configuration)
		{
			QUIC_SETTINGS Settings = { 0 };
			Settings.IdleTimeoutMs = 100000;
			Settings.IsSet.IdleTimeoutMs = TRUE;

			InitNetworkSettings(Settings, true);
			//Settings.HandshakeIdleTimeoutMs = 10000;
			//Settings.IsSet.HandshakeIdleTimeoutMs = true;

			if (QUIC_FAILED(Status = m_msQuic->ConfigurationOpen(m_registration, &Alpn, 1, &Settings, sizeof(Settings), NULL, &m_configuration)))
				return m_logger.Error(L"ConfigurationOpen failed, 0x%x!", Status);

			if (!LoadCredentials(true))
				return false;
		}

		HQUIC connection;
		static auto StaticConnectionCallback = [](HQUIC Connection, void* Context, QUIC_CONNECTION_EVENT* Event) { return ((NetworkBackendQuic*)Context)->ConnectionCallback(Connection, Event); };
		if (QUIC_FAILED(Status = m_msQuic->ConnectionOpen(m_registration, StaticConnectionCallback, this, &connection)))
			return m_logger.Error(L"ConnectionOpen failed, 0x%x!", Status);

		auto connPtr = new Connection{ *this, connection };
		Connection& conn = *connPtr;
		conn.connectionHandle = connection;

		SCOPED_WRITE_LOCK(m_connectionsLock, lock);
		auto insres = m_connections.try_emplace(connection, connPtr);
		UBA_ASSERT(insres.second);
		lock.Leave();

		auto connectionGuard = MakeGuard([&]()
			{
				SCOPED_WRITE_LOCK(m_connectionsLock, lock);
				if (m_connections.erase(connection))
					conn.Release();
			});

		conn.ready.Create(true);

		static auto StaticStreamCallback = [](HQUIC Stream, void* Context, QUIC_STREAM_EVENT* Event) { return ((NetworkBackendQuic&)((Connection*)Context)->backend).StreamCallback(*(Connection*)Context, Stream, Event); };

		char target[512];
		sprintf_s(target, sizeof_array(target), "%ls", ip);

		if (QUIC_FAILED(Status = m_msQuic->ConnectionStart(connection, m_configuration, QUIC_ADDRESS_FAMILY_UNSPEC, target, port)))
			return m_logger.Error(L"ConnectionStart failed, 0x%x!", Status);

		if (!conn.ready.IsSet(60*1000))
			return m_logger.Error(L"Failed to wait for msquic connection.");

		if (!conn.connected)
		{
			if (timedOut)
			{
				*timedOut = true;
				Sleep(200);
			}
			return false;
		}

		sockaddr remoteAddr;
		TraverseRemoteAddresses(m_logger, ip, port, [&remoteAddr](const sockaddr& remoteSockaddr)
			{
				remoteAddr = remoteSockaddr;
				return false;
			});

		if (!connectedFunc(&conn, remoteAddr, timedOut))
			return false;

		connectionGuard.Cancel();
		return true;
	}

	bool NetworkBackendQuic::Connect(Logger& logger, const sockaddr& remoteSocketAddr, const ConnectedFunc& connectedFunc, bool* timedOut, const wchar_t* nameHint)
	{
		UBA_ASSERT(false);
		return false;
	}

	QUIC_STATUS NetworkBackendQuic::ConnectionCallback(HQUIC connection, QUIC_CONNECTION_EVENT* Event)
	{
		switch (Event->Type) {
		case QUIC_CONNECTION_EVENT_CONNECTED:
		{
			m_logger.Info(L"[conn][%p] Connected", connection); // The handshake has completed for the connection.
			SCOPED_READ_LOCK(m_connectionsLock, lock);
			Connection& conn = *m_connections.find(connection)->second;
			lock.Leave();

			conn.connected = true;
			conn.ready.Set();
			//ClientSend(connection);
			break;
		}
		case QUIC_CONNECTION_EVENT_SHUTDOWN_INITIATED_BY_TRANSPORT:
		{
			// The connection has been shut down by the transport. Generally, this is the expected way for the connection to shut down with this protocol, since we let idle timeout kill the connection.
			if (Event->SHUTDOWN_INITIATED_BY_TRANSPORT.Status == QUIC_STATUS_CONNECTION_IDLE)
				m_logger.Info(L"[conn][%p] Successfully shut down on idle.", connection);
			//else
			//	m_logger.Info(L"[conn][%p] Shut down by transport, 0x%x", connection, Event->SHUTDOWN_INITIATED_BY_TRANSPORT.Status);
			CloseConnection(connection);
			break;
		}
		case QUIC_CONNECTION_EVENT_SHUTDOWN_INITIATED_BY_PEER:
		{
			m_logger.Info(L"[conn][%p] Shut down by peer, 0x%llu", connection, (unsigned long long)Event->SHUTDOWN_INITIATED_BY_PEER.ErrorCode); // The connection was explicitly shut down by the peer.
			CloseConnection(connection);
			break;
		}
		case QUIC_CONNECTION_EVENT_SHUTDOWN_COMPLETE:
			//m_logger.Info(L"[conn][%p] All done", connection); // The connection has completed the shutdown process and is ready to be safely cleaned up.
			if (!Event->SHUTDOWN_COMPLETE.AppCloseInProgress)
				CloseConnection(connection);
				//m_msQuic->ConnectionClose(connection);
			break;

		case QUIC_CONNECTION_EVENT_RESUMPTION_TICKET_RECEIVED:
			// A resumption ticket (also called New Session Ticket or NST) was received from the server.
			m_logger.Info(L"[conn][%p] Resumption ticket received (%u bytes):", connection, Event->RESUMPTION_TICKET_RECEIVED.ResumptionTicketLength);
			break;

		case QUIC_CONNECTION_EVENT_PEER_STREAM_STARTED:
		{
			StreamCreated(connection, Event->PEER_STREAM_STARTED.Stream);
			break;
		}
		case QUIC_CONNECTION_EVENT_STREAMS_AVAILABLE:
			break;

		case QUIC_CONNECTION_EVENT_DATAGRAM_STATE_CHANGED:
			break;

		case QUIC_CONNECTION_EVENT_PEER_NEEDS_STREAMS:
			UBA_ASSERT(false);
			break;

		default:
			printf("");
			break;
		}
		return QUIC_STATUS_SUCCESS;
	}

	void NetworkBackendQuic::StreamCreated(HQUIC connection, HQUIC streamHandle)
	{
		SCOPED_WRITE_LOCK(m_connectionsLock, lock);
		auto findIt = m_connections.find(connection);
		UBA_ASSERT(findIt != m_connections.end());
		Connection& conn = *findIt->second;
		conn.AddRef();
		lock.Leave();

		static auto StaticStreamCallback = [](HQUIC Stream, void* Context, QUIC_STREAM_EVENT* Event) { return ((NetworkBackendQuic&)((Connection*)Context)->backend).StreamCallback(*(Connection*)Context, Stream, Event); };
		m_msQuic->SetCallbackHandler(streamHandle, (void*)StaticStreamCallback, &conn);
	}

	void NetworkBackendQuic::StreamShutdown(Connection& conn, HQUIC streamHandle)
	{
		auto findIt = conn.recvStreams.find(streamHandle);
		if (findIt != conn.recvStreams.end())
		{
			auto& stream = findIt->second;
			if (void* bodyData = stream.bodyData)
			{
				stream.bodyData = nullptr;
				conn.bodyCallback(conn.context, true, stream.headerData, stream.bodyContext, nullptr);
			}
			conn.recvStreams.erase(findIt);
		}
		m_msQuic->StreamClose(streamHandle);
		conn.Release();
	}

	void NetworkBackendQuic::CloseConnection(HQUIC connection)
	{
		SCOPED_WRITE_LOCK(m_connectionsLock, lock);
		auto findIt = m_connections.find(connection);
		if (findIt == m_connections.end())
			return;
		Connection* conn = findIt->second;
		m_connections.erase(findIt);
		lock.Leave();

		CloseConnection(*conn);
		conn->Release();
	}

	void NetworkBackendQuic::CloseConnection(Connection& conn)
	{
		conn.connected = false;

		if (HQUIC handle = conn.connectionHandle)
			m_msQuic->ConnectionShutdown(handle, QUIC_CONNECTION_SHUTDOWN_FLAG_NONE, 0);

		if (auto disconnectCallback = conn.disconnectCallback)
		{
			conn.disconnectCallback = nullptr;
			disconnectCallback(conn.context, &conn);
		}

		conn.ready.Set();
	}

	void NetworkBackendQuic::InitNetworkSettings(QUIC_SETTINGS& Settings, bool isClient)
	{
		Settings.PeerUnidiStreamCount = 10000;
		Settings.IsSet.PeerUnidiStreamCount = TRUE;

		Settings.StreamRecvWindowDefault = SendMaxSize;
		Settings.IsSet.StreamRecvWindowDefault = true;

		Settings.MaxOperationsPerDrain = 8;
		Settings.IsSet.MaxOperationsPerDrain = true;

		Settings.SendBufferingEnabled = false;
		Settings.IsSet.SendBufferingEnabled = true;
	
		//Settings.StreamRecvBufferDefault = 256 * 1024;
		//Settings.IsSet.StreamRecvBufferDefault = true;

		if (!isClient)
		{
			//Settings.SendBufferingEnabled = false;
			//Settings.IsSet.SendBufferingEnabled = true;
		}
	}

	bool NetworkBackendQuic::LoadCredentials(bool isClient)
	{
		QUIC_STATUS Status = QUIC_STATUS_SUCCESS;

		if (isClient)
		{
			QUIC_CREDENTIAL_CONFIG CredConfig;
			memset(&CredConfig, 0, sizeof(CredConfig));
			CredConfig.Type = QUIC_CREDENTIAL_TYPE_NONE;
			CredConfig.Flags = QUIC_CREDENTIAL_FLAG_CLIENT;
			if (true) // Unsecure
				CredConfig.Flags |= QUIC_CREDENTIAL_FLAG_NO_CERTIFICATE_VALIDATION;

			if (QUIC_FAILED(Status = m_msQuic->ConfigurationLoadCredential(m_configuration, &CredConfig)))
				return m_logger.Error(L"ConfigurationLoadCredential failed, 0x%x!", Status);

			return true;
		}

		typedef struct QUIC_CREDENTIAL_CONFIG_HELPER {
			QUIC_CREDENTIAL_CONFIG CredConfig;
			union {
				QUIC_CERTIFICATE_HASH CertHash;
				QUIC_CERTIFICATE_HASH_STORE CertHashStore;
				QUIC_CERTIFICATE_FILE CertFile;
				QUIC_CERTIFICATE_FILE_PROTECTED CertFileProtected;
			};
		} QUIC_CREDENTIAL_CONFIG_HELPER;


		QUIC_CREDENTIAL_CONFIG_HELPER Config;
		memset(&Config, 0, sizeof(Config));

		const char* Cert = "4433475BB0F3E338BD68CB343194E8654679DC4A";
		uint32_t CertHashLen =
			DecodeHexBuffer(
				Cert,
				sizeof(Config.CertHash.ShaHash),
				Config.CertHash.ShaHash);
		if (CertHashLen != sizeof(Config.CertHash.ShaHash)) {
			return FALSE;
		}
		Config.CredConfig.Type = QUIC_CREDENTIAL_TYPE_CERTIFICATE_HASH;
		Config.CredConfig.CertificateHash = &Config.CertHash;
		Config.CredConfig.Flags |= QUIC_CREDENTIAL_FLAG_NO_CERTIFICATE_VALIDATION;

		if (isClient)
			Config.CredConfig.Flags |= QUIC_CREDENTIAL_FLAG_CLIENT;

		if (QUIC_FAILED(Status = m_msQuic->ConfigurationLoadCredential(m_configuration, &Config.CredConfig)))
			return m_logger.Error(L"ConfigurationLoadCredential failed, 0x%x!", Status);

		return true;
	}
}

#endif // UBA_USE_QUIC
