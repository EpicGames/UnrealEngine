// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaNetworkBackendTcp.h"
#include "UbaConfig.h"
#include "UbaDefinitions.h"
#include "UbaDirectoryIterator.h"
#include "UbaEnvironment.h"
#include "UbaEvent.h"
#include "UbaFile.h"
#include "UbaHash.h"
#include "UbaPlatform.h"
#include "UbaStringBuffer.h"
#include "UbaTimer.h"

#if PLATFORM_WINDOWS
#include <iphlpapi.h>
#include <ipifcons.h>
#include <Mstcpip.h>
#include <qos2.h>
#pragma comment (lib, "Netapi32.lib")
#pragma comment (lib, "Ws2_32.lib")
#pragma comment(lib, "IPHLPAPI.lib") // For GetAdaptersInfo
#pragma comment(lib, "qwave.lib")
#define WSABUFINIT(a, b) { u32(b), (char*)a }
#else
#include "UbaLinuxNetworkWrappers.h"
#include <netdb.h>
#include <netinet/tcp.h>
#include <arpa/inet.h>
#include <ifaddrs.h>
#include <poll.h>
#include <sys/uio.h>
#define SOCKET_ERROR -1
#define WSAHOST_NOT_FOUND 0
#define WSAEADDRINUSE EADDRINUSE
#define addrinfoW addrinfo
#define GetAddrInfoW getaddrinfo
#define FreeAddrInfoW freeaddrinfo
#define strcpy_s(a, b, c) strcpy(a, c)
#define WSABUF iovec
#define WSABUFINIT(a, b) { (void*)a, b }
#endif

#include "UbaSocket.h"

#if PLATFORM_WINDOWS
#include "UbaSocketWin.h"
#else
namespace uba
{
	#include "UbaSocket.inl"
}
#endif

#define UBA_LOG_SOCKET_ERRORS UBA_DEBUG
#define UBA_EMULATE_BAD_INTERNET 0

#define UBA_USE_OVERLAPPED_SEND PLATFORM_WINDOWS
#define UBA_USE_OVERLAPPED_SEND_WITH_LOCK 0
#define UBA_USE_IOCP PLATFORM_WINDOWS

namespace uba
{
	constexpr u32 MaxHeaderSize = 33;
	typedef void* WsaEvent;
	struct ThreadedSendContext;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	#define UBA_SEND_WAKEUP(conn) ((conn).threadedSendContext ? (conn).threadedSendContext->sendWakeupSocket : InvalidSocket)

	bool ShutdownSocket(Logger& logger, Socket s, Socket recvWakeupSocket, Socket sendWakeupSocket, const tchar* hint);
	bool CloseSocket(Logger& logger, Socket s, const tchar* hint);
	bool SetKeepAlive(Logger& logger, Socket socket);
	bool SetUserTimeout(Logger& logger, Socket socket, u32 timeoutMs);
	bool SetBlocking(Logger& logger, Socket socket, bool blocking);
	bool SetTimeout(Logger& logger, Socket socket, u32 timeoutMs);
	bool SetNoLinger(Logger& logger, Socket socket);
	bool SetRecvBuf(Logger& logger, Socket socket, u32 windowSize);
	bool SetSendBuf(Logger& logger, Socket socket, u32 windowSize);
	bool SetSocketPriority(Logger& logger, Socket socket);
	bool DisableNagle(Logger& logger, Socket socket);
	void LogTcpInfo(Logger& logger, Socket socket);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct NetworkBackendTcp::ListenEntry
	{
		StringBuffer<128> ip;
		u16 port;
		bool linger = false;
		ListenConnectedFunc connectedFunc;
		Event listening;
		Atomic<Socket> socket = InvalidSocket;
		Atomic<Socket> wakeupSocket = InvalidSocket;
		sockaddr_in addr;
		Thread thread;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct SendBlock
	{
		const u8* data;
		u32 dataLen;
		SendBlock* next;
		EventSlim sent;
	};

	struct ThreadedSendContext
	{
		SendBlock* firstBlock = nullptr;
		SendBlock* lastBlock = nullptr;
		EventSlim blockQueueEvent;
		Futex blockQueueLock;
		Thread thread;
		Socket sendWakeupSocket = InvalidSocket;
		~ThreadedSendContext() { if (sendWakeupSocket != InvalidSocket) SocketClose(sendWakeupSocket); }
	};

	struct NetworkBackendTcp::Connection
	{
		Connection(Logger& l, Socket s, bool linger) : logger(l), socket(s), ready(EventResetType_Manual)
		{
			CreateGuid(uid);
			if (!linger)
				recvWakeupSocket = SocketCreateWakeupSocket();
		}
		~Connection()
		{
			delete threadedSendContext;
			if (recvWakeupSocket != InvalidSocket)
				SocketClose(recvWakeupSocket);
		}

		Logger& logger;
		Atomic<Socket> socket;
		Atomic<bool> shutdownCalled = false;

		Event ready;
		Guid uid;
		u32 headerSize = 0;
		
		u32 recvTimeoutMs = 0;
		void* recvTimeoutContext = nullptr;
		RecvTimeoutCallback* recvTimeoutCallback = nullptr;

		void* recvContext = nullptr;
		RecvHeaderCallback* headerCallback = nullptr;
		RecvBodyCallback* bodyCallback = nullptr;
		const tchar* recvHint = TC("");

		void* dataSentContext = nullptr;
		DataSentCallback* dataSentCallback = nullptr;

		void* dataRecvContext = nullptr;
		DataSentCallback* dataRecvCallback = nullptr;

		void* disconnectContext = nullptr;
		DisconnectCallback* disconnectCallback = nullptr;

		#if !UBA_USE_OVERLAPPED_SEND_WITH_LOCK
		Futex timeoutLock;
		#endif

		Futex sendLock;
		CriticalSection shutdownLock;

		Thread recvThread;

		bool allowLess = false;

		// Temporary state for iocp
		#if UBA_USE_IOCP
		OVERLAPPED overlapped;
		WSABUF wsaBuf = {};
		u8 header[MaxHeaderSize];
		u8* bodyData = nullptr;
		u32 bodySize = 0;
		void* bodyContext = nullptr;
		bool receivingHeader = true;
		#endif

		ThreadedSendContext* threadedSendContext = nullptr;

		Connection(const Connection&) = delete;
		void operator=(const Connection&) = delete;

		Socket recvWakeupSocket = InvalidSocket;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct NetworkBackendTcp::RecvCache
	{
		u8 bytes[256*1024];
		u32 pos = 0;
		u32 written = 0;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void NetworkBackendTcpCreateInfo::Apply(const Config& config, const tchar* tableName)
	{
		const ConfigTable* tablePtr = config.GetTable(tableName);
		if (!tablePtr)
			return;
		const ConfigTable& table = *tablePtr;
		table.GetValueAsBool(disableNagle, TC("DisableNagle"));
		table.GetValueAsU32(statusUpdateSeconds, TC("StatusUpdateSeconds"));

		#if PLATFORM_WINDOWS
		table.GetValueAsBool(useOverlappedSend, TC("UseOverlappedSend"));
		table.GetValueAsU32(iocpWorkerCount, TC("IocpWorkerCount"));
		#endif

		table.GetValueAsU32(recvBufferSize, TC("RecvBufferSize"));
		table.GetValueAsU32(sendBufferSize, TC("SendBufferSize"));
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool InitNetwork(Logger& logger)
	{
#if PLATFORM_WINDOWS
		WSADATA wsaData;
		if (int res = WSAStartup(MAKEWORD(2, 2), &wsaData))
			return logger.Error(TC("WSAStartup failed (%d)"), res);

		if (IsRunningWine())
		{
			if (void* wineDll = GetUbaWineModule())
				SocketInitWine(logger, (HMODULE)wineDll);
			else
				logger.Warning(TC("Failed to load UbaWine.dll.so (%s)"), LastErrorToText().data);

		}
#else
		struct sigaction sa = { { SIG_IGN } };
		sigaction(SIGPIPE, &sa, NULL); // Needed for broken pipe that can happen if helpers crash
#endif
		return true;
	}

	bool InitNetworkOnce(Logger& logger)
	{
		static bool initOnce = [](Logger& logger) { return InitNetwork(logger); }(logger);
		return initOnce;
	}

	bool NetworkBackendTcp::EnsureInitialized(Logger& logger)
	{
		if (!InitNetworkOnce(logger))
			return false;

		#if PLATFORM_WINDOWS
		SCOPED_FUTEX(m_initLock, lock);
		if (m_initDone)
			return m_initSuccess;
		m_initDone = true;

		#if UBA_USE_IOCP
		if (m_iocpWorkerCount)
		{
			m_iocpHandle = CreateIoCompletionPort(INVALID_HANDLE_VALUE, NULL, 0, 0);

			for (u32 i=0; i!=m_iocpWorkerCount; ++i)
				m_iocpThreads.emplace_back().Start([this]() { ThreadIocp(); return 0; }, TC("UbaIocp"));
		}
		#endif

		m_initSuccess = true;
		#endif
		return true;
	}

	NetworkBackendTcp::NetworkBackendTcp(const NetworkBackendTcpCreateInfo& info, const tchar* prefix)
		: m_logger(info.logWriter, prefix)
	{
		m_disableNagle = info.disableNagle;

		#if PLATFORM_WINDOWS
		m_useOverlappedSend = info.useOverlappedSend && EventIsNative && !IsRunningWine();
		m_iocpWorkerCount = u16(info.iocpWorkerCount);
		#endif

		m_recvBufferSize = info.recvBufferSize;
		m_sendBufferSize = info.sendBufferSize;

		if (info.statusUpdateSeconds)
		{
			m_tcpStatusLoop.Create(EventResetType_Manual);
			m_tcpStatusThread.Start([this, sus = info.statusUpdateSeconds]() { ThreadStatus(sus); return 0; }, TC("UbaTcpStat"));
		}
	}

	NetworkBackendTcp::~NetworkBackendTcp()
	{
		StopListen();

		SCOPED_FUTEX(m_connectionsLock, lock);
		for (auto& conn : m_connections)
		{
			SCOPED_CRITICAL_SECTION(conn.shutdownLock, lock2);
			if (conn.socket == InvalidSocket)
				continue;
			Socket s = conn.socket;
			conn.socket = InvalidSocket;
			conn.shutdownCalled = true;
			auto context = conn.threadedSendContext;
			if (context)
				context->blockQueueEvent.Set();
			ShutdownSocket(conn.logger, s, conn.recvWakeupSocket, UBA_SEND_WAKEUP(conn), TC("Dtor"));
			lock2.Leave();
			conn.recvThread.Wait();
			if (context)
				context->thread.Wait();
			CloseSocket(conn.logger, s, TC("Dtor"));
		}
		m_connections.clear();

		#if UBA_USE_IOCP
		if (m_iocpHandle)
		{
			for (u64 i=0; i!=m_iocpThreads.size(); ++i)
				PostQueuedCompletionStatus(m_iocpHandle, 0, 1, NULL);
			for (Thread& t : m_iocpThreads)
				t.Wait();
			CloseHandle(m_iocpHandle);
		}
		#endif

		m_tcpStatusLoop.Set();
		m_tcpStatusThread.Wait();
	}

	void NetworkBackendTcp::Shutdown(void* connection)
	{
		auto& conn = *(Connection*)connection;
		SCOPED_CRITICAL_SECTION(conn.shutdownLock, lock);
		bool shutdownCalled = conn.shutdownCalled;
		conn.shutdownCalled = true;
		if (conn.socket != InvalidSocket && !shutdownCalled)
			ShutdownSocket(conn.logger, conn.socket, conn.recvWakeupSocket, UBA_SEND_WAKEUP(conn), TC("Shutdown"));
	}

	bool NetworkBackendTcp::Send(Logger& logger, void* connection, const void* data, u32 dataSize, SendContext& sendContext, const tchar* sendHint)
	{
		auto& conn = *(Connection*)connection;
		sendContext.isUsed = true;

		bool res = SendSocket(conn, logger, data, dataSize, sendHint);

		sendContext.isFinished = true;

		m_totalSend += dataSize;

		if (auto c = conn.dataSentCallback)
			c(conn.dataSentContext, dataSize);
		return res;
	}

	void NetworkBackendTcp::SetDataSentCallback(void* connection, void* context, DataSentCallback* callback)
	{
		auto& conn = *(Connection*)connection;
		conn.dataSentCallback = callback;
		conn.dataSentContext = context;
	}

	void NetworkBackendTcp::SetDataReceivedCallback(void* connection, void* context, DataReceivedCallback* callback)
	{
		auto& conn = *(Connection*)connection;
		conn.dataRecvCallback = callback;
		conn.dataRecvContext = context;
	}

	void NetworkBackendTcp::TraverseConnections(const Function<void(void*)>& func)
	{
		SCOPED_FUTEX(m_connectionsLock, lock);
		for (auto& conn : m_connections)
			func(&conn);
	}

	void NetworkBackendTcp::SetRecvCallbacks(void* connection, void* context, u32 headerSize, RecvHeaderCallback* h, RecvBodyCallback* b, const tchar* recvHint)
	{
		UBA_ASSERT(h);
		UBA_ASSERT(headerSize <= MaxHeaderSize);
		auto& conn = *(Connection*)connection;

		SCOPED_CRITICAL_SECTION(conn.shutdownLock, lock);
		UBA_ASSERTF(conn.disconnectCallback, TC("SetDisconnectCallback must be called before SetRecvCallbacks"));
		conn.recvContext = context;
		conn.headerSize = headerSize;
		conn.headerCallback = h;
		conn.bodyCallback = b;
		conn.recvHint = recvHint;
		conn.ready.Set();

		#if UBA_USE_IOCP
		if (m_iocpHandle && !conn.wsaBuf.buf)
			PostIocpRead(conn, conn.header, headerSize);
		#endif
	}

	void NetworkBackendTcp::SetRecvTimeout(void* connection, u32 timeoutMs, void* context, RecvTimeoutCallback* callback)
	{
		auto& conn = *(Connection*)connection;
		conn.recvTimeoutMs = timeoutMs;
		conn.recvTimeoutContext = context;
		conn.recvTimeoutCallback = callback;
	}

	void NetworkBackendTcp::SetThreadedSend(void* connection)
	{
		auto& conn = *(Connection*)connection;
		if (conn.threadedSendContext)
			return;
		auto context = new ThreadedSendContext();
		context->blockQueueEvent.Create(EventResetType_Manual);
		context->sendWakeupSocket = SocketCreateWakeupSocket();
		context->thread.Start([this, connPtr = &conn] { ThreadSend(*connPtr); return 0; }, TC("UbaTcpSnd"));
		conn.threadedSendContext = context;
	}

	void NetworkBackendTcp::SetDisconnectCallback(void* connection, void* context, DisconnectCallback* callback)
	{
		auto& conn = *(Connection*)connection;
		SCOPED_CRITICAL_SECTION(conn.shutdownLock, lock);
		conn.disconnectCallback = callback;
		conn.disconnectContext = context;
	}

	void NetworkBackendTcp::SetAllowLessThanBodySize(void* connection, bool allow)
	{
		auto& conn = *(Connection*)connection;
		SCOPED_CRITICAL_SECTION(conn.shutdownLock, lock);
		conn.allowLess = allow;
	}

	void NetworkBackendTcp::SetPriority(void* connection)
	{
		auto& conn = *(Connection*)connection;
		SCOPED_CRITICAL_SECTION(conn.shutdownLock, lock);
		SetSocketPriority(m_logger, conn.socket);
	}

	bool NetworkBackendTcp::StartListen(Logger& logger, u16 port, const tchar* ip, const ListenConnectedFunc& connectedFunc, bool linger)
	{
		if (!EnsureInitialized(logger))
			return false;

		SCOPED_FUTEX(m_listenEntriesLock, lock);

		auto prevListenEntryCount = int(m_listenEntries.size());

		auto AddAddr = [&](const tchar* addr)
			{
				m_listenEntries.emplace_back();
				auto& entry = m_listenEntries.back();
				entry.ip.Append(addr);
				entry.port = port;
				entry.connectedFunc = connectedFunc;
				entry.linger = linger;
			};

		if (ip && *ip)
		{
			AddAddr(ip);
		}
		else
		{
			TraverseNetworkAddresses(logger, [&](const StringBufferBase& addr)
				{
					AddAddr(addr.data);
					return true;
				});
			AddAddr(TC("127.0.0.1"));
		}

		if (m_listenEntries.empty())
		{
			logger.Warning(TC("No host addresses found for UbaServer. Will not be able to use remote workers"));
			return false;
		}

		auto skipCount = prevListenEntryCount;
		for (auto& e : m_listenEntries)
		{
			if (skipCount-- > 0)
				continue;
			e.listening.Create(EventResetType_Manual);
			e.wakeupSocket = SocketCreateWakeupSocket();
			e.thread.Start([this, &logger, &e]
				{
					ThreadListen(logger, e);
					return 0;
				}, TC("UbaTcpListen"));
		}

		bool success = true;
		skipCount = prevListenEntryCount;
		for (auto& e : m_listenEntries)
		{
			if (skipCount-- > 0)
				continue;
			if (!e.listening.IsSet(4000))
				success = false;
			if (e.socket == InvalidSocket)
				success = false;
			e.listening.Destroy();
		}
		return success;
	}

	void NetworkBackendTcp::StopListen()
	{
		SCOPED_FUTEX(m_listenEntriesLock, lock);
		for (auto& e : m_listenEntries)
		{
			e.socket = InvalidSocket;
			if (e.wakeupSocket != InvalidSocket)
				SocketWakeup(e.wakeupSocket);
		}
		for (auto& e : m_listenEntries)
			e.thread.Wait();
		for (auto& e : m_listenEntries)
		{
			if (e.wakeupSocket == InvalidSocket)
				continue;
			SocketClose(e.wakeupSocket);
			e.wakeupSocket = InvalidSocket;
		}
		m_listenEntries.clear();
	}

	bool NetworkBackendTcp::ThreadListen(Logger& logger, ListenEntry& entry)
	{
		addrinfoW hints;
		memset(&hints, 0, sizeof(hints));
		hints.ai_family = AF_INET; //AF_UNSPEC; (Skip AF_INET6)
		hints.ai_socktype = SOCK_STREAM;
		hints.ai_protocol = IPPROTO_TCP;
		hints.ai_flags = AI_PASSIVE;

		// Resolve the server address and port
		struct addrinfoW* result = NULL;
		StringBuffer<32> portStr;
		portStr.AppendValue(entry.port);
		int res = GetAddrInfoW(entry.ip.data, portStr.data, &hints, &result);

		auto listenEv = MakeGuard([&]() { entry.listening.Set(); });

		if (res != 0)
			return logger.Error(TC("getaddrinfo failed (%d)"), res);

		UBA_ASSERT(result);
		auto addrGuard = MakeGuard([result]() { FreeAddrInfoW(result); });

		// Create a socket for listening to connections
		Socket listenSocket = SocketCreate(result->ai_family, result->ai_socktype, result->ai_protocol);
		if (listenSocket == InvalidSocket)
			return logger.Error(TC("socket failed (%s)"), LastErrorToText(SocketGetLastError()).data);

		auto listenSocketCleanup = MakeGuard([&]() { CloseSocket(logger, listenSocket, TC("listen cleanup")); });

		if (SocketReuseAddr(listenSocket) == SOCKET_ERROR)
			return logger.Error(TC("setsockopt SO_REUSEADDR failed (error: %s)"), LastErrorToText(SocketGetLastError()).data);

		// Setup the TCP listening socket

		u32 retryCount = 0;
		while (true)
		{
			res = SocketBind(listenSocket, result->ai_addr, (socklen_t)result->ai_addrlen);
			if (res != SOCKET_ERROR)
				break;
			int lastError = SocketGetLastError();
			if (lastError != WSAEADDRINUSE)
				return logger.Error(TC("bind %s:%hu failed (%s)"), entry.ip.data, entry.port, LastErrorToText(lastError).data);
			if (retryCount == 5)
				return logger.Info(TC("bind %s:%hu failed because address/port is in use. Some other process is already using this address/port (retried %u times)"), entry.ip.data, entry.port, retryCount).ToFalse();
			++retryCount;
			Sleep(retryCount*50);
		}

		entry.addr = *(sockaddr_in*)result->ai_addr;
		if (entry.addr.sin_addr.s_addr == 0) // if 0.0.0.0 then we store 127.0.0.1
			entry.addr.sin_addr.s_addr = htonl(127 << 24 | 1);

		addrGuard.Execute();

		res = SocketListen(listenSocket);
		if (res == SOCKET_ERROR)
			return logger.Error(TC("Listen failed (%s)"), LastErrorToText(SocketGetLastError()).data);

		if (!SetKeepAlive(logger, listenSocket))
			return false;

		if (!SetUserTimeout(logger, listenSocket, UserTimeoutSeconds * 1000))
			return false;

		logger.Info(TC("Listening on %s:%hu"), entry.ip.data, entry.port);
		entry.socket = listenSocket;

		listenEv.Execute();

		while (true)
		{
			int revents = 0;
			int pollRes = SocketPoll2(listenSocket, PollIN, -1, entry.wakeupSocket, &revents);

			if (entry.socket == InvalidSocket)
				break;

			if (pollRes < 0)
			{
				int lastError = SocketGetLastError();
				logger.Warning(TC("WSAPoll returned error %s"), LastErrorToText(lastError).data);
				break;
			}

			if (!pollRes) // timed out
				continue;

			if (revents & (PollERR | PollHUP | PollNVAL))
			{
				logger.Warning(TC("WSAPoll returned successful but with unexpected flags: %u"), revents);
				continue;
			}

			sockaddr remoteSockAddr = { 0 }; // for TCP/IP
			Socket clientSocket = SocketAccept(listenSocket, &remoteSockAddr, sizeof(remoteSockAddr));

			if (clientSocket == InvalidSocket)
			{
				if (entry.socket != InvalidSocket)
					logger.Info(TC("Accept failed with WSA error: %s"), LastErrorToText(SocketGetLastError()).data);
				break;
			}

			if (m_disableNagle && !DisableNagle(logger, clientSocket))
			{
				CloseSocket(logger, clientSocket, TC("disable nagle"));
				continue;
			}

			if (!SetKeepAlive(logger, clientSocket))
			{
				CloseSocket(logger, clientSocket, TC("keep alive"));
				continue;
			}

			if (!SetUserTimeout(logger, clientSocket, UserTimeoutSeconds * 1000))
			{
				CloseSocket(logger, clientSocket, TC("user timeout"));
				continue;
			}

			if (!entry.linger)
				SetNoLinger(logger, clientSocket);

			// For some reason performance gets destroyed on wine when changing these
			if (IsWindows && !IsRunningWine())
			{
				if (m_recvBufferSize)
					SetRecvBuf(logger, clientSocket, m_recvBufferSize);
				if (m_sendBufferSize)
					SetSendBuf(logger, clientSocket, m_sendBufferSize);
			}

			SCOPED_FUTEX(m_connectionsLock, lock);
			auto it = m_connections.emplace(m_connections.end(), logger, clientSocket, entry.linger);
			auto& conn = *it;
			#if UBA_USE_IOCP
			if (m_iocpHandle)
				CreateIoCompletionPort((HANDLE)ToPlatformSocket(clientSocket), m_iocpHandle, (ULONG_PTR)&conn, 0);
			else
			#endif
				conn.recvThread.Start([this, connPtr = &conn] { ThreadRecv(*connPtr); return 0; }, TC("UbaTcpRecv"));

			lock.Leave();

			if (!entry.connectedFunc(&conn, remoteSockAddr))
			{
				conn.shutdownCalled = true;
				ShutdownSocket(logger, clientSocket, conn.recvWakeupSocket, UBA_SEND_WAKEUP(conn), TC("ThreadListen"));
				conn.ready.Set();
				conn.recvThread.Wait();
				SCOPED_FUTEX(m_connectionsLock, lock2);
				m_connections.erase(it);
				continue;
			}
		}

		return true;
	}

	void NetworkBackendTcp::ThreadRecv(Connection& connection)
	{
		ElevateCurrentThreadPriority();
		
		auto& logger = connection.logger;

		if (connection.ready.IsSet(60000)) // This should never time out!
		{
			SetBlocking(logger, connection.socket, false);

			RecvCache recvCache;

			bool isFirst = true;
			while (connection.socket != InvalidSocket)
			{
				void* bodyContext = nullptr;
				u8* bodyData = nullptr;
				u32 bodySize = 0;

				u8 headerData[MaxHeaderSize];
				if (!RecvSocket(connection, recvCache, headerData, connection.headerSize, TC(""), isFirst, false))
					break;
				isFirst = false;

				auto hc = connection.headerCallback;
				if (!hc)
				{
					logger.Error(TC("Tcp connection header callback not set"));
					break;
				}

				if (!hc(connection.recvContext, connection.uid, headerData, bodyContext, bodyData, bodySize))
					break;

				u32 received = connection.headerSize + bodySize;
				m_totalRecv += received;
				if (auto c = connection.dataRecvCallback)
					c(connection.dataRecvContext, received);

				if (!bodySize)
					continue;

				bool success = RecvSocket(connection, recvCache, bodyData, bodySize, TC("Body"), false, connection.allowLess);

				auto bc = connection.bodyCallback;
				if (!bc)
				{
					logger.Error(TC("Tcp connection body callback not set"));
					break;
				}

				if (!bc(connection.recvContext, !success, headerData, bodyContext, bodyData, bodySize))
					break;
				if (!success)
					break;
			}
		}
		else
		{
			logger.Warning(TC("Tcp connection timed out waiting for recv thread to be ready"));
		}

		SCOPED_CRITICAL_SECTION(connection.shutdownLock, lock2);
		Socket s = connection.socket;

		{
			SCOPED_FUTEX(connection.sendLock, lock);
			connection.socket = InvalidSocket;
		}

		if (auto context = connection.threadedSendContext)
			context->blockQueueEvent.Set();

		if (auto cb = connection.disconnectCallback)
		{
			auto context = connection.disconnectContext;
			connection.disconnectCallback = nullptr;
			connection.disconnectContext = nullptr;
			cb(context, connection.uid, &connection);
		}

		if (s == InvalidSocket)
			return;
		if (!connection.shutdownCalled)
		{
			connection.shutdownCalled = true;
			ShutdownSocket(logger, s, connection.recvWakeupSocket, UBA_SEND_WAKEUP(connection), TC("ThreadRecv"));
		}
		CloseSocket(logger, s, TC("ThreadRecv"));
	}

	bool NetworkBackendTcp::Connect(Logger& logger, const tchar* ip, const ConnectedFunc& connectedFunc, u16 port, bool* timedOut)
	{
		if (!EnsureInitialized(logger))
			return false;

		u64 startTime = GetTime();

		if (timedOut)
			*timedOut = false;

		bool connected = false;
		bool success = true;
		TraverseRemoteAddresses(logger, ip, port, [&](const sockaddr& remoteSockaddr)
			{
				bool timedOut2 = false;
				connected = Connect(logger, remoteSockaddr, connectedFunc, &timedOut2, ip);
				if (connected)
					return false;
				if (timedOut2)
					return true;
				success = false;
				return false;
			});

		if (connected)
			return true;

		if (!success)
			return false;

		if (!timedOut)
			return false;

		*timedOut = true;
		int connectTimeMs = int(TimeToMs(GetTime() - startTime));
		int timeoutMs = 2000;
		if (connectTimeMs < timeoutMs)
			Sleep(u32(timeoutMs - connectTimeMs));
		return false;
	}

	bool NetworkBackendTcp::Connect(Logger& logger, const sockaddr& remoteSocketAddr, const ConnectedFunc& connectedFunc, bool* timedOut, const tchar* nameHint)
	{
		// Create a socket for connecting to server

		Socket socketFd = SocketCreate(remoteSocketAddr.sa_family, SOCK_STREAM, IPPROTO_TCP);
		if (socketFd == InvalidSocket)
			return logger.Error(TC("socket failed (%s)"), LastErrorToText(SocketGetLastError()).data);

		// Create guard in case we fail to connect (will be cancelled further down if we succeed)
		auto socketClose = MakeGuard([&]() { CloseSocket(logger, socketFd, TC("Connect")); });

		// Set to non-blocking just for the connect call (we want to control the connect timeout after connect using select instead)
		if (!SetBlocking(logger, socketFd, false))
			return false;

		// Connect to server.
		int res = SocketConnect(socketFd, &remoteSocketAddr, sizeof(remoteSocketAddr), false);
		if (res != 0)
			return logger.Error(TC("SocketConnect failed"));

		int timeoutMs = 2000;
		if (nameHint && (Equals(nameHint, TC("localhost")) || Equals(nameHint, TC("127.0.0.1"))))
			timeoutMs = 100;

		int revents = 0;
		int pollRes = SocketPoll(socketFd, PollOUT, timeoutMs, &revents);

		if (pollRes < 0)
		{
			int lastError = SocketGetLastError();
			logger.Warning(TC("WSAPoll returned error %s (%s)"), LastErrorToText(lastError).data, nameHint);
			return false;
		}

		u16 validFlags = PollERR | PollHUP; // Treat hangup as timeout (since we want retry if that happens). Also treat error as timeout. This is needed for Wine agent to be able to retry
		if (!pollRes || revents & validFlags)
		{
			if (timedOut)
				*timedOut = true;
			return false;
		}

		if (revents & PollNVAL)
		{
			logger.Warning(TC("WSAPoll returned successful but with unexpected flags: %u"), revents);
			return false;
		}

		// Return to blocking since we want select to block
		if (!SetBlocking(logger, socketFd, true))
			return false;

		if (SocketCheckConnect(socketFd, timedOut) != 0)
			return false;

		if (m_disableNagle && !DisableNagle(logger, socketFd))
			return false;

		if (!SetKeepAlive(logger, socketFd))
			return false;

		if (!SetUserTimeout(logger, socketFd, UserTimeoutSeconds * 1000))
			return false;

		SetNoLinger(logger, socketFd);

		// For some reason performance gets destroyed on wine when changing these
		if (IsWindows && !IsRunningWine())
		{
			if (m_recvBufferSize)
				SetRecvBuf(logger, socketFd, m_recvBufferSize);
			if (m_sendBufferSize)
				SetSendBuf(logger, socketFd, m_sendBufferSize);
		}

		// Socket is good, cancel the socket close scope and break out of the loop.
		socketClose.Cancel();

		bool linger = false;
		SCOPED_FUTEX(m_connectionsLock, lock);
		auto it = m_connections.emplace(m_connections.end(), logger, socketFd, linger);
		auto& conn = *it;
		#if UBA_USE_IOCP
		if (m_iocpHandle)
			CreateIoCompletionPort((HANDLE)ToPlatformSocket(socketFd), m_iocpHandle, (ULONG_PTR)&conn, 0);
		else
		#endif
			conn.recvThread.Start([this, connPtr = &conn] { ThreadRecv(*connPtr); return 0; }, TC("UbaTcpRecv"));

		lock.Leave();

		if (!connectedFunc(&conn, remoteSocketAddr, timedOut))
		{
			conn.shutdownCalled = true;
			ShutdownSocket(logger, conn.socket, conn.recvWakeupSocket, UBA_SEND_WAKEUP(conn), TC("Connect"));
			conn.ready.Set();
			conn.recvThread.Wait();
			UBA_ASSERT(!conn.threadedSendContext);
			SCOPED_FUTEX(m_connectionsLock, lock2);
			m_connections.erase(it);
			return false;
		}

		//char* ip = inet_ntoa(((sockaddr_in*)const_cast<sockaddr*>(&remoteSocketAddr))->sin_addr);
		if (nameHint)
			logger.Detail(TC("Connected to %s:%u (%s)"), nameHint, ((sockaddr_in&)remoteSocketAddr).sin_port, GuidToString(conn.uid).str);
		else
			logger.Detail(TC("Connected using sockaddr (%s)"), GuidToString(conn.uid).str);

		return true;
	}

	void NetworkBackendTcp::DeleteConnection(void* connection)
	{
		SCOPED_FUTEX(m_connectionsLock, lock);
		for (auto it=m_connections.begin();it!=m_connections.end(); ++it)
		{
			Connection& c = *it;
			if (&c != connection)
				continue;
			c.ready.Set();
			it = m_connections.erase(it);
			break;
		}
	}

	void NetworkBackendTcp::SetSendBufferSize(void* connection, u32 bytes)
	{
		auto& conn = *(Connection*)connection;
		SCOPED_CRITICAL_SECTION(conn.shutdownLock, lock);
		SetSendBuf(m_logger, conn.socket, bytes);
	}

	void NetworkBackendTcp::SetRecvBufferSize(void* connection, u32 bytes)
	{
		auto& conn = *(Connection*)connection;
		SCOPED_CRITICAL_SECTION(conn.shutdownLock, lock);
		SetRecvBuf(m_logger, conn.socket, bytes);
	}

	void NetworkBackendTcp::GetTotalSendAndRecv(u64& outSend, u64& outRecv)
	{
		outSend = m_totalSend;
		outRecv = m_totalRecv;
	}

	void NetworkBackendTcp::Validate(Logger& logger, const Vector<void*>& connections, bool full)
	{
		logger.Info(TC("  NetworkBackendTcp"));

		PrintTcpStatistics(logger, 0, nullptr);

		SCOPED_FUTEX(m_connectionsLock, lock);
		for (auto ptr : connections)
		{
			auto& c = *(Connection*)ptr;
			LogTcpInfo(logger, c.socket);
		}
	}

	void NetworkBackendTcp::ThreadSend(Connection& connection)
	{
		auto& logger = connection.logger;

		if (!connection.ready.IsSet(60000)) // This should never time out!
			return;

		ElevateCurrentThreadPriority();

		SendBlock* first = nullptr;
		SendBlock* last = nullptr;

		const tchar* hint = TC("ThreadSend");

		auto& context = *connection.threadedSendContext;

		auto closeGuard = MakeGuard([&]()
			{
				while (first) { auto cur = first; first = first->next; cur->sent.Set(); }
				SCOPED_FUTEX(context.blockQueueLock, queueLock);
				first = context.firstBlock;
				context.firstBlock = (SendBlock*)~0ull;
				context.lastBlock = nullptr;
				queueLock.Leave();
				while (first) { auto cur = first; first = first->next; cur->sent.Set(); }
			});

		while (connection.socket != InvalidSocket)
		{
			SCOPED_FUTEX(context.blockQueueLock, queueLock);
			SendBlock* newFirst = context.firstBlock;
			SendBlock* newLast = context.lastBlock;
			context.firstBlock = nullptr;
			context.lastBlock = nullptr;
			if (!first && !newFirst)
				context.blockQueueEvent.Reset();
			queueLock.Leave();

			if (!first && !newFirst)
			{
				context.blockQueueEvent.IsSet();
				continue;
			}

			if (newFirst)
			{
				if (first)
				{
					UBA_ASSERT(last);
					last->next = newFirst;
					last = newLast;
				}
				else
				{
					first = newFirst;
					last = newLast;
				}
			}

			Socket socket = connection.socket;
			if (socket == InvalidSocket)
				break;

			SendBuf bufs[256];
			u32 bufCount = 0;
			for (SendBlock* it = first; it && bufCount < sizeof_array(bufs); it = it->next, ++bufCount)
				bufs[bufCount] = { it->dataLen, it->data };

			int res = SocketSend(socket, bufs, bufCount);
			if (res != SOCKET_ERROR)
			{
				u32 bytesSent = res;
				while (bytesSent)
				{
					if (bytesSent >= first->dataLen)
					{
						bytesSent -= first->dataLen;
						auto temp = first;
						first = first->next;
						temp->dataLen = 0;
						temp->sent.Set();
					}
					else
					{
						// only part of block was sent
						first->dataLen -= bytesSent;
						first->data += bytesSent;
						bytesSent = 0;
					}
				}
				continue;
			}

			if (!HandleSendError(connection, logger, hint))
				return;
		}

		SCOPED_CRITICAL_SECTION(connection.shutdownLock, lock2);
		SCOPED_FUTEX(connection.sendLock, lock);
		Socket s = connection.socket;
		if (s == InvalidSocket)
			return;
		if (connection.shutdownCalled)
			return;
		connection.shutdownCalled = true;
		ShutdownSocket(logger, s, connection.recvWakeupSocket, UBA_SEND_WAKEUP(connection), TC("ThreadSend"));
	}

	bool NetworkBackendTcp::HandleSendError(Connection& connection, Logger& logger, const tchar* hint)
	{
		Socket socket = connection.socket;
		if (socket == InvalidSocket)
			return false;

		bool retry = false;
		bool shouldPoll = SocketShouldPoll(&retry);
		if (retry)
			return true;

		if (!shouldPoll)
		{
			#if UBA_LOG_SOCKET_ERRORS
			logger.Info(TC("SendSocket - send returned an error for socket %llu: %s (%s)"), socket.internal, LastErrorToText(SocketGetLastError()).data, hint);
			#endif
			return false;
		}

		u32 timeoutMs = 10 * 1000;

		u32 timeoutCounter = 0;
		while (true)
		{
			int revents = 0;
			int res = SocketPoll2(socket, PollOUT, timeoutMs, UBA_SEND_WAKEUP(connection), &revents);
			if (res > 0 && revents == 0) // wakeup fired (no events on real socket) — re-check shutdown and loop back
			{
				if (connection.shutdownCalled) return false;
				continue;
			}
			if (!res)
			{
				if (connection.shutdownCalled)
					return false;
				++timeoutCounter;
				if (timeoutCounter == 4)
					logger.Info(TC("SendSocket - WSAPoll returned timeout after 40 seconds for socket %llu (%s)"), socket.internal, hint);
				if (timeoutCounter < 6*10) // Loop for 10 minutes
					continue;
				logger.Info(TC("SendSocket - WSAPoll returned timeout after 10 minutes for socket %llu (%s)"), socket.internal, hint);
				return false;
			}
			if (res < 0)
			{
				#if UBA_LOG_SOCKET_ERRORS
				logger.Info(TC("SendSocket - WSAPoll returned an error for socket %llu: %s (%s)"), socket.internal, LastErrorToText(SocketGetLastError()).data, hint);
				#endif
				return false;
			}
			if (revents & (PollERR | PollHUP | PollNVAL))
			{
				#if UBA_LOG_SOCKET_ERRORS
				logger.Info(TC("SendSocket - WSAPoll revents contained error flags for socket %llu: %s (%s)"), socket.internal, LastErrorToText(SocketGetLastError()).data, hint);
				#endif
				return false;
			}
			break;
		}
		return true;
	}

	bool NetworkBackendTcp::SendSocket(Connection& connection, Logger& logger, const void* b, u64 bufferLen, const tchar* hint)
	{
		#if UBA_EMULATE_BAD_INTERNET
		if ((rand() % 10000) == 0)
		{
			connection.logger.Info(TC("BAD INTERNET"));
			Sleep(10000);
		}
		#endif

		if (auto context = connection.threadedSendContext)
		{
			SendBlock newBlock { (const u8*)b, u32(bufferLen), nullptr };

			newBlock.sent.Create(EventResetType_Manual);

			SCOPED_FUTEX(context->blockQueueLock, queueLock);
			if (context->lastBlock)
				context->lastBlock->next = &newBlock;
			else
			{
				if (context->firstBlock == (SendBlock*)~0ull)
					return false;
				context->firstBlock = &newBlock;
				context->blockQueueEvent.Set();
			}
			context->lastBlock = &newBlock;
			queueLock.Leave();

			newBlock.sent.IsSet();
			return newBlock.dataLen == 0;
		}


#if UBA_USE_OVERLAPPED_SEND
		if (m_useOverlappedSend)
		{
			Event ev(EventResetType_Manual);

			// Right now we are experimenting with if we can ignore taking a lock around the entire thing and only the WSASend.
			// Documentation is slightly unclear and network forums claim the order of data is correct even though multiple
			// threads call WSASend and then wait on event.
			#if UBA_USE_OVERLAPPED_SEND_WITH_LOCK
			SCOPED_FUTEX(connection.sendLock, lock);
			#endif

			char* buffer = (char*)b;
			u64 left = bufferLen;
			while (left)
			{
				OVERLAPPED overlapped {};
				overlapped.hEvent = ev.GetHandle();
				WSABUF buf { u32(left), buffer };

				#if !UBA_USE_OVERLAPPED_SEND_WITH_LOCK
				SCOPED_FUTEX(connection.sendLock, lock);
				#endif

				Socket socket = connection.socket;
				if (socket == InvalidSocket)
					return false;
				int res = WSASend(ToPlatformSocket(socket), &buf, 1, NULL, 0, &overlapped, NULL);

				#if !UBA_USE_OVERLAPPED_SEND_WITH_LOCK
				lock.Leave();
				#endif

				if (res == SOCKET_ERROR)
				{
					u32 lastError = WSAGetLastError();
					if (lastError != WSA_IO_PENDING)
					{
						#if UBA_LOG_SOCKET_ERRORS
						logger.Info(TC("WSASend - error for socket %llu: %s (%s)"), socket.internal, LastErrorToText(lastError).data, hint);
						#endif
						return false;
					}
				}

				if (!ev.IsSet(38*1000))
				{
					#if !UBA_USE_OVERLAPPED_SEND_WITH_LOCK
					SCOPED_FUTEX(connection.timeoutLock, timeoutLock);
					#endif

					if (ev.IsSet(2*1000))
						break;
					logger.Info(TC("SendSocket - WSASend returned timeout after 40 seconds for socket %llu (%s)"), socket.internal, hint);
					u64 startTime = GetTime();
					while (true)
					{
						PrintTcpStatistics(logger, 0, nullptr);
						LogTcpInfo(logger, socket);

						if (ev.IsSet(4*1000))
							break;

						u64 sinceStartSeconds = TimeToMs(GetTime() - startTime)/1000;
						if (sinceStartSeconds >= DefaultNetworkSendTimeoutSeconds)
						{
							logger.Info(TC("SendSocket - WSASend returned timeout after 10 minutes for socket %llu (%s)"), socket.internal, hint);
							return false;
						}
					}
				}

				DWORD bytesSent;
				DWORD flags;
				if (!WSAGetOverlappedResult(ToPlatformSocket(socket), &overlapped, &bytesSent, FALSE, &flags))
				{
					#if UBA_LOG_SOCKET_ERRORS
					logger.Info(TC("WSAGetOverlappedResult - error for socket %llu: %s (%s)"), socket.internal, LastErrorToText().data, hint);
					#endif
					return false;
				}

				buffer += bytesSent;
				left -= bytesSent;
				if (left)
				{
					#if !UBA_USE_OVERLAPPED_SEND_WITH_LOCK
					//#if UBA_LOG_SOCKET_ERRORS
					logger.Warning(TC("SendSocket - WSASend did not send all data in one call for socket %llu. Note that this might be allowed behavior. Hard to read out of documentation. (%s)"), socket.internal, hint);
					//#endif
					return false;
					#else
					ev.Reset();
					#endif
				}
			}
			return true;
		}
#endif

		SCOPED_FUTEX(connection.sendLock, lock);
		Socket socket = connection.socket;
		if (socket == InvalidSocket)
			return false;

		char* buffer = (char*)b;
		u64 left = bufferLen;
		while (left)
		{
			SendBuf buf { u32(left), buffer };
			int sent = SocketSend(socket, &buf, 1);
			if (sent != SOCKET_ERROR)
			{
				buffer += sent;
				left -= sent;
				continue;
			}

			if (!HandleSendError(connection, logger, hint))
				return false;
		}
		return true;
	}

	bool NetworkBackendTcp::RecvSocket(Connection& connection, RecvCache& recvCache, void* b, u32& bufferLen, const tchar* hint, bool isFirstCall, bool allowLess)
	{
		#if UBA_EMULATE_BAD_INTERNET
		if ((rand() % 10000) == 0)
		{
			connection.logger.Info(TC("BAD INTERNET"));
			Sleep(10000);
		}
		#endif

		Socket socket = connection.socket;

		u32 retryCount = 4;

		u8* buffer = (u8*)b;
		u32 recvLeft = bufferLen;
		while (recvLeft)
		{
			if (recvCache.written)
			{
				u32 toCopy = Min(recvCache.written - recvCache.pos, recvLeft);
				memcpy(buffer, recvCache.bytes + recvCache.pos, toCopy);
				recvCache.pos += toCopy;
				if (recvCache.pos == recvCache.written)
				{
					recvCache.pos = 0;
					recvCache.written = 0;
				}
				recvLeft -= toCopy;
				buffer += toCopy;
				if (!recvLeft)
					return true;
			}

			int read;

			{
				RecvBuf bufs[2] = { { recvLeft, buffer }, { sizeof(recvCache.bytes), recvCache.bytes } };
				read = SocketRecv2(socket, bufs, 2);
				if (read > 0)
				{
					if (u32(read) > recvLeft)
					{
						recvCache.written = read - recvLeft;
						return true;
					}
					recvLeft -= read;
					buffer += read;

					if (!allowLess)
						continue;

					bufferLen = read;
					return true;
				}
			}

			if (read == 0)
			{
				#if UBA_LOG_SOCKET_ERRORS
				connection.logger.Info(TC("RecvSocket - recv gracefully closed by peer for socket %llu and connection %s (%s%s)"), socket.internal, GuidToString(connection.uid).str, connection.recvHint, hint);
				#endif
				return false;
			}

			UBA_ASSERT(read == SOCKET_ERROR);

			bool retry = false;
			bool shouldPoll = SocketShouldPoll(&retry);
			if (retry)
				continue;

			if (shouldPoll)
			{
				if (retryCount > 0)
				{
					--retryCount;
					continue;
				}
				u32 timeoutMs = connection.recvTimeoutMs;
				if (!timeoutMs)
					timeoutMs = DefaultNetworkReceiveTimeoutSeconds * 1000; // 10 minutes time out

				// Since a socket shutdown does not signal the poll to stop we need to timeout earlier and check shutdownCalled and then poll again
				// We timeout 10 seconds at the time until we reach the "real" timeout
				u32 timeoutSliceMs = Min(timeoutMs, 10u*1000);
				u64 startTime = GetTime();
				while (true)
				{
					int revents = 0;
					int res = SocketPoll2(socket, PollIN, timeoutSliceMs, connection.recvWakeupSocket, &revents);
					if (res > 0 && revents == 0) // wakeup fired (no events on real socket) — re-check shutdown and loop back
					{
						if (connection.shutdownCalled)
							return false;
						continue;
					}
					if (!res)
					{
						if (connection.shutdownCalled)
							return false;

						u64 now = GetTime();
						u32 timePassedMs = u32(TimeToMs(now - startTime));
						if (timePassedMs < timeoutMs)
						{
							// If less than 10s left to timeout we just timeout to that time
							timeoutSliceMs = Min(timeoutMs - timePassedMs + 1000U, timeoutSliceMs);
							continue;
						}

						if (connection.recvTimeoutCallback)
						{
							if (connection.recvTimeoutCallback(connection.recvTimeoutContext, timeoutMs, connection.recvHint, hint))
								break;
							return false;
						}
						connection.logger.Info(TC("RecvSocket - WSAPoll returned timeout for socket %llu and connection %s after %s (%s%s)"), socket.internal, GuidToString(connection.uid).str, TimeToText(MsToTime(timeoutMs)).str, connection.recvHint, hint);
						return false;
					}
					if (res < 0)
					{
						#if UBA_LOG_SOCKET_ERRORS
						connection.logger.Info(TC("RecvSocket - WSAPoll returned an err-or for socket %llu and connection %s: %s (%s%s)"), socket.internal, GuidToString(connection.uid).str, LastErrorToText(SocketGetLastError()).data, connection.recvHint, hint);
						#endif
						return false;
					}
					if ((revents & PollIN) == 0)
					{
						#if UBA_LOG_SOCKET_ERRORS
						connection.logger.Info(TC("RecvSocket - WSAPoll revents contained error for socket %llu and connection %s: %s (%s%s)"), socket.internal, GuidToString(connection.uid).str, LastErrorToText(SocketGetLastError()).data, connection.recvHint, hint);
						#endif
						return false;
					}
					break;
				}

				continue;
			}

			#if !PLATFORM_WINDOWS
			if (!isFirstCall && errno != ECONNRESET)
				return connection.logger.Warning(TC("RecvSocket - recv err-or on socket %llu and connection %s: %s (%s%s)"), socket.internal, GuidToString(connection.uid).str, strerror(errno), connection.recvHint, hint);
			#endif

			#if UBA_LOG_SOCKET_ERRORS
			connection.logger.Info(TC("RecvSocket - read returned an err-or for socket %llu and connection %s: %s (%s%s)"), socket.internal, GuidToString(connection.uid).str, LastErrorToText(SocketGetLastError()).data, connection.recvHint, hint);
			#endif
			return false;
		}
		return true;
	}
	
	#if UBA_USE_IOCP
	void NetworkBackendTcp::ThreadIocp()
	{
		ElevateCurrentThreadPriority();

		while (true)
		{
			DWORD bytesTransferred = 0;
			ULONG_PTR completionKey = 0;
			OVERLAPPED* overlapped = nullptr;
			BOOL result = GetQueuedCompletionStatus(m_iocpHandle, &bytesTransferred, &completionKey, (OVERLAPPED**)&overlapped, INFINITE);

			if (!result && !overlapped)
			{
				m_logger.Info(TC("GetQueuedCompletionStatus error (%s)"), LastErrorToText(SocketGetLastError()).data);
				break;
			}
			if (completionKey == 1)
				break;

			auto& connection = *(Connection*)completionKey;
			auto& logger = connection.logger;

			if (bytesTransferred == 0 || !result)
			{
				SCOPED_CRITICAL_SECTION(connection.shutdownLock, lock2);
				Socket s = connection.socket;
				CancelIoEx((HANDLE)ToPlatformSocket(s), overlapped);

				{
					SCOPED_FUTEX(connection.sendLock, lock);
					connection.socket = InvalidSocket;
				}

				// We need to marshal shutdown and callback handling out of iocp thread since callback can call things that requires iocp to loop
				connection.recvThread.Start([this, s, connPtr = &connection]
					{
						Connection& connection = *connPtr;
						auto& logger = connection.logger;

						SCOPED_CRITICAL_SECTION(connection.shutdownLock, lock2);
						if (auto cb = connection.disconnectCallback)
						{
							auto context = connection.disconnectContext;
							connection.disconnectCallback = nullptr;
							connection.disconnectContext = nullptr;
							cb(context, connection.uid, &connection);
						}

						if (s != InvalidSocket)
						{
							ShutdownSocket(logger, s, connection.recvWakeupSocket, UBA_SEND_WAKEUP(connection), TC("ThreadRecv"));
							CloseSocket(logger, s, TC("ThreadRecv"));
						}
						return 0;

					}, TC("UbaTcpSdwn"));
				continue;
			}

			bool isSend = overlapped != &connection.overlapped;
			if (isSend)
				continue;

			UBA_ASSERT(bytesTransferred <= connection.wsaBuf.len);
			connection.wsaBuf.len -= bytesTransferred;
			if (connection.wsaBuf.len)
			{
				u8* newPos = (u8*)connection.wsaBuf.buf + bytesTransferred;
				PostIocpRead(connection, newPos, connection.wsaBuf.len);
				continue;
			}

			if (connection.receivingHeader)
			{
				m_totalRecv += connection.headerSize;

				auto hc = connection.headerCallback;
				if (!hc)
				{
					logger.Error(TC("Tcp connection header callback not set"));
					continue;
				}

				u8*& bodyData = connection.bodyData;
				u32& bodySize = connection.bodySize;
				bodyData = nullptr;
				bodySize = 0;
				if (!hc(connection.recvContext, connection.uid, connection.header, connection.bodyContext, bodyData, bodySize))
					continue;

				if (connection.socket == InvalidSocket)
					continue;

				if (!bodySize)
				{
					PostIocpRead(connection, connection.header, connection.headerSize);
					continue;
				}

				connection.receivingHeader = false;
				PostIocpRead(connection, bodyData, bodySize);
			}
			else
			{
				auto bc = connection.bodyCallback;
				if (!bc)
				{
					logger.Error(TC("Tcp connection body callback not set"));
					continue;
				}

				bool success = true;
				if (!bc(connection.recvContext, !success, connection.header, connection.bodyContext, connection.bodyData, connection.bodySize))
					continue;
				if (!success)
					continue;

				m_totalRecv += connection.bodySize;

				if (connection.socket == InvalidSocket)
					continue;

				connection.receivingHeader = true;
				PostIocpRead(connection, connection.header, connection.headerSize);
			}
		}
	}

	bool NetworkBackendTcp::PostIocpRead(Connection& connection, u8* data, u32 dataSize)
	{
		UBA_ASSERT(data);
		UBA_ASSERT(dataSize < 1*1024*1024); // Sanity
		ZeroMemory(&connection.overlapped, sizeof(OVERLAPPED));
		connection.wsaBuf.buf = (char*)data;
		connection.wsaBuf.len = dataSize;

		Socket socket = connection.socket;

		DWORD flags = 0;
		int ret = WSARecv(ToPlatformSocket(socket), &connection.wsaBuf, 1, NULL, &flags, &connection.overlapped, NULL);
		if (ret == SOCKET_ERROR && WSAGetLastError() != WSA_IO_PENDING)
		{
			connection.logger.Info(TC("WSARecv failed for socket %llu trying to receive %u bytes (%s)"), socket.internal, dataSize, LastErrorToText(WSAGetLastError()).data);
			PostQueuedCompletionStatus(m_iocpHandle, 0, (ULONG_PTR)&connection, &connection.overlapped );
			return false;
		}
		return true;
	}
	#endif

	void NetworkBackendTcp::ThreadStatus(u32 statusUpdateSeconds)
	{
		u32 statusUpdateMs = statusUpdateSeconds*1000;

		#if PLATFORM_WINDOWS
		MIB_TCPSTATS_LH prevStats{};
		#endif

		while (!m_tcpStatusLoop.IsSet(statusUpdateMs))
		{
			#if PLATFORM_WINDOWS
			PrintTcpStatistics(m_logger, statusUpdateSeconds, &prevStats);
			#endif

			SCOPED_FUTEX(m_connectionsLock, lock);
			for (auto& conn : m_connections)
			{
				SCOPED_CRITICAL_SECTION(conn.shutdownLock, lock2);
				if (conn.socket == InvalidSocket)
					continue;
				SCOPED_FUTEX(conn.sendLock, sendLock);
				Socket s = conn.socket;
				if (s == InvalidSocket)
					continue;
				//LogTcpInfo(m_logger, s);

				#if 1
				int sendBufSize = 0;
				SocketGetSendBuf(s, sendBufSize);

				int recvBufSize = 0;
				SocketGetRecvBuf(s, recvBufSize);

				char alg[16] = { 0 };
				SocketGetCongestionAlgorithm(s, alg, sizeof(alg));

				int prio = 0;
				#if PLATFORM_LINUX
				socklen_t optlen = sizeof(int);
				getsockopt(ToPlatformSocket(s), SOL_SOCKET, SO_PRIORITY, (char*)&prio, &optlen);
				#endif

				m_logger.Info(TC("%llu - Send: %s, Recv: %s, Prio %i%hs%hs"), s.internal, BytesToText(sendBufSize).str, BytesToText(recvBufSize).str, prio, *alg ? ", Alg: " : "", alg);
				#endif

			}
		}
	}

	bool NetworkBackendTcp::CheckEnvironment(Logger& logger, bool printTips)
	{
		if (!InitNetworkOnce(logger))
			return false;

		if (!printTips)
			return true;

		bool (*GetTcpAutoTuning)(int* outReadMin, int* outReadDefault, int* outReadMax, int* outWriteMin, int* outWriteDefault, int* outWriteMax) = nullptr;

		#if PLATFORM_WINDOWS
		if (!GetTcpAutoTuning && IsRunningWine())
			GetTcpAutoTuning = (decltype(GetTcpAutoTuning))(void*)GetProcAddress((HMODULE)GetUbaWineModule(), "GetTcpAutoTuning");
		#else
		GetTcpAutoTuning = UnixGetTcpAutoTuning;
		#endif

		if (!IsWindows || IsRunningWine())
		{
			Socket tempSocket = SocketCreate();
			char alg[16] = { 0 };
			SocketGetCongestionAlgorithm(tempSocket, alg, sizeof(alg));
			SocketClose(tempSocket);
			if (strncmp(alg, "bbr", 3) != 0)
				logger.Info(TC("  TIP: Enable BBR congestion algorithm and fq (Fair Queue) queuing discipline (Current: %hs)"), alg);
		}

		if (GetTcpAutoTuning)
		{
			int readMin, readDefault, readMax, writeMin, writeDefault, writeMax;
			if (GetTcpAutoTuning(&readMin, &readDefault, &readMax, &writeMin, &writeDefault, &writeMax))
			{
				if (readMax < 1024*1024)
					logger.Info(TC("  TIP: Increase tcp window auto tuning read max to 4-16 MiB (Current: %s %s %s)"), BytesToText(readMin, true).str, BytesToText(readDefault, true).str, BytesToText(readMax, true).str);
				if (writeMax < 1024*1024)
					logger.Info(TC("  TIP: Increase tcp window auto tuning write max to 4-16 MiB (Current: %s %s %s)"), BytesToText(writeMin, true).str, BytesToText(writeDefault, true).str, BytesToText(writeMax, true).str);
			}
		}

		// sudo modprobe tcp_bbr
		// sudo sysctl -w net.core.default_qdisc=fq
		// sudo sysctl -w net.ipv4.tcp_congestion_control=bbr
		// sudo sysctl -w net.ipv4.tcp_rmem="4096 131072 16777216"
		// sudo sysctl -w net.ipv4.tcp_wmem="4096 65536 33554432\"

		// Persist
		// 
		// echo "tcp_bbr" | sudo tee /etc/modules-load.d/bbr.conf
		// echo "net.core.default_qdisc=fq" | sudo tee -a /etc/sysctl.conf
		// echo "net.ipv4.tcp_congestion_control=bbr" | sudo tee -a /etc/sysctl.conf
		// sudo sysctl -p

		return true;
	}

	bool NetworkBackendTcp::KillTcpHogs(Logger& logger, u16 port)
	{
#if PLATFORM_WINDOWS
		if (IsRunningWine())
		{
			using KillTcpHogsFunc = bool(u16);
			if (auto KillTcpHogs = (KillTcpHogsFunc*)(void*)GetProcAddress((HMODULE)GetUbaWineModule(), "KillTcpHogs"))
				return KillTcpHogs(port);
			return logger.Error(TC("Make sure UbaWine.dll.so is available to current binary"));
		}
		return logger.Error(TC("No support for killing tcp hogs."));
#else
		StringBuffer<> lsofCommand;
		lsofCommand.Appendf("lsof -i :%u -sTCP:LISTEN -Pn -t", u32(port));

		FILE* lsof = popen(lsofCommand.data, "r");
		if (!lsof)
			return logger.Error(TC("Failed run lsof while trying to kill processes holding port %u (not installed?)"), u32(port));

		char pidStr[16];
		while (fgets(pidStr, sizeof(pidStr), lsof))
		{
			pid_t pid = (pid_t)atoi(pidStr);
			if (pid <= 0)
				continue;
			if (kill(pid, SIGKILL) != 0)
				return logger.Error("Failed to kill process %d while trying to kill processes holding port %u", pid, u32(port));
			logger.Info("Process %d killed successfully", pid);
		}
		pclose(lsof);
		return true;
#endif
	}

	void NetworkBackendTcp::PrintTcpStatistics(Logger& logger, u32 statusUpdateSeconds, void* prevStatsPtr)
	{
		#if PLATFORM_WINDOWS
		auto prevStats = (MIB_TCPSTATS_LH*)prevStatsPtr;
		MIB_TCPSTATS_LH stats{};
		if (GetTcpStatisticsEx(&stats, AF_INET) != NO_ERROR)
		{
			logger.Info(TC("GetTcpStatisticsEx failed"));
			return;
		}

		INT64 recv       = stats.dwInSegs;
		INT64 sent       = stats.dwOutSegs;
		INT64 retrans    = stats.dwRetransSegs;
		INT64 inerrs     = stats.dwInErrs;
		INT64 outrsts    = stats.dwOutRsts;

		StringBuffer<128> temp;

		if (prevStats && prevStats->dwRtoAlgorithm != 0)
		{
			recv       -= prevStats->dwInSegs;
			sent       -= prevStats->dwOutSegs;
			retrans    -= prevStats->dwRetransSegs;
			inerrs     -= prevStats->dwInErrs;
			outrsts    -= prevStats->dwOutRsts;

			temp.Appendf(TC("%us delta"), statusUpdateSeconds);
			*prevStats = stats;
		}

		double rrate = sent ? (100.0 * double(retrans) / double(sent)) : 0;
		logger.Info(TC("%s  recv=%lld  sent=%lld  retrans=%lld  (%.2f%%)  inErr=%lld  RST=%lld"), temp.data, recv, sent, retrans, rrate, inerrs, outrsts);
		#endif
	}


	////////////////////////////////////////////////////////////////////////////////////////////////////

	void TraverseNetworkAddresses(Logger& logger, const Function<bool(const StringBufferBase& addr)>& func)
	{
#if PLATFORM_WINDOWS
		// Fallback code for some cloud setups where we can't use the dns to find out ip addresses. (note it always work by providing the adapter we want to listen on)
		IP_ADAPTER_INFO info[16];
		ULONG bufLen = sizeof(info);
		if (GetAdaptersInfo(info, &bufLen) != ERROR_SUCCESS)
		{
			logger.Info(TC("GetAdaptersInfo failed (%s)"), LastErrorToText(SocketGetLastError()).data);
			return;
		}
		for (IP_ADAPTER_INFO* it = info; it; it = it->Next)
		{
			if (it->Type != MIB_IF_TYPE_ETHERNET && it->Type != IF_TYPE_IEEE80211)
				continue;
			for (IP_ADDR_STRING* s = &it->IpAddressList; s; s = s->Next)
			{
				StringBuffer<128> ip;
				ip.Appendf(TC("%hs"), s->IpAddress.String);
				if (ip.Equals(L"0.0.0.0"))
					continue;
				if (!func(ip))
					return;
			}
		}
#else
		struct ifaddrs* ifaddr;
		if (getifaddrs(&ifaddr) == -1)
		{
			logger.Info("getifaddrs failed");
			return;
		}
		auto g = MakeGuard([ifaddr]() { freeifaddrs(ifaddr); });

		for (struct ifaddrs* ifa = ifaddr; ifa; ifa = ifa->ifa_next)
		{
			if (ifa->ifa_addr == nullptr)
				continue;

			int family = ifa->ifa_addr->sa_family;
			if (family != AF_INET)
				continue;

			StringBuffer<NI_MAXHOST> ip;
			int s = getnameinfo(ifa->ifa_addr, sizeof(struct sockaddr_in), ip.data, ip.capacity, NULL, 0, NI_NUMERICHOST);
			if (s != 0)
				continue;
			ip.count = strlen(ip.data);
			if (ip.StartsWith("169.254") || ip.Equals("127.0.0.1"))
				continue;
			if (!func(ip))
				return;
		}
#endif
	}

	bool TraverseRemoteAddresses(Logger& logger, const tchar* addr, u16 port, const Function<bool(const sockaddr& remoteSockaddr)>& func)
	{
		addrinfoW  hints;
		memset(&hints, 0, sizeof(hints));
		hints.ai_family = AF_INET; //AF_UNSPEC; (Skip AF_INET6)
		hints.ai_socktype = SOCK_STREAM;
		hints.ai_protocol = IPPROTO_TCP;

		StringBuffer<32> portStr;
		portStr.AppendValue(port);

		// Resolve the server address and port
		addrinfoW* remoteAddrInfo = nullptr;
		int res = GetAddrInfoW(addr, portStr.data, &hints, &remoteAddrInfo);
		if (res != 0)
		{
			if (res == WSAHOST_NOT_FOUND)
				return logger.Error(TC("Invalid server address '%s'"), addr);
			//logger.Error(TC("GetAddrInfoW failed with error: %s"), getErrorText(res).c_str());
			return false;
		}

		auto addrCleanup = MakeGuard([&]() { if (remoteAddrInfo) FreeAddrInfoW(remoteAddrInfo); });

		auto addrInfoIt = remoteAddrInfo;
		// Loop through and attempt to connect to an address until one succeeds
		for (; addrInfoIt != NULL; addrInfoIt = addrInfoIt->ai_next)
			if (!func(*addrInfoIt->ai_addr))
				return true;
		return true;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	HttpConnection::HttpConnection()
	{
		m_socket = InvalidSocket.internal;
		*m_host = 0;
	}

	HttpConnection::~HttpConnection()
	{
		if (m_socket != InvalidSocket.internal)
		{
			LoggerWithWriter logger(g_nullLogWriter);
			CloseSocket(logger, {m_socket}, TC("HttpDtor"));
		}
	}

	bool HttpConnection::Connect(Logger& logger, const char* host)
	{
		if (!InitNetworkOnce(logger))
			return false;

		hostent* hostent = gethostbyname(host);
		if (hostent == NULL)
			return logger.Error(TC("HttpConnection: gethostbyname error (%s)"), host);

		char* ntoaRes = inet_ntoa(*(struct in_addr*)*(hostent->h_addr_list));
		if (!ntoaRes)
			return logger.Error(TC("HttpConnection: inet_ntoa returned null"));
		unsigned long in_addr = inet_addr(ntoaRes);
		if (in_addr == INADDR_NONE)
			return logger.Error(TC("HttpConnection: inet_addr returned INADDR_NONE (%s)"), ntoaRes);

		protoent* protoent = getprotobyname("tcp");
		if (protoent == NULL)
			return logger.Error(TC("HttpConnection: getprotobyname returned null for tcp"));

		Socket sock = SocketCreate(AF_INET, SOCK_STREAM, protoent->p_proto);
		if (sock == InvalidSocket)
			return logger.Error(TC("HttpConnection: socket failed (%s)"), LastErrorToText(SocketGetLastError()).data);
		auto socketClose = MakeGuard([sock]() { SocketClose(sock); });

		if (m_connectTimeOutMs)
			SetTimeout(logger, sock, m_connectTimeOutMs);

		sockaddr_in sockaddr_in;
		sockaddr_in.sin_addr.s_addr = in_addr;
		sockaddr_in.sin_family = AF_INET;
		sockaddr_in.sin_port = htons(80);

		if (SocketConnect(sock, (struct sockaddr*)&sockaddr_in, sizeof(sockaddr_in)) == -1)
			return false;// logger.Error(TC("HttpConnection: connect error"));

		socketClose.Cancel();

		if (m_connectTimeOutMs)
			SetTimeout(logger, sock, 10000); // 10 seconds timeout

		strcpy_s(m_host, sizeof_array(m_host), host);
		m_socket = sock.internal;
		return true;
	}

	bool HttpConnection::Query(Logger& logger, const char* type, StringBufferBase& outResponse, u32& outStatusCode, const char* host, const char* path, const char* header, u32 timeoutMs)
	{
		// TODO: Fix so we reuse socket connection for multiple queries
		// Will need to change "Connection: close" and also know where end of message is

		if (*m_host)// && _stricmp(m_host, host) != 0)
		{
			CloseSocket(logger, {m_socket}, TC("HttpQuery"));
			m_socket = InvalidSocket.internal;
			*m_host = 0;
		}

		if (m_socket == InvalidSocket.internal)
			if (!Connect(logger, host))
				return false;

		char request[1024];
		int requestLen = snprintf(request, sizeof_array(request), "%s /%s HTTP/1.1\r\nHost: %s\r\nUser-Agent: uba\r\nConnection: close\r\n%s\r\n", type, path, m_host, header);
		if (requestLen >= sizeof_array(request))
			return logger.Error(TC("STACK BUFFER TOO SMALL!"));

		SetTimeout(logger, {m_socket}, timeoutMs);

		int totalBytesSent = 0;
		while (totalBytesSent < requestLen)
		{
			SendBuf buf { u32(requestLen - totalBytesSent), request + totalBytesSent };
			int bytesSent = SocketSend({m_socket}, &buf, 1);
			if (bytesSent == -1)
				return logger.Error(TC("HttpConnection: socket send error (%hs)"), host);
			totalBytesSent += bytesSent;
		}

		//logger.Warning(TC("REQUEST:\r\n%hs\r\n"), request);

#if PLATFORM_WINDOWS
#pragma warning(push)
#pragma warning(disable:6386) // analyzer claims that buf can have buffer overrun.. but can't see how that can happen
#endif

		u32 readPos = 0;
		char buf[4*1024];
		int bytesRead = 0;
		while ((bytesRead = SocketRecv({m_socket}, buf + readPos, sizeof(buf) - readPos)) > 0)
			readPos += bytesRead;

		if (bytesRead == SOCKET_ERROR)
			return logger.Error(TC("HttpConnection: socket recv error after reading %u bytes - %s (%hs %hs)"), readPos, LastErrorToText(SocketGetLastError()).data, m_host, path);

		if (readPos == sizeof(buf))
			return logger.Error(TC("HttpConnection: buffer overflow"));

		buf[readPos] = 0;

#if PLATFORM_WINDOWS
#pragma warning(pop)
#endif

		//logger.Warning(TC("RESPONSE:\r\n%hs\r\n"), buf);

		char* firstSpace = strchr(buf, ' '); // After version (where status code starts)
		if (!firstSpace)
			return logger.Error(TC("HttpConnection: first space not found (read %u)"), readPos);

		char* secondSpace = strchr(firstSpace + 1, ' '); // after status code
		if (!secondSpace)
			return logger.Error(TC("HttpConnection: second space not found"));

		*secondSpace = 0;
		outStatusCode = strtoul(firstSpace + 1, nullptr, 10);

		if (outStatusCode != 200)
			return false;

		char* bodyStart = strstr(secondSpace + 1, "\r\n\r\n");
		if (!bodyStart)
			return logger.Error(TC("HttpConnection: no body found"));

		outResponse.Append(bodyStart + 4);
		return true;
	}

	void HttpConnection::SetConnectTimeout(u32 timeOutMs)
	{
		m_connectTimeOutMs = timeOutMs;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool ShutdownSocket(Logger& logger, Socket s, Socket recvWakeupSocket, Socket sendWakeupSocket, const tchar* hint)
	{
		// Wake recv/send threads via their wakeup sockets. SocketWakeup on InvalidSocket is a no-op
		// from the kernel's POV (send returns -1, harmless). No FIN goes on the wire — the actual
		// connection teardown happens later in CloseSocket() with SO_LINGER {1,0} → RST.
		(void)logger; (void)s; (void)hint;
		if (recvWakeupSocket == InvalidSocket)
			SocketShutdown(s);
		else
			SocketWakeup(recvWakeupSocket);
		if (sendWakeupSocket != InvalidSocket)
			SocketWakeup(sendWakeupSocket);
		return true;
	}

	bool CloseSocket(Logger& logger, Socket s, const tchar* hint)
	{
		if (s == InvalidSocket)
			return true;
		if (SocketClose(s) != SOCKET_ERROR)
			return true;
		logger.Info(TC("Failed to close socket %llu in %s (%s)"), s.internal, hint, LastErrorToText(SocketGetLastError()).data);
		return false;
	}

	bool SetBlocking(Logger& logger, Socket socket, bool blocking)
	{
		if (SocketSetBlocking(socket, blocking) != 0)
			return logger.Error(TC("Setting non blocking socket failed (error: %s)"), LastErrorToText(SocketGetLastError()).data);
		return true;
	}

	bool DisableNagle(Logger& logger, Socket socket)
	{
		if (SocketSetNoDelay(socket) != 0)
			return logger.Error(TC("setsockopt TCP_NODELAY error: (error: %s)"), LastErrorToText(SocketGetLastError()).data);
		return true;
	}

	bool SetTimeout(Logger& logger, Socket socket, u32 timeoutMs)
	{
		if (SocketSetTimeout(socket, timeoutMs) != 0)
			return logger.Error(TC("setsockopt SO_SNDTIMEO error: (error: %s)"), LastErrorToText(SocketGetLastError()).data);
		return true;
	}

	bool SetNoLinger(Logger& logger, Socket socket)
	{
		// l_onoff=1 + l_linger=0 means close() (and kernel-driven cleanup on
		// process death — crashes, SIGKILL, power-off) emits a TCP RST instead
		// of FIN. The connection 4-tuple is gone the moment the kernel processes
		// the close: no FIN_WAIT_2, no TIME_WAIT, port is immediately rebindable.
		//
		// UBA's protocol already assumes any in-flight message can be lost on
		// disconnect (workers and the visualizer both reconnect/resync), so the
		// loss of "be polite, drain send buffer" is irrelevant. The win is that
		// `start UbaNinja ? connect visualizer ? kill UbaNinja ? start again`
		// always works in <1 s, instead of waiting up to ~60 s for the kernel
		// state machine to clear.
		if (SocketSetLinger(socket, 0) != 0)
			return logger.Error(TC("setsockopt SO_LINGER error: (error: %s)"), LastErrorToText(SocketGetLastError()).data);
		return true;
	}

	bool SetRecvBuf(Logger& logger, Socket socket, u32 windowSize)
	{
		if (SocketSetRecvBuf(socket, windowSize) == 0)
			return true;
		return logger.Error(TC("setsockopt SO_RCVBUF error: (error: %s)"), LastErrorToText(SocketGetLastError()).data);
	}

	bool SetSendBuf(Logger& logger, Socket socket, u32 windowSize)
	{
		if (SocketSetSendBuf(socket, windowSize) == 0)
			return true;
		return logger.Error(TC("setsockopt SO_SNDBUF error: (error: %s)"), LastErrorToText(SocketGetLastError()).data);
	}

	bool SetSocketPriority(Logger& logger, Socket socket)
	{
		if (SocketSetPriority(socket) == 0)
			return true;
		return logger.Error(TC("setsockopt SO_PRIORITY failed (%s)"), LastErrorToText(SocketGetLastError()).data);
	}

	bool SetKeepAlive(Logger& logger, Socket socket) // This will make sure that WSAPoll exits when the network cable is pulled
	{
		if (SocketSetKeepAlive(socket, KeepAliveIdleSeconds, KeepAliveIntervalSeconds) != 0)
			return logger.Error(TC("SocketSetKeepAlive (error: %s)"), LastErrorToText(SocketGetLastError()).data);
		return true;
	}

	bool SetUserTimeout(Logger& logger, Socket socket, u32 timeoutMs)
	{
		if (SocketSetUserTimeout(socket, (int)timeoutMs) == 0)
			return true;
		#if PLATFORM_WINDOWS
		if (IsRunningWine() && !GetUbaWineModule()) // Wine does not have TCP_MAXRT implemented (at least not 11.2)
			return true;
		#endif
		return logger.Error(TC("SocketSetUserTimeout (error: %s)"), LastErrorToText(SocketGetLastError()).data);
	}

	#if !defined(TCP_INFO_v0) && PLATFORM_WINDOWS
	enum TCPSTATE {};
	struct TCP_INFO_v0
	{
		TCPSTATE State;
		ULONG    Mss;
		ULONG64  ConnectionTimeMs;
		BOOLEAN  TimestampsEnabled;
		ULONG    RttUs;
		ULONG    MinRttUs;
		ULONG    BytesInFlight;
		ULONG    Cwnd;
		ULONG    SndWnd;
		ULONG    RcvWnd;
		ULONG    RcvBuf;
		ULONG64  BytesOut;
		ULONG64  BytesIn;
		ULONG    BytesReordered;
		ULONG    BytesRetrans;
		ULONG    FastRetrans;
		ULONG    DupAcksIn;
		ULONG    TimeoutEpisodes;
		UCHAR    SynRetrans;
	};
	#endif

	#ifndef SIO_TCP_INFO     // older MinGW headers
	#define SIO_TCP_INFO  _WSAIORW(IOC_VENDOR,0x50)
	#endif


#if PLATFORM_WINDOWS
	struct LinuxTcpInfo        
	{
		u8 tcpi_state;         
		u8 tcpi_ca_state;
		u8 tcpi_retransmits;
		u8 tcpi_probes;
		u8 tcpi_backoff;
		u8 tcpi_options;
		u8 tcpi_snd_wscale : 4,
		   tcpi_rcv_wscale : 4;

		u32 tcpi_rto;          
		u32 tcpi_ato;
		u32 tcpi_snd_mss;
		u32 tcpi_rcv_mss;

		u32 tcpi_unacked;      
		u32 tcpi_sacked;
		u32 tcpi_lost;
		u32 tcpi_retrans;      
		u32 tcpi_fackets;

		u32 tcpi_last_data_sent;
		u32 tcpi_last_ack_sent;
		u32 tcpi_last_data_recv;
		u32 tcpi_last_ack_recv;

		u32 tcpi_pmtu;
		u32 tcpi_rcv_ssthresh;
		u32 tcpi_rtt;           
		u32 tcpi_rttvar;        
		u32 tcpi_snd_ssthresh;
		u32 tcpi_snd_cwnd;      
		u32 tcpi_advmss;
		u32 tcpi_reordering;

		u32 tcpi_rcv_rtt;
		u32 tcpi_rcv_space;     

		u32 tcpi_total_retrans; 

		u64 tcpi_pacing_rate;    
		u64 tcpi_max_pacing_rate;
		u64 tcpi_bytes_acked;    
		u64 tcpi_bytes_received; 
		u64 tcpi_segs_out;       
		u64 tcpi_segs_in;        

		u32 tcpi_notsent_bytes;  
		u32 tcpi_min_rtt;        
		u32 tcpi_data_segs_in;
		u32 tcpi_data_segs_out;

		u64 tcpi_delivery_rate;  

		u64 tcpi_busy_time;      
		u64 tcpi_rwnd_limited;   
		u64 tcpi_sndbuf_limited; 

		u32 tcpi_delivered;
		u32 tcpi_delivered_ce;

		u64 tcpi_bytes_sent;
		u64 tcpi_bytes_retrans;
		u64 tcpi_dsack_dups;
		u64 tcpi_reord_seen;

		u32 tcpi_rtt_min;
		u32 tcpi_rcv_rtt_min;
	};
#endif


	void LogTcpInfo(Logger& logger, Socket socket)
	{
	#if PLATFORM_WINDOWS
		#if 0
		if (IsRunningWine())
		{
			static HMODULE wineDll = (HMODULE)GetUbaWineModule();
			if (!wineDll)
				return false;
			using GetLinuxTcpInfoFunc = int WINAPI(Socket, void*, int*);
			static auto GetLinuxTcpInfo = (GetLinuxTcpInfoFunc*)GetProcAddress(wineDll, "GetLinuxTcpInfo");
			if (!GetLinuxTcpInfo)
				return logger.Error(TC("GetLinuxTcpInfo not found in UbaWine dll"));
			LinuxTcpInfo ti {};
			int size = sizeof(LinuxTcpInfo);
			int res = GetLinuxTcpInfo(socket, &ti, &size);
			if (res != 0)
				return logger.Error(TC("GetLinuxTcpInfo failed with error %i"), res);

			u32 bytes_inflight_est = ti.tcpi_unacked * ti.tcpi_snd_mss + ti.tcpi_notsent_bytes;
			logger.Info(TC("%i - RTT=%uus  InFlight=%u  Retrans=%u (Tot %u)  Dupacks=%u  State=%u RcvWnd=%u  RcvScale=%u"), (int)socket, ti.tcpi_rtt, bytes_inflight_est, ti.tcpi_retrans, ti.tcpi_total_retrans, ti.tcpi_snd_cwnd, ti.tcpi_state, ti.tcpi_rcv_space, ti.tcpi_rcv_wscale);
			return true;
		}
		#endif
		
		//TCP_INFO_v0 ti;
		//ULONG version = 0; // Specify 0 to retrieve the v0 version of this structure.
		//ULONG bytesReturned;
		//if (WSAIoctl(socket, SIO_TCP_INFO, &version, sizeof(ULONG), &ti, sizeof(TCP_INFO_v0), &bytesReturned, NULL, NULL) == SOCKET_ERROR)
		//	return logger.Error(TC("WSAIoctl SIO_TCP_INFO failed (%s)"), LastErrorToText(SocketGetLastError()).data);
		//logger.Info(TC("RTT=%uÂµs  InFlight=%u  Retrans=%u  Dupacks=%u  State=%u RcvWnd=%u  RcvBuf=%u"), ti.RttUs, ti.BytesInFlight, ti.BytesRetrans, ti.DupAcksIn, ti.State, ti.RcvWnd, ti.RcvBuf);
	#else
		/*
		struct tcp_info ti = {};
		socklen_t len = sizeof(ti);
		if (getsockopt(s, IPPROTO_TCP, TCP_INFO, &ti, &len) == 0)
		{
			printf("%s  rtt=%uÂµs  unacked=%u  retrans=%u  snd_cwnd=%u  state=%u\n",
				   tag,
				   ti.tcpi_rtt, ti.tcpi_unacked, ti.tcpi_retrans,
				   ti.tcpi_snd_cwnd, ti.tcpi_state);
		}
		*/
	#endif
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
