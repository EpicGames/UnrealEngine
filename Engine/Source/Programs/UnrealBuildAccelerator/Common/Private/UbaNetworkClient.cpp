// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaNetworkClient.h"
#include "UbaConfig.h"
#include "UbaCrypto.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaEnvironment.h"
#include "UbaNetworkBackendTcp.h"
#include "UbaNetworkMessage.h"
#include <stdlib.h>
#include <stdio.h>

namespace uba
{
	void NetworkClientCreateInfo::Apply(Config& config, const tchar* tableName)
	{
		const ConfigTable* tablePtr = config.GetTable(tableName);
		if (!tablePtr)
			return;
		const ConfigTable& table = *tablePtr;
		table.GetValueAsU32(recvBufferSize, TC("RecvBufferSize"));
		table.GetValueAsU32(sendBufferSize, TC("SendBufferSize"));
		table.GetValueAsBool(useThreadedSend, TC("UseThreadedSend"));
	}

	struct NetworkClient::Connection
	{
		Connection(NetworkClient& o) : owner(o), disconnectedEvent(EventResetType_Manual) {}
		NetworkClient& owner;
		void* backendConnection = nullptr;
		Atomic<u32> connected = 0;
		Event disconnectedEvent;
		NetworkBackend* backend = nullptr;
		Guid cryptoIV;

		#if UBA_TRACK_NETWORK_TIMES
		Atomic<u64> lastSendTime = 0;
		Atomic<u64> lastHeaderRecvTime = 0;
		Atomic<u64> lastBodyRecvTime = 0;
		#endif
	};

	NetworkClient::NetworkClient(bool& outCtorSuccess, const NetworkClientCreateInfo& info, const tchar* name)
	: WorkManagerImpl(info.workerCount == ~0u ? GetLogicalProcessorCount() : info.workerCount, TC("UbaWrk/NetClnt"))
	,	m_logWriter(info.logWriter)
	,	m_logger(info.logWriter, SetGetPrefix(name))
	,	m_isConnected(EventResetType_Manual)
	,	m_isOrWasConnected(EventResetType_Manual)
	{
		outCtorSuccess = true;

		u32 fixedSendSize = Max(info.sendSize, (u32)(4*1024));
		fixedSendSize = Min(fixedSendSize, (u32)(SendMaxSize));
		if (info.sendSize != fixedSendSize)
			m_logger.Detail(TC("Adjusted msg size to %u to stay inside limits"), fixedSendSize);
		m_sendSize = fixedSendSize;
		m_receiveTimeoutSeconds = info.receiveTimeoutSeconds;
		m_useMessagePriority = info.useMessagePriority;
		m_useThreadedSend = info.useThreadedSend;
		m_recvBufferSize = info.recvBufferSize;
		m_sendBufferSize = info.sendBufferSize;

		m_connectionsIt = m_connections.end();

		if (info.cryptoKey128)
		{
			m_cryptoKey = Crypto::CreateKey(m_logger, info.cryptoKey128);
			if (m_cryptoKey == InvalidCryptoKey)
				outCtorSuccess = false;
		}

		#if UBA_TRACK_NETWORK_TIMES
		m_startTime = GetTime();
		#endif
	}

	NetworkClient::~NetworkClient()
	{
		UBA_ASSERTF(m_connections.empty(), TC("Client still has connections (%llu). %s"), m_connections.size(), m_isDisconnecting ? TC("") : TC("Disconnect has not been called"));

		if (m_cryptoKey)
			Crypto::DestroyKey(m_cryptoKey);
	}

	bool NetworkClient::Connect(NetworkBackend& backend, const tchar* ip, u16 port, bool* timedOut)
	{
		return backend.Connect(m_logger, ip, [&](void* connection, const sockaddr& remoteSocketAddr, bool* timedOut)
			{
				return AddConnection(backend, connection, timedOut);
			}, port, timedOut);
	}

	bool NetworkClient::AddConnection(NetworkBackend& backend, void* backendConnection, bool* timedOut)
	{
		struct RecvContext
		{
			RecvContext(NetworkClient& c, NetworkBackend& b, void* bc) : client(c), backend(b), backendConnection(bc), recvEvent(EventResetType_Manual), exitScopeEvent(EventResetType_Manual)
			{
			}

			~RecvContext()
			{
				Wait();
			}

			bool Wait()
			{
				if (error)
					backend.Shutdown(backendConnection);
				else
					backend.SetRecvTimeout(backendConnection, client.m_receiveTimeoutSeconds*1000, nullptr, nullptr);
				return exitScopeEvent.IsSet(~0u) && error == 0;
			}

			NetworkClient& client;
			NetworkBackend& backend;
			void* backendConnection;
			Event recvEvent;
			Event exitScopeEvent;
			Atomic<u8> error = 255;
		};

		backend.SetRecvTimeout(backendConnection, 10*1000, nullptr, nullptr); // 10 seconds timeout during adding connection. RecvContext set it to right time in ctor

		if (IsWindows && !IsRunningWine())
		{
			if (m_recvBufferSize)
				backend.SetRecvBufferSize(backendConnection, m_recvBufferSize);
			if (m_sendBufferSize)
				backend.SetSendBufferSize(backendConnection, m_sendBufferSize);
		}

		if (m_useThreadedSend)
			backend.SetThreadedSend(backendConnection);

		RecvContext rc(*this, backend, backendConnection);

		// The only way out of this function is to get a call to one of the below callbacks since exitScopeEvent must be set.

		backend.SetDisconnectCallback(backendConnection, &rc, [](void* context, const Guid& connectionUid, void* connection)
			{
				auto& rc = *(RecvContext*)context;
				if (rc.error == 0)
					rc.error = 4;
				rc.recvEvent.Set();
				rc.exitScopeEvent.Set();
			});

		u32 recvSize = 1 + sizeof(Guid);
		if (m_cryptoKey !=InvalidCryptoKey)
			recvSize += sizeof(Guid);

		backend.SetRecvCallbacks(backendConnection, &rc, recvSize, [](void* context, const Guid& connectionUid, u8* headerData, void*& outBodyContext, u8*& outBodyData, u32& outBodySize)
			{
				auto& rc = *(RecvContext*)context;
				NetworkClient& client = rc.client;

				rc.error = *headerData;
				Guid serverUid = *(Guid*)(headerData+1);
				Guid cryptoIV;
				
				CryptoKey key = client.m_cryptoKey;
				if (key != InvalidCryptoKey)
					cryptoIV = *(Guid*)(headerData+17);

				if (serverUid == Guid())
					rc.error = 5;

				if (!rc.error)
				{
					SCOPED_FUTEX(client.m_serverUidLock, lock);
					if (client.m_serverUid == Guid())
						client.m_serverUid = serverUid;
					else if (client.m_serverUid != serverUid) // Seems like two different servers tried to connect to this client.. keep the first one and ignore the others
						rc.error = 6;
				}

				if (!rc.error)
				{
					if (key != InvalidCryptoKey)
					{
						Guid initVector;
						if (!Crypto::Decrypt(client.m_logger, key, (u8*)&cryptoIV, sizeof(cryptoIV), initVector))
							return false;
					}

					if (!client.ConnectedCallback(rc.backend, rc.backendConnection, cryptoIV))
						rc.error = 4;
				}

				if (rc.error != 0)
					return false;

				rc.recvEvent.Set();
				rc.exitScopeEvent.Set();
				return true;

			}, nullptr, TC("Connecting"));


		StackBinaryWriter<1024> handshakeData;
		if (m_cryptoKey != InvalidCryptoKey)
		{
			handshakeData.WriteByte(1);
			// If we have a crypto key we start by sending a predefined 128 bytes blob that is encrypted.
			// If server decrypt it to the same blob, we're good on that part
			u8* encryptedBuffer = handshakeData.AllocWrite(sizeof(EncryptionHandshakeString));//[1024];
			memcpy(encryptedBuffer, EncryptionHandshakeString, sizeof(EncryptionHandshakeString));
			Guid initVector;
			if (!Crypto::Encrypt(m_logger, m_cryptoKey, encryptedBuffer, sizeof(EncryptionHandshakeString), initVector))
				return false;
		}

		handshakeData.WriteU32(SystemNetworkVersion);
		handshakeData.WriteBytes(&m_uid, sizeof(m_uid));

		NetworkBackend::SendContext sendContext;
		if (!backend.Send(m_logger, backendConnection, handshakeData.GetData(), u32(handshakeData.GetPosition()), sendContext, TC("Handshake")))
			return false;

		if (!rc.recvEvent.IsSet(20*1000)) // This can not happen. Since both callbacks are using rc we can't leave this function until we know we are not in the callbacks
			return m_logger.Info(TC("Timed out after 20 seconds waiting for connection response from server.")).ToFalse();

		m_isOrWasConnected.Set();

		if (rc.error == 1) // Bad version
			return m_logger.Error(TC("Version mismatch with server"));

		if (rc.error == 2)
			return m_logger.Error(TC("Server failed to receive client uid"));

		if (rc.error == 3)
		{
			if (!timedOut)
				return m_logger.Error(TC("Server does not allow new clients"));
			*timedOut = true;
			Sleep(1000); // Kind of ugly, but we want the retry-clients to keep retrying so we pretend it is a timeout
			return false;
		}

		if (rc.error == 4)
		{
			if (!timedOut)
				return m_logger.Error(TC("Server disconnected"));
			*timedOut = true;
			Sleep(1000); // Kind of ugly, but we want the retry-clients to keep retrying so we pretend it is a timeout
			return false;
		}

		if (rc.error == 5)
			return m_logger.Error(TC("A connection to a server with uid zero was requested."));

		if (rc.error == 6)
			return m_logger.Warning(TC("A connection to a server with different uid was requested. Ignore"));

		SCOPED_FUTEX(m_onConnectedFunctionsLock, lock);

		if (m_connectionCount.fetch_add(1) != 0)
			return true;

		for (auto& f : m_onConnectedFunctions)
			f();
		m_isConnected.Set();
		lock.Leave();

		return rc.Wait();
	}

	constexpr u32 SendHeaderSize = 6;
	constexpr u32 ReceiveHeaderSize = 5;

	void NetworkClient::DisconnectCallback(void* context, const Guid& connectionUid, void* connection)
	{
		auto& c = *(Connection*)context;
		c.owner.OnDisconnected(c, 1);
		c.disconnectedEvent.Set();
	}

	bool NetworkClient::ConnectedCallback(NetworkBackend& backend, void* backendConnection, const Guid& cryptoIV)
	{
		SCOPED_WRITE_LOCK(m_connectionsLock, lock);
		if (m_isDisconnecting)
			return false;
		m_connections.emplace_back(*this);
		Connection* connection = &m_connections.back();
		connection->backendConnection = backendConnection;
		connection->connected = 1;
		connection->backend = &backend;
		connection->cryptoIV = cryptoIV;

		#if 0 // Haven't really seen any difference by setting priority on socket. Will need more testing
		if (m_useMessagePriority && m_connections.size() > 3)
		{
			auto& firstConnection = m_connections.front();
			firstConnection.backend->SetPriority(firstConnection.backendConnection);
		}
		#endif

		SCOPED_FUTEX(m_connectionsItLock, l); // Take this lock to make sure callbacks are set before connection is used
		m_connectionsIt = --m_connections.end();

		m_logger.Detail(TC("Connected to server... (0x%p)"), backendConnection);
		lock.Leave();

		backend.SetRecvTimeout(backendConnection, m_receiveTimeoutSeconds*1000, nullptr, nullptr);

		backend.SetDisconnectCallback(backendConnection, connection, DisconnectCallback);
		backend.SetRecvCallbacks(backendConnection, connection, ReceiveHeaderSize, ReceiveResponseHeader, ReceiveResponseBody, TC("ReceiveMessageResponse"));
		return true;
	}

	bool NetworkClient::ReceiveResponseHeader(void* context, const Guid& connectionUid, u8* headerData, void*& outBodyContext, u8*& outBodyData, u32& outBodySize)
	{
		auto& connection = *(Connection*)context;
		auto& client = connection.owner;

		#if UBA_TRACK_NETWORK_TIMES
		connection.lastHeaderRecvTime = GetTime();
		#endif

		u16 messageId = u16(headerData[0] << 8) | u16((*(u32*)(headerData + 1) & 0xff000000) >> 24);
		u32 messageSize = *(u32*)(headerData + 1) & 0x00FFFFFF;

		if (messageSize == MessageKeepAliveSize) // Keep alive message
		{
			u8 data[6] = { SystemMessageType_KeepAlive, 0, 1, 0, 0, 0 };
			NetworkBackend::SendContext sendContext;
			return connection.backend->Send(client.m_logger, connection.backendConnection, data, sizeof(data), sendContext, TC("KeepAliveNoResponse"));
		}

		NetworkMessage* msg;
		{
			LOG_STALL_SCOPE(client.m_logger, 5, TC("Took more than %s to get message from id"));
			SCOPED_READ_LOCK(client.m_activeMessagesLock, lock);
			if (!connection.connected)
				return false;
			if (messageId >= client.m_activeMessages.size())
				return client.m_logger.Error(TC("Message id %u is higher than max %u"), messageId, u32(client.m_activeMessages.size()));
			msg = client.m_activeMessages[messageId];
		}

		if (!msg)
			return false;

		if (messageSize == MessageErrorSize || messageSize == MessageErrorSize - ReceiveHeaderSize)  // ReceiveHeaderSize is removed from size in server send
		{
			msg->m_error = 1;
			msg->Done();
			return true;
		}
		
		if (!messageSize)
		{
			++client.m_recvCount;
			msg->Done();
			return true;
		}

		if (messageSize > msg->m_responseCapacity)
		{
			u8 serviceIdAndMessageType = msg->m_sendWriter->GetData()[0];
			u8 serviceId = serviceIdAndMessageType >> 6;
			u8 messageType = serviceIdAndMessageType & 0b111111;
			client.m_logger.Error(TC("Message size is %u but reader capacity is only %u (serviceId %u, messageType %u)"), messageSize, msg->m_responseCapacity, u32(serviceId), u32(messageType));
			msg->m_error = 1;
			msg->Done();
			return false;
		}

		msg->m_responseSize = messageSize;

		outBodyContext = msg;
		outBodyData = (u8*)msg->m_response;
		outBodySize = messageSize;

		++client.m_recvCount;
		client.m_recvBytes += ReceiveHeaderSize + messageSize;


		return true;
	}

	bool NetworkClient::ReceiveResponseBody(void* context, bool recvError, u8* headerData, void* bodyContext, u8* bodyData, u32 bodySize)
	{
		auto& msg = *(NetworkMessage*)bodyContext;
		if (recvError)
		{
			// It could be that message already had done called and went out of scope when we get this since there can be tcp buffer with multiple messages in it
			auto& connection = *(Connection*)context;
			auto& client = connection.owner;
			bool found = false;
			SCOPED_WRITE_LOCK(client.m_activeMessagesLock, lock);
			for (auto m : client.m_activeMessages)
				found = m == &msg;
			lock.Leave();
			if (!found)
			{
				if (client.m_isDisconnecting)
					return false;
				u16 messageId = u16(headerData[0] << 8) | u16((*(u32*)(headerData + 1) & 0xff000000) >> 24);
				u32 messageSize = *(u32*)(headerData + 1) & 0x00FFFFFF;
				return client.m_logger.Error(TC("NetworkClient got receive response body for message that does not exist (Id: %u, Size: %u)"), u32(messageId), messageSize);
			}
			msg.m_error = 2;
		}
		#if UBA_TRACK_NETWORK_TIMES
		if (msg.m_connection)
			msg.m_connection->lastBodyRecvTime = GetTime();
		#endif

		msg.Done();
		return true;
	}

	void NetworkClient::Disconnect(bool flushWork)
	{
		auto fg = MakeGuard([&]()
			{
				if (!flushWork)
					return;
				if (FlushWork(30*1000))
					return;
				m_logger.Error(TC("NetworkClient has waited 30 seconds for all work to finish... something is stuck"));
				PrintAllCallstacks(m_logger);
			});

		{
			SCOPED_WRITE_LOCK(m_connectionsLock, connectionsLock);
			if (m_isDisconnecting)
			{
				// We can't return straight away here if m_connections is not empty since it might actively disconnect on another thread
				while (!m_connections.empty())
				{
					connectionsLock.Leave();
					Sleep(10);
					connectionsLock.Enter();
				}
				return;
			}

			m_isDisconnecting = true;
			for (auto& c : m_connections)
				if (c.connected)
					c.backend->Shutdown(c.backendConnection);
		}

		for (auto& c : m_connections)
			c.disconnectedEvent.IsSet(~0u);

		if (m_connectionCount != 0)
		{
			// I think this could happen if m_isDisconnecting is set while a connection is in progress.
			// it might early out in ConnectedCallback which will prevent connection->connected from being set to 1
			// which means that NetworkClient::OnDisconnected will not enter the code that decrement m_connectionCount
			m_logger.Info(TC("Should be zero connections here. %llu still active (was Disconnect called while a connect was being made?)"), m_connectionCount.load());

			SCOPED_READ_LOCK(m_onDisconnectedFunctionsLock, lock);
			for (auto& f : m_onDisconnectedFunctions)
				f();
		}

		{
			SCOPED_WRITE_LOCK(m_connectionsLock, lock2);
			for (auto& c : m_connections)
				c.backend->DeleteConnection(c.backendConnection);
			m_connections.clear();
			m_connectionsIt = m_connections.end();
		}
	}

	bool NetworkClient::StartListen(NetworkBackend& backend, u16 port, const tchar* ip)
	{
		return backend.StartListen(m_logger, port, ip, [&](void* connection, const sockaddr& remoteSockAddr)
			{
				return AddConnection(backend, connection, nullptr);
			});
	}

	bool NetworkClient::SetConnectionCount(u32 count)
	{
		StackBinaryWriter<64> writer;
		NetworkMessage msg(*this, SystemServiceId, SystemMessageType_SetConnectionCount, writer); // Connection count
		writer.WriteU32(count);
		return msg.Send();
	}

	bool NetworkClient::SendKeepAlive()
	{
		StackBinaryWriter<64> writer;
		NetworkMessage msg(*this, SystemServiceId, SystemMessageType_KeepAlive, writer);
		return msg.Send();
	}

	bool NetworkClient::FetchConfig(Config& config)
	{
		StackBinaryWriter<64> writer;
		NetworkMessage msg(*this, SystemServiceId, SystemMessageType_FetchConfig, writer);
		writer.WriteByte(0); // Need to have a body
		StackBinaryReader<SendMaxSize> reader;
		if (!msg.Send(reader))
			return false;
		u64 textLen = reader.Read7BitEncoded();
		return config.LoadFromText(m_logger, (const char*)reader.GetPositionData(), textLen);
	}

	bool NetworkClient::IsConnected(u32 waitTimeoutMs)
	{
		return m_isConnected.IsSet(waitTimeoutMs);
	}

	bool NetworkClient::IsOrWasConnected(u32 waitTimeoutMs)
	{
		return m_isOrWasConnected.IsSet(waitTimeoutMs);
	}

	void NetworkClient::PrintSummary(Logger& logger)
	{
		SCOPED_READ_LOCK(m_connectionsLock, lock);
		u32 connectionsCount = u32(m_connections.size());
		lock.Leave();

		logger.Info(TC("  ----- Uba client stats summary ------"));
		logger.Info(TC("  SendTotal          %8u %9s"), m_sendTimer.count.load(), TimeToText(m_sendTimer.time).str);
		logger.Info(TC("     Bytes                    %9s"), BytesToText(m_sendBytes).str);
		logger.Info(TC("  RecvTotal          %8u %9s"), m_recvCount.load(), BytesToText(m_recvBytes).str);
		if (m_cryptoKey)
		{
			logger.Info(TC("  EncryptTotal       %8u %9s"), m_encryptTimer.count.load(), TimeToText(m_encryptTimer.time).str);
			logger.Info(TC("  DecryptTotal       %8u %9s"), m_decryptTimer.count.load(), TimeToText(m_decryptTimer.time).str);
		}
		logger.Info(TC("  MaxActiveMessages  %8u"), m_activeMessageIdMax);
		logger.Info(TC("  Connections        %8u"), connectionsCount);
		logger.Info(TC("  SendSize Set/Max  %9s %9s"), BytesToText(m_sendSize).str, BytesToText(SendMaxSize).str);
		logger.Info(TC(""));
	}

	void NetworkClient::ValidateNetwork(Logger& logger, bool full)
	{
		UnorderedMap<NetworkBackend*, Vector<void*>> backends;
		{
			LogStallScope lss(logger, LogEntryType_Info, 1, TC("  Connections lock took %s"));
			SCOPED_WRITE_LOCK(m_connectionsLock, lock);
			lss.Leave();

			u32 connectionIndex = 0;
			for (auto& c : m_connections)
			{
				if (full)
				{
					#if UBA_TRACK_NETWORK_TIMES
					logger.Info(TC("  Connection %u - LastSend %s, LastHeaderRecv: %s, LastBodyRecv: %s"), connectionIndex++, TimeToText(c.lastSendTime - m_startTime).str, TimeToText(c.lastHeaderRecvTime - m_startTime).str, TimeToText(c.lastBodyRecvTime - m_startTime).str);
					#else
					logger.Info(TC("  Connection %u"), connectionIndex++);
					#endif
				}
				backends[c.backend].push_back(c.backendConnection);
			}
		}

		{
			LogStallScope lss(logger, LogEntryType_Info, 1, TC("  ConnectionsIterator lock took %s"));
			SCOPED_FUTEX(m_connectionsItLock, lock);
		}

		if (full)
		{
			LogStallScope lss(logger, LogEntryType_Info, 1, TC("  ActiveMessages lock took %s"));
			SCOPED_WRITE_LOCK(m_activeMessagesLock, lock);
			lss.Leave();
			logger.Info(TC("  Active messages"));
			u64 now = GetTime();
			for (auto m : m_activeMessages)
				if (m)
				{
					u64 sendTime = 0;
					#if UBA_TRACK_NETWORK_TIMES
					sendTime = m->m_sendTime;
					#endif

					logger.Info(TC("     %s (%u): %s"), MessageToString(m->GetServiceId(), m->GetMessageType()).data, u32(m->m_id), TimeToText(now - sendTime).str);
				}
		}

		for (auto& kv : backends)
			kv.first->Validate(logger, kv.second, full);
	}

	void NetworkClient::RegisterOnConnected(const OnConnectedFunction& function)
	{
		SCOPED_FUTEX(m_onConnectedFunctionsLock, lock);
		m_onConnectedFunctions.push_back(function);
		if (!m_isConnected.IsSet(0))
			return;
		lock.Leave();
		function();
	}

	void NetworkClient::RegisterOnDisconnected(const OnDisconnectedFunction& function)
	{
		SCOPED_WRITE_LOCK(m_onDisconnectedFunctionsLock, lock);
		m_onDisconnectedFunctions.push_back(function);
	}

	void NetworkClient::RegisterOnVersionMismatch(OnVersionMismatchFunction&& function)
	{
		m_versionMismatchFunction = std::move(function);
	}

	void NetworkClient::InvokeVersionMismatch(const CasKey& exeKey, const CasKey& dllKey)
	{
		if (m_versionMismatchFunction)
			m_versionMismatchFunction(exeKey, dllKey);
	}


	u64 NetworkClient::GetMessageHeaderSize()
	{
		return SendHeaderSize;
	}

	u64 NetworkClient::GetMessageReceiveHeaderSize()
	{
		return ReceiveHeaderSize;
	}

	u64 NetworkClient::GetMessageMaxSize()
	{
		return m_sendSize;
	}

	NetworkBackend* NetworkClient::GetFirstConnectionBackend()
	{
		SCOPED_READ_LOCK(m_connectionsLock, connectionLock);
		if (m_connections.empty())
			return nullptr;
		return m_connections.front().backend;
	}

	void NetworkClient::TraverseConnections(const Function<void(NetworkBackend&, void*)>& func)
	{
		SCOPED_WRITE_LOCK(m_connectionsLock, lock);
		for (auto& c : m_connections)
			func(*c.backend, c.backendConnection);
	}

	void NetworkClient::OnDisconnected(Connection& connection, u32 reason)
	{
		if (connection.connected.exchange(0) == 1)
		{
			m_logger.Detail(TC("Disconnected from server... (0x%p) (%u)"), connection.backendConnection, reason);

			connection.backend->Shutdown(connection.backendConnection);

			if (m_connectionCount.fetch_sub(1) == 1)
			{
				m_isConnected.Reset();
				SCOPED_READ_LOCK(m_onDisconnectedFunctionsLock, lock);
				for (auto& f : m_onDisconnectedFunctions)
					f();
			}
		}

		SCOPED_WRITE_LOCK(m_activeMessagesLock, lock);
		for (auto m : m_activeMessages)
		{
			if (m && m->m_connection == &connection)
			{
				m->m_error = 3;
				m->Done(false);
			}
		}
	}

	bool NetworkClient::Send(NetworkMessage& message, void* response, u32 responseCapacity, bool async)
	{
		SCOPED_READ_LOCK(m_connectionsLock, connectionLock);
		SCOPED_FUTEX(m_connectionsItLock, connectionItLock);
		if (m_connectionsIt == m_connections.end())
		{
			if (m_isDisconnecting)
				message.m_error = 11;
			else if (!m_connections.empty())
				message.m_error = 12; // should never happen
			else
				message.m_error = 6;
			if (async)
				message.Done(false);
			return false;
		}

		// Skip connections that has disconnected
		Connection* connectionPtr = nullptr;
		
		if (m_useMessagePriority)
		{
			Connection* firstConnectionPtr = &m_connections.front();
			if (message.m_priority == HasPriority && firstConnectionPtr->connected)
			{
				connectionPtr = firstConnectionPtr;
			}
			else
			{
				connectionPtr = &*m_connectionsIt;
				u32 loopCount = u32(m_connections.size()) - 1;
				while (loopCount--)
				{
					++m_connectionsIt;
					if (m_connectionsIt == m_connections.end())
					{
						m_connectionsIt = m_connections.begin();
						++m_connectionsIt;
					}
					if (connectionPtr->connected)
						break;
					connectionPtr = &*m_connectionsIt;
				}
			}
		}
		else
		{
			connectionPtr = &*m_connectionsIt;
			Connection* connectionPtrStart = connectionPtr;
			while (true)
			{
				++m_connectionsIt;
				if (m_connectionsIt == m_connections.end())
					m_connectionsIt = m_connections.begin();
				if (connectionPtr->connected)
					break;
				connectionPtr = &*m_connectionsIt;
				if (connectionPtr == connectionPtrStart)
					break;
			}
		}

		connectionItLock.Leave();
		connectionLock.Leave();

		Connection& connection = *connectionPtr;

		message.m_response = response;
		message.m_responseCapacity = responseCapacity;
		message.m_connection = &connection;
		message.m_cryptoIV = connection.cryptoIV;

		BinaryWriter& writer = *message.m_sendWriter;

		u16 messageId = 0;
		EventSlim gotResponse;


		if (response)
		{
			if (!async)
			{
				if (!gotResponse.Create(EventResetType_Manual))
				{
					m_logger.Error(TC("Failed to create event, this should not happen?!?"));
					message.m_error = 13;
					OnDisconnected(connection, 13);
					return false;
				}
			}

			LOG_STALL_SCOPE(m_logger, 5, TC("Took more than %s to get message id"));

			while (true)
			{
				SCOPED_WRITE_LOCK(m_activeMessagesLock, lock);
				if (m_availableMessageIds.empty())
				{
					if (!connection.connected)
					{
						message.m_error = 7;
						if (async)
							message.Done(false);
						return false;
					}

					if (m_activeMessageIdMax == 65534)
					{
						lock.Leave();
						m_logger.Info(TC("Reached max limit of active message ids (65534). Waiting 1 second"));
						Sleep(100u + u32(rand()) % 900u);
						continue;
					}
					messageId = m_activeMessageIdMax++;

					if (m_activeMessages.size() < m_activeMessageIdMax)
						m_activeMessages.resize(size_t(m_activeMessageIdMax) + 1024);
				}
				else
				{
					messageId = m_availableMessageIds.back();
					m_availableMessageIds.pop_back();
				}

				UBA_ASSERT(!m_activeMessages[messageId]);
				m_activeMessages[messageId] = &message;

				message.m_id = messageId;
				message.m_sendContext.flags = NetworkBackend::SendFlags_ExternalWait;
				if (!async)
				{
					UBA_ASSERT(!message.m_doneFunc);
					message.m_doneUserData = &gotResponse;
					message.m_doneFunc = [](bool error, void* userData) { ((EventSlim*)userData)->Set(); };
				}
				break;
			}
		}

		UBA_ASSERT(messageId < 65535);

		u32 sendSize = u32(writer.GetPosition());
		u8* data = writer.GetData();
		data[1] = messageId >> 8;
		u32 dataSize = sendSize - 6;
		UBA_ASSERTF(dataSize || data[0] == 1, TC("NetworkMessage must have data size of at least 1."));
		*(u32*)(data + 2) = dataSize | u32(messageId) << 24;

		//m_logger.Debug(TC("Send: %u, %u, %u, %u"), data[0], data[1], data[2], sendSize - 7);

		u32 bodySize = sendSize - SendHeaderSize;
		if (m_cryptoKey && bodySize)
		{
			TimerScope ts(m_encryptTimer);
			Guid iv = connection.cryptoIV;
			if (!Crypto::Encrypt(m_logger, m_cryptoKey, data + SendHeaderSize, bodySize, iv))
			{
				message.m_error = 8;
				OnDisconnected(connection, 8);
				return false;
			}
		}


		m_sendBytes += sendSize;

		{
			TimerScope ts(m_sendTimer);
			#if UBA_TRACK_NETWORK_TIMES
			connection.lastSendTime = ts.start;
			message.m_sendTime = ts.start;
			#endif
			if (!connection.backend->Send(m_logger, connection.backendConnection, data, sendSize, message.m_sendContext, TC("Message")))
			{
				message.m_error = 9;
				OnDisconnected(connection, 9);
				return false;
			}
		}

		// Theory for long timeouts waiting for gotResponse..
		// There could have been a disconnect that happened just before the message above was added to m_activeMessages (thus calling Done on all active messages)
		// But still backend Send succeeded on same socket... this would mean that gotResponse is never set and we get the timeout
		// We need to explicitly check here so we are still connected and if not, call OnDisconnected
		if (!connection.connected)
		{
			message.m_error = 9;
			OnDisconnected(connection, 10);
			return false;
		}

		if (async)
			return true;

		if (response)
		{
			u64 waitStart = GetTime();
			u32 timeoutMs = 10 * 60 * 1000;
			if (!gotResponse.IsSet(timeoutMs))
			{
				m_logger.Warning(TC("Timed out after %s waiting for message response from server. (%s)"), TimeToText(GetTime() - waitStart, true).str, MessageToString(message.GetServiceId(), message.GetMessageType()).data);
				message.m_error = 4;
				OnDisconnected(connection, 4);
			}
			else if (m_cryptoKey && !message.m_error && message.m_responseSize)
			{
				TimerScope ts(m_decryptTimer);
				Guid iv = connection.cryptoIV;
				if (!Crypto::Decrypt(m_logger, m_cryptoKey, (u8*)message.m_response, message.m_responseSize, iv))
				{
					message.m_error = 5;
					OnDisconnected(connection, 5);
				}
			}
		}
		return !message.m_error;
	}

	const tchar* NetworkClient::SetGetPrefix(const tchar* originalPrefix)
	{
		CreateGuid(m_uid);
		StringBuffer<512> b;
		b.Appendf(TC("%s (%s)"), originalPrefix, GuidToString(m_uid).str);
		m_prefix = b.data;
		return m_prefix.c_str();
	}

	NetworkMessage::NetworkMessage(NetworkClient& client, u8 serviceId, u8 messageType, BinaryWriter& sendWriter, MessagePriority priority)
	{
		Init(client, serviceId, messageType, sendWriter, priority);
	}

	NetworkMessage::~NetworkMessage()
	{
		UBA_ASSERT(!m_id);
	}

	void NetworkMessage::Init(NetworkClient& client, u8 serviceId, u8 messageType, BinaryWriter& sendWriter, MessagePriority priority)
	{
		m_client = &client;
		m_sendWriter = &sendWriter;
		m_priority = priority;

		// Header (SendHeaderSize):
		// 1 byte    - 2 bits for serviceid, 6 bits for messagetype
		// 2 byte    - message id
		// 3 byte    - message size
		UBA_ASSERT(sendWriter.GetPosition() == 0);
		UBA_ASSERT((serviceId & 0b11) == serviceId);
		UBA_ASSERT((messageType & 0b111111) == messageType);
		u8* data = sendWriter.AllocWrite(SendHeaderSize);
		data[0] = u8(serviceId << 6) | messageType;
	}

	bool NetworkMessage::Send()
	{
		return m_client->Send(*this, nullptr, 0, false);
	}

	bool NetworkMessage::Send(Timer& outTimer)
	{
		TimerScope ts(outTimer);
		return Send();
	}

	bool NetworkMessage::Send(BinaryReader& response)
	{
		if (!m_client->Send(*this, (u8*)response.GetPositionData(), u32(response.GetLeft()), false))
			return false;
		response.SetSize(response.GetPosition() + m_responseSize);
		return true;
	}

	bool NetworkMessage::Send(BinaryReader& response, Timer& outTimer)
	{
		TimerScope ts(outTimer);
		return Send(response);
	}

	bool NetworkMessage::SendAsync(BinaryReader& response, DoneFunc* func, void* userData)
	{
		UBA_ASSERT(!m_doneFunc);
		m_doneFunc = func;
		m_doneUserData = userData;
		return m_client->Send(*this, (u8*)response.GetPositionData(), u32(response.GetLeft()), true);
	}

	bool NetworkMessage::ProcessAsyncResults(BinaryReader& response)
	{
		if (m_error)
			return false;

		if (m_client->m_cryptoKey)
		{
			UBA_ASSERT(!response.GetPosition());
			TimerScope ts(m_client->m_decryptTimer);
			if (!Crypto::Decrypt(m_client->m_logger, m_client->m_cryptoKey, (u8*)m_response, m_responseSize, m_cryptoIV))
			{
				m_error = 10;
				return false;
			}
		}
		response.SetSize(response.GetPosition() + m_responseSize);
		return true;
	}

	u8 NetworkMessage::GetServiceId()
	{
		return m_sendWriter ? m_sendWriter->GetData()[0] >> 6 : 0;
	}

	u8 NetworkMessage::GetMessageType()
	{
		return m_sendWriter ? m_sendWriter->GetData()[0] & 63 : 0;
	}

	void NetworkMessage::Done(bool shouldLock)
	{
		bool hasId = false;
		auto returnId = [&]()
			{
				if (m_id)
				{
					m_client->m_availableMessageIds.push_back(m_id);
					m_client->m_activeMessages[m_id] = nullptr;
					m_id = 0;
					hasId = true;
				}
			};

		if (shouldLock)
		{
			SCOPED_WRITE_LOCK(m_client->m_activeMessagesLock, lock);
			returnId();
		}
		else
		{
			returnId();
		}
		if (hasId)
			m_doneFunc(m_error != 0, m_doneUserData);
	}
}
