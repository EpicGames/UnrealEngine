// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaNetworkServer.h"
#include "UbaBottleneck.h"
#include "UbaConfig.h"
#include "UbaCrypto.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaEnvironment.h"
#include "UbaPlatform.h"
#include "UbaTrace.h"

namespace uba
{
	void NetworkServerCreateInfo::Apply(Config& config, const tchar* tableName)
	{
		const ConfigTable* tablePtr = config.GetTable(tableName);
		if (!tablePtr)
			return;
		const ConfigTable& table = *tablePtr;
		table.GetValueAsU32(recvBufferSize, TC("RecvBufferSize"));
		table.GetValueAsU32(sendBufferSize, TC("SendBufferSize"));
		table.GetValueAsU32(workerCount, TC("WorkerCount"));
		table.GetValueAsU32(sendSize, TC("SendSize"));
		table.GetValueAsU32(receiveTimeoutSeconds, TC("ReceiveTimeoutSeconds"));
		table.GetValueAsBool(logConnections, TC("LogConnections"));
		table.GetValueAsBool(useKeepAlive, TC("UseKeepAlive"));
		table.GetValueAsBool(useMessagePriority, TC("UseMessagePriority"));
		table.GetValueAsBool(useThreadedSend, TC("UseThreadedSend"));
	}

	struct NetworkServer::Client
	{
	public:
		Client(const Guid& uid_, u32 id_) : uid(uid_), id(id_) {}
		Guid uid;
		u32 id;
		u32 queuedConnectionCount = 0;
		u32 storageVersion = 0;
		Atomic<u32> refCount = 0;
		Atomic<u32> connectionCount = 0;
		Atomic<u64> sendBytes = 0;
		Atomic<u64> recvBytes = 0;
	};

	struct NetworkServer::WorkerContext
	{
		WorkerContext(NetworkServer& s) : server(s), workAvailable(EventResetType_Auto)
		{
			writeMemSize = server.m_sendSize;
			writeMem = new u8[writeMemSize];
		}

		~WorkerContext()
		{
			delete[] writeMem;
		}

		NetworkServer& server;
		EventSlim workAvailable;
		u8* writeMem = nullptr;
		u32 writeMemSize = 0;

		Vector<u8> buffer;
		Connection* connection = nullptr;
		u32 dataSize = 0;
		u8 serviceId = 0;
		u8 messageType = 0;
		u16 id = 0;
	};

	class NetworkServer::Worker
	{
	public:
		Worker() {}
		~Worker()
		{
			UBA_ASSERT(!m_inUse);
			m_context->connection = nullptr;
			m_loop = false;
			m_context->workAvailable.Set();
			m_thread.Wait();
			delete m_context;
			m_context = nullptr;
		}

		void Start(NetworkServer& server)
		{
			m_context = new WorkerContext(server);
			m_loop = true;
			m_thread.Start([&]() { ThreadWorker(server); return 0; }, TC("UbaWrkNetwSrv"));
		}

		void Stop(NetworkServer& server)
		{
			m_loop = false;
			SCOPED_FUTEX(server.m_availableWorkersLock, lock);
			while (m_inUse)
			{
				m_context->workAvailable.Set();
				lock.Leave();
				if (m_thread.Wait(5))
					break;
				lock.Enter();
			}
		}

		void ThreadWorker(NetworkServer& server);
		void Update(WorkerContext& context);
		void DoAdditionalWorkAndSignalAvailable(NetworkServer& server);

		Worker* m_nextWorker = nullptr;
		Worker* m_prevWorker = nullptr;

		WorkerContext* m_context = nullptr;

		Atomic<bool> m_loop = false;
		Atomic<bool> m_inUse = false;
		Thread m_thread;

		Worker(const Worker&) = delete;
	};
	thread_local NetworkServer::Worker* t_worker;

	class NetworkServer::Connection
	{
	public:
		static bool DefaultRecvTimeout(void* context, u32 timeoutMs, const tchar* recvHint, const tchar* hint)
		{
			auto& conn = *(Connection*)context;
			u32 clientId = ~0u;
			if (auto c = conn.m_client)
				clientId = c->id;
			conn.m_server.m_logger.Warning(TC("Connection %u (Client %u) timed out after %u seconds (%s%s)"), conn.m_id, clientId, timeoutMs/1000, recvHint, hint);
			return false;
		}


		Connection(NetworkServer& server, NetworkBackend& backend, void* backendConnection, const sockaddr& remoteSockAddr, bool requiresCrypto, CryptoKey cryptoKey, u32 id)
		:	m_server(server)
		,	m_backend(backend)
		,	m_remoteSockAddr(remoteSockAddr)
		,	m_cryptoKey(cryptoKey)
		,	m_disconnectCallbackCalled(EventResetType_Manual)
		,	m_id(id)
		,	m_backendConnection(backendConnection)
		{
			m_backend.SetDisconnectCallback(m_backendConnection, this, [](void* context, const Guid& connectionUid, void* connection)
				{
					auto& conn = *(Connection*)context;
					conn.Disconnect(TC("Backend"));
					conn.m_disconnectCallbackCalled.Set();
				});

			m_backend.SetDataSentCallback(m_backendConnection, this, [](void* context, u32 bytes)
				{
					auto& conn = *(Connection*)context;
					if (auto c = conn.m_client)
						c->recvBytes += bytes;
					conn.m_server.m_sendBytes += bytes;
				});

			// We expect fast response on these messages. If we don't, then something is wrong
			// This is on a new socket so there should be no contention for networking here.
			// No message communication can happen until it has gone through these first steps
			u32 initialTimeoutMs = Min(10u*1000, m_server.m_receiveTimeoutMs);
			m_backend.SetRecvTimeout(m_backendConnection, initialTimeoutMs, this, DefaultRecvTimeout);

			if (requiresCrypto)
				m_backend.SetRecvCallbacks(m_backendConnection, this, 1, ReceiveHandshakeHeader, ReceiveHandshakeBody, TC("ReceiveHandshake"));
			else
				m_backend.SetRecvCallbacks(m_backendConnection, this, 4, ReceiveVersion, nullptr, TC("ReceiveVersion"));
		}

		~Connection()
		{
			BeginStop();
			EndStop();
			if (m_cryptoKey)
				Crypto::DestroyKey(m_cryptoKey);
		}

		void Disconnect(const tchar* reason)
		{
			if (m_disconnectCalled.fetch_add(1) != 0)
				return;
			SetShouldDisconnect();
			int refCount = --m_refCount;
			if (refCount == 1) // Will disconnect in send if there are active workers
				TestDisconnect();
			//else
			//	m_server.m_logger.Detail(TC("Connection %u disconnected while it has %u active workers (%s)"), m_id, refCount - 1, reason);
		}

		bool BeginStop()
		{
			Disconnect(TC("Stop"));
			return true;
		}

		bool EndStop()
		{
			u64 startTime = GetTime();
			bool waitReported = false;
			while (m_refCount)
			{
				u64 waitTime = GetTime() - startTime;
				u64 waitTimeMs = TimeToMs(waitTime);
				if (waitTimeMs < 5*1000)
				{
					Sleep(5);
					continue;
				}

				if (!waitReported)
				{
					waitReported = true;
					m_server.m_logger.Warning(TC("Connection %u has waited %s to stop... something is stuck (Pending refs: %u)"), m_id, TimeToText(waitTime).str, m_refCount.load());
					PrintAllCallstacks(m_server.m_logger);
				}

				if (waitTimeMs > 60*1000) // Wait 60 seconds
					return false;

				Sleep(200);
			}

			if (waitReported)
				m_server.m_logger.Info(TC("Connection %u wait for active workers ended after %s"), m_id, TimeToText(GetTime() - startTime).str);

			if (m_disconnectCallbackCalled.IsSet(30*1000)) // This should never time out!
				return true;

			m_backend.SetDisconnectCallback(m_backendConnection, nullptr, [](void* context, const Guid& connectionUid, void* connection) {});
			m_backend.SetRecvCallbacks(m_backendConnection, nullptr, 0, [](void*, const Guid&, u8*, void*&, u8*&, u32&) { return false; }, nullptr, TC("Stopped"));
			m_server.m_logger.Info(TC("Disconnect callback event timed out. This should never happen!!"));
			#if UBA_DEBUG
			PrintAllCallstacks(m_server.m_logger);
			#endif
			return true;
		}

		bool SendInitialResponse(u8 value, const Guid& cryptoIV = {})
		{
			u8 data[33];
			*data = value;
			*(Guid*)(data+1) = m_server.m_uid;
			u32 dataSize = 1 + sizeof(Guid);
			if (m_cryptoKey != InvalidCryptoKey)
			{
				*(Guid*)(data+17) = cryptoIV;
				dataSize += sizeof(Guid);
			}
			NetworkBackend::SendContext context(NetworkBackend::SendFlags_Async);
			return m_backend.Send(m_server.m_logger, m_backendConnection, data, dataSize, context, TC("UidResponse"));
		}

		static bool ReceiveHandshakeHeader(void* context, const Guid& connectionUid, u8* headerData, void*& outBodyContext, u8*& outBodyData, u32& outBodySize)
		{
			u8* handshakeData = new u8[sizeof(EncryptionHandshakeString)];
			outBodyData = handshakeData;
			outBodySize = sizeof(EncryptionHandshakeString);
			return true;
		}

		static bool ReceiveHandshakeBody(void* context, bool recvError, u8* headerData, void* bodyContext, u8* bodyData, u32 bodySize)
		{
			auto& conn = *(Connection*)context;
			u8* handshakeData = bodyData;
			auto g = MakeGuard([handshakeData]() { delete[] handshakeData; });

			auto& logger = conn.m_server.m_logger;

			if (bodySize != sizeof(EncryptionHandshakeString))
				return logger.Warning(TC("Connection %u Crypto mismatch... (body size was %u, expected %u)"), conn.m_id, bodySize, sizeof(EncryptionHandshakeString));

			auto TestHandshake = [&](CryptoKey key)
			{
				u8 temp[sizeof(EncryptionHandshakeString)];
				memcpy(temp, handshakeData, sizeof(temp));
				Guid initVector;
				if (!Crypto::Decrypt(logger, key, temp, sizeof(EncryptionHandshakeString), initVector))
					return false;
				return memcmp(temp, EncryptionHandshakeString, sizeof(EncryptionHandshakeString)) == 0;
			};

			if (conn.m_cryptoKey != InvalidCryptoKey)
			{
				if (!TestHandshake(conn.m_cryptoKey))
					return logger.Warning(TC("Connection %u Crypto mismatch... (Handshake string is encrypted with different key)"), conn.m_id);
			}
			else
			{
				SCOPED_FUTEX(conn.m_server.m_cryptoKeysLock, lock);
				auto& keys = conn.m_server.m_cryptoKeys;
				u64 time = GetTime();
				for (auto it=keys.begin(); it!=keys.end();)
				{
					auto& entry = *it;
					if (entry.expirationTime < time)
					{
						it = keys.erase(it);
						continue;
					}
					++it;

					CryptoKey key = Crypto::DuplicateKey(logger, entry.key);
					auto keyGuard = MakeGuard([&]() { Crypto::DestroyKey(key); });
					if (!TestHandshake(key))
						continue;
					keyGuard.Cancel();
					conn.m_cryptoKey = key;
					conn.m_cryptoUserData = entry.userData;
					break;
				}
				if (conn.m_cryptoKey == InvalidCryptoKey)
					return logger.Warning(TC("Connection %u Crypto mismatch... (Handshake string is encrypted with different key than any registered keys)"), conn.m_id);
			}

			conn.m_backend.SetRecvCallbacks(conn.m_backendConnection, &conn, 4, ReceiveVersion, nullptr, TC("ReceiveVersion"));

			return true;
		}

		static bool ReceiveVersion(void* context, const Guid& connectionUid, u8* headerData, void*& outBodyContext, u8*& outBodyData, u32& outBodySize)
		{
			auto& conn = *(Connection*)context;
			u32 clientVersion = *(u32*)headerData;

			if (clientVersion < SystemNetworkVersionMin || clientVersion > SystemNetworkVersionRoute)
			{
				conn.SendInitialResponse(1);
				return false;
			}
			conn.m_clientVersion = clientVersion;

			// Ok, we got a version, meaning that there is someone on the other side of the connection. Set to the actual timeout.. 10 minutes or so
			conn.m_backend.SetRecvTimeout(conn.m_backendConnection, conn.m_server.m_receiveTimeoutMs, &conn, DefaultRecvTimeout);

			bool hasRoute = clientVersion >= SystemNetworkVersionRoute;
			conn.m_backend.SetRecvCallbacks(conn.m_backendConnection, &conn, hasRoute ? sizeof(Guid)*2 : sizeof(Guid), ReceiveClientUid, nullptr, TC("ReceiveClientUid"));

			return true;
		}

		static bool KeepAliveRecvTimeout(void* context, u32 timeoutMs, const tchar* recvHint, const tchar* hint)
		{
			auto& conn = *(Connection*)context;
			++conn.m_recvTimeoutCount;
			if (!conn.SendKeepAlive())
				return false;
			conn.m_backend.SetRecvTimeout(conn.m_backendConnection, KeepAliveIntervalSeconds*1000, context, KeepAliveRecvTimeout);
			if (conn.m_recvTimeoutCount < KeepAliveProbeCount)
				return true;
			constexpr u32 totalTimeoutSeconds = KeepAliveIdleSeconds + KeepAliveIntervalSeconds*KeepAliveProbeCount;
			u32 clientId = ~0u;
			if (auto c = conn.m_client)
				clientId = c->id;
			conn.m_server.m_logger.Warning(TC("Connection %u (Client %u) timed out after %u seconds (%s%s)"), conn.m_id, clientId, totalTimeoutSeconds, recvHint, hint);
			return false;
		}

		static bool ReceiveClientUid(void* context, const Guid& connectionUid, u8* headerData, void*& outBodyContext, u8*& outBodyData, u32& outBodySize)
		{
			auto& conn = *(Connection*)context;
			Guid clientUid = *(Guid*)headerData;
			if (conn.m_clientVersion >= SystemNetworkVersionRoute)
				conn.m_route = *(Guid*)(headerData + sizeof(Guid));
			return conn.ProcessConnectedClient(clientUid, connectionUid);
		}

		bool ProcessConnectedClient(const Guid& clientUid, const Guid& connectionUid)
		{
			auto& conn = *this;
			auto& server = conn.m_server;

			if (!server.m_allowNewClients)
			{
				SCOPED_READ_LOCK(server.m_clientsLock, clientsLock);
				bool found = false;
				for (auto& kv : server.m_clients)
					found |= kv.second.uid == clientUid;
				if (!found)
				{
					conn.SendInitialResponse(3);
					return false;
				}
			}

			conn.m_backend.SetRecvTimeout(conn.m_backendConnection, server.m_receiveTimeoutMs, &conn, DefaultRecvTimeout);

			constexpr u32 HeaderSize = 6;
			conn.m_backend.SetRecvCallbacks(conn.m_backendConnection, &conn, HeaderSize, ReceiveMessageHeader, ReceiveMessageBody, TC("ReceiveMessage"));

			// If keep alive we change timeout to 60 seconds
			if (server.m_useKeepAlive)
				conn.m_backend.SetRecvTimeout(conn.m_backendConnection, KeepAliveIdleSeconds*1000, &conn, KeepAliveRecvTimeout);

			Guid cryptoIV;
			if (conn.m_cryptoKey != InvalidCryptoKey)
			{
				CreateGuid(cryptoIV);
				conn.m_cryptoIV = cryptoIV;
				Guid initVector;
				if (!Crypto::Encrypt(server.m_logger, conn.m_cryptoKey, (u8*)&cryptoIV, sizeof(cryptoIV), initVector))
					return false;
			}

			if (!conn.SendInitialResponse(0, cryptoIV))
				return false;

			SCOPED_FUTEX(conn.m_shutdownLock, shutdownLock);

			SCOPED_WRITE_LOCK(server.m_clientsLock, clientsLock);
			u32 clientId = 0;
			for (auto& kv : server.m_clients)
				if (kv.second.uid == clientUid)
					clientId = kv.second.id;
			if (!clientId)
				clientId = ++server.m_clientCounter;
			Client& client = server.m_clients.try_emplace(clientId, clientUid, clientId).first->second;
			++client.refCount;
			clientsLock.Leave();

			conn.m_client = &client;

			u32 oldCount = client.connectionCount.fetch_add(1);
			if (oldCount == 0)
			{
				if (server.m_onConnectionFunction)
					if (!server.m_onConnectionFunction(clientUid, clientId))
					{
						--client.refCount;
						client.connectionCount.fetch_sub(1);
						conn.m_client = nullptr;
						return false;
					}

				if (server.m_logConnections)
					server.m_logger.Detail(TC("Client %u (%s) connected on connection %u (%s)"), clientId, GuidToString(clientUid).str, conn.m_id, GuidToString(connectionUid).str);
			}
			else
			{
				if (server.m_logConnections)
					server.m_logger.Detail(TC("Client %u (%s) additional connection %u (%s) connected"), clientId, GuidToString(clientUid).str, conn.m_id, GuidToString(connectionUid).str);

				#if 0 // Haven't really seen any difference by setting priority on socket. Will need more testing
				if (m_useMessagePriority && oldCount == 1)
				{
					SCOPED_READ_LOCK(server.m_connectionsLock, lock);
					for (auto& conIt : server.m_connections)
					{
						if (conIt.m_client != &client)
							continue;
						conIt.m_backend.SetPriority(conIt.m_backendConnection);
						server.m_logger.Info(TC("Priority set on first connection for client"));
						break;
					}
				}
				#endif
			}

			return true;
		}

		static bool ReceiveMessageHeader(void* context, const Guid& connectionUid, u8* headerData, void*& outBodyContext, u8*& outBodyData, u32& outBodySize)
		{
			auto& conn = *(Connection*)context;

			u8 serviceIdAndMessageType = headerData[0];
			u8 serviceId = serviceIdAndMessageType >> 6;
			u8 messageType = serviceIdAndMessageType & 0b111111;
			u16 messageId = u16(headerData[1] << 8) | u16((*(u32*)(headerData + 2) & 0xff000000) >> 24);
			u32 messageSize = *(u32*)(headerData + 2) & 0x00ffffff;

			if (messageSize > SendMaxSize)
				return conn.m_server.m_logger.Warning(TC("Client %u Got message size %u which is larger than max %u. Protocol error? (serviceId %u, messageType %u, messageId %u)"), conn.m_client->id, messageSize, SendMaxSize, u32(serviceId), u32(messageType), u32(messageId));
			if (serviceId >= sizeof(NetworkServer::m_workerFunctions))
				return conn.m_server.m_logger.Warning(TC("Client %u Got message with service id %u which is out of range. Protocol error?"), conn.m_client->id, serviceId);

			if (conn.m_recvTimeoutCount) // This can only be set if keepalive is enabled.
			{
				conn.m_recvTimeoutCount = 0; // Getting any message means we reset the timeout counter and time.
				conn.m_backend.SetRecvTimeout(conn.m_backendConnection, KeepAliveIdleSeconds*1000, context, KeepAliveRecvTimeout);
			}

			if (serviceId == SystemServiceId && messageType == SystemMessageType_KeepAlive) // Keep alive
				return true;

			LOG_STALL_SCOPE(conn.m_server.m_logger, 5, TC("PopWorker took more than %s"));

			//m_logger.Debug(TC("Recv: %u, %u, %u, %u"), serviceId, messageType, id, size);
			Worker* worker = conn.m_server.PopWorker();
			if (!worker)
				return false;
			if (!worker->m_context)
				return conn.m_server.m_logger.Error(TC("Client %u - Popped worker which has no context"), conn.m_client->id);
			auto& wc = *worker->m_context;
			wc.id = messageId;
			wc.serviceId = serviceId;
			wc.messageType = messageType;
			wc.dataSize = messageSize;
			wc.connection = &conn;
			if (wc.buffer.size() < messageSize)
				wc.buffer.resize(size_t(Min(messageSize + 1024u, SendMaxSize)));
			outBodyContext = worker;
			outBodyData = wc.buffer.data();
			outBodySize = messageSize;
			return true;
		}

		static bool ReceiveMessageBody(void* context, bool recvError, u8* headerData, void* bodyContext, u8* bodyData, u32 bodySize)
		{
			auto& conn = *(Connection*)context;
			auto worker = (Worker*)bodyContext;

			if (recvError)
			{
				conn.m_server.PushWorker(worker);
				return false;
			}

			// If the connection is being shut down, don't dispatch new work. A client can
			// drop connection at any time, so the local lifecycle must already tolerate
			// un-drained bytes. Recycling the worker (instead of bumping m_refCount)
			// lets EndStop() unblock as soon as already-running workers finish, rather than
			// waiting on every late message — typically SendFileToServer pushes for processes
			// we just cancelled.
			if (conn.m_shouldDisconnect)
			{
				conn.m_server.PushWorker(worker);
				return true;
			}

			auto& wc = *worker->m_context;

			conn.m_client->sendBytes += wc.dataSize;
			conn.m_server.m_recvBytes += wc.dataSize;
			++conn.m_server.m_recvCount;

			++conn.m_refCount;
			wc.workAvailable.Set();
			return true;
		}

		void Send(const void* data, u32 bytes, const tchar* sendHint)
		{
			TimerScope ts(m_sendTimer);
			NetworkBackend::SendContext context;
			if (!m_backend.Send(m_server.m_logger, m_backendConnection, data, bytes, context, sendHint))
				SetShouldDisconnect();
		}

		bool SetShouldDisconnect()
		{
			SCOPED_FUTEX(m_shutdownLock, lock);
			bool isConnected = !m_shouldDisconnect;
			m_shouldDisconnect = true;
			return isConnected;
		}

		void Release()
		{
			if (--m_refCount == 1)
				TestDisconnect();
		}

		void TestDisconnect()
		{
			SCOPED_FUTEX(m_shutdownLock, lock);
			if (!m_shouldDisconnect)
				return;
			if (m_disconnected)
				return;
			m_disconnected = true;
			lock.Leave();
			m_backend.Shutdown(m_backendConnection);
			if (m_client && m_client->connectionCount.fetch_sub(1) == 1)
			{
				SCOPED_READ_LOCK(m_server.m_onDisconnectFunctionsLock, l);
				for (auto& entry : m_server.m_onDisconnectFunctions)
					entry.function(m_client->uid, m_client->id);
				if (m_server.m_logConnections)
					m_server.m_logger.Detail(TC("Client %u (%s) disconnected"), m_client->id, GuidToString(m_client->uid).str);
			}
			--m_refCount; // drop the guard ref — unblocks EndStop() once we have fully returned
		}

		bool SendKeepAlive()
		{
			NetworkBackend::SendContext sendContext;
			constexpr u32 HeaderSize = 5;
			u16 messageId = 0;
			u32 bodySize = MessageKeepAliveSize;
			u8 data[5];
			data[0] = messageId >> 8;
			*(u32*)(data + 1) = bodySize | u32(messageId << 24);
			return m_backend.Send(m_server.m_logger, m_backendConnection, data, HeaderSize, sendContext, TC("KeepAlive"));
		}


		NetworkServer& m_server;
		NetworkBackend& m_backend;
		Futex m_shutdownLock;
		Client* m_client = nullptr;
		sockaddr m_remoteSockAddr;
		CryptoKey m_cryptoKey;
		Guid m_cryptoIV;
		Guid m_route;
		u32 m_clientVersion = 0;
		Event m_disconnectCallbackCalled;
		Atomic<int> m_refCount = 2; // Starts at 2: Disconnect() drops the connection ref, TestDisconnect() drops the guard ref on its way out.
		Atomic<int> m_disconnectCalled = 0;
		Atomic<bool> m_disconnected = false;
		u32 m_id = 0;
		u32 m_recvTimeoutCount = 0;
		bool m_shouldDisconnect = false;
		void* m_backendConnection = nullptr;

		Timer m_sendTimer;
		Timer m_encryptTimer;
		Timer m_decryptTimer;

		u64 m_cryptoUserData = 0;

		Connection(const Connection& o) = delete;
		void operator=(const Connection& o) = delete;
	};

	const Guid& ConnectionInfo::GetUid() const
	{
		return ((NetworkServer::Connection*)internalData)->m_client->uid;
	}

	u32 ConnectionInfo::GetId() const
	{
		return ((NetworkServer::Connection*)internalData)->m_client->id;
	}

	bool ConnectionInfo::GetName(StringBufferBase& out) const
	{
		#if PLATFORM_WINDOWS
		auto& remoteSockAddr = ((NetworkServer::Connection*)internalData)->m_remoteSockAddr;
		if (!InetNtopW(AF_INET, &remoteSockAddr, out.data, out.capacity))
			return false;
		out.count = u32(wcslen(out.data));
		return true;
		#else
		UBA_ASSERT(false);
		return false;
		#endif
	}

	u64 ConnectionInfo::GetCryptoUserData() const
	{
		auto& conn = *(NetworkServer::Connection*)internalData;
		return conn.m_cryptoUserData;
	}

	bool ConnectionInfo::ShouldDisconnect() const
	{
		auto& conn = *(NetworkServer::Connection*)internalData;
		SCOPED_FUTEX(conn.m_shutdownLock, lock);
		return conn.m_shouldDisconnect;
	}

	void ConnectionInfo::SetStorageVersion(u32 version) const
	{
		auto& conn = *(NetworkServer::Connection*)internalData;
		conn.m_client->storageVersion = version;
	}

	u32 ConnectionInfo::GetStorageVersion() const
	{
		auto& conn = *(NetworkServer::Connection*)internalData;
		return conn.m_client->storageVersion;
	}

	void NetworkServer::Worker::Update(WorkerContext& context)
	{
		auto& server = context.server;

		// This is only additional work
		if (!context.connection)
			return;

		auto& connection = *context.connection;
		context.connection = nullptr;

		WorkerRec& rec = server.m_workerFunctions[context.serviceId];
		TrackWorkScope tws(server, rec.toString(context.messageType), ColorWork);

		CryptoKey cryptoKey = connection.m_cryptoKey;
		if (cryptoKey)
		{
			//TrackHintScope ths(tws, TCV("Decrypt"));
			TimerScope ts(connection.m_decryptTimer);
			Guid iv = connection.m_cryptoIV;
			if (!Crypto::Decrypt(server.m_logger, cryptoKey, context.buffer.data(), context.dataSize, iv))
			{
				connection.SetShouldDisconnect();
				connection.Release();
				return;
			}
		}

		BinaryReader reader(context.buffer.data(), 0, context.dataSize);

		constexpr u32 HeaderSize = 5; // 2 byte id, 3 bytes size

		BinaryWriter writer(context.writeMem, 0, context.writeMemSize);
		u8* idAndSizePtr = writer.AllocWrite(HeaderSize);
			
		u32 size;

		MessageInfo mi;
		mi.type = context.messageType;
		mi.connectionId = connection.m_id;
		mi.messageId = context.id;

		{
			//TrackHintScope ths(tws, TCV("HandleMessage"));
			if (!rec.func)
			{
				server.m_logger.Error(TC("WORKER FUNCTION NOT FOUND. id: %u, serviceid: %u type: %s, client: %u"), context.id, context.serviceId, rec.toString(context.messageType).data, connection.m_client->id);
				connection.SetShouldDisconnect();
				size = MessageErrorSize;
			}
			else if (!rec.func({&connection}, {tws}, mi, reader, writer))
			{
				if (connection.SetShouldDisconnect())
				{
					#if UBA_DEBUG
					server.m_logger.Error(TC("WORKER FUNCTION FAILED. id: %u, serviceid: %u type: %s, client: %u"), context.id, context.serviceId, rec.toString(context.messageType).data, connection.m_client->id);
					#endif
				}
				size = MessageErrorSize;
			}
			else
			{
				size = u32(writer.GetPosition());
			}
		}

		if (mi.messageId)
		{
			UBA_ASSERT(size < (1 << 24));
				
			u32 bodySize = u32(size - HeaderSize);
			if (cryptoKey && size != MessageErrorSize && bodySize)
			{
				//TrackHintScope ths(tws, TCV("Encrypt"));
				TimerScope ts(connection.m_encryptTimer);
				u8* bodyData = writer.GetData() + HeaderSize;
				Guid iv = connection.m_cryptoIV;
				if (!Crypto::Encrypt(server.m_logger, cryptoKey, bodyData, bodySize, iv))
				{
					connection.SetShouldDisconnect();
					size = MessageErrorSize;
					bodySize = u32(size - HeaderSize);
				}
			}

			idAndSizePtr[0] = context.id >> 8;
			*(u32*)(idAndSizePtr + 1) = bodySize | u32(context.id << 24);

			// This can happen for proxy servers in a valid situation
			//if (size == MessageErrorSize)
			//	UBA_ASSERT(false);

			//TrackHintScope ths(tws, TCV("Send"));
			connection.Send(writer.GetData(), size == MessageErrorSize ? HeaderSize : size, TC("MessageResponse"));
		}
			
		connection.Release();
	}

	void NetworkServer::Worker::ThreadWorker(NetworkServer& server)
	{
		t_worker = this;
		while (m_context->workAvailable.IsSet(~0u) && m_loop)
		{
			Update(*m_context);
			DoAdditionalWorkAndSignalAvailable(m_context->server);
		}

		t_worker = nullptr;

		if (m_inUse) // I have no idea how this can happen.. should not be possible. There is a path somewhere where it can leave while still being in use
			server.PushWorker(this);
	}

	void NetworkServer::Worker::DoAdditionalWorkAndSignalAvailable(NetworkServer& server)
	{
		auto& additionalWorkQueue = server.m_additionalWorkQueue;
		BottleneckTicket ticket;

		while (true)
		{
			while (true)
			{
				Work* work = additionalWorkQueue.Dequeue(ticket);
				if (!work)
					break;

				#if UBA_TRACK_WORK
				TrackWorkScope tws(server, work->desc, ColorWork);
				#else
				TrackWorkScope tws;
				#endif

				work->func({tws});

				additionalWorkQueue.Finish(work, ticket);
			}

			// Both locks needs to be taken to verify if additional work
			// is present before making ourself available to avoid
			// a race where AddWork would not see this thread in the
			// available list after adding some work.
			SCOPED_FUTEX(server.m_availableWorkersLock, lock1);
			SCOPED_FUTEX_READ(additionalWorkQueue.lock, lock2);
			// Verify there is not additional work while we hold both lock
			// and only add ourself as available if no additional work is present.
			if (!additionalWorkQueue.IsEmptyNoLock())
				continue;
			server.PushWorkerNoLock(this);
			break;
		}
	}

	const tchar* g_typeStr[] = { TC("0"), TC("1"), TC("2"), TC("3"), TC("4"), TC("5"), TC("6"), TC("7"), TC("8"), TC("9"), TC("10"), TC("11"), TC("12") };

	static StringView GetMessageTypeToName(u8 type)
	{
		if (type <= 12)
			return ToView(g_typeStr[type]);
		return ToView(TC("NUMBER HIGHER THAN 12"));
	}

	NetworkServer::NetworkServer(bool& outCtorSuccess, const NetworkServerCreateInfo& info, const tchar* name)
	:	m_logger(info.logWriter, name)
	{
		outCtorSuccess = true;

		u32 workerCount;
		if (info.workerCount == 0)
			workerCount = GetLogicalProcessorCount();
		else
			workerCount = Min(Max(info.workerCount, (u32)(1u)), (u32)(1024u));
		m_maxWorkerCount = workerCount;

		#if UBA_DEBUG
		m_logger.Info(TC("Created in DEBUG"));
		#endif

		u32 fixedSendSize = Max(info.sendSize, (u32)(4*1024));
		fixedSendSize = Min(fixedSendSize, (u32)(SendMaxSize));
		if (info.sendSize != fixedSendSize)
			m_logger.Detail(TC("Adjusted msg size to %u to stay inside limits"), fixedSendSize);
		m_sendSize = fixedSendSize;
		m_receiveTimeoutMs = info.receiveTimeoutSeconds * 1000;
		m_logConnections = info.logConnections;
		m_useKeepAlive = info.useKeepAlive;
		m_useMessagePriority = info.useMessagePriority;
		m_useThreadedSend = info.useThreadedSend;

		m_recvBufferSize = info.recvBufferSize;
		m_sendBufferSize = info.sendBufferSize;

		#if PLATFORM_MAC
		m_useKeepAlive = true; // Always run keep alive on mac since the built-in one has a probe interval of 1 minute or something like that... so timeout is always 10 minutes
		#endif


		m_workerFunctions[SystemServiceId].toString = GetMessageTypeToName;
		m_workerFunctions[SystemServiceId].func = [this](const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer)
			{
				return HandleSystemMessage(connectionInfo, messageInfo.type, reader, writer);
			};

		if (!CreateGuid(m_uid))
			outCtorSuccess = false;
	}

	NetworkServer::~NetworkServer()
	{
		UBA_ASSERT(m_connections.empty());
		FlushWorkers();
		for (auto& entry : m_cryptoKeys)
			Crypto::DestroyKey(entry.key);
	}

	bool NetworkServer::StartListen(NetworkBackend& backend, u16 port, const tchar* ip, bool requiresCrypto)
	{
		return backend.StartListen(m_logger, port, ip, [this, &backend, requiresCrypto](void* connection, const sockaddr& remoteSockAddr)
			{
				return AddConnection(backend, connection, remoteSockAddr, requiresCrypto, InvalidCryptoKey);
			});
	}
	
	void NetworkServer::DisallowNewClients()
	{
		m_allowNewClients = false;
	}

	void NetworkServer::DisconnectClients()
	{
		{
			SCOPED_FUTEX(m_availableWorkersLock, lock);
			if (!m_workersEnabled) // Already disconnecting.
				return;
			m_workersEnabled = false;
			while (PopWorkerRequest* req = m_firstRequest)
			{
				m_firstRequest = req->next;
				req->next = nullptr;
				req->ev.Set();
			}
			m_lastRequest = nullptr;
		}
		{
			SCOPED_FUTEX(m_addConnectionsLock, lock);
			m_addConnections.clear();
		}

		{
			SCOPED_WRITE_LOCK(m_connectionsLock, lock);
			List<Connection> connections(std::move(m_connections));
			lock.Leave();

			bool success = true;
			for (auto& c : connections)
				success = c.BeginStop() && success;

			for (auto& c : connections)
			{
				success = c.EndStop() && success;
				m_sendTimer += c.m_sendTimer;
				m_encryptTimer += c.m_encryptTimer;
				m_decryptTimer += c.m_decryptTimer;
			}

			// If stopping connections fail we need to abort because we will most likely run into a deadlock when deleting the workers.
			if (!success)
			{
				m_logger.Info(TC("Failed to stop connection(s) in a graceful way. Will abort process"));
				abort(); // TODO: Does this produce core dump on windows?
			}
		}

		FlushWorkers();

		SCOPED_WRITE_LOCK(m_connectionsLock, lock);
		UBA_ASSERT(m_connections.empty());
		m_allClientsDisconnected = true;
		m_workersEnabled = true;
	}

	void NetworkServer::ShutdownActiveConnections()
	{
		SCOPED_READ_LOCK(m_connectionsLock, lock);
		for (auto& c : m_connections)
			if (!c.m_disconnected)
				c.m_backend.Shutdown(c.m_backendConnection);
	}

	bool NetworkServer::RegisterCryptoKey(const u8* cryptoKey128, u64 expirationTime, u64 cryptoUserData)
	{
		CryptoKey key = Crypto::CreateKey(m_logger, cryptoKey128);
		if (key == InvalidCryptoKey)
			return false;
		SCOPED_FUTEX(m_cryptoKeysLock, lock);
		m_cryptoKeys.push_back(CryptoEntry{key, expirationTime, cryptoUserData});
		return true;
	}

	void NetworkServer::SetClientsConfig(const Config& config)
	{
		config.SaveToText(m_logger, m_clientsConfig);
	}

	void NetworkServer::SetTrace(Trace* trace)
	{
		m_trace = trace;
	}

	bool NetworkServer::AddClient(NetworkBackend& backend, const tchar* ip, u16 port, const u8* cryptoKey128)
	{
		SCOPED_FUTEX(m_addConnectionsLock, lock);
		if (!m_workersEnabled)
			return false;

		for (auto it = m_addConnections.begin(); it != m_addConnections.end();)
		{
			if (it->Wait(0))
				it = m_addConnections.erase(it);
			else
				++it;
		}

		CryptoKey cryptoKey = InvalidCryptoKey;
		if (cryptoKey128)
		{
			cryptoKey = Crypto::CreateKey(m_logger, cryptoKey128);
			if (cryptoKey == InvalidCryptoKey)
				return false;
		}

		Event done(EventResetType_Manual);
		bool success = false;

		m_addConnections.emplace_back([this, &success , &done, &backend, ip2 = TString(ip), port, cryptoKey]()
			{
				// TODO: Should this retry?
				success = backend.Connect(m_logger, ip2.c_str(), [this, &backend, cryptoKey](void* connection, const sockaddr& remoteSocketAddr, bool* timedOut)
					{
						return AddConnection(backend, connection, remoteSocketAddr, cryptoKey != InvalidCryptoKey, cryptoKey);
					}, port, nullptr);
				if (!success)
					Crypto::DestroyKey(cryptoKey);
				done.Set();
				return 0;
			}, TC("UbaAddConSrv"));
		
		lock.Leave();
		done.IsSet();
		return success;
	}

	bool NetworkServer::HasConnectInProgress()
	{
		SCOPED_FUTEX(m_addConnectionsLock, lock);
		for (auto it = m_addConnections.begin(); it != m_addConnections.end();)
		{
			if (it->Wait(0))
				it = m_addConnections.erase(it);
			else
				++it;
		}
		return !m_addConnections.empty();
	}

	void NetworkServer::PrintSummary(Logger& logger)
	{
		if (!m_maxActiveConnections)
			return;

		m_maxCreatedWorkerCount = Max(m_createdWorkerCount, m_maxCreatedWorkerCount);
		StringBuffer<> workers;
		workers.Appendf(TC("%u/%u"), m_maxCreatedWorkerCount, m_maxWorkerCount);

		logger.Info(TC("  ----- Uba server stats summary ------"));
		logger.Info(TC("  MaxActiveConnections           %6u"), m_maxActiveConnections);
		logger.Info(TC("  SendTotal          %8u %9s"), m_sendTimer.count.load(), TimeToText(m_sendTimer.time).str);
		logger.Info(TC("     Bytes                    %9s"), BytesToText(m_sendBytes).str);
		logger.Info(TC("  RecvTotal          %8u %9s"), m_recvCount.load(), BytesToText(m_recvBytes.load()).str);
		if (m_encryptTimer.count || m_decryptTimer.count)
		{
			logger.Info(TC("  EncryptTotal       %8u %9s"), m_encryptTimer.count.load(), TimeToText(m_encryptTimer.time).str);
			logger.Info(TC("  DecryptTotal       %8u %9s"), m_decryptTimer.count.load(), TimeToText(m_decryptTimer.time).str);
		}
		logger.Info(TC("  WorkerCount                 %9s"), workers.data);
		logger.Info(TC("  SendSize Set/Max  %9s %9s"), BytesToText(m_sendSize).str, BytesToText(SendMaxSize).str);
		logger.Info(TC(""));
	}

	void NetworkServer::RegisterService(u8 serviceId, const WorkerFunction& function, TypeToNameFunction* typeToNameFunc)
	{
		UBA_ASSERTF(serviceId != 0, TC("ServiceId 0 is reserved by system"));
		WorkerRec& rec = m_workerFunctions[serviceId];
		UBA_ASSERT(!rec.func);
		rec.func = function;
		rec.toString = typeToNameFunc;
		if (!typeToNameFunc)
			rec.toString = GetMessageTypeToName;
	}

	void NetworkServer::UnregisterService(u8 serviceId)
	{
		SCOPED_WRITE_LOCK(m_connectionsLock, lock);
		UBA_ASSERTF(m_connections.empty(), TC("Unregistering service while still having live connections"));
		WorkerRec& rec = m_workerFunctions[serviceId];
		rec.func = {};
		//rec.toString = nullptr; // Keep this for now, we want to be able to output stats
	}

	void NetworkServer::RegisterOnClientConnected(u8 id, const OnConnectionFunction& func)
	{
		UBA_ASSERT(!m_onConnectionFunction);
		m_onConnectionFunction = func;
	}

	void NetworkServer::UnregisterOnClientConnected(u8 id)
	{
		SCOPED_WRITE_LOCK(m_connectionsLock, lock);
		UBA_ASSERT(m_connections.empty());
		m_onConnectionFunction = {};
	}

	void NetworkServer::RegisterOnClientDisconnected(u8 id, const OnDisconnectFunction& func)
	{
		SCOPED_WRITE_LOCK(m_onDisconnectFunctionsLock, l);
		m_onDisconnectFunctions.emplace_back(OnDisconnectEntry{id, func});
	}

	void NetworkServer::UnregisterOnClientDisconnected(u8 id)
	{
		SCOPED_WRITE_LOCK(m_onDisconnectFunctionsLock, l);
		for (auto it = m_onDisconnectFunctions.begin(); it != m_onDisconnectFunctions.end(); ++it)
		{
			if (it->id != id)
				continue;
			m_onDisconnectFunctions.erase(it);
			return;
		}
	}

	void NetworkServer::AddWork(WorkFunction&& work, u32 count, const tchar* desc, const Color& color, WorkPriority priority, Bottleneck* bottleneck)
	{
		m_additionalWorkQueue.Enqueue(std::move(work), count, desc, color, priority, bottleneck, m_workTracker != nullptr);

		SCOPED_FUTEX(m_availableWorkersLock, lock2);
		if (!m_workersEnabled)
			return;
		while (count--)
		{
			Worker* worker = PopWorkerNoLock();
			if (!worker)
				break;
			UBA_ASSERT(worker->m_inUse);
			worker->m_context->connection = nullptr;
			worker->m_context->workAvailable.Set();
		}
	}

	void NetworkServer::DoWork(u32 count)
	{
		while (count--)
			if (!DoAdditionalWork())
				return;
	}

	u32 NetworkServer::GetWorkerCount()
	{
		return m_maxWorkerCount;
	}

	bool NetworkServer::FlushWork(u32 timeoutMs)
	{
		u64 start = GetTime();
		if (!timeoutMs)
			timeoutMs = 2*60*60*1000; // Two hours is the same as forever

		while (true)
		{
			bool noAdditionalWork = m_additionalWorkQueue.IsEmpty();

			SCOPED_FUTEX_READ(m_availableWorkersLock, lock2);
			bool noActiveWorkers = !m_firstActiveWorker;
			lock2.Leave();

			if (noAdditionalWork && noActiveWorkers)
				return true;

			Sleep(50);

			u64 delta = GetTime() - start;
			if (TimeToMs(delta) > timeoutMs)
				return m_logger.Error(TC("NetworkServer::FlushWork timed out after %s"), TimeToText(delta).str);
		}
	}

	MutableLogger& NetworkServer::GetLogger()
	{
		return m_logger;
	}

	u64 NetworkServer::GetTotalSentBytes()
	{
		return m_sendBytes;
	}

	u64 NetworkServer::GetTotalRecvBytes()
	{
		return m_recvBytes;
	}

	Timer& NetworkServer::GetTotalSentTimer()
	{
		return m_sendTimer;
	}

	u32 NetworkServer::GetClientCount()
	{
		SCOPED_READ_LOCK(m_clientsLock, lock)
		return u32(m_clients.size());
	}

	u32 NetworkServer::GetConnectionCount()
	{
		SCOPED_READ_LOCK(m_connectionsLock, lock);
		u32 count = 0;
		for (auto& con : m_connections)
			if (!con.m_disconnected)
				++count;
		return count;
	}

	void NetworkServer::GetClientStats(ClientStats& out, u32 clientId)
	{
		SCOPED_READ_LOCK(m_clientsLock, lock);
		auto findIt = m_clients.find(clientId);
		if (findIt == m_clients.end())
			return;
		Client& c = findIt->second;
		out.send += c.sendBytes;
		out.recv += c.recvBytes;
		out.connectionCount += c.connectionCount;
	}

	bool NetworkServer::IsConnectedSlow(u32 clientId)
	{
		SCOPED_READ_LOCK(m_clientsLock, lock);
		auto findIt = m_clients.find(clientId);
		if (findIt == m_clients.end())
			return false;
		Client& c = findIt->second;
		if (c.connectionCount == 0)
			return false;
		lock.Leave();

		SCOPED_READ_LOCK(m_connectionsLock, lock2);
		for (auto& con : m_connections)
		{
			if (con.m_client != &c)
				continue;
			if (!con.m_disconnectCalled)
				return true;
		}
		return false;
	}

	void NetworkServer::ResetTotalStats()
	{
		m_sendTimer = {};
		m_sendBytes = 0;
		m_recvBytes = 0;
	}

	bool NetworkServer::DoAdditionalWork()
	{
		BottleneckTicket ticket;

		Work* work = m_additionalWorkQueue.Dequeue(ticket);
		if (work)
		{
			#if UBA_TRACK_WORK
			TrackWorkScope tws(*this, work->desc, ColorWork);
			#else
			TrackWorkScope tws;
			#endif
			work->func({tws});

			m_additionalWorkQueue.Finish(work, ticket);
			return true;
		}

		SCOPED_FUTEX(m_availableWorkersLock, lock);
		if (m_createdWorkerCount != m_maxWorkerCount)
			return false;
		lock.Leave();

		auto worker = t_worker;
		if (!worker)
			return false;

		auto oldContext = worker->m_context;
		WorkerContext context(*this);
		worker->m_context = &context;

		PushWorker(worker);
		bool workAvail = context.workAvailable.IsSet(10);
		lock.Enter();
		if (worker->m_inUse)
		{
			lock.Leave();
			if (!workAvail)
				context.workAvailable.IsSet(~0u);
			worker->Update(context);
			UBA_ASSERT(worker->m_inUse);
		}
		else
		{
			// Take worker back from free list
			if (m_firstAvailableWorker == worker)
				m_firstAvailableWorker = worker->m_nextWorker;
			else
				worker->m_prevWorker->m_nextWorker = worker->m_nextWorker;
			if (worker->m_nextWorker)
				worker->m_nextWorker->m_prevWorker = worker->m_prevWorker;
			worker->m_prevWorker = nullptr;
			worker->m_nextWorker = m_firstActiveWorker;
			if (m_firstActiveWorker)
				m_firstActiveWorker->m_prevWorker = worker;
			m_firstActiveWorker = worker;
			worker->m_inUse = true;
		}
			
		worker->m_context = oldContext;
		return true;
	}

	bool NetworkServer::SendResponse(const MessageInfo& info, const u8* body, u32 bodySize)
	{
		UBA_ASSERT(info.connectionId);
		UBA_ASSERT(info.messageId);

		LOG_STALL_SCOPE(m_logger, 5, TC("NetworkServer::SendResponse took more than %s"));

		SCOPED_READ_LOCK(m_connectionsLock, lock);
		Connection* found = nullptr;
		for (auto& it : m_connections)
		{
			if (it.m_id != info.connectionId)
				continue;
			if (!it.m_disconnected)
				found = &it;
			break;
		}
		if (!found)
			return false;
		Connection& connection = *found;

		u8 buffer[SendMaxSize];

		constexpr u32 HeaderSize = 5; // 2 byte id, 3 bytes size

		BinaryWriter writer(buffer, 0, sizeof_array(buffer));
		u8* idAndSizePtr = writer.AllocWrite(HeaderSize);

		if (body)
		{
			writer.WriteBytes(body, bodySize);

			if (connection.m_cryptoKey && bodySize)
			{
				TimerScope ts(connection.m_encryptTimer);
				u8* bodyData = writer.GetData() + HeaderSize;
				Guid iv = connection.m_cryptoIV;
				if (!Crypto::Encrypt(m_logger, connection.m_cryptoKey, bodyData, bodySize, iv))
				{
					connection.SetShouldDisconnect();
					return false;
				}
			}
		}
		else
		{
			bodySize = MessageErrorSize;
			connection.SetShouldDisconnect();
		}

		idAndSizePtr[0] = info.messageId >> 8;
		*(u32*)(idAndSizePtr + 1) = bodySize | u32(info.messageId << 24);

		connection.Send(writer.GetData(), u32(writer.GetPosition()), TC("MessageResponse"));
		return true;
	}

	bool NetworkServer::SendKeepAlive()
	{
		SCOPED_READ_LOCK(m_connectionsLock, lock);
		for (auto& it : m_connections)
			if (!it.SendKeepAlive())
				return false;
		return true;
	}

	NetworkServer::Worker* NetworkServer::PopWorker()
	{
		while (true)
		{
			SCOPED_FUTEX(m_availableWorkersLock, lock);
			if (!m_workersEnabled)
				return nullptr;
			if (auto worker = PopWorkerNoLock())
				return worker;
			
			PopWorkerRequest req;
			req.ev.Create(EventResetType_Manual);

			if (!m_firstRequest)
				m_firstRequest = &req;
			else
				m_lastRequest->next = &req;
			m_lastRequest = &req;

			lock.Leave();

			// TODO: We should log something here
			//if (!req.ev.IsSet(20*1000))
			//{
				//m_logger.Info(TC("Waited 20 seconds for available worker. This should not happen!"));
				req.ev.IsSet();
			//}

			if (req.worker)
				return req.worker;
		}
	}

	NetworkServer::Worker* NetworkServer::PopWorkerNoLock()
	{
		Worker* worker = m_firstAvailableWorker;
		if (worker)
		{
			m_firstAvailableWorker = worker->m_nextWorker;
			if (m_firstAvailableWorker)
				m_firstAvailableWorker->m_prevWorker = nullptr;
		}
		else
		{
			if (m_createdWorkerCount == m_maxWorkerCount)
				return nullptr;

			worker = new Worker();
			worker->Start(*this);
			++m_createdWorkerCount;
		}

		if (m_firstActiveWorker)
			m_firstActiveWorker->m_prevWorker = worker;
		worker->m_nextWorker = m_firstActiveWorker;
		m_firstActiveWorker = worker;
		worker->m_inUse = true;

		return worker;
	}

	void NetworkServer::PushWorker(Worker* worker)
	{
		SCOPED_FUTEX(m_availableWorkersLock, lock);

		PushWorkerNoLock(worker);
	}

	void NetworkServer::PushWorkerNoLock(Worker* worker)
	{
		UBA_ASSERT(worker->m_inUse);

		if (PopWorkerRequest* first = m_firstRequest)
		{
			m_firstRequest = first->next;
			if (!m_firstRequest)
				m_lastRequest = nullptr;
			first->worker = worker;
			first->ev.Set();
			return;
		}


		if (worker->m_prevWorker)
			worker->m_prevWorker->m_nextWorker = worker->m_nextWorker;
		else
			m_firstActiveWorker = worker->m_nextWorker;
		if (worker->m_nextWorker)
			worker->m_nextWorker->m_prevWorker = worker->m_prevWorker;

		if (m_firstAvailableWorker)
			m_firstAvailableWorker->m_prevWorker = worker;
		worker->m_prevWorker = nullptr;
		worker->m_nextWorker = m_firstAvailableWorker;
		worker->m_inUse = false;
		m_firstAvailableWorker = worker;
	}

	void NetworkServer::FlushWorkers()
	{
		SCOPED_FUTEX(m_availableWorkersLock, lock);
		while (auto worker = m_firstActiveWorker)
		{
			lock.Leave();
			worker->Stop(*this);
			lock.Enter();
		}

		UBA_ASSERT(m_firstActiveWorker == nullptr);

		auto worker = m_firstAvailableWorker;
		while (worker)
		{
			auto temp = worker;
			worker = worker->m_nextWorker;
			delete temp;
		}
		m_firstAvailableWorker = nullptr;
		m_maxCreatedWorkerCount = Max(m_createdWorkerCount, m_maxCreatedWorkerCount);
		m_createdWorkerCount = 0;
	}

	void NetworkServer::RemoveDisconnectedConnections()
	{
		bool updateClients = false;

		for (auto it=m_connections.begin(); it!=m_connections.end();)
		{
			Connection& con = *it;
			if (!con.m_disconnected)
			{
				++it;
				continue;
			}
			m_sendTimer += con.m_sendTimer;
			auto& backend = con.m_backend;
			void* backendConnection = con.m_backendConnection;
			if (auto client = con.m_client)
				--client->refCount;
			it = m_connections.erase(it);
			backend.DeleteConnection(backendConnection);
			updateClients = true;
		}

		if (!updateClients)
			return;

		SCOPED_WRITE_LOCK(m_clientsLock, lock);
		for (auto it=m_clients.begin(); it!=m_clients.end();)
		{
			if (it->second.refCount)
				++it;
			else
				it = m_clients.erase(it);
		}
	}

	bool NetworkServer::HandleSystemMessage(const ConnectionInfo& connectionInfo, u8 messageType, BinaryReader& reader, BinaryWriter& writer)
	{
		switch (messageType)
		{
			case SystemMessageType_SetConnectionCount:
			{
				LOG_STALL_SCOPE(m_logger, 5, TC("SystemMessageType_SetConnectionCount took more than %s"));
				u32 connectionCount = reader.ReadU32();
				u32 clientId = connectionInfo.GetId();

				SCOPED_READ_LOCK(m_clientsLock, lock);
				auto findIt = m_clients.find(clientId);
				if (findIt == m_clients.end())
					return true;
				Client& c = findIt->second;

				u32 currentCount = c.connectionCount + c.queuedConnectionCount;
				if (currentCount >= connectionCount)
					return true;
				u32 toAdd = connectionCount - currentCount;
				c.queuedConnectionCount += toAdd;
				m_logger.Detail(TC("Client %u requested %u connections. Has %u, queue %u"), c.id, connectionCount, c.connectionCount.load(), c.queuedConnectionCount);
				lock.Leave();

				u32 connectionId = ((NetworkServer::Connection*)connectionInfo.internalData)->m_id;

				SCOPED_FUTEX(m_addConnectionsLock, lock2);
				for (u32 i = 0; i != toAdd; ++i)
				{
					m_addConnections.emplace_back([this, connectionId]()
						{
							auto cg = MakeGuard([&]()
								{
									SCOPED_READ_LOCK(m_clientsLock, lock);
									auto findIt = m_clients.find(connectionId);
									if (findIt != m_clients.end())
										--findIt->second.queuedConnectionCount;
								});

							Connection* conn = nullptr;
							SCOPED_READ_LOCK(m_connectionsLock, lock);
							for (auto& c : m_connections)
								if (c.m_id == connectionId)
									conn = &c;
							if (!conn || conn->m_disconnected)
								return 0;
							auto& backend = conn->m_backend;
							auto remoteSockAddr = conn->m_remoteSockAddr;
							CryptoKey cryptoKey = InvalidCryptoKey;
							if (conn->m_cryptoKey)
							{
								cryptoKey = Crypto::DuplicateKey(m_logger, conn->m_cryptoKey);
								if (cryptoKey == InvalidCryptoKey)
									return 0;
							}
							lock.Leave();

							bool success = backend.Connect(m_logger, remoteSockAddr, [this, &backend, &cryptoKey](void* connection, const sockaddr& remoteSocketAddr, bool* timedOut)
								{
									return AddConnection(backend, connection, remoteSocketAddr, cryptoKey != InvalidCryptoKey, cryptoKey);
								}, nullptr);

							if (!success)
								Crypto::DestroyKey(cryptoKey);
							return 0;
						});
				}
				return true;
			}
			case SystemMessageType_FetchConfig:
			{
				writer.Write7BitEncoded(m_clientsConfig.size());
				writer.WriteBytes(m_clientsConfig.data(), m_clientsConfig.size());
				return true;
			}
			case SystemMessageType_GetTraceInformation:
			{
				if (!m_trace)
				{
					writer.WriteU32(0);
					return true;
				}
				Trace& trace = *m_trace;

				u32 remotePos = reader.ReadU32();
				u32 localPos = trace.GetPublishedSize();

				writer.WriteU32(localPos);
				u32 toWrite = Min(localPos - remotePos, u32(writer.GetCapacityLeft()));
				writer.WriteBytes(trace.m_memoryBegin + remotePos, toWrite);
				return true;
			}
		}
		return false;
	}

	bool NetworkServer::AddConnection(NetworkBackend& backend, void* backendConnection, const sockaddr& remoteSocketAddr, bool requiresCrypto, CryptoKey cryptoKey)
	{
		LOG_STALL_SCOPE(m_logger, 5, TC("NetworkServer::AddConnection took more than %s"));
		SCOPED_WRITE_LOCK(m_connectionsLock, lock);

		RemoveDisconnectedConnections();

		if (!m_workersEnabled || m_allClientsDisconnected)
		{
			// Just to prevent errors in log
			backend.SetDisconnectCallback(backendConnection, nullptr, [](void*, const Guid&, void*) {});
			backend.SetRecvCallbacks(backendConnection, nullptr, 0, [](void*, const Guid&, u8*, void*&, u8*&, u32&) { return false; }, nullptr, TC("Disconnecting"));
			return false;
		}

		if (IsWindows && !IsRunningWine())
		{
			if (m_recvBufferSize)
				backend.SetRecvBufferSize(backendConnection, m_recvBufferSize);
			if (m_sendBufferSize)
				backend.SetSendBufferSize(backendConnection, m_sendBufferSize);
		}

		if (m_useThreadedSend)
			backend.SetThreadedSend(backendConnection);

		m_connections.emplace_back(*this, backend, backendConnection, remoteSocketAddr, requiresCrypto, cryptoKey, m_connectionIdCounter++);
		m_maxActiveConnections = Max(m_maxActiveConnections, u32(m_connections.size()));
		return true;
	}
}
