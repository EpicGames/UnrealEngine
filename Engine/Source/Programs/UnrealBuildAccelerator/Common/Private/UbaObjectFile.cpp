// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaObjectFile.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaCompressedFileHeader.h"
#include "UbaFileAccessor.h"
#include "UbaObjectFileCoff.h"
#include "UbaObjectFileElf.h"
#include "UbaObjectFileImportLib.h"
#include "UbaObjectFileLLVMIR.h"
#include "UbaObjectFileMachO.h"
#include <oodle2.h>
#include <algorithm>

namespace uba
{
	u8 SymbolFileVersion = 3; // v3: added explicit-exports section (verbatim COFF /EXPORT: names + isData)

	ObjectFile* ObjectFile::OpenAndParse(Logger& logger, ObjectFileParseMode parseMode, const tchar* filename)
	{
		UBA_ASSERT(*filename);
		auto file = new FileAccessor(logger, filename);
		auto fileGuard = MakeGuard([&]() { delete file; });

		if (!file->OpenMemoryRead())
			return nullptr;

		ObjectFile* objectFile = Parse(logger, parseMode, file->GetData(), file->GetSize(), filename);
		if (!objectFile)
			return nullptr;

		fileGuard.Cancel();
		objectFile->m_file = file;
		return objectFile;
	}

	ObjectFile* ObjectFile::Parse(Logger& logger, ObjectFileParseMode parseMode, u8* data, u64 dataSize, const tchar* hint)
	{
		ObjectFile* objectFile = nullptr;

		bool ownsData = false;
		if (dataSize >= sizeof(CompressedFileHeader) && ((CompressedFileHeader*)data)->IsValid())
		{
			u64 decompressedSize = *(u64*)(data + sizeof(CompressedFileHeader));
			u8* readPos = data + sizeof(CompressedFileHeader) + 8;

			u8* decompressedData = (u8*)malloc(decompressedSize);
			u8* writePos = decompressedData;

			OO_SINTa decoredMemSize = OodleLZDecoder_MemorySizeNeeded(OodleLZ_Compressor_Kraken);
			void* decoderMem = malloc(decoredMemSize);
			auto mg = MakeGuard([decoderMem]() { free(decoderMem); });

			u64 left = decompressedSize;
			while (left)
			{
				u32 compressedBlockSize = *(u32*)readPos;
				readPos += 4;
				u32 decompressedBlockSize = *(u32*)readPos;
				readPos += 4;

				OO_SINTa decompLen = OodleLZ_Decompress(readPos, (OO_SINTa)compressedBlockSize, writePos, (OO_SINTa)decompressedBlockSize,
					OodleLZ_FuzzSafe_Yes, OodleLZ_CheckCRC_No, OodleLZ_Verbosity_None, NULL, 0, NULL, NULL, decoderMem, decoredMemSize);
				if (decompLen != decompressedBlockSize)
				{
					logger.Error(TC("Failed to decompress file %s (Compressed size %llu)"), hint, dataSize);
					return nullptr;
				}

				readPos += compressedBlockSize;
				writePos += decompressedBlockSize;
				left -= decompressedBlockSize;
			}

			data = decompressedData;
			dataSize = decompressedSize;
			ownsData = true;
		}

		if (IsElfFile(data, dataSize))
			objectFile = new ObjectFileElf();
		else if (IsMachOFile(data, dataSize))
			objectFile = new ObjectFileMachO();
		else if (IsLLVMIRFile(data, dataSize))
			objectFile = new ObjectFileLLVMIR();
		else if (IsCoffFile(data, dataSize))
			objectFile = new ObjectFileCoff();
		else if (IsImportLib(data, dataSize))
			objectFile = new ObjectFileImportLib();
		else
		{
			if (ownsData)
				free(data);
			logger.Error(TC("Unknown object file format (Size %llu). Maybe msvc FE IL? (%s)"), dataSize, hint);
			return nullptr;
		}

		objectFile->m_data = data;
		objectFile->m_dataSize = dataSize;
		objectFile->m_ownsData = ownsData;

		if (objectFile->Parse(logger, parseMode, hint))
			return objectFile;

		if (ownsData)
			free(data);
		delete objectFile;
		return nullptr;
	}

	bool ObjectFile::CopyMemoryAndClose()
	{
		u8* data = (u8*)malloc(m_dataSize);
		UBA_ASSERT(false);
		if (!data)
			return false;
		memcpy(data, m_data, m_dataSize);
		if (m_ownsData)
			free(m_data);
		m_data = data;
		m_ownsData = true;
		delete m_file;
		m_file = nullptr;
		return true;
	}

	bool ObjectFile::WriteImportsAndExports(Logger& logger, MemoryBlock& memoryBlock, bool verbose, u8 version)
	{
		auto write = [&](const void* data, u64 dataSize) { memcpy(memoryBlock.Allocate(dataSize, 1, TC("ObjectFile::WriteImportsAndExports")), data, dataSize); };
		if (!WriteImportsAndExports(logger, write, verbose, version))
			return false;
		return true;
	}

	bool ObjectFile::WriteImportsAndExports(Logger& logger, const tchar* exportsFilename, bool verbose, u8 version)
	{
		FileAccessor exportsFile(logger, exportsFilename);
		if (!exportsFile.CreateWrite())
			return false;

		char buffer[256*1024];
		u64 bufferPos = 0;
		auto flush = [&]() { exportsFile.Write(buffer, bufferPos); bufferPos = 0; };
		auto write = [&](const void* data, u64 dataSize) { if (bufferPos + dataSize > sizeof(buffer)) flush(); memcpy(buffer + bufferPos, data, dataSize); bufferPos += dataSize; };

		if (!WriteImportsAndExports(logger, write, verbose, version))
			return false;

		flush();

		return exportsFile.Close();
	}

	template<typename WriteFunc>
	bool ObjectFile::WriteImportsAndExports(Logger& logger, const WriteFunc& write, bool verbose, u8 version)
	{
		write(&version, 1);
		write(&m_type, 1);
		write(&verbose, 1);

		{
			// Sort imports based on name
			Vector<const std::string*> sortedImports;
			sortedImports.resize(m_imports.size());
			u32 i = 0;
			for (auto& symbol : m_imports)
				sortedImports[i++] = &symbol;
			std::sort(sortedImports.begin(), sortedImports.end(), [](const std::string* a, const std::string* b) { return *a < *b; });

			// Write all imports
			for (auto symbol : sortedImports)
			{
				write(symbol->c_str(), symbol->size());
				write("\n", 1);
			}
			write("\n", 1);
		}


		if (verbose)
		{
			// Write all exports
			for (auto& kv : m_exports)
			{
				write(kv.second.symbol.c_str(), kv.second.symbol.size());
				if (kv.second.isData)
					write(",DATA", 5);
				write("\n", 1);
			}
			write("\n", 1);
		}
		else
		{
			// Sort exports based on stringkey
			struct SortedExport { const StringKey* key; u8 isData; };
			Vector<SortedExport> sortedExports;
			sortedExports.resize(m_exports.size());
			u32 i = 0;
			for (auto& kv : m_exports)
				sortedExports[i++] = { &kv.first, kv.second.isData };
			std::sort(sortedExports.begin(), sortedExports.end(), [](const SortedExport& a, const SortedExport& b) { return *a.key < *b.key; });

			// Write all exports
			u32 count = u32(sortedExports.size());
			write(&count, sizeof(count));
			for (auto& e : sortedExports)
			{
				write(e.key, sizeof(StringKey));
				write(&e.isData, 1);
			}
		}

		// Explicit exports — verbatim names (length-prefixed) + isData. Sorted by name so the .exi is deterministic.
		// Unlike the export set above (which can be a hash), these names go into the import lib literally, so the
		// string must survive the round-trip regardless of 'verbose'. Only present in v3+ (omitted for legacy v2).
		if (version >= 3)
		{
			Vector<const UnorderedExports::value_type*> sortedExplicit;
			sortedExplicit.reserve(m_explicitExports.size());
			for (auto& kv : m_explicitExports)
				sortedExplicit.push_back(&kv);
			std::sort(sortedExplicit.begin(), sortedExplicit.end(), [](const UnorderedExports::value_type* a, const UnorderedExports::value_type* b) { return a->second.symbol < b->second.symbol; });

			u32 explicitCount = u32(sortedExplicit.size());
			write(&explicitCount, sizeof(explicitCount));
			for (auto* kv : sortedExplicit)
			{
				const std::string& name = kv->second.symbol;
				u32 len = u32(name.size());
				write(&len, sizeof(len));
				write(name.data(), len);
				write(&kv->second.isData, 1);
			}
		}
		return true;
	}

	const char* ObjectFile::GetLibName()
	{
		UBA_ASSERT(false);
		return "";
	}

	ObjectFile::~ObjectFile()
	{
		if (m_ownsData)
			free(m_data);
		delete m_file;
	}

	void ObjectFile::RemoveExportedSymbol(const char* symbol)
	{
		m_exports.erase(ToStringKeyRaw(symbol, strlen(symbol)));
	}

	const tchar* ObjectFile::GetFileName() const
	{
		return m_file->GetFileName();
	}

	const UnorderedSymbols& ObjectFile::GetImports() const
	{
		return m_imports;
	}

	const UnorderedExports& ObjectFile::GetExports() const
	{
		return m_exports;
	}

	const UnorderedExports& ObjectFile::GetExplicitExports() const
	{
		return m_explicitExports;
	}

	const UnorderedSymbols& ObjectFile::GetPotentialDuplicates() const
	{
		return m_potentialDuplicates;
	}

	bool ObjectFile::CreateExtraFile(Logger& logger, const StringView& extraObjFilename, const StringView& moduleName, const StringView& platform, const AllExternalImports& allExternalImports, const UnorderedSymbols& allInternalImports, const AllExports& allExports, const AllExports& allExplicitExports, const ExtraExports& extraExports, bool includeExportsInFile)
	{
		ObjectFileCoff objectFileCoff;
		ObjectFileElf objectFileElf;
		
		MemoryBlock memoryBlock(TC("CreateExtraFile"), 64*1024*1024);

		bool res;
		if (extraObjFilename.EndsWith(TCV(".obj")))
			res = ObjectFileCoff::CreateExtraFile(logger, platform, memoryBlock, allExternalImports, allInternalImports, allExports, allExplicitExports, includeExportsInFile);
		else if (extraObjFilename.EndsWith(TCV(".dynlist")))
			res = CreateVersionScript(logger, memoryBlock, allExternalImports, allInternalImports, allExports, extraExports, includeExportsInFile, true);
		else if (extraObjFilename.EndsWith(TCV(".ldscript")))
			res = CreateVersionScript(logger, memoryBlock, allExternalImports, allInternalImports, allExports, extraExports, includeExportsInFile, false);
		else if (extraObjFilename.EndsWith(TCV(".emd")))
			res = CreateEmdFile(logger, memoryBlock, moduleName, allExternalImports, allInternalImports, allExports, includeExportsInFile);
		else if (extraObjFilename.EndsWith(TCV(".exp")))
			res = CreateExportedSymbolsList(logger, memoryBlock, allExternalImports, allInternalImports, allExports, extraExports);
		else
			res = ObjectFileElf::CreateExtraFile(logger, platform, memoryBlock, allExternalImports, allInternalImports, allExports, includeExportsInFile);

		if (!res)
			return false;

		FileAccessor extraFile(logger, extraObjFilename.data);
		if (!extraFile.CreateWrite())
			return false;

		if (!extraFile.Write(memoryBlock.memory, memoryBlock.writtenSize))
			return false;

		return extraFile.Close();
	}

	bool SymbolFile::ParseFile(Logger& logger, const tchar* filename)
	{
		UBA_ASSERT(*filename);
		FileAccessor symFile(logger, filename);
		if (!symFile.OpenMemoryRead())
			return false;
		if (symFile.GetSize() == 0)
			return logger.Error(TC("%s - Import/export file corrupt (size 0)"), filename);

		auto readPos = (const char*)symFile.GetData();

		u8 version = *(u8*)readPos++;
		if (version < 2 || version > SymbolFileVersion)
			return logger.Error(TC("%s - Import/export file version mismatch (application supports 2..%u, file version %u)"), filename, SymbolFileVersion, version);

		type = *(const ObjectFileType*)readPos++;
		bool verbose = *(bool*)readPos++;

		while (*readPos != '\n')
		{
			auto start = readPos;
			while (*readPos != '\n')
				++readPos;
			imports.insert(std::string(start, readPos - start));
			++readPos;
		}
		++readPos;

		if (verbose)
		{
			while (*readPos != '\n')
			{
				auto start = readPos;(void)start;
				const char* comma = nullptr;
				while (*readPos != '\n')
				{
					if (*readPos == ',')
						comma = readPos;
					++readPos;
				}

				auto end = readPos;

				ExportInfo info;
				if (comma)
				{
					end = comma;
					info.isData = true;
				}
				//info.symbol.assign(start, end - start);
				exports.emplace(ToStringKeyRaw(start, end - start), std::move(info));
				++readPos;
			}
			++readPos; // consume the empty-line terminator written after the verbose export list
		}
		else
		{
			u32 count = *(u32*)readPos;
			readPos += sizeof(u32);
			while (count--)
			{
				StringKey key = *(StringKey*)readPos;
				readPos += sizeof(StringKey);
				u8 isData = *readPos != 0;
				++readPos;
				ExportInfo info;
				info.isData = isData;
				exports.emplace(key, info);
			}
		}

		// Explicit exports — verbatim names (length-prefixed) + isData. Mirrors WriteImportsAndExports.
		// Only present in v3+; legacy v2 files leave explicitExports empty (old UbaObjTool behavior).
		if (version >= 3)
		{
			u32 explicitCount = *(u32*)readPos;
			readPos += sizeof(u32);
			while (explicitCount--)
			{
				u32 len = *(u32*)readPos;
				readPos += sizeof(u32);
				const char* name = readPos;
				readPos += len;
				u8 isData = *readPos != 0;
				++readPos;
				ExportInfo info;
				info.symbol.assign(name, len);
				info.isData = isData;
				explicitExports.emplace(ToStringKeyRaw(name, len), std::move(info));
			}
		}
		return true;
	}

	bool ObjectFile::CreateVersionScript(Logger& logger, MemoryBlock& memoryBlock, const AllExternalImports& allExternalImports, const UnorderedSymbols& allInternalImports, const AllExports& allExports, const ExtraExports& extraExports, bool includeExportsInFile, bool isDynList)
	{
		auto WriteString = [&](const char* str, u64 strLen) { memcpy(memoryBlock.Allocate(strLen, 1, TC("")), str, strLen); };

		bool isFirst = true;
		auto WriteSymbol = [&](const char* str, u64 strLen)
			{
				if (isFirst)
					WriteString("global:\n", 8);
				WriteString(str, strLen);
				WriteString(";\n", 2);
				isFirst = false;
			};

		//WriteString("VERSION ", 8);
		WriteString("{\n", 2);

		for (auto& imp : allExternalImports)
		{
			StringKey impKey = ToStringKeyRaw(imp.c_str(), imp.size());
			auto findIt = allExports.find(impKey);
			if (findIt != allExports.end())
				WriteSymbol(imp.c_str(), imp.size());
		}

		for (auto& symbol : extraExports)
			WriteSymbol(symbol.c_str(), symbol.size());

		if (!isDynList)
			WriteString("local: *;\n", 10);
		else if (isFirst)
			WriteSymbol("ThisIsAnUnrealEngineModule", 26); // Workaround for tool not liking empty lists

		WriteString("};", 2);

		return true;
	}

	bool ObjectFile::CreateExportedSymbolsList(Logger& logger, MemoryBlock& memoryBlock, const AllExternalImports& allExternalImports, const AllInternalImports& allInternalImports, const AllExports& allExports, const ExtraExports& extraExports)
	{
		// Apple ld64 / ld-prime -exported_symbols_list format: one symbol per line.
		// Symbol names retain the leading underscore from the Mach-O C ABI.
		auto WriteString = [&](const char* str, u64 strLen) { memcpy(memoryBlock.Allocate(strLen, 1, TC("")), str, strLen); };
		auto WriteSymbol = [&](const char* str, u64 strLen)
			{
				WriteString(str, strLen);
				WriteString("\n", 1);
			};

		bool symbolAdded = false;

		for (auto& imp : allExternalImports)
		{
			StringKey impKey = ToStringKeyRaw(imp.c_str(), imp.size());
			auto findIt = allExports.find(impKey);
			if (findIt == allExports.end())
				continue;
			WriteSymbol(imp.c_str(), imp.size());
			symbolAdded = true;
		}

		for (auto& symbol : extraExports)
		{
			WriteSymbol(symbol.c_str(), symbol.size());
			symbolAdded = true;
		}

		if (!symbolAdded)
			WriteSymbol("_ThisIsAnUnrealEngineModule", 27); // ld64 warns (not errors) on a missing symbol; keeps an empty list legal

		return true;
	}

	bool ObjectFile::CreateEmdFile(Logger& logger, MemoryBlock& memoryBlock, const StringView& moduleName, const AllExternalImports& allExternalImports, const UnorderedSymbols& allInternalImports, const AllExports& allExports, bool includeExportsInFile)
	{
		auto WriteString = [&](const char* str, u64 strLen) { memcpy(memoryBlock.Allocate(strLen, 1, TC("")), str, strLen); };

		char moduleName2[256];
		u32 moduleNameLen = StringBuffer<>(moduleName.data).Parse(moduleName2, 256) - 1;

		WriteString("Library: ", 9);
		WriteString(moduleName2, moduleNameLen);
		WriteString(" { export: {\n", 13);

		bool symbolAdded = false;

		for (auto& imp : allExternalImports)
		{
			StringKey impKey = ToStringKeyRaw(imp.c_str(), imp.size());
			auto findIt = allExports.find(impKey);
			if (findIt == allExports.end())
				continue;
			WriteString(imp.c_str(), imp.size());
			WriteString("\n", 1);
			symbolAdded = true;
		}

		if (!symbolAdded)
			WriteString("ThisIsAnUnrealEngineModule\n", 27); // Workaround for tool not liking empty lists

		WriteString("}}", 2);

		return true;
	}
}
