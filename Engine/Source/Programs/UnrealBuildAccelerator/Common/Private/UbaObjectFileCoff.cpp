// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaObjectFileCoff.h"
#include "UbaFileAccessor.h"

namespace uba
{
	constexpr u16 ImageFileMachineUnknown = 0;
	constexpr u8 ImageSizeofShortName = 8; // IMAGE_SIZEOF_SHORT_NAME
	constexpr u16 ImageSymClassExternal = 0x0002; // IMAGE_SYM_CLASS_EXTERNAL
	constexpr u16 ImageSymClassStatic = 0x0003; // IMAGE_SYM_CLASS_STATIC
	constexpr u32 ImageSymUndefined = 0; // IMAGE_SYM_UNDEFINED

	constexpr u32 ImageScnCntCode = 0x00000020; // IMAGE_SCN_CNT_CODE
	constexpr u32 ImageScnCntInitializedData = 0x00000040; // IMAGE_SCN_CNT_INITIALIZED_DATA
	constexpr u32 ImageScnLnkInfo = 0x00000200; // IMAGE_SCN_LNK_INFO
	constexpr u32 ImageScnLnkRemove = 0x00000800; // IMAGE_SCN_LNK_REMOVE
	constexpr u32 ImageScnLnkComdat = 0x00001000; // IMAGE_SCN_LNK_COMDAT
	constexpr u32 ImageScnAlign1Bytes = 0x00100000; // IMAGE_SCN_ALIGN_1BYTES
	constexpr u32 ImageScnLnkNRelocOvfl = 0x01000000; // IMAGE_SCN_LNK_NRELOC_OVFL
	constexpr u32 ImageScnMemExecute = 0x20000000; // IMAGE_SCN_MEM_EXECUTE
	constexpr u32 ImageScnMemRead = 0x40000000; // IMAGE_SCN_MEM_READ

	const u16 ImageRelAmd64Addr64 = 0x0001; // IMAGE_REL_AMD64_ADDR64


	#pragma pack(push)
	#pragma pack(1)

	struct ImageFileHeader // IMAGE_FILE_HEADER
	{
		u16 Machine;
		u16 NumberOfSections;
		u32 TimeDateStamp;
		u32 PointerToSymbolTable;
		u32 NumberOfSymbols;
		u16 SizeOfOptionalHeader;
		u16 Characteristics;
	};

	struct AnonObjectHeaderBigobj // ANON_OBJECT_HEADER_BIGOBJ
	{
		u16 Sig1;            // Must be IMAGE_FILE_MACHINE_UNKNOWN
		u16 Sig2;            // Must be 0xffff
		u16 Version;         // >= 2 (implies the Flags field is present)
		u16 Machine;         // Actual machine - IMAGE_FILE_MACHINE_xxx
		u32 TimeDateStamp;
		Guid ClassID;         // {D1BAA1C7-BAEE-4ba9-AF20-FAF66AA4DCB8}
		u32 SizeOfData;      // Size of data that follows the header
		u32 Flags;           // 0x1 -> contains metadata
		u32 MetaDataSize;    // Size of CLR metadata
		u32 MetaDataOffset;  // Offset of CLR metadata

		// bigobj specifics
		u32 NumberOfSections; // extended from WORD
		u32 PointerToSymbolTable;
		u32 NumberOfSymbols;
	};

	struct ImageSectionHeader // IMAGE_SECTION_HEADER
	{
		u8 Name[ImageSizeofShortName];
		union
		{
			u32 PhysicalAddress;
			u32 VirtualSize;
		} Misc;
		u32 VirtualAddress;
		u32 SizeOfRawData;
		u32 PointerToRawData;
		u32 PointerToRelocations;
		u32 PointerToLinenumbers;
		u16 NumberOfRelocations;
		u16 NumberOfLinenumbers;
		u32 Characteristics;
	};
	static_assert(sizeof(ImageSectionHeader) == 40);

	struct ImageRelocation // IMAGE_RELOCATION
	{
		union
		{
			u32 VirtualAddress;
			u32 RelocCount;
		};
		u32 SymbolTableIndex;
		u16 Type;
	};


	struct ImageSymbolEx // IMAGE_SYMBOL_EX
	{
		union
		{
			char ShortName[8];
			struct
			{
				u32 Short; // if 0, use LongName
				u32 Long; // offset into string table
			} Name;
			u32 LongName[2]; // PBYTE  [2]
		} N;
		u32 Value;
		u32 SectionNumber;
		u16 Type;
		u8 StorageClass;
		u8 NumberOfAuxSymbols;
	};

	struct ImageSymbol // IMAGE_SYMBOL
	{
		union
		{
			u8 ShortName[8];
			struct
			{
				u32   Short; // if 0, use LongName
				u32   Long; // offset into string table
			} Name;
			u32   LongName[2]; // PBYTE [2]
		} N;
		u32 Value;
		u16 SectionNumber;
		u16 Type;
		u8 StorageClass;
		u8 NumberOfAuxSymbols;
	};

	#pragma pack(pop)

	bool IsBigObj(const u8* data, u64 size)
	{
		if (size < sizeof(AnonObjectHeaderBigobj))
			return false;
		auto& header = *(AnonObjectHeaderBigobj *)data;
		if (header.Sig1 != ImageFileMachineUnknown)
			return false;
		if (header.Sig2 != 0xffff)
			return false;
		if (header.Version < 2)
			return false;
		constexpr u8 bigObjClassId[16] = { 0xc7, 0xa1, 0xba, 0xd1, 0xee, 0xba, 0xa9, 0x4b, 0xaf, 0x20, 0xfa, 0xf6, 0x6a, 0xa4, 0xdc, 0xb8 };
		if (header.ClassID != *(const Guid*)bigObjClassId)
			return false;
		return true;
	}

	bool IsCoffFile(const u8* data, u64 dataSize)
	{
		if (IsBigObj(data, dataSize))
			return true;
		if (dataSize < sizeof(ImageFileHeader) + 8)
			return false;

		// TODO: This is not a solid way to identify a coff file.. don't know how this should be done
		auto header = *(ImageFileHeader*)data;
		if (header.Machine != 0x8664) // TODO: Add whatever other machines supported (ARM)
			return false;
		if (header.SizeOfOptionalHeader != 0) // Should always be 0
			return false;
		if (header.Characteristics != 0) // Should always be 0
			return false;
		// We expect .text, .rdata or .drectve sections first.. this is also hacky.. but don't know how to verify that a file is a coff file
		const u8* firstSection = data + sizeof(ImageFileHeader);
		return memcmp(firstSection, ".text", 5) == 0 || memcmp(firstSection, ".drectve", 8) == 0 || memcmp(firstSection, ".rdata", 6) == 0;
	}

	template<typename SymbolType>
	ObjectFile::AnsiStringView GetSymbolName(SymbolType& symbol, const u8* data, u32 stringTableMemPos)
	{
		if (symbol.N.Name.Short == 0)
		{
			auto name = (const char*)(data + stringTableMemPos + symbol.N.Name.Long);
			return { name, name + strlen(name) };
		}
		auto shortName = (char*)symbol.N.ShortName;
		return { shortName, shortName + strnlen(shortName, ImageSizeofShortName) };
	}

	ObjectFileCoff::ObjectFileCoff()
	{
		m_type = ObjectFileType_Coff;
	}

	bool ObjectFileCoff::Parse(Logger& logger, ObjectFileParseMode parseMode, const tchar* hint)
	{
		m_isBigObj = IsBigObj(m_data, m_dataSize);
		
		if (m_isBigObj)
		{
			auto& header = *(AnonObjectHeaderBigobj*)m_data;
			m_info.symbolsMemPos = header.PointerToSymbolTable;
			m_info.symbolCount = header.NumberOfSymbols;
			m_info.stringTableMemPos = header.PointerToSymbolTable + header.NumberOfSymbols * sizeof(ImageSymbolEx);
			m_info.sectionsMemOffset = sizeof(AnonObjectHeaderBigobj);
			m_info.sectionCount = header.NumberOfSections;
		}
		else
		{
			auto& header = *(ImageFileHeader*)m_data;
			m_info.symbolsMemPos = header.PointerToSymbolTable;
			m_info.symbolCount = header.NumberOfSymbols;
			m_info.stringTableMemPos = header.PointerToSymbolTable + header.NumberOfSymbols * sizeof(ImageSymbol);
			m_info.sectionsMemOffset = sizeof(ImageFileHeader);
			m_info.sectionCount = header.NumberOfSections;
		}

		if (parseMode == ObjectFileParseMode_Exports)
		{
			if (!ParseExports(m_exports))
				return false;
		}
		else
		{
			if (m_isBigObj)
				ParseAllSymbols<ImageSymbolEx>();
			else
				ParseAllSymbols<ImageSymbol>();

			// Also record what this file explicitly exports (its .drectve /EXPORT: directives), kept separate from
			// the full defined-symbol set in m_exports. These are the symbols that must end up in the import lib
			// even when no in-build dependent imports them. Verbatim names (needed downstream by the lib writer).
			if (!ParseExports(m_explicitExports))
				return false;
		}
		return true;
	}

	bool ObjectFileCoff::ParseExports(UnorderedExports& outExports)
	{
		auto sections = (ImageSectionHeader*)(m_data + m_info.sectionsMemOffset);
		for (u32 i=0; i!=m_info.sectionCount; ++i)
		{
			if (strncmp((char*)sections[i].Name, ".drectve", 8) != 0)
				continue;
			m_info.directiveSectionMemOffset = u64((u8*)(sections + i) - m_data);
			break;
		}
		if (!m_info.directiveSectionMemOffset)
			return true;

		auto directiveSection = (ImageSectionHeader*)(m_data + m_info.directiveSectionMemOffset);
		u8* directiveData = m_data + directiveSection->PointerToRawData;
		u8* directiveEnd = directiveData + directiveSection->SizeOfRawData;

		static constexpr u8 utf8Bom[3] = { 0xef, 0xbb, 0xbf };
		UBA_ASSERT(memcmp(directiveData, utf8Bom, 3) != 0);

		std::string tmp;

		u32 index = 0;

		auto str = (char*)directiveData;
		auto strEnd = (char*)directiveEnd;
		while (str)
		{
			char* exportStr = strstr(str, "/EXPORT:");
			if (!exportStr)
				break;
			exportStr += 8;
			char* exportEnd;
			if (*exportStr == '\"')
			{
				++exportStr;
				exportEnd = strchr(exportStr, '\"');
				str = exportEnd + 1;
			}
			else
			{
				exportEnd = (char*)memchr(exportStr, ' ', strEnd - exportStr);
				str = exportEnd;
				if (!exportEnd)
					exportEnd = strEnd;
				else
					++str;
			}

			tmp.assign(exportStr, exportEnd);
			bool isData = false;
			if (const char* comma = strchr(tmp.c_str(), ','))
			{
				isData = true;
				tmp = tmp.substr(0, comma - tmp.data());
			}
			StringKey key = ToStringKeyRaw(tmp.data(), tmp.size());
			outExports.emplace(key, ExportInfo{tmp, isData, index++});
		}
		return true;
	}

	template<typename SymbolType>
	void ObjectFileCoff::ParseAllSymbols()
	{
		u32 index = 0;
		std::string symbolString;

		auto symbols = (SymbolType*)(m_data + m_info.symbolsMemPos);
		auto sections = (ImageSectionHeader*)(m_data + m_info.sectionsMemOffset);

		for (u32 i=0, e=m_info.symbolCount; i!=e; ++i)
		{
			auto& symbol = symbols[i];
			i += symbol.NumberOfAuxSymbols;

			if (symbol.StorageClass != ImageSymClassExternal && symbol.StorageClass != ImageSymClassStatic)
				continue;

			// Skip COMDAT section definition symbols — these are section headers, not real code/data symbols.
			// They always have: STATIC storage class, Value=0 (section offset), Type=0 (no type info), 1 aux record.
			if (symbol.StorageClass == ImageSymClassStatic && symbol.Value == 0 && symbol.Type == 0 && symbol.NumberOfAuxSymbols == 1)
				continue;

			AnsiStringView symbolName = GetSymbolName(symbol, m_data, m_info.stringTableMemPos);

			if (symbolName[0] == '$') // exception handling etc
				continue;

			if (symbol.SectionNumber != ImageSymUndefined)
			{
				if (symbolName.StartsWith("__", 2)) // compiler-generated internal symbols
					continue;
				if (symbolName[0] == '.') // sections
					continue;

				u8 isData = 0;
				if (symbol.SectionNumber != decltype(symbol.SectionNumber)(-1) && symbol.SectionNumber != decltype(symbol.SectionNumber)(-2))
				{
					UBA_ASSERTF(symbol.SectionNumber <= m_info.sectionCount, TC("%i / %i"), symbol.SectionNumber, m_info.sectionCount);
					auto& section = sections[symbol.SectionNumber-1];
					if (memcmp(section.Name, ".data", 6) == 0 || memcmp(section.Name, ".rdata", 7) == 0 || memcmp(section.Name, ".bss", 5) == 0)
						isData = 1;
				}
				symbolName.ToString(symbolString);
				StringKey symbolKey = ToStringKeyRaw(symbolString.data(), symbolString.size());
				auto res = m_exports.emplace(symbolKey, ExportInfo{symbolString, isData, index});
				UBA_ASSERT(res.second || (res.first->second.isData == isData && res.first->second.symbol == symbolString));
				if (res.second)
					++index;
			}
			else
			{
				UBA_ASSERT(symbol.StorageClass == ImageSymClassExternal);
				symbolName.ToString(symbolString);
				m_imports.emplace(symbolString);
			}
		}
	}

	bool ObjectFileCoff::CreateExtraFile(Logger& logger, const StringView& platform, MemoryBlock& memoryBlock, const AllExternalImports& allExternalImports, const AllInternalImports& allInternalImports, const AllExports& allExports, const AllExports& allExplicitExports, bool includeExportsInFile)
	{
		std::string tmp;

		u32 totalStringSize = 0;
		UnorderedSymbols neededLoopbacks;
		for (auto& symbol : allInternalImports)
		{
			if (strncmp(symbol.data(), "__imp_", 6) != 0)
				continue;
			tmp = symbol.substr(6);
			StringKey key = ToStringKeyRaw(tmp.data(), tmp.size());
			if (allExports.find(key) == allExports.end())
				continue;
			if (neededLoopbacks.insert(symbol).second)
				totalStringSize += u32(symbol.size()) + 1;
		}
		u32 loopbackCount = u32(neededLoopbacks.size());

		auto allocate = [&](u64 size) { return memoryBlock.Allocate(size, 1, TC("")); };
		auto write = [&](const void* data, u64 size) { memcpy(allocate(size), data, size); };

		// File header (allocator returns zeroed memory)
		auto& header = *(ImageFileHeader*)allocate(sizeof(ImageFileHeader));
		header.Machine = 0x8664;

		// Section for loopback data
		auto& rdataSection = *(ImageSectionHeader*)allocate(sizeof(ImageSectionHeader));
		memcpy(rdataSection.Name, ".rdata\0\0", 8);
		rdataSection.Characteristics = ImageScnCntInitializedData | ImageScnMemRead;
		u16 rdataSectionIndex0 = header.NumberOfSections; // 0-based index
		++header.NumberOfSections;

		// Optional .drectve with exports
		if (includeExportsInFile)
		{
			auto& directiveSection = *(ImageSectionHeader*)allocate(sizeof(ImageSectionHeader));
			memcpy(directiveSection.Name, ".drectve", 8);
			directiveSection.Characteristics = ImageScnAlign1Bytes | ImageScnLnkInfo | ImageScnLnkRemove;
			++header.NumberOfSections;

			auto writeExport = [&](AnsiStringView symbol, bool isData)
			{
				write("/EXPORT:", 8);
				write(symbol.strBegin, symbol.Length());
				if (isData)
					write(",DATA", 5);
				write(" ", 1);
			};

			u32 directiveRawDataStart = u32(memoryBlock.writtenSize);
			directiveSection.PointerToRawData = directiveRawDataStart;

			// Dedup so a symbol that is both demanded by a dependent and explicitly exported only emits once.
			UnorderedSet<StringKey> writtenExports;

			for (auto& imp : allExternalImports)
			{
				AnsiStringView impSym(imp);
				if (impSym.StartsWith("__imp_", 6))
					impSym = impSym.Skip(6);
				StringKey impKey = ToStringKeyRaw(impSym.strBegin, impSym.Length());
				auto findIt = allExports.find(impKey);
				if (findIt == allExports.end())
					continue;
				if (!writtenExports.insert(impKey).second)
					continue;
				writeExport(impSym, findIt->second.isData != 0);
			}

			tmp = "ThisIsAnUnrealEngineModule";
			StringKey isUnrealKey = ToStringKeyRaw(tmp.data(), tmp.size());
			if (allExports.find(isUnrealKey) != allExports.end())
				if (writtenExports.insert(isUnrealKey).second)
					writeExport(tmp, false);

			// Explicit exports (source files' own /EXPORT: directives) — must be exported even if no dependent
			// imports them, so they are not subject to the allExternalImports ∩ allExports intersection above.
			for (auto& kv : allExplicitExports)
				if (writtenExports.insert(kv.first).second)
					writeExport(AnsiStringView(kv.second.symbol), kv.second.isData != 0);

			write("", 1);
			directiveSection.SizeOfRawData = u32(memoryBlock.writtenSize) - directiveRawDataStart;
		}

		// Raw data for loopback pointers (8 bytes each, zero-initialized)
		u32 dataRawPos = u32(memoryBlock.writtenSize);
		allocate(loopbackCount * 8);

		bool useOverflow = loopbackCount > 0xFFFF;

		// Relocations for each 8-byte slot (pointing at the base symbol, i.e., "foo")
		u32 relocFilePos = u32(memoryBlock.writtenSize);
		u32 relocRecordCount = useOverflow ? (loopbackCount + 1) : loopbackCount;
		auto relocations = (ImageRelocation*)allocate(relocRecordCount * sizeof(ImageRelocation));
		rdataSection.PointerToRawData   = dataRawPos;
		rdataSection.SizeOfRawData      = loopbackCount * 8;
		rdataSection.PointerToRelocations = relocFilePos;
		rdataSection.NumberOfRelocations  = u16(loopbackCount);

		if (useOverflow)
		{
			rdataSection.Characteristics |= ImageScnLnkNRelocOvfl;
			rdataSection.NumberOfRelocations = 0xFFFF;

			// First entry is the overflow header
			// +1 to include this overflow header record itself (per PE/COFF NRELOC_OVFL)
			relocations[0].VirtualAddress   = loopbackCount + 1;
			relocations[0].SymbolTableIndex = 0;
			relocations[0].Type             = 0;
			++relocations;
		}

		for (u32 i = 0; i != loopbackCount; ++i)
		{
			auto& relocation = relocations[i];
			relocation.VirtualAddress   = 8 * i;   // offset in section
			relocation.SymbolTableIndex = i;       // refers to first half of symbols (undefined "foo")
			relocation.Type             = ImageRelAmd64Addr64;
		}

		// Symbol table
		header.PointerToSymbolTable = u32(memoryBlock.writtenSize);
		header.NumberOfSymbols = loopbackCount * 2;
		auto symbols = (ImageSymbol*)allocate(header.NumberOfSymbols * sizeof(ImageSymbol));

		// String table: write BOTH "foo" and "__imp_foo" as independent entries
		u64 stringStart = memoryBlock.writtenSize;
		auto& stringTableSize = *(u32*)allocate(4);
		Vector<u32> offBase;    offBase.reserve(loopbackCount);    // "foo"
		Vector<u32> offImpBase; offImpBase.reserve(loopbackCount); // "__imp_foo"

		for (auto& s : neededLoopbacks)
		{
			// s == "__imp_foo"
			std::string base = s.substr(6); // "foo"

			offBase.push_back(u32(memoryBlock.writtenSize - stringStart));
			write(base.data(), base.size() + 1);

			offImpBase.push_back(u32(memoryBlock.writtenSize - stringStart));
			write(s.data(), s.size() + 1);
		}
		stringTableSize = u32(memoryBlock.writtenSize - stringStart); // includes the 4-byte size field

		// First N symbols: undefined "foo" (targets for relocations)
		for (u32 i = 0; i != loopbackCount; ++i)
		{
			auto& sym = symbols[i];
			sym.N.Name.Long  = offBase[i];             // points to "foo"
			sym.SectionNumber = ImageSymUndefined;
			sym.StorageClass  = ImageSymClassExternal;
			// Type/Value/Aux are 0 from allocator
		}

		// Next N symbols: defined "__imp_foo" in our .rdata section at offset i*8
		for (u32 i = 0; i != loopbackCount; ++i)
		{
			auto& sym = symbols[i + loopbackCount];
			sym.N.Name.Long   = offImpBase[i];         // points to "__imp_foo"
			sym.SectionNumber = (short)(rdataSectionIndex0 + 1); // 1-based section index
			sym.StorageClass  = ImageSymClassExternal;
			sym.Value         = i * 8;                 // offset of the pointer slot
		}

		return true;
	}
}
