// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaObjectFile.h"

namespace uba
{

	class ObjectFileCoff : public ObjectFile
	{
	public:
		ObjectFileCoff();
		virtual bool Parse(Logger& logger, ObjectFileParseMode parseMode, const tchar* hint) override;

		static bool CreateExtraFile(Logger& logger, const StringView& platform, MemoryBlock& memoryBlock, const AllExternalImports& allExternalImports, const AllInternalImports& allInternalImports, const UnorderedExports& allSharedExports, const UnorderedExports& allExplicitExports, bool includeExportsInFile);

	private:
		struct Info;

		bool ParseExports(UnorderedExports& outExports);
		template<typename SymbolType> void ParseAllSymbols();

		struct Info
		{
			u32 sectionsMemOffset = 0;
			u32 sectionCount = 0;
			u64 directiveSectionMemOffset = 0;
			u32 stringTableMemPos = 0;
			u32 symbolsMemPos = 0;
			u32 symbolCount = 0;
		};

		bool m_isBigObj = false;
		Info m_info;

		UnorderedSymbols m_loopbacksToAdd;
		UnorderedSymbols m_toRemove;
	};

	bool IsCoffFile(const u8* data, u64 dataSize);
}
