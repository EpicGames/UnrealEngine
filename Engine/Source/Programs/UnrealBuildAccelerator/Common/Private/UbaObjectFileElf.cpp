// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaObjectFileElf.h"
#include "UbaElf.h"
#include "UbaFileAccessor.h"

namespace uba
{
	struct Elf64Sym
	{
		u32	st_name;		// Symbol name (string tbl index)
		u8	st_info;		// Symbol type and binding
		u8	st_other;		// Symbol visibility
		u16	st_shndx;		// Section index
		u64	st_value;		// Symbol value
		u64	st_size;		// Symbol size
	};

	struct Elf64Rel
	{
		u64	r_offset;
		u64	r_info;
	};
 
	struct Elf64Rela
	{
		u64	r_offset;
		u64	r_info;
		u64	r_addend;
	};

	#define ELF64_R_SYM(i)			((i) >> 32)
	#define ELF64_R_TYPE(i)			((i) & 0xffffffff)
	#define ELF64_R_INFO(sym,type)		((((u64) (sym)) << 32) + (type))

	#define ELF32_ST_VISIBILITY(o)	((o) & 0x03)
	#define ELF64_ST_VISIBILITY(o)	ELF32_ST_VISIBILITY (o)

	#define EM_X86_64	62		// AMD x86-64 architecture

	#define SHT_PROGBITS  1		// Program data
	#define SHT_SYMTAB	  2		// Symbol table
	#define SHT_STRTAB	  3
	#define SHT_RELA	  4		// Relocation entries with addends
	#define SHT_DYNAMIC	  6		// Dynamic linking information
	#define SHT_REL		  9		// Relocation entries, no addends
	#define SHT_DYNSYM	  11	// Dynamic linker symbol table
	#define SHT_SYMTAB_SHNDX  18 // Extended section indeces

	#define STT_NOTYPE	0		// Symbol type is unspecified
	#define STT_OBJECT	1		// Symbol is a data object
	#define STT_FUNC	2		// Symbol is a code object
	#define STT_SECTION	3		// Symbol associated with a section

	#define STB_LOCAL	0		// Local symbol
	#define STB_GLOBAL	1		// Global symbol
	#define STB_WEAK	2		// Weak symbol

	#define STV_DEFAULT	0		// Default symbol visibility rules
	#define STV_INTERNAL	1	// Processor specific hidden class
	#define STV_HIDDEN	2		// Sym unavailable in other modules
	#define STV_PROTECTED	3	// Not preemptible, not exported

	#define ELF32_ST_BIND(val)		(((unsigned char) (val)) >> 4)
	#define ELF32_ST_TYPE(val)		((val) & 0xf)
	#define ELF32_ST_INFO(bind, type)	(((bind) << 4) + ((type) & 0xf))

	// Both Elf32_Sym and Elf64_Sym use the same one-byte st_info field.
	#define ELF64_ST_BIND(val)		ELF32_ST_BIND(val)
	#define ELF64_ST_TYPE(val)		ELF32_ST_TYPE (val)
	#define ELF64_ST_INFO(bind, type)	ELF32_ST_INFO ((bind), (type))

	bool IsElfFile(const u8* data, u64 dataSize)
	{
		constexpr u8 magic[] = { 0x7f, 'E', 'L', 'F' };
		return dataSize > 4 && memcmp(data, magic, sizeof(magic)) == 0;
	}


	ObjectFileElf::ObjectFileElf()
	{
		m_type = ObjectFileType_Elf;
	}


	// names = the string table that owns this symbol's section (.strtab for SHT_SYMTAB, .dynstr for SHT_DYNSYM).
	const char* GetSymbolName(const Elf64SectionHeader* sections, const Elf64Sym& symbol, const char* names, const char* sectionNamesTable)
	{
		u8 symbolType = ELF64_ST_TYPE(symbol.st_info);
		if (symbolType == STT_FUNC || symbolType == STT_NOTYPE || symbolType == STT_OBJECT)
			return names + symbol.st_name;
		if (symbolType == STT_SECTION && symbol.st_shndx != 65535)
			return sectionNamesTable + sections[symbol.st_shndx].sh_name;
		return "";
	}

	bool ObjectFileElf::Parse(Logger& logger, ObjectFileParseMode parseMode, const tchar* hint)
	{
		auto& header = *(Elf64FileHeader*)m_data;
		if (header.e_ident[4] != 2) // 64-bit  (1 is 32-bit)
			return false;
		if (header.e_ident[5] != 1) // Little endian
			return false;
		if (header.e_ident[6] != 1) // Version
			return false;
		//if (header.e_type != 1 && header.e_type != 0xfe0c) // Relocatable file
		//	return false;

		UBA_ASSERT(sizeof(Elf64SectionHeader) == header.e_shentsize);

		auto sections = (Elf64SectionHeader*)(m_data + header.e_shoff);

		auto& namesSection = sections[header.e_shstrndx];
		UBA_ASSERT(namesSection.sh_type == SHT_STRTAB);
		char* sectionNamesTable = (char*)m_data + namesSection.sh_offset;

		u64 sectionCount = header.e_shnum ? header.e_shnum : sections[0].sh_size;

		for (u64 i=0; i!=sectionCount; ++i)
		{
			auto& section = sections[i];
			char* sectionName = sectionNamesTable + section.sh_name;

			if (section.sh_type == SHT_STRTAB)
			{
				if (strcmp(sectionName, ".strtab") == 0)
					m_symTableNamesOffset = section.sh_offset;
				else if (strcmp(sectionName, ".dynstr") == 0)
					m_dynTableNamesOffset = section.sh_offset;
			}
			else if (section.sh_type == SHT_PROGBITS)
			{
				if (strcmp(sectionName, ".linker_cmd") == 0)
					m_useVisibilityForExports = false;
			}
		}

		char* symTableNames = (char*)m_data + m_symTableNamesOffset;
		char* dynTableNames = (char*)m_data + m_dynTableNamesOffset;

		for (u64 i=0; i!=sectionCount; ++i)
		{
			auto& section = sections[i]; 

			char* sectionName = sectionNamesTable + section.sh_name;
			//printf("TYPE: %3llu - %14u %-30s  Offset: 0x%llx Size: %llu\n", i, section.sh_type, sectionName, section.sh_offset, section.sh_size);

			if (section.sh_type == SHT_SYMTAB || section.sh_type == SHT_DYNSYM)
			{
				UBA_ASSERT(section.sh_entsize == sizeof(Elf64Sym));
				auto symbols = (Elf64Sym*)(m_data + section.sh_offset);
				u64 symbolCount = section.sh_size / sizeof(Elf64Sym);
				for (u64 j=0; j!=symbolCount; ++j)
				{
					auto& symbol = symbols[j];
					const char* names = section.sh_type == SHT_DYNSYM ? dynTableNames : symTableNames;
					const char* symbolName = GetSymbolName(sections, symbol, names, sectionNamesTable);
					if (!*symbolName)
						continue;

					u8 symbolType = ELF64_ST_TYPE(symbol.st_info);
					u32 symbolBinding = ELF64_ST_BIND(symbol.st_info);
					u32 symbolVisibility = ELF64_ST_VISIBILITY(symbol.st_other);

					// Keep hidden/internal globals as exports too, tagged isHidden. Visible symbols are auto-exported;
					// hidden ones are candidates that downstream logic (the merge strip pass) promotes to exported only
					// when a dependent imports them. Dropping hidden here would empty the .exi for hidden-by-default
					// platforms (e.g. Sony), breaking export promotion.
					bool isHidden = !(symbolVisibility == STV_DEFAULT || symbolVisibility == STV_PROTECTED);

					if (symbolBinding == STB_GLOBAL)
					{
						if (symbolType != STT_NOTYPE)
						{
							std::string tmp = symbolName;
							StringKey key = ToStringKeyRaw(tmp.data(), tmp.size());
							m_exports.emplace(key, ExportInfo{std::move(tmp), u8(symbolType == STT_OBJECT), 0, symbol.st_size, u8(isHidden)});
						}
						else
							m_imports.emplace(symbolName);
					}
					else if (symbolBinding == STB_WEAK)
					{
						if (symbolType != STT_NOTYPE)
						{
							std::string tmp = symbolName;
							StringKey key = ToStringKeyRaw(tmp.data(), tmp.size());
							m_exports.emplace(key, ExportInfo{std::move(tmp), u8(symbolType == STT_OBJECT), 0, symbol.st_size, u8(isHidden)});
						}
					}
					else if (symbolBinding == STB_LOCAL)
					{
						if (symbolName[0] == '.')
							continue;
						std::string tmp = symbolName;
						StringKey key = ToStringKeyRaw(tmp.data(), tmp.size());
						m_exports.emplace(key, ExportInfo{std::move(tmp), u8(symbolType == STT_OBJECT), 0, symbol.st_size, u8(isHidden)});
					}
				}
			}
			else if (!m_useVisibilityForExports && section.sh_type == SHT_PROGBITS)
			{
				if (strcmp(sectionName, ".linker_cmd") == 0)
				{
					u8* it = m_data + section.sh_offset;
					u8* end = it + section.sh_size;
					while (it < end)
					{
						it += 4;
						std::string symbolName((char*)it);
						u64 symbolLen = symbolName.size();
						StringKey key = ToStringKeyRaw(symbolName.data(), symbolLen);
						m_exports.emplace(key, ExportInfo{std::move(symbolName), 0});
						it += symbolLen + 1;
					}
				}
			}
		}

		return true;
	}

	bool ObjectFileElf::CreateExtraFile(Logger& logger, const StringView& platform, MemoryBlock& memoryBlock, const AllExternalImports& allExternalImports, const AllInternalImports& allInternalImports, const AllExports& allExports, bool includeExportsInFile)
	{
		u8* data = (u8*)memoryBlock.Allocate(sizeof(Elf64FileHeader) + sizeof(Elf64SectionHeader) + 1, 1, TC(""));
		auto& header = *(Elf64FileHeader*)data;

		header.e_ident[0] = 0x7f;
		header.e_ident[1] = 'E';
		header.e_ident[2] = 'L';
		header.e_ident[3] = 'F';
		header.e_ident[4] = 2;
		header.e_ident[5] = 1;
		header.e_ident[6] = 1;
		header.e_ident[7] = platform.Equals(TCV("PS4")) ? 9 : 0; // OS ABI identification
		header.e_type = 1;
		header.e_machine = EM_X86_64;
		header.e_version = 1;
		header.e_ehsize = sizeof(Elf64FileHeader);
		header.e_flags = 0;//0x04000000;
		header.e_shoff = sizeof(Elf64FileHeader);
		header.e_shentsize = sizeof(Elf64SectionHeader);
		header.e_shnum = 1;
		header.e_shstrndx = 0;

		auto& section = *(Elf64SectionHeader*)(data + sizeof(Elf64FileHeader));
		section.sh_name = 0;
		section.sh_type = 3;
		section.sh_flags = 0;//0xC35d00;
		section.sh_addralign = 1;
		section.sh_offset = sizeof(Elf64FileHeader) + sizeof(Elf64SectionHeader);
		section.sh_size = 1;
		return true;
	}
}
