// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaObjectFile.h"

namespace uba
{
	class ObjectFileElf : public ObjectFile
	{
	public:
		ObjectFileElf();
		virtual bool Parse(Logger& logger, ObjectFileParseMode parseMode, const tchar* hint) override;

		static bool CreateExtraFile(Logger& logger, const StringView& platform, MemoryBlock& memoryBlock, const AllExternalImports& allExternalImports, const AllInternalImports& allInternalImports, const AllExports& allExports, bool includeExportsInFile);

	private:
		u64 m_symTableNamesOffset = 0;
		u64 m_dynTableNamesOffset = 0;
		bool m_useVisibilityForExports = true;
	};

	bool IsElfFile(const u8* data, u64 dataSize);
}
