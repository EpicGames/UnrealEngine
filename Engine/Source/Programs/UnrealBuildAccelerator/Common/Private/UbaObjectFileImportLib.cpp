// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaObjectFileImportLib.h"

namespace uba
{
	bool IsImportLib(const u8* data, u64 dataSize)
	{
		return dataSize > 5 && memcmp(data, "!<arch>", 5) == 0;
	}

	bool ObjectFileImportLib::Parse(Logger& logger, ObjectFileParseMode parseMode, const tchar* hint)
	{
#if PLATFORM_WINDOWS

		u8* pos = m_data;
		pos += IMAGE_ARCHIVE_START_SIZE;
		//auto& header = *(IMAGE_ARCHIVE_MEMBER_HEADER*)pos;
		pos += sizeof(IMAGE_ARCHIVE_MEMBER_HEADER);
		
		u32 symbolCount = _byteswap_ulong(*(u32*)pos);
		pos += sizeof(u32);

		u32* symbolOffsets = (u32*)pos;
		pos += sizeof(u32)*symbolCount;

		Vector<StringKey> impSymbols;
		for (u32 i=0; i!=symbolCount; ++i)
		{
			u32 symbolOffset = _byteswap_ulong(symbolOffsets[i]);(void)symbolOffset;
			auto symbolStr = (char*)pos;
			std::string symbol(symbolStr);
			pos += symbol.size() + 1;
			if (i == 0)
			{
				m_libName = symbol.data() + strlen("__IMPORT_DESCRIPTOR_");
			}
			if (i < 3) // Skip the predefined symbols
				continue;
			if (strncmp(symbol.data(), "__imp_", 6) == 0)
			{
				//impSymbols.push_back(symbolStr + 6);
				impSymbols.push_back(ToStringKeyRaw(symbolStr + 6, symbol.size() - 6));
				continue;
			}

			StringKey key = ToStringKeyRaw(symbol.data(), symbol.size());
			m_exports.try_emplace(key, ExportInfo{ std::move(symbol), true, i });
		}

		for (auto& symbol : impSymbols)
		{
			auto findIt = m_exports.find(symbol);
			if (findIt != m_exports.end())
				findIt->second.isData = false;
		}

		return true;
#else
		return false;
#endif
	}

	const char* ObjectFileImportLib::GetLibName()
	{
		return m_libName.c_str();
	}
}
