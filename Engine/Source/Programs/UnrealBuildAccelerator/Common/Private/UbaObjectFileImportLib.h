// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaObjectFile.h"

namespace uba
{
	class ObjectFileImportLib : public ObjectFile
	{
	public:
		virtual bool Parse(Logger& logger, ObjectFileParseMode parseMode, const tchar* hint) override;
		virtual const char* GetLibName() override;

	private:
		std::string m_libName;
	};


	bool IsImportLib(const u8* data, u64 dataSize);
}
