// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaObjectFileLLVMIR.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaMap.h"
#include "UbaObjectFileCoff.h"

#define UBA_LOG_STREAM 0

namespace uba
{
	// https://llvm.org/docs/BitCodeFormat.html

	bool IsRawBitcode(const u8* data, u64 dataSize)
	{
		constexpr u8 magic[] = { 'B', 'C', 0xc0, 0xde };
		return dataSize >= 4 && memcmp(data, magic, sizeof(magic)) == 0;
	}

	bool IsWrappedBitcode(const u8* data, u64 dataSize)
	{
		constexpr u8 wrapperMagic[] = { 0xDE, 0xC0, 0x17, 0x0B };
		return dataSize >= 4 && memcmp(data, wrapperMagic, sizeof(wrapperMagic)) == 0;
	}

	bool IsLLVMIRFile(const u8* data, u64 dataSize)
	{
		return IsRawBitcode(data, dataSize) || IsWrappedBitcode(data, dataSize);
	}


	#define BLOCK_IDS \
		BLOCK_ID(MODULE_BLOCK_ID) \
		BLOCK_ID(PARAMATTR_BLOCK_ID) \
		BLOCK_ID(PARAMATTR_GROUP_BLOCK_ID) \
		BLOCK_ID(CONSTANTS_BLOCK_ID) \
		BLOCK_ID(FUNCTION_BLOCK_ID) \
		BLOCK_ID(IDENTIFICATION_BLOCK_ID) \
		BLOCK_ID(VALUE_SYMTAB_BLOCK_ID) \
		BLOCK_ID(METADATA_BLOCK_ID) \
		BLOCK_ID(METADATA_ATTACHMENT_ID) \
		BLOCK_ID(TYPE_BLOCK_ID_NEW) \
		BLOCK_ID(USELIST_BLOCK_ID) \
		BLOCK_ID(MODULE_STRTAB_BLOCK_ID) \
		BLOCK_ID(GLOBALVAL_SUMMARY_BLOCK_ID) \
		BLOCK_ID(OPERAND_BUNDLE_TAGS_BLOCK_ID) \
		BLOCK_ID(METADATA_KIND_BLOCK_ID) \
		BLOCK_ID(STRTAB_BLOCK_ID) \
		BLOCK_ID(FULL_LTO_GLOBALVAL_SUMMARY_BLOCK_ID) \
		BLOCK_ID(SYMTAB_BLOCK_ID) \
		BLOCK_ID(SYNC_SCOPE_NAMES_BLOCK_ID) \

	enum BlockIDs : u8
	{
		BLOCKINFO_BLOCK_ID = 0,
		BEFORE_FIRST_APPLICATION_BLOCKID = 7,
		#define BLOCK_ID(x) x,
		BLOCK_IDS
		#undef BLOCK_ID
	};

	const tchar* BlockIdToString(u32 id)
	{
		switch(id)
		{
		case 0: return TC("BLOCKINFO_BLOCK_ID");
		#define BLOCK_ID(x) case x: return TC(#x);
		BLOCK_IDS
		#undef BLOCK_ID
		}
		return TC("UNKNOWN_BLOCK_ID");
	}

	#define MODULE_CODES \
		MODULE_CODE(VERSION, 1) \
		MODULE_CODE(TRIPLE, 2) \
		MODULE_CODE(DATALAYOUT, 3) \
		MODULE_CODE(ASM, 4) \
		MODULE_CODE(SECTIONNAME, 5) \
		MODULE_CODE(DEPLIB, 6) \
		MODULE_CODE(GLOBALVAR, 7) \
		MODULE_CODE(FUNCTION, 8) \
		MODULE_CODE(ALIAS_OLD, 9) \
		MODULE_CODE(GCNAME, 11) \
		MODULE_CODE(COMDAT, 12) \
		MODULE_CODE(VSTOFFSET, 13) \
		MODULE_CODE(ALIAS, 14) \
		MODULE_CODE(METADATA_VALUES_UNUSED, 15) \
		MODULE_CODE(SOURCE_FILENAME, 16) \
		MODULE_CODE(HASH, 17) \
		MODULE_CODE(IFUNC, 18)

	enum ModuleCodes {
		#define MODULE_CODE(x, v) MODULE_CODE_##x = v,
		MODULE_CODES
		#undef MODULE_CODE
	};

	const tchar* ModuleCodeToString(u32 code)
	{
		switch(code)
		{
		#define MODULE_CODE(x, v) case MODULE_CODE_##x: return TC("MODULE_CODE_" #x);
		MODULE_CODES
		#undef MODULE_CODE
		}
		return TC("MODULE_CODE_UNKNOWN");
	}

	enum ObjectFileLLVMIR::BlockInfoCodes : u8
	{
	  BLOCKINFO_CODE_SETBID = 1,
	  BLOCKINFO_CODE_BLOCKNAME = 2,
	  BLOCKINFO_CODE_SETRECORDNAME = 3
	};

	enum ObjectFileLLVMIR::FixedAbbrevIDs : u8
	{
		END_BLOCK = 0,
		ENTER_SUBBLOCK = 1,
		DEFINE_ABBREV = 2,
		UNABBREV_RECORD = 3,
		FIRST_APPLICATION_ABBREV = 4
	};

	enum ObjectFileLLVMIR::EntryKind : u8
	{
		EntryKind_Error,
		EntryKind_EndBlock,
		EntryKind_SubBlock,
		EntryKind_Record
	};

	enum ObjectFileLLVMIR::Encoding : u8
	{
		Encoding_Fixed = 1,
		Encoding_VBR   = 2,
		Encoding_Array = 3,
		Encoding_Char6 = 4,
		Encoding_Blob  = 5 
	};

	enum
	{
		AF_DontPopBlockAtEnd = 1,
		AF_DontAutoprocessAbbrevs = 2
	};

	struct ObjectFileLLVMIR::AbbrevOp
	{
		explicit AbbrevOp(u64 v) : val(v), isLiteral(true) {}
		explicit AbbrevOp(Encoding e, u64 data = 0) : val(data), isLiteral(false), encoding(e) {}
		u64 val;
		bool isLiteral : 1;
		u32 encoding   : 3;
	};

	struct ObjectFileLLVMIR::Abbrev
	{
		Vector<AbbrevOp> operands;
	};

	struct ObjectFileLLVMIR::Entry
	{
		EntryKind kind;
		u32 id;
	};

	struct ObjectFileLLVMIR::BlockInfo
	{
		struct Record
		{
			u32 blockId = 0;
			Vector<AbbrevPtr> abbrevs;
			std::string name;
			Vector<std::pair<u32, std::string>> recordNames;
		};

		Vector<Record> records;

		const Record* GetBlockInfo(u32 blockId) const
		{
			if (!records.empty() && records.back().blockId == blockId)
				return &records.back();
			for (const Record& bi : records)
				if (bi.blockId == blockId)
					return &bi;
			return nullptr;
		}

		Record& GetOrCreateBlockInfo(u32 blockId)
		{
			if (const Record* bi = GetBlockInfo(blockId))
				return *const_cast<Record*>(bi);
			records.emplace_back();
			records.back().blockId = blockId;
			return records.back();
		}
	};

	class ObjectFileLLVMIR::BitStreamReader
	{
	public:
		BitStreamReader(ObjectFileLLVMIR& owner, Logger& logger, u8* data, u64 dataSize)
		:	m_owner(owner)
		,	m_logger(logger)
		,	m_begin(data)
		,	m_pos(data)
		,	m_end(data + dataSize)
		{
		}

		u64 GetCurrentBitNo() const
		{
			return u64(m_pos - m_begin)*8 - m_wordBits;
		}

		bool CanSkipToPos(u64 pos) const
		{
			return pos <= u64(m_end - m_begin);
		}

		void SkipToEnd()
		{
			m_pos = m_end;
		}

		void JumpToBit(u64 bitNo)
		{
			u64 byteNo = u64(bitNo/8) & ~(sizeof(u32)-1);
			u32 wordBitNo = u32(bitNo & (sizeof(u32)*8-1));
			m_pos = m_begin + byteNo;
			m_wordBits = 0;
			if (wordBitNo)
				Read(wordBitNo);
		}

		u32 Read(u32 bits)
		{
			if (m_wordBits >= bits)
			{
				#if !defined( __clang_analyzer__ )
				u32 res = m_word & (~0u >> (32u - bits));
				#else
				u32 res = 0;
				#endif
				m_word >>= (bits & 0x1f);
				m_wordBits -= bits;
				return res;
			}
			u32 res = m_wordBits ? m_word : 0;
			u32 bitsLeft = bits - m_wordBits;

			UBA_ASSERT((m_end - m_pos) >= 4);
			m_word = *(u32*)m_pos;
			m_wordBits = 32;
			m_pos += 4;

			u32 res2 = m_word & (~0u >> (32 - bitsLeft));
			m_word >>= (bitsLeft & 0x1f);
			m_wordBits -= bitsLeft;

			res |= res2 << (bits - bitsLeft);

			return res;
		}

		u32 ReadVBR(u32 bits)
		{
			u32 piece = Read(bits);
			u32 maskBitOrder = (bits - 1);
			u32 mask = 1u << maskBitOrder;

			if ((piece & mask) == 0)
			  return piece;

			u32 result = 0;
			u32 nextBit = 0;

			while (true)
			{
				result |= (piece & (mask - 1)) << nextBit;

				if ((piece & mask) == 0)
					return result;

				nextBit += bits-1;
				UBA_ASSERT(nextBit < 32);
				piece = Read(bits);
			}
		}

		u64 ReadVBR64(u32 bits)
		{
			u32 piece = Read(bits);
			u32 maskBitOrder = (bits - 1);
			u32 mask = 1u << maskBitOrder;

			if ((piece & mask) == 0)
			  return piece;

			u64 result = 0;
			u32 nextBit = 0;

			while (true)
			{
				result |= u64(piece & (mask - 1)) << nextBit;

				if ((piece & mask) == 0)
					return result;

				nextBit += bits-1;
				UBA_ASSERT(nextBit < 64);
				piece = Read(bits);
			}
		}

		u32 ReadCode()
		{
			return Read(m_currentCodeSize);
		}

		void SkipToFourByteBoundary()
		{
			m_wordBits = 0;
		}

		bool IsDone()
		{
			return m_pos == m_end;
		}

		void SkipBlock()
		{
			ReadVBR(4); // CodeLenWidth
			SkipToFourByteBoundary();
			size_t numFourBytes = Read(32); // BlockSizeWidth
			size_t skipTo = GetCurrentBitNo() + numFourBytes * 4 * 8;
			JumpToBit(skipTo);
		}

		void EnterSubBlock(u32& outNumWords, u32 blockId)
		{
			auto& block = m_blockScope.emplace_back();
			block.prevCodeSize = m_currentCodeSize;
			block.prevAbbrevs.swap(m_curAbbrevs);

			if (m_blockInfo)
				if (const BlockInfo::Record* record = m_blockInfo->GetBlockInfo(blockId))
					m_curAbbrevs.insert(m_curAbbrevs.end(), record->abbrevs.begin(), record->abbrevs.end());

			m_currentCodeSize = ReadVBR(4);
			UBA_ASSERT(m_currentCodeSize);

			SkipToFourByteBoundary();
			u32 numWords = Read(32);
			UBA_ASSERT(numWords);
			outNumWords = numWords;
		}

		BlockInfo ReadBlockInfoBlock(bool readBlockInfoNames)
		{
			u32 numWords;
			EnterSubBlock(numWords, BLOCKINFO_BLOCK_ID);

			BlockInfo newBlockInfo;

			Vector<u64> record;
			BlockInfo::Record* curBlockInfo = nullptr;

			while (true)
			{
				Entry entry = AdvanceSkippingSubblocks(AF_DontAutoprocessAbbrevs);

				switch (entry.kind)
				{
				case EntryKind_SubBlock:
				case EntryKind_Error:
					UBA_ASSERT(false);
					return {};
				case EntryKind_EndBlock:
					return newBlockInfo;
				case EntryKind_Record:
					break;
				}

				if (entry.id == DEFINE_ABBREV)
				{
					if (!curBlockInfo)
					{
						UBA_ASSERT(false);
						return {};
					}
					ReadAbbrevRecord();
					curBlockInfo->abbrevs.push_back(std::move(m_curAbbrevs.back()));
					m_curAbbrevs.pop_back();
					continue;
				}

				record.clear();
				u32 code = ReadRecord(record, entry.id);
				switch (code)
				{
				default:
					break;
				case BLOCKINFO_CODE_SETBID:
					if (record.size() < 1)
					{
						UBA_ASSERT(false);
						return {};
					}
					curBlockInfo = &newBlockInfo.GetOrCreateBlockInfo(u32(record[0]));
					break;
				case BLOCKINFO_CODE_BLOCKNAME: {
					if (!curBlockInfo)
					{
						UBA_ASSERT(false);
						return {};
					}
					UBA_ASSERT(!readBlockInfoNames);
					break;
				}
				case BLOCKINFO_CODE_SETRECORDNAME: {
					if (!curBlockInfo)
					{
						UBA_ASSERT(false);
						return {};
					}
					UBA_ASSERT(!readBlockInfoNames);
					break;
				}
				}
			}
		}

		bool ReadAbbrevRecord()
		{
			auto abbv = std::make_shared<Abbrev>();

			u32 numOpInfo = ReadVBR(5);
			for (u32 i = 0; i != numOpInfo; ++i)
			{
				bool isLiteral = Read(1) != 0;
				if (isLiteral)
				{
					u64 op = ReadVBR64(8);
					abbv->operands.emplace_back(op);
					continue;
				}

				Encoding encoding = Encoding(Read(3));
				UBA_ASSERT(encoding >= 1 && encoding <= 5);
				if (encoding == Encoding_Fixed || encoding == Encoding_VBR)
				{
					u64 data = ReadVBR64(5);
					if (data == 0)
						abbv->operands.emplace_back(0);
					else
						abbv->operands.emplace_back(encoding, data);
				}
				else
					abbv->operands.emplace_back(encoding);
			}

			m_curAbbrevs.push_back(abbv);
			return true;
		}

		u64 ReadAbbreviatedField(const AbbrevOp& op)
		{
			switch (op.encoding)
			{
			case Encoding_Fixed:
				return Read(u32(op.val));
			case Encoding_VBR:
				return ReadVBR64(u32(op.val));
			case Encoding_Char6:
				return "abcdefghijklmnopqrstuvwxyzABCDEFGHIJKLMNOPQRSTUVWXYZ0123456789._"[Read(6)];
			default:
				UBA_ASSERT(false);
				return ~0ull;
			}
		}

		void SkipRecord(u32 abbrevId)
		{
			if (abbrevId == UNABBREV_RECORD)
			{
				ReadVBR(6);
				u32 numElts = ReadVBR(6);
				for (u32 i = 0; i != numElts; ++i)
					ReadVBR64(6);
				return;
			}

			u32 abbrevIndex = abbrevId - FIRST_APPLICATION_ABBREV;
			Abbrev& abbv = *m_curAbbrevs[abbrevIndex];
			UBA_ASSERT(!abbv.operands.empty());
			AbbrevOp& codeOp = abbv.operands[0];
			if (!codeOp.isLiteral)
				ReadAbbreviatedField(codeOp);

			for (u64 i = 1, e = abbv.operands.size(); i != e; ++i) {
				const AbbrevOp& op = abbv.operands[i];
				if (op.isLiteral)
					continue;

				if (op.encoding != Encoding_Array && op.encoding != Encoding_Blob)
				{
					ReadAbbreviatedField(op);
					continue;
				}

				if (op.encoding == Encoding_Array)
				{
					u32 numElts = ReadVBR(6);

					const AbbrevOp& eltEnc = abbv.operands[++i];

					switch (eltEnc.encoding)
					{
					default:
						UBA_ASSERT(false);
						return;
					case Encoding_Fixed:
						JumpToBit(GetCurrentBitNo() + u64(numElts) * eltEnc.val);
						break;
					case Encoding_VBR:
						for (; numElts; --numElts)
							ReadVBR64(u32(eltEnc.val));
						break;
					case Encoding_Char6:
						JumpToBit(GetCurrentBitNo() + numElts * 6);
					}
					continue;
				}

				UBA_ASSERT(op.encoding == Encoding_Blob);
				u32 numElts = ReadVBR(6);
				SkipToFourByteBoundary();

				u64 newEnd = GetCurrentBitNo() + AlignUp(numElts, 4) * 8;
				if (!CanSkipToPos(newEnd/8))
				{
					SkipToEnd();
					break;
				}
				JumpToBit(newEnd);
			}
			return;
		}

		void ReadRecordOperands(Vector<u64>& outVals, const Vector<AbbrevOp>& operands, std::string* blob = nullptr, u32 blockId = 0)
		{
			for (u64 i = 1, e = operands.size(); i != e; ++i) {
				const AbbrevOp& op = operands[i];
				if (op.isLiteral)
				{
					outVals.push_back(op.val);
					continue;
				}

				if (op.encoding != Encoding_Array && op.encoding != Encoding_Blob)
				{
					u64 val = ReadAbbreviatedField(op);(void)val;
					outVals.push_back(val);
					continue;
				}

				if (op.encoding == Encoding_Array)
				{
					u32 numElts = ReadVBR(6);
					outVals.reserve(outVals.size() + numElts);

					const AbbrevOp& eltEnc = operands[++i];
					UBA_ASSERT(!eltEnc.isLiteral);

					switch (eltEnc.encoding)
					{
					default:
						UBA_ASSERT(false);
						return;
					case Encoding_Fixed:
						for (; numElts; --numElts)
							outVals.push_back(Read(u32(eltEnc.val)));
						break;
					case Encoding_VBR:
						for (; numElts; --numElts)
							outVals.push_back(ReadVBR64(u32(eltEnc.val)));
						break;
					case Encoding_Char6:
						for (; numElts; --numElts)
							outVals.push_back("abcdefghijklmnopqrstuvwxyzABCDEFGHIJKLMNOPQRSTUVWXYZ0123456789._"[Read(6)]);
					}
					continue;
				}

				UBA_ASSERT(op.encoding == Encoding_Blob);
				u32 numElts = ReadVBR(6);
				SkipToFourByteBoundary();  // 32-bit alignment

				size_t curBitPos = GetCurrentBitNo();
				size_t newEnd = curBitPos + AlignUp(numElts, 4) * 8;

				UBA_ASSERT(CanSkipToPos(newEnd/8));
				JumpToBit(newEnd);

				if (blockId == STRTAB_BLOCK_ID)
				{
					m_owner.m_strTabPos = u64(curBitPos/8);
					m_owner.m_strTabSize = numElts;
				}

				char* ptr = (char*)(m_begin + (curBitPos/8));
				if (blob)
					*blob = std::string(ptr, numElts);
				else
				{
					auto* uptr = reinterpret_cast<const unsigned char*>(ptr);
					outVals.insert(outVals.end(), uptr, uptr + numElts);
				}
			}
		}

		bool CanBeExported(u32 blockId, u32 code)
		{
			if (blockId != MODULE_BLOCK_ID)
				return false;
			switch (code)
			{
			case MODULE_CODE_FUNCTION:
			case MODULE_CODE_GLOBALVAR:
			case MODULE_CODE_IFUNC:
			case MODULE_CODE_ALIAS:
			case MODULE_CODE_ALIAS_OLD:
				return true;
			default:
				return false;
			}
		}

		u32 ReadRecord(Vector<u64>& outVals, u32 abbrevId, std::string* blob = nullptr, u32 blockId = 0)
		{
			if (abbrevId == UNABBREV_RECORD)
			{
				u32 code = ReadVBR(6);

				if (CanBeExported(blockId, code))
					m_owner.m_globalVarOrFunctionRecords.push_back(BitStreamEntry{u64(m_pos - m_begin), m_word, m_wordBits, code});

				u32 numElts = ReadVBR(6);

				outVals.reserve(outVals.size() + numElts);
				for (u32 i = 0; i != numElts; ++i)
					outVals.push_back(ReadVBR64(6));

#if UBA_LOG_STREAM
				if (blockId == MODULE_BLOCK_ID)
				{
					auto str = ModuleCodeToString(code);
					u32 indent = 2;
					m_logger.Info(TC("%s%s (%u)"), TC("        ") + 8 - indent, str, code);
				}
#endif
				return code;
			}

			u32 abbrevIndex = abbrevId - FIRST_APPLICATION_ABBREV;
			UBA_ASSERT(abbrevIndex < m_curAbbrevs.size());
			Abbrev& abbv = *m_curAbbrevs[abbrevIndex];
			UBA_ASSERT(!abbv.operands.empty());
			AbbrevOp& codeOp = abbv.operands[0];
			u32 code;
			if (codeOp.isLiteral)
				code = u32(codeOp.val);
			else
			{
				UBA_ASSERT(codeOp.encoding != Encoding_Array && codeOp.encoding != Encoding_Blob);
				code = u32(ReadAbbreviatedField(codeOp));
			}

			if (CanBeExported(blockId, code))
			{
				UBA_ASSERT(!abbv.operands.empty());
				m_owner.m_globalVarOrFunctionRecords.push_back(BitStreamEntry{u64(m_pos - m_begin), m_word, m_wordBits, code, abbv.operands });
			}

#if UBA_LOG_STREAM
			if (blockId == MODULE_BLOCK_ID)
			{
				auto str = ModuleCodeToString(code);
				u32 indent = 2;
				m_logger.Info(TC("%s%s (%u)"), TC("        ") + 8 - indent, str, code);
			}
#endif

			ReadRecordOperands(outVals, abbv.operands, blob, blockId);

			return code;
		}

		Entry Advance(u32 flags)
		{
			while (true)
			{
				UBA_ASSERT(!IsDone());

				u32 code = ReadCode();
				if (code == END_BLOCK)
				{
					if (flags & AF_DontPopBlockAtEnd)
						return { EntryKind_EndBlock };
					UBA_ASSERT(!m_blockScope.empty());
					SkipToFourByteBoundary();
					m_currentCodeSize = m_blockScope.back().prevCodeSize;
					m_curAbbrevs = std::move(m_blockScope.back().prevAbbrevs);
					m_blockScope.pop_back();
					return { EntryKind_EndBlock };
				}

				if (code == ENTER_SUBBLOCK)
				{
					u32 subBlockId =  ReadVBR(8);
					return { EntryKind_SubBlock, subBlockId };
				}

				if (code == DEFINE_ABBREV && !(flags & AF_DontAutoprocessAbbrevs))
				{
					ReadAbbrevRecord();
					continue;
				}

				return { EntryKind_Record, code };
			}
		}

		Entry AdvanceSkippingSubblocks(u32 flags)
		{
			while (true)
			{
				Entry entry = Advance(flags);
				if (entry.kind != EntryKind_SubBlock)
					return entry;
				SkipBlock();
			}
		}

		ObjectFileLLVMIR& m_owner;
		Logger& m_logger;
		u32 m_word = 0;
		u32 m_wordBits = 0;
		u8* m_begin;
		u8* m_pos;
		u8* m_end;

		u32 m_currentCodeSize = 2;

		struct Block
		{
			u32 prevCodeSize;
			Vector<AbbrevPtr> prevAbbrevs;
		};
		Vector<Block> m_blockScope;
		Vector<AbbrevPtr> m_curAbbrevs;
		BlockInfo* m_blockInfo = nullptr;
	};

	ObjectFileLLVMIR::ObjectFileLLVMIR()
	{
		m_type = ObjectFileType_LLVMIR;
	}

	ObjectFileLLVMIR::~ObjectFileLLVMIR() = default;

	u32 GetDllStorageIndex(u32 code)
	{
		if (code == MODULE_CODE_FUNCTION)
			return 13;
		if (code == MODULE_CODE_GLOBALVAR)
			return 12;
		if (code == MODULE_CODE_ALIAS)
			return 7;
		if (code == MODULE_CODE_ALIAS_OLD)
			return 6;
		UBA_ASSERTF(false, TC("module code %s not supported"), ModuleCodeToString(code));
		return ~0u;
	}

	u32 GetLinkageIndex(u32 code)
	{
		if (code == MODULE_CODE_FUNCTION)
			return 5;
		if (code == MODULE_CODE_GLOBALVAR)
			return 5;
		if (code == MODULE_CODE_ALIAS)
			return 3;
		if (code == MODULE_CODE_ALIAS_OLD)
			return 2;
		UBA_ASSERTF(false, TC("module code %s not supported"), ModuleCodeToString(code));
		return ~0u;
	}

	bool ObjectFileLLVMIR::Parse(Logger& logger, ObjectFileParseMode parseMode, const tchar* hint)
	{
		BitStreamReader reader(*this, logger, m_data, m_dataSize);

		if (IsWrappedBitcode(m_data, m_dataSize))
		{
			reader.Read(32);
			u32 version = reader.Read(32);(void)version;
			u32 bitcodeOffset = reader.Read(32);(void)bitcodeOffset;
			u32 bitcodeSize = reader.Read(32);(void)bitcodeSize;
			u32 cpuType = reader.Read(32);(void)cpuType;
			reader.JumpToBit(bitcodeOffset*8);
			reader.m_end = reader.m_pos + bitcodeSize;
		}

		reader.Read(32); // Skip magic

		BlockInfo blockInfo;
		reader.m_blockInfo = &blockInfo;

		while (!reader.IsDone())
		{
			u32 code = reader.ReadCode();(void)code;
			UBA_ASSERT(code == ENTER_SUBBLOCK);

			u32 blockId = reader.ReadVBR(8);

			u64 subBlockBitStart = reader.GetCurrentBitNo();(void)subBlockBitStart;
			ParseBlock(logger, reader, blockInfo, blockId, 0);
		}

		Vector<u64> recordData;
		std::string name;

		Set<std::string> recordsWithOds;

		u32 index = 0;
		for (auto& record : m_globalVarOrFunctionRecords)
		{
			reader.m_pos = m_data + record.pos;
			reader.m_word = record.word;
			reader.m_wordBits = record.wordBits;

			recordData.clear();
			if (record.operands.empty())
			{
				u32 numElts = reader.ReadVBR(6);
				recordData.reserve(recordData.size() + numElts);
				for (u32 i = 0; i != numElts; ++i)
					recordData.push_back(reader.ReadVBR64(6));
			}
			else
			{
				reader.ReadRecordOperands(recordData, record.operands);
			}

			u64* recIt = recordData.data();
			u64 recSize = recordData.size();

			if (recIt[0] + recIt[1] >= m_strTabSize)
				continue;
			name.assign((char*)(m_data + m_strTabPos) + recIt[0], recIt[1]);

			u32 dllStorageIndex = GetDllStorageIndex(record.code);

			u64 dllStorage = ~0ull;
			if (recSize > dllStorageIndex)
				dllStorage = recIt[dllStorageIndex];

			// Left here for debugging purposes :)
			//if (name.find("RemoveHighlight") != -1)
			//	printf("");

			bool isExported = false;
			u32 linkageIndex = GetLinkageIndex(record.code);
			u64 linkage = 0;
			if (recSize > linkageIndex)
			{
				linkage = recIt[linkageIndex];
				if (linkage == 9) // PrivateLinkage
					continue;
				if (linkage == 19) // LinkOnceODRLinkage
				{
					recordsWithOds.insert(name);
					continue;
				}

				// ExternalLinkage || DLLImportLinkage || DLLExportLinkage || LinkOnceODRAutoHideLinkage || WeakODRLinkage
				if (parseMode == ObjectFileParseMode_All)
				{
					isExported = linkage == 0 || linkage == 5 || linkage == 6 || linkage == 15 || linkage == 17;
					if (dllStorage == DllStorage_Import)
						isExported = false;
				}
				else
					isExported = dllStorage == DllStorage_Export;
			}

			if (record.code == MODULE_CODE_GLOBALVAR)
			{
				bool hasInitializer = recordData[4] != 0;
				if (!hasInitializer)
					isExported = false;

				if (isExported)
				{
					u64 valueType = recSize > 2 ? recIt[2] : 0;
					u64 dataSize = valueType < m_typeSizes.size() ? m_typeSizes[valueType] : 0;
					u32 typeAlign = valueType < m_typeAligns.size() ? u32(m_typeAligns[valueType]) : 0;
					// GLOBALVAR alignment field is encoded as log2(align)+1 (0 = use the type's natural alignment).
					u32 explicitAlign = (recSize > 6 && recIt[6] > 0) ? (1u << u32(recIt[6] - 1)) : 0;
					u32 dataAlign = explicitAlign > typeAlign ? explicitAlign : typeAlign;
					m_exports.emplace(ToStringKeyRaw(name.data(), name.size()), ExportInfo{name, true /*isData*/, index++, dataSize, 0 /*isHidden*/, dataAlign});
				}
				else if (name.find(".str") == -1)
				{
					if (recordsWithOds.find(name) != recordsWithOds.end())
						continue;
					m_imports.emplace(name);
				}
			}
			else if (record.code == MODULE_CODE_FUNCTION)
			{
				bool isProto = recordData[4] != 0;

				if (linkage == 12) // thunk import?!? (like _ZThn840_N21UFortHUDElementWidget15RemoveHighlightEv)
					isProto = true;

				if (isProto)
				{
					if (recordsWithOds.find(name) != recordsWithOds.end())
						continue;
					m_imports.emplace(name);
				}
				else if (isExported)
				{
					m_exports.emplace(ToStringKeyRaw(name.data(), name.size()), ExportInfo{name, false, index++});
				}
			}
			else if (record.code == MODULE_CODE_ALIAS)
			{
				if (isExported)
					m_exports.emplace(ToStringKeyRaw(name.data(), name.size()), ExportInfo{name, false, index++});
			}
			(void)index;
		}
		return true;
	}

	// TYPE_BLOCK record codes (subset needed to size global variables).
	enum TypeCodes
	{
		TYPE_CODE_NUMENTRY = 1, TYPE_CODE_VOID = 2, TYPE_CODE_FLOAT = 3, TYPE_CODE_DOUBLE = 4,
		TYPE_CODE_LABEL = 5, TYPE_CODE_OPAQUE = 6, TYPE_CODE_INTEGER = 7, TYPE_CODE_POINTER = 8,
		TYPE_CODE_FUNCTION_OLD = 9, TYPE_CODE_HALF = 10, TYPE_CODE_ARRAY = 11, TYPE_CODE_VECTOR = 12,
		TYPE_CODE_X86_FP80 = 13, TYPE_CODE_FP128 = 14, TYPE_CODE_PPC_FP128 = 15, TYPE_CODE_METADATA = 16,
		TYPE_CODE_X86_MMX = 17, TYPE_CODE_STRUCT_ANON = 18, TYPE_CODE_STRUCT_NAME = 19, TYPE_CODE_STRUCT_NAMED = 20,
		TYPE_CODE_FUNCTION = 21, TYPE_CODE_TOKEN = 22, TYPE_CODE_BFLOAT = 23, TYPE_CODE_X86_AMX = 24,
		TYPE_CODE_OPAQUE_POINTER = 25, TYPE_CODE_TARGET_TYPE = 26,
	};

	static u64 SizeRoundUp(u64 v, u64 a) { return a <= 1 ? v : ((v + a - 1) / a) * a; }

	// Records the byte size/alignment of each TYPE_BLOCK type, in stream order, so GLOBALVAR exports can be given a
	// real st_size. Type operands only reference earlier type ids (pointers are opaque, size 8), so one pass suffices.
	// DataLayout is the x86_64/arm64 default (pointer = 8 bytes).
	void ObjectFileLLVMIR::AddType(u32 typeCode, const Vector<u64>& ops)
	{
		auto push = [&](u64 size, u64 align) { m_typeSizes.push_back(size); m_typeAligns.push_back(align); };
		auto sizeOf = [&](u64 t) { return t < m_typeSizes.size() ? m_typeSizes[t] : 0; };
		auto alignOf = [&](u64 t) { return t < m_typeAligns.size() ? m_typeAligns[t] : 1; };

		switch (typeCode)
		{
		case TYPE_CODE_NUMENTRY: // table size hint, not a type
			return;
		case TYPE_CODE_STRUCT_NAME: // names the following STRUCT_NAMED; not its own type entry
			return;

		case TYPE_CODE_VOID: case TYPE_CODE_LABEL: case TYPE_CODE_METADATA: case TYPE_CODE_TOKEN:
		case TYPE_CODE_OPAQUE: case TYPE_CODE_FUNCTION_OLD: case TYPE_CODE_FUNCTION:
			push(0, 1); return;

		case TYPE_CODE_HALF: case TYPE_CODE_BFLOAT: push(2, 2); return;
		case TYPE_CODE_FLOAT: push(4, 4); return;
		case TYPE_CODE_DOUBLE: push(8, 8); return;
		case TYPE_CODE_X86_FP80: case TYPE_CODE_FP128: case TYPE_CODE_PPC_FP128: push(16, 16); return;
		case TYPE_CODE_X86_MMX: push(8, 8); return;
		case TYPE_CODE_POINTER: case TYPE_CODE_OPAQUE_POINTER: push(8, 8); return;

		case TYPE_CODE_INTEGER:
		{
			u64 bytes = ((ops.empty() ? 0 : ops[0]) + 7) / 8;
			u64 align = bytes <= 1 ? 1 : bytes <= 2 ? 2 : bytes <= 4 ? 4 : bytes <= 8 ? 8 : 16;
			push(SizeRoundUp(bytes, align), align);
			return;
		}
		case TYPE_CODE_ARRAY:
		{
			u64 num = ops.size() > 0 ? ops[0] : 0;
			u64 elt = ops.size() > 1 ? ops[1] : 0;
			push(num * sizeOf(elt), alignOf(elt));
			return;
		}
		case TYPE_CODE_VECTOR:
		{
			u64 num = ops.size() > 0 ? ops[0] : 0;
			u64 elt = ops.size() > 1 ? ops[1] : 0;
			u64 total = num * sizeOf(elt);
			u64 align = 1; while (align < total) align <<= 1;
			push(SizeRoundUp(total, align), align);
			return;
		}
		case TYPE_CODE_STRUCT_ANON:
		case TYPE_CODE_STRUCT_NAMED:
		{
			bool packed = !ops.empty() && ops[0] != 0;
			u64 offset = 0, structAlign = 1;
			for (u64 i = 1; i < ops.size(); ++i)
			{
				u64 ma = packed ? 1 : alignOf(ops[i]);
				offset = SizeRoundUp(offset, ma) + sizeOf(ops[i]);
				if (ma > structAlign) structAlign = ma;
			}
			push(SizeRoundUp(offset, structAlign), structAlign);
			return;
		}
		default:
			push(0, 1); // unknown / target-extension types: no size claimed
			return;
		}
	}

	bool ObjectFileLLVMIR::ParseBlock(Logger& logger, BitStreamReader& reader, BlockInfo& blockInfo, u32 blockId, u32 indent)
	{
		if (blockId == BLOCKINFO_BLOCK_ID)
		{
			uint64_t blockBitStart = reader.GetCurrentBitNo();
			blockInfo = reader.ReadBlockInfoBlock(true);
			reader.JumpToBit(blockBitStart);
		}

		u32 numWords = 0;
		reader.EnterSubBlock(numWords, blockId);

		#if UBA_LOG_STREAM
		const tchar* str = BlockIdToString(blockId);
		logger.Info(TC("%s%s (%u)"), TC("        ") + 8 - indent, str, blockId);
		#endif

		Vector<u64> record;
		std::string blob;

		while (true)
		{
			UBA_ASSERT(!reader.IsDone());
		
			Entry entry = reader.Advance(AF_DontAutoprocessAbbrevs);
			switch (entry.kind)
			{
			case EntryKind_Record:
				break;
			case EntryKind_EndBlock:
				return true;
			case EntryKind_SubBlock:
			{
				u64 subBlockBitStart = reader.GetCurrentBitNo();(void)subBlockBitStart;
				ParseBlock(logger, reader, blockInfo, entry.id, indent + 2);
				continue;
			}
			default:
				UBA_ASSERT(false);
			}

			if (entry.id == DEFINE_ABBREV)
			{
				reader.ReadAbbrevRecord();
				continue;
			}

			if (blockId == MODULE_BLOCK_ID || blockId == STRTAB_BLOCK_ID)
			{
				record.clear();
				reader.ReadRecord(record, entry.id, nullptr, blockId);
			}
			else if (blockId == TYPE_BLOCK_ID_NEW)
			{
				record.clear();
				u32 typeCode = reader.ReadRecord(record, entry.id, nullptr, blockId);
				AddType(typeCode, record);
			}
			else
				reader.SkipRecord(entry.id);

			#if UBA_LOG_STREAM
			//logger.Log(LogEntryType_Info, blob);
			#endif
		}
	}
}
