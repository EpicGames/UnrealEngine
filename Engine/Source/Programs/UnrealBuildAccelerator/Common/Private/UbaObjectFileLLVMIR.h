// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaObjectFile.h"
#include <memory>

namespace uba
{
	class ObjectFileLLVMIR : public ObjectFile
	{
	public:
		ObjectFileLLVMIR();
		~ObjectFileLLVMIR();

		virtual bool Parse(Logger& logger, ObjectFileParseMode parseMode, const tchar* hint) override;

	private:
		enum BlockInfoCodes : u8;
		enum Encoding : u8;
		enum EntryKind : u8;
		enum FixedAbbrevIDs : u8;
		struct Abbrev;
		struct AbbrevOp;
		struct BlockInfo;
		struct Entry;
		using AbbrevPtr = std::shared_ptr<Abbrev>;

		class BitStreamReader;
		bool ParseBlock(Logger& logger, BitStreamReader& reader, BlockInfo& blockInfo, u32 blockId, u32 indent);

		// Resolves a TYPE_BLOCK record (in stream order) to its byte size, used to give GLOBALVAR exports a real
		// st_size. Copy relocations against a data symbol require a non-zero size, which bitcode otherwise omits.
		void AddType(u32 typeCode, const Vector<u64>& ops);
		Vector<u64> m_typeSizes;
		Vector<u64> m_typeAligns;

		enum DllStorage : u8
		{
			DllStorage_None,
			DllStorage_Export,
			DllStorage_Import,
		};

		struct BitStreamEntry
		{
			u64 pos;
			u32 word;
			u32 wordBits;
			u32 code;
			Vector<AbbrevOp> operands;
		};

		Vector<BitStreamEntry> m_globalVarOrFunctionRecords;

		u64 m_strTabPos;
		u64 m_strTabSize;
	};


	bool IsLLVMIRFile(const u8* data, u64 dataSize);
}
