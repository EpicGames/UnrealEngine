// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaObjectFileMachO.h"

namespace uba
{
	// Mach-O on-disk structures and constants. Defined locally so the parser
	// builds on Linux/Windows without the macOS SDK. Assumes a little-endian host.

	struct MachHeader64
	{
		u32 magic;
		u32 cputype;
		u32 cpusubtype;
		u32 filetype;
		u32 ncmds;
		u32 sizeofcmds;
		u32 flags;
		u32 reserved;
	};

	struct MachLoadCommand
	{
		u32 cmd;
		u32 cmdsize;
	};

	struct MachSymtabCommand
	{
		u32 cmd;
		u32 cmdsize;
		u32 symoff;
		u32 nsyms;
		u32 stroff;
		u32 strsize;
	};

	struct MachNlist64
	{
		u32 n_strx;
		u8  n_type;
		u8  n_sect;
		u16 n_desc;
		u64 n_value;
	};

	// Local Mach-O constants. Named to avoid colliding with macros from
	// <mach-o/loader.h> and <mach-o/nlist.h> when this file is built on macOS.
	constexpr u32 MachOMagic64 = 0xFEEDFACFu;
	constexpr u32 MachOLcSymtab = 0x2u;

	constexpr u8 MachONStab = 0xE0;
	constexpr u8 MachONType = 0x0E;
	constexpr u8 MachONExt  = 0x01;

	constexpr u8 MachONUndf = 0x0;
	constexpr u8 MachONAbs  = 0x2;
	constexpr u8 MachONIndr = 0xA;
	constexpr u8 MachONSect = 0xE;

	bool IsMachOFile(const u8* data, u64 dataSize)
	{
		return dataSize >= 4 && *(const u32*)data == MachOMagic64;
	}

	ObjectFileMachO::ObjectFileMachO()
	{
		m_type = ObjectFileType_MachO;
	}

	bool ObjectFileMachO::Parse(Logger& logger, ObjectFileParseMode parseMode, const tchar* hint)
	{
		if (m_dataSize < sizeof(MachHeader64))
			return false;

		auto& header = *(const MachHeader64*)m_data;
		if (header.magic != MachOMagic64)
			return false;

		u64 lcStart = sizeof(MachHeader64);
		if (lcStart + header.sizeofcmds > m_dataSize)
			return false;

		const u8* lcPtr = m_data + lcStart;
		const u8* lcEnd = lcPtr + header.sizeofcmds;

		bool foundSymtab = false;
		MachSymtabCommand symtab = {};

		for (u32 i = 0; i != header.ncmds; ++i)
		{
			if (lcPtr + sizeof(MachLoadCommand) > lcEnd)
				return false;
			auto& lc = *(const MachLoadCommand*)lcPtr;
			if (lc.cmdsize < sizeof(MachLoadCommand) || lcPtr + lc.cmdsize > lcEnd)
				return false;

			if (lc.cmd == MachOLcSymtab)
			{
				if (lc.cmdsize < sizeof(MachSymtabCommand))
					return false;
				symtab = *(const MachSymtabCommand*)lcPtr;
				foundSymtab = true;
			}

			lcPtr += lc.cmdsize;
		}

		if (!foundSymtab)
			return true;

		u64 symtabBytes = (u64)symtab.nsyms * sizeof(MachNlist64);
		if (symtab.symoff + symtabBytes > m_dataSize)
			return false;
		if ((u64)symtab.stroff + symtab.strsize > m_dataSize)
			return false;

		m_symtabOffset = symtab.symoff;
		m_symbolCount = symtab.nsyms;
		m_strtabOffset = symtab.stroff;
		m_strtabSize = symtab.strsize;

		auto symbols = (const MachNlist64*)(m_data + m_symtabOffset);
		const char* strtab = (const char*)(m_data + m_strtabOffset);

		auto addExport = [&](const char* name, u64 nameLen)
			{
				std::string tmp(name, nameLen);
				StringKey key = ToStringKeyRaw(tmp.data(), tmp.size());
				m_exports.emplace(key, ExportInfo{ std::move(tmp), false, 0 });
			};

		for (u32 j = 0; j != m_symbolCount; ++j)
		{
			const MachNlist64& sym = symbols[j];

			if ((sym.n_type & MachONStab) != 0)
				continue;

			if (sym.n_strx == 0 || sym.n_strx >= m_strtabSize)
				continue;

			const char* name = strtab + sym.n_strx;
			if (!*name)
				continue;

			u64 nameLen = strnlen(name, m_strtabSize - sym.n_strx);
			if (nameLen == 0)
				continue;

			u8 ntype = sym.n_type & MachONType;
			bool isExt = (sym.n_type & MachONExt) != 0;

			if (isExt && ntype == MachONUndf)
			{
				if (sym.n_value != 0)
					addExport(name, nameLen); // common symbol (uninitialized global)
				else
					m_imports.emplace(name, nameLen);
			}
			else if (isExt && (ntype == MachONSect || ntype == MachONAbs || ntype == MachONIndr))
			{
				addExport(name, nameLen);
			}
			else if (!isExt && ntype == MachONSect)
			{
				if (name[0] == 'L')
					continue;
				addExport(name, nameLen);
			}
		}

		return true;
	}
}
