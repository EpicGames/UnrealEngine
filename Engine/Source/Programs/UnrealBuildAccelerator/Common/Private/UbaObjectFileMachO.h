// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaObjectFile.h"

namespace uba
{
	class ObjectFileMachO : public ObjectFile
	{
	public:
		ObjectFileMachO();
		virtual bool Parse(Logger& logger, ObjectFileParseMode parseMode, const tchar* hint) override;

	private:
		u64 m_symtabOffset = 0;
		u32 m_symbolCount = 0;
		u64 m_strtabOffset = 0;
		u32 m_strtabSize = 0;
	};

	bool IsMachOFile(const u8* data, u64 dataSize);
}
