// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaProcess.h"
#include "UbaApplicationRules.h"
#include "UbaDefinitions.h"
#include "UbaEnvironment.h"
#include "UbaFileAccessor.h"
#include "UbaProtocol.h"
#include "UbaProcessStats.h"
#include "UbaProcessUtils.h"
#include "UbaStorage.h"

#if PLATFORM_WINDOWS
#include "UbaDetoursPayload.h"
#include <winternl.h>
#include <Psapi.h>
#include <detours/detours.h>
#else
#include <wchar.h>
#include <poll.h>
#include <stdio.h>
#include <spawn.h>

// These headers are used for tracking child and beyond
// processes and making sure they clean up properly
// Linux uses PR_SET_CHILD_SUBREAPER
// Mac has to roll it's own solution
#if PLATFORM_LINUX
#include <sys/prctl.h>
#include <sys/resource.h>
#include <fcntl.h>
#include <unistd.h>
#elif PLATFORM_MAC
#include <sys/types.h>
#include <sys/sysctl.h>
#include <fcntl.h>
#include <unistd.h>
#endif

extern char **environ;
#endif

#define UBA_DEBUG_TRACK_PROCESS 0 // UBA_DEBUG_LOGGER

#if !PLATFORM_WINDOWS
#define EXCEPTION_ACCESS_VIOLATION 128 + SIGSEGV
#define STATUS_STACK_BUFFER_OVERRUN 128 + SIGSEGV
#endif

//////////////////////////////////////////////////////////////////////////////

namespace uba
{
	// The /EXTRACTEXPORTS<Version> token (on the compile command line, interpreted+stripped by UBA) both triggers symbol extraction and selects the .exi format version to write.
	static constexpr const tchar g_extractExportsStr[] = TC("/extractexports");

	void Process::AddRef()
	{
		++m_refCount;
	}

	void Process::Release()
	{
		UBA_ASSERT(m_refCount);
		if (!--m_refCount)
			delete this;
	}

	struct ProcessImpl::PipeReader
	{
		PipeReader(ProcessImpl& p, LogEntryType lt) : process(p), logType(lt) {}
		~PipeReader()
		{
			Flush();
		}

		void Flush()
		{
			if (!currentString.empty())
				process.InternalLogLine(false, TString(currentString), logType);
			currentString.clear();
		}

		void ReadData(char* buf, u32 readCount)
		{
			char* startPos = buf;
			char* end = buf + readCount;

			while (startPos < end)
			{
				char* endOfLine = (char*)memchr(startPos, '\n', end - startPos);
				if (!endOfLine)
				{
					currentString.append(TString(startPos, end));
					return;
				}

				char* lineEnd = endOfLine;
				if (lineEnd > startPos && lineEnd[-1] == '\r')
					--lineEnd;

				currentString.append(TString(startPos, lineEnd));

				process.InternalLogLine(false, TString(currentString), logType);
				currentString.clear();

				startPos = endOfLine + 1;
			}
		}

		ProcessImpl& process;
		LogEntryType logType;
		TString currentString;
	};

	struct ImageInfo
	{
		CasKey key;
		u64 fileSize = 0;
		bool isPEImage = false;
		bool is64Bit = true;
		bool isX64 = false;
		bool isArm64 = false;
		bool isDotnet = false;
	};

	bool GetImageInfo(ImageInfo& out, Logger& logger, const tchar* application, bool calculateCas)
	{
		FileAccessor fa(logger, application);
		if (!fa.OpenMemoryRead())
			return false;
		out.fileSize = fa.GetSize();
		u8* data = fa.GetData();

		if (calculateCas)
		{
			CasKeyHasher hasher;
			hasher.Update(data, out.fileSize);
			out.key = ToCasKey(hasher, false);
		}

		if (data[0] == 'M' && data[1] == 'Z')
		{
			u32 offset = *(u32*)(data + 0x3c);
			u32* signaturePos = (u32*)(data + offset);

			out.isPEImage = *signaturePos == 0x00004550;
			if (out.isPEImage)
			{
				u16 machine = *(u16*)(signaturePos + 1);
				out.isX64 = machine == 0x8664;
				out.isArm64 = machine == 0xaa64;

				u16 magic = *(u16*)(data + offset + 0x18);
				out.is64Bit = (magic == 0x20B);

				if (out.fileSize > offset + 0x18 + 0x70 + 4)
					out.isDotnet = (*(u32*)(data + offset + 0x18 + 0x70)) != 0;
			}
		}
		return fa.Close();
	}

	#if !PLATFORM_WINDOWS
	SharedEvent& CreateSharedEvent(u8* mem, u32 index)
	{
		if (UseSemaphoreForSharedEvents())
			return *new SharedEvent();
		else
			return *(mem ? (new (mem + sizeof(SharedEvent)*index) SharedEvent) : (SharedEvent*)nullptr);
	}
	#endif

	ProcessImpl::ProcessImpl(Session& session, u32 id, ProcessImpl* parent, bool detourEnabled)
	:	m_session(session)
	,	m_parentProcess(parent)
	,	m_id(id)
	,	m_comMemory(detourEnabled ? m_session.m_processCommunicationAllocator.Alloc(TC("")) : FileMappingAllocation())
		#if !PLATFORM_WINDOWS
	,	m_cancelEvent(CreateSharedEvent(m_comMemory.memory, 0))
	,	m_writeEvent(CreateSharedEvent(m_comMemory.memory, 1))
	,	m_readEvent(CreateSharedEvent(m_comMemory.memory, 2))
		#endif
	,	m_detourEnabled(detourEnabled)
	,	m_shared(parent ? parent->m_shared : *new Shared)
	,	m_messageThreadDone(EventResetType_Manual)
	{
		if (!m_comMemory.memory)
			return;

		#if !PLATFORM_WINDOWS
		if (UseSemaphoreForSharedEvents())
		{
			StringBuffer<256> name;
			name.Append(TCV("uba")).AppendValue(m_comMemory.handle.internal).Append('_').AppendValue(m_comMemory.offset);
			u64 len = name.count;
			name.Append("_cancel");
			m_cancelEvent.Create(name, true);
			name.Resize(len).Append("_read");
			m_writeEvent.Create(name, true);
			name.Resize(len).Append("_write");
			m_readEvent.Create(name, true);
			return;
		}
		#endif

		m_cancelEvent.Create(EventResetType_Manual);
		m_writeEvent.Create(EventResetType_Auto);
		m_readEvent.Create(EventResetType_Auto);
	}

	struct ProcessImpl::OverflowedWrittenFilesMessage
	{
		u8 buffer[CommunicationMemSize];
		u64 written = 0;
		OverflowedWrittenFilesMessage* next = nullptr;
	};

	ProcessImpl::~ProcessImpl()
	{
		UBA_ASSERT(m_refCount == 0);
		{
			#if !PLATFORM_WINDOWS
			SCOPED_FUTEX(m_comMemoryLock, lock);
			#endif
			if (m_comMemory.memory)
				m_cancelEvent.Set();
		}

		if (m_messageThread)
			m_messageThreadDone.IsSet();

		if (m_comMemory.memory)
		{
			InternalDestroyEvents();
			m_session.m_processCommunicationAllocator.Free(m_comMemory);
		}

		while (auto msg = m_firstOverflowMessage)
		{
			m_firstOverflowMessage = msg->next;
			delete msg;
		}

		if (!m_parentProcess)
		{
			for (auto& pair : m_shared.writtenFiles)
				if (pair.second.memoryHandle.IsValid())
					m_session.m_sharedMemory.CloseHandle(m_session.m_logger, pair.second.memoryHandle, pair.second.name.c_str());
			for (auto& pair : m_shared.activeFiles)
				m_session.m_logger.Warning(TC("Process %s still has active file %s that was never closed (refcount %u)"), m_startInfo.GetDescription(), pair.second.name.c_str(), pair.second.refCount);
			delete &m_shared;
		}
	}

	bool ProcessImpl::Start(const ProcessStartInfo& startInfo, bool runningRemote, Vector<tchar>& environment, bool async, void* userDataOverride)
	{
		m_startTime = GetTime();

		m_startInfo = startInfo;
		m_userDataOverride = userDataOverride;
		m_runningRemote = runningRemote;

		FixPathSeparators(m_startInfo.workingDirStr.data());
		FixPathSeparators(m_startInfo.logFileStr.data());

		if (IsAbsolutePath(m_startInfo.application))
		{
			StringBuffer<256> temp2;
			FixPath(m_startInfo.applicationStr.data(), nullptr, 0, temp2);
			m_startInfo.applicationStr = temp2.data;
			m_startInfo.application = m_startInfo.applicationStr.data();
		}
		else
			FixPathSeparators(m_startInfo.applicationStr.data());


		m_startInfo.Expand();

		const tchar* extractExportsPos = nullptr;
		if (Contains(m_startInfo.arguments, g_extractExportsStr, true, &extractExportsPos))
		{
			const tchar* after = extractExportsPos + (sizeof_array(g_extractExportsStr) - 1); // skip "/extractexports"
			StringView sv(after, TStrlen(after));
			u32 exiVersion = 2; // bare "/extractexports" == version 2
			if (sv.count && sv.data[0] != ' ')
				sv.Parse(exiVersion);
			m_extractExportsVersion = u8(exiVersion);
		}

		StringBuffer<> realApplication(m_startInfo.application);
		const tchar* realWorkingDir = m_startInfo.workingDir;

		if (!m_session.PrepareProcess(*this, m_parentProcess != nullptr, realApplication, realWorkingDir))
		{
			m_exitCode = 44324;
			CallExitedFunc();
			m_hasExited = true;
			m_messageThreadDone.Set();
			return false;
		}

		m_realApplication = realApplication.data;
		m_realWorkingDir = realWorkingDir;
		if (realWorkingDir == startInfo.workingDir)
			m_realWorkingDir = m_startInfo.workingDir;

		if (m_parentProcess)
			m_waitForParent.Create(EventResetType_Manual);
		else if (!m_shared.overlay.handle.IsValid())
			m_session.CreateOverlay(m_shared.overlay);

		if (!m_startInfo.overlayFilesStr.empty())
		{
			UBA_ASSERT(!m_parentProcess);
			if (!m_session.PopulateOverlayFiles(m_shared.overlay, m_startInfo.overlayFilesStr))
			{
				m_session.DestroyOverlay(m_shared.overlay);
				m_exitCode = 44325;
				CallExitedFunc();
				m_hasExited = true;
				m_messageThreadDone.Set();
				return false;
			}
		}

		UBA_ASSERT(m_startInfo.rules);

		// If running remote we can't use mspdbsrv (not supported yet).. so instead embed information in .obj file
		// TODO: This should be placed somewhere else it feels like.
		#if PLATFORM_WINDOWS
		if (runningRemote && (m_startInfo.rules->index == SpecialRulesIndex_ClExe || m_startInfo.rules->index == SpecialRulesIndex_LinkExe))
		{
			tchar* pos = nullptr;
			if (Contains(m_startInfo.argumentsStr.data(), L"/FS ", true, (const tchar**)&pos))
				memcpy(pos, L"/Z7", 6);
		}
		#endif

		m_session.ProcessAdded(*this, 0);

		// This is a little bit hacky but we only need this dir and not its parents. So we write only a dir entry with no children or parents
		if (!m_parentProcess)
		{
			m_shared.m_tempPathId = m_session.PopTempDirectory(m_shared.m_tempPath);
			OverlayEditScope scope(m_shared.overlay);
			m_session.RegisterNonRootedDir(scope, ToView(m_session.m_tempPath).RemoveSuffix(1)); // For some reason if this is not registered ShaderCompileWorker with ios target will fail
			m_session.RegisterNonRootedDir(scope, ToView(m_shared.m_tempPath).RemoveSuffix(1));
		}

		if (async)
		{
			m_session.m_processMessageThreads.AddWork([this, &environment](const WorkContext& context)
				{
					m_messageThread = context.thread;
					ThreadRun(environment);
					ThreadExit();
					m_messageThreadDone.Set();
				}, 1, TC("UbaProcMsg"));
		}
		else
		{
			// This is needed to handle cltr-c. Otherwise this thread might exit before the detoured process which in turn might do things after ctrl-c which can cause deadlocks in detoured process
			#if !PLATFORM_WINDOWS
			sigset_t newMask;
			sigset_t oldMask;
			sigemptyset(&newMask);
			sigaddset(&newMask, SIGINT);
			pthread_sigmask(SIG_BLOCK, &newMask, &oldMask);
			#endif

			ThreadRun(environment);
			ThreadExit();

			#if !PLATFORM_WINDOWS
			pthread_sigmask(SIG_BLOCK, &oldMask, nullptr);
			#endif

			m_messageThreadDone.Set();
		}
		return true;
	}

	bool ProcessImpl::IsActive()
	{
		if (m_nativeProcessHandle == InvalidProcHandle)
		{
			//m_session.m_logger.Info(TC("IsActive false 1"), LastErrorToText().data);
			return false;
		}

		#if PLATFORM_WINDOWS
		DWORD waitRes = WaitForSingleObject((HANDLE)m_nativeProcessHandle, 0);
		if (waitRes == WAIT_TIMEOUT)
			return true;
		if (waitRes != WAIT_OBJECT_0)
		{
			m_session.m_logger.Error(TC("WaitForSingleObject failed on handle %llu id %u returning %u (%s)"), u64(m_nativeProcessHandle), m_nativeProcessId, waitRes, LastErrorToText().data);
			return false;
		}

		DWORD exitCode = STILL_ACTIVE;
		if (!GetExitCodeProcess((HANDLE)m_nativeProcessHandle, &exitCode))
		{
			m_nativeProcessExitCode = ~0u;
			m_session.m_logger.Error(TC("GetExitCodeProcess failed (%s)"), LastErrorToText().data);
			return false;
		}
		if (exitCode == STILL_ACTIVE)
			return true;
		if (!m_gotExitMessage && exitCode != EXCEPTION_ACCESS_VIOLATION && exitCode != STATUS_STACK_BUFFER_OVERRUN)
		{
			StringBuffer<> err;

			if (m_messageCount == 0) // This is bad.. bad binaries?
			{
				err.Append(TCV("ERROR: Process did not start properly. "));

				ImageInfo imageInfo;
				bool machineIsArm = IsArmBinary;
				if (exitCode == 1398)
					err.Appendf(TC("UbaDetours.dll has a different version than " UBA_BINARY));
				else if (!GetImageInfo(imageInfo, m_session.m_logger, m_realApplication.c_str(), true))
					err.Appendf(TC("Failed to load %s"), m_realApplication.c_str());
				else if (!imageInfo.isPEImage)
					err.Append(TCV("Doesn't seem to be a PE executable"));
				else if (!imageInfo.is64Bit)
					err.Append(TCV("Doesn't seem to be a 64-bit executable"));
				else if (imageInfo.isDotnet)
					err.Append(TCV("Dotnet binary"));
				else if (!imageInfo.isArm64 && !imageInfo.isX64)
					err.Append(TCV("Unknown image architecture"));
				else if (!machineIsArm && imageInfo.isArm64)
					err.Appendf(TC("Machine is x64 and image is arm64"));

				if (exitCode != 1398)
					err.Appendf(TC(" (GetExitCodeProcess returned 0x%x - %s Size: %llu, CasKey: %s)"), exitCode, m_realApplication.c_str(), imageInfo.fileSize, CasKeyString(imageInfo.key).str);
			}

			if (err.IsEmpty())
			{
				// Ok, this is a bit hacky putting this here.. but theory is that process could have exited and exit message is still in the shared memory
				// We've seen builds on the farm that has a successful process but also error below
				if (m_readEvent.IsSet(0))
					return true;

				err.Appendf(TC("ERROR: Process %llu %s (%s) not active but did not get exit message. Received %u messages (GetExitCodeProcess returned 0x%x)"), u64(m_nativeProcessHandle), m_startInfo.GetDescription(), m_realApplication.c_str(), m_messageCount, exitCode);
			}
			LogLine(false, err.data, LogEntryType_Error);
			m_nativeProcessExitCode = UBA_EXIT_CODE(666);
		}
		return false;

		#else

		if (m_parentProcess)
		{
#if PLATFORM_LINUX // Can't do wait on grandchildren on Linux.. but since we use PR_SET_CHILD_SUBREAPER we should once parent is gone and child is orphaned
			if (m_parentProcess->m_nativeProcessId != 0)
				return true;
			// Cancel zeros m_parentProcess->m_nativeProcessId without reaping; waitid would return ECHILD here. Stay active until cancel propagates.
			if (m_cancelled || m_parentProcess->m_cancelled)
				return true;
#elif PLATFORM_MAC
			if (m_gotExitMessage)
				return false;
			// No child should outlive the parent but there could be a bit of race here between threads
			if (m_parentProcess->m_hasExited)
			{
				u64 timeSinceParentExitedMs = TimeToMs(GetTime() - m_parentProcess->m_processStats.exitTime);
				if (timeSinceParentExitedMs > 5*1000)
				{
					m_session.m_logger.Info(TC("Child process %s outlived parent for 5 seconds. Will treat it as not active"), m_realApplication.c_str());
					return false;
				}
			}
#endif
		}

		siginfo_t signalInfo;
		while (m_nativeProcessId != 0)
		{
			memset(&signalInfo, 0, sizeof(signalInfo));
			int res = waitid(P_PID, (unsigned int)m_nativeProcessId, &signalInfo, WEXITED | WNOHANG | WNOWAIT);
			StringBuffer<> err;

			if (res)
			{
				UBA_ASSERT(res == -1);
				if (errno == EINTR)
					continue;
				if (errno == ECHILD)
				{
#if PLATFORM_MAC
					if (m_messageCount == 0)
					{
						err.Appendf(TC("NO MESSAGES RECEIVED BEFORE EXIT. Use \"UbaCli -validate %s\" to verify DYLD env support"), m_startInfo.application);
						LogLine(false, err.data, LogEntryType_Error);
						m_nativeProcessExitCode = UBA_EXIT_CODE(667);
						return false;
					}
					// Mac has no PR_SET_CHILD_SUBREAPER, so waitid(P_PID, grandchild_pid, ...) always returns ECHILD even while the grandchild is alive.
					// UBT spawns clang++ which spawns cc1 — cc1 is a grandchild. Treating ECHILD here as "exited" causes ProcessImpl::~ProcessImpl
					// to destroy the comm-slot events while cc1 is still mid-RPC, deadlocking it on Destroyed events.
					// We've already received messages (m_messageCount > 0), so trust the comm channel's exit message to mark exit instead.
					return true;
#else
					// ECHILD: process unreachable (WSL2/CHILD_SUBREAPER reparenting, or child reaped via another path). Mark exited so the message loop stops.
					m_nativeProcessId = 0;
					return false;
#endif
				}
				UBA_ASSERTF(false, "waitid failed with error: %u (%s)", errno, strerror(errno));
				break;
			}
			else
			{
				if (signalInfo.si_pid != (pid_t)m_nativeProcessId)
					return true;

				const char* codeType = nullptr;
				const char* extraString = "";
				switch (signalInfo.si_code)
				{
				case CLD_KILLED:
					codeType = "killed";
					break;
				case CLD_DUMPED:
					codeType = "killed";
					extraString = " (dumped core)";
					break;
				case CLD_STOPPED:
					codeType = "stopped";
					break;
				case CLD_TRAPPED:
					codeType = "trapped";
					break;
				case CLD_CONTINUED:
					codeType = "continued";
					break;
				}

				u32 nativeProcessId = m_nativeProcessId;
				// Don't clear m_nativeProcessId — waitid used WNOWAIT (peek-only); the actual reap happens in InternalExitProcess's blocking waitpid loop.
				m_nativeProcessExitCode = signalInfo.si_status;

				if (!codeType) // Is null if graceful exit (CLD_EXITED)
				{
					// posix_spawn convention: _exit(127) = exec failed, _exit(126) = file action failed / not executable.
					if ((signalInfo.si_status == 127 || signalInfo.si_status == 126) && m_messageCount == 0)
					{
						err.Appendf(TC("Process %u (%s) exited %i before detour handshake — likely exec/file-action failure (missing binary, wrong arch, ETXTBSY, bad interpreter, or file_action error). Working dir: %s."),
							nativeProcessId, m_startInfo.GetDescription(), signalInfo.si_status, m_realWorkingDir);
						LogLine(false, err.data, LogEntryType_Error);
						m_nativeProcessExitCode = UBA_EXIT_CODE(666); // We do exit code 666 to trigger non-uba retry on the outside
						return false;
					}
					break;
				}

				err.Appendf(TC("Process %u (%s) %s by signal %i. Received %u messages. Execution time: %s. (%s)"), nativeProcessId, m_startInfo.GetDescription(), codeType, signalInfo.si_status, m_messageCount, TimeToText(GetTime() - m_startTime).str, ToView(m_startInfo.logFileStr).GetFileName().data);
				LogLine(false, err.data, LogEntryType_Error);
				m_nativeProcessExitCode = UBA_EXIT_CODE(666); // We do exit code 666 to trigger non-uba retry on the outside
				return false;
			}
		}

		// There is a small race condition between this process polling and exit message.
		// Detoured process can't wait for exit message response and then close the shared memory because it might end up closing another process memory
		// .. so solution is to do one more poll from here to make sure we pick up the message before leaving.
		if (!m_gotExitMessage)
		{
			if (m_doOneExtraCheckForExitMessage)
			{
				m_doOneExtraCheckForExitMessage = false;
				return true;
			}

			StringBuffer<> err;
			err.Appendf(TC("ERROR: Process %u (%s) not active but did not get exit message. Received %u messages. Signal code: %i. Exit value or signal: %i. Execution time: %s."), m_nativeProcessId, m_startInfo.GetDescription(), m_messageCount, signalInfo.si_code, signalInfo.si_status, TimeToText(GetTime() - m_startTime).str);
			LogLine(false, err.data, LogEntryType_Error);
			m_nativeProcessExitCode = UBA_EXIT_CODE(666);

			#if PLATFORM_MAC
			if (m_messageCount == 0)
			{
				err.Clear().Appendf(TC("Use \"UbaCli -validate %s\" to verify DYLD env support"), m_startInfo.application);
				ProcessImpl* p = m_parentProcess ? m_parentProcess : this;
				p->LogLine(false, err.data, LogEntryType_Error);
				m_nativeProcessExitCode = UBA_EXIT_CODE(667);
			}
			#endif
		}

		//m_session.m_logger.Info(TC("IsActive false (no parent)"), LastErrorToText().data);
		return false;
		#endif
	}

	bool ProcessImpl::IsCancelled()
	{
		return m_cancelled;
	}

	bool ProcessImpl::HasFailedMessage()
	{
		return m_firstMessageError != MessageType_None;
	}

	bool ProcessImpl::WaitForExit(u32 millisecondsTimeout)
	{
		return m_messageThreadDone.IsSet(millisecondsTimeout);
	}

	u64 ProcessImpl::GetTotalProcessorTime() const
	{
		return m_processStats.cpuTime;
	}

	u64 ProcessImpl::GetTotalWallTime() const
	{
		return m_processStats.wallTime;
	}

	u64 ProcessImpl::GetPeakMemory() const
	{
		return m_processStats.peakJobMemory > 0 ? m_processStats.peakJobMemory : m_processStats.peakMemory;
	}

	bool ProcessImpl::Cancel()
	{
		SCOPED_FUTEX(m_logLinesLock, l); // Abuse exiting mutex to not waste memory
		if (!m_cancelAllowed)
			return m_cancelled;
		m_cancelled = true;
		#if PLATFORM_WINDOWS
		m_cancelEvent.Set();
		#else
		SCOPED_FUTEX(m_comMemoryLock, lock);
		if (m_comMemory.memory)
			m_cancelEvent.Set();
		#endif
		return true;
	}

	void ProcessImpl::TraverseOutputFiles(const Function<void(StringView file)>& func)
	{
		SCOPED_FUTEX_READ(m_shared.writtenFilesLock, lock);
		for (auto& kv : m_shared.writtenFiles)
		{
			WrittenFile& wf = kv.second;
			func(wf.name);
		}
	}

	bool ProcessImpl::WaitForRead()
	{
		// This number could be bigger but
		// 1. It would delay ctrl-c
		// 2. If processes crashes and does not send exit message
		constexpr u32 readEventTimeout = 1000;

		while (true)
		{
			if (m_readEvent.IsSet(readEventTimeout))
				return !m_cancelled;

			if (!IsActive())
				return m_readEvent.IsSet(0); // Do one more check

			if (m_cancelled)
				return false;
		}
	}

	void ProcessImpl::SetWritten()
	{
		m_writeEvent.Set();
	}

	void ProcessImpl::ThreadRun(Vector<tchar>& environment)
	{
		KernelStatsScope kernelStatsScope(m_kernelStats);
		StorageStatsScope storageStatsScope(m_storageStats);
		SessionStatsScope sessionStatsScope(m_sessionStats);

		if (HandleSpecialApplication())
			return;

		if (!m_session.ProcessThreadStart(*this))
		{
			m_logLines.push_back({ TString(TC("Session::ProcessThreadStart failed")), LogEntryType_Error });
			return;
		}

		u8* comMemory = m_comMemory.memory;
		u64 comMemorySize = CommunicationMemSize;
		#if !PLATFORM_WINDOWS
		if (!UseSemaphoreForSharedEvents())
		{
			comMemory += sizeof(SharedEvent) * 3;
			comMemorySize -= sizeof(SharedEvent) * 3;
		}
		#endif

		u32 retryCount = 0; // Do not allow retry

		u32 exitCode = ~0u;

		while (!IsCancelled())
		{
			exitCode = InternalCreateProcess(environment, m_comMemory.handle, m_comMemory.offset);

			#if !PLATFORM_WINDOWS
			PipeReader outReader(*this, LogEntryType_Info);
			PipeReader errReader(*this, LogEntryType_Error);
			Atomic<bool> pollStdDone = false;
			Thread pollStdThread([&]()
			{
				while (!pollStdDone)
					if (!PollStdPipes(outReader, errReader, -1))
						return false;
				return true;
			}, TC("PollStd"));
			#endif
			
			if (exitCode == 0)
			{
				m_session.ProcessNativeCreated(*this);

				bool loop = m_detourEnabled;
				while (loop && WaitForRead())
				{
					u64 startTime = GetTime();
					BinaryReader reader(comMemory, 0, comMemorySize);
					BinaryWriter writer(comMemory, 0, comMemorySize);

					#if UBA_PROTOCOL_GUARD
					u32 deadbeef = reader.ReadU32();
					if (deadbeef != 0xdeadbeef)
						m_session.m_logger.Error(TC("SHOULD NOT HAPPEN%s"), deadbeef == 0xbaadf00d ? TC(" (Seems like nothing was written?)") : TC(""));
					writer.WriteU32(0xbaadf00d);
					#endif

					loop = HandleMessage(reader, writer);
					SetWritten();
					m_processStats.hostTotalTime += GetTime() - startTime;
					++m_messageCount;
				}
			}

			m_exitTime = GetTime();

			m_session.ReleaseDirTableSnapshot(m_dirTableSnapshot); // Should be null, but just in case (if it crashed or something)
			m_dirTableSnapshot = nullptr;

			// Process can't be cancelled after this.
			SCOPED_FUTEX(m_logLinesLock, cancelLock); // Abuse exiting mutex to not waste memory
			m_cancelAllowed = false;
			bool cancelled = IsCancelled();
			cancelLock.Leave();

			auto& rules = *m_startInfo.rules;

			bool writeSuccess = true;
			auto writeFiles = MakeGuard([&]()
				{
					if (!cancelled && (m_startInfo.writeOutputFilesOnFail || rules.IsExitCodeSuccess(exitCode)))
					{
						if (!m_parentProcess && m_extractExportsVersion)
							m_session.ExtractSymbolsForProcessTree(*this);
						writeSuccess = WriteFilesToDisk();
					}
				});

			if (rules.ShouldWriteFilesBeforeWaitOnExit())
				writeFiles.Execute();

			if (exitCode == 0)
				exitCode = InternalExitProcess(cancelled);

			// For some reason a parent can exit before a child. I've seen it happen on ClangEditor win64 and also with some experimental features enabled on msvc
			// We need to wait also because we need all written files to be added in shared file system
			WaitForChildrenExit();

			#if !PLATFORM_WINDOWS
			pollStdDone = true;
			pollStdThread.Wait();
			PollStdPipes(outReader, errReader, -1);
			outReader.Flush();
			errReader.Flush();
			#endif
			
			if (cancelled)
			{
				if (!m_parentProcess)
					m_session.ProcessCancelled(*this);
			}
			else if (exitCode == 0)
			{
				writeFiles.Execute(); // might already have been written
			}

			if (exitCode == 0 && !writeSuccess)
				exitCode = UBA_EXIT_CODE(1);

			u64 childMaxPeakMemory = 0;
			u64 childMaxDetoursMemory = 0;
			for (auto& cph : m_childProcesses)
			{
				auto& child = *(ProcessImpl*)cph.m_process;
				m_processStats.Add(child.m_processStats);
				m_sessionStats.Add(child.m_sessionStats);
				m_storageStats.Add(child.m_storageStats);
				m_kernelStats.Add(child.m_kernelStats);

				// We assume children are running one at the time.. this is an so-so estimation
				childMaxPeakMemory = Max(childMaxPeakMemory, child.m_processStats.peakMemory.load());
				childMaxDetoursMemory = Max(childMaxDetoursMemory, child.m_processStats.detoursMemory.load());

				u64 size = m_trackedInputs.size();
				m_trackedInputs.resize(size + child.m_trackedInputs.size());
				memcpy(m_trackedInputs.data() + size, child.m_trackedInputs.data(), child.m_trackedInputs.size());
			}
			AtomicMax(m_processStats.peakMemory, m_processStats.peakMemory + childMaxPeakMemory);
			AtomicMax(m_processStats.detoursMemory, m_processStats.detoursMemory + childMaxDetoursMemory);

			if (exitCode == 0 && m_firstMessageError != 0)
				exitCode = UBA_EXIT_CODE(1);

			bool isChild = m_parentProcess != nullptr;

			if (cancelled || isChild)
				break;

			if (retryCount == 0)
				break;
			--retryCount;

			if (exitCode == EXCEPTION_ACCESS_VIOLATION)
				m_session.m_logger.Warning(TC("Process exited with access violation. Will do one retry."));
			else if (exitCode == STATUS_STACK_BUFFER_OVERRUN)
				m_session.m_logger.Warning(TC("Process exited with stack buffer overflow. Will do one retry."));
			else
				break;

			// TODO: This should probably do much more cleanup..
			// This is the retry cleanup if the process crashes
			m_logLines.clear();
			m_trackedInputs.clear();
			m_shared.writtenFiles.clear();
		}

		#if PLATFORM_WINDOWS
		if (m_accountingJobObject)
		{
			JOBOBJECT_BASIC_ACCOUNTING_INFORMATION accountingInformation = {};
			if (QueryInformationJobObject(m_accountingJobObject, JOBOBJECTINFOCLASS::JobObjectBasicAccountingInformation, &accountingInformation, sizeof(accountingInformation), NULL))
				AtomicMax(m_processStats.cpuTime, u64(accountingInformation.TotalUserTime.QuadPart + accountingInformation.TotalKernelTime.QuadPart));
			JOBOBJECT_EXTENDED_LIMIT_INFORMATION limitInformation = {};
			if (QueryInformationJobObject(m_accountingJobObject, JOBOBJECTINFOCLASS::JobObjectExtendedLimitInformation, &limitInformation, sizeof(limitInformation), NULL))
			{
				m_processStats.peakProcessMemory = limitInformation.PeakProcessMemoryUsed;
				m_processStats.peakJobMemory = limitInformation.PeakJobMemoryUsed;

				// Non-detoured processes never send an exit message, so peakMemory (the cross-platform field
				// the trace/visualizer reports on) is otherwise left 0. Seed it from the job-object peak.
				if (!m_detourEnabled)
					AtomicMax(m_processStats.peakMemory, limitInformation.PeakJobMemoryUsed);
			}
			CloseHandle(m_accountingJobObject);
		}
		#endif

		if (m_nativeProcessExitCode == EmergencyShutdownExitCode)
		{
			m_session.EmergencyShutdown();
		}

		if (m_exitCode == ~0u)
		{
			if (IsCancelled())
				m_exitCode = ProcessCancelExitCode;
			else
				m_exitCode = exitCode;
		}
	}

	void ProcessImpl::ThreadExit()
	{
		m_processStats.wallTime = GetTime() - m_startTime;

		KernelStats::GetGlobal().Add(m_kernelStats);

		#if UBA_DEBUG_TRACK_PROCESS
		m_session.m_debugLogger->Info(TC("ProcessExitedStart (%u)"), m_id);
		#endif

		UBA_ASSERTF(IsCancelled() || !m_parentProcess || !m_parentProcess->m_hasExited || m_parentProcess->m_cancelled, TC("Process %s exited after parent exited"), m_startInfo.GetDescription());

		m_hasExited = true;

		if (m_comMemory.memory)
		{
			#if !PLATFORM_WINDOWS
			SCOPED_FUTEX(m_comMemoryLock, lock);
			#endif

			InternalDestroyEvents();
			m_session.m_processCommunicationAllocator.Free(m_comMemory);
			m_comMemory = {};
		}

		CallExitedFunc();

		if (!m_parentProcess)
		{
			CleanupWrittenFiles();

			//m_session.PrintOverlay(m_shared.overlay);

			for (auto& memoryHandle : m_shared.sharedMemoryHandles)
				m_session.m_sharedMemory.CloseHandle(m_session.m_logger, memoryHandle, TC(""));
			m_shared.sharedMemoryHandles.clear();
			m_session.DestroyOverlay(m_shared.overlay);

			m_session.PushTempDirectory(m_shared.m_tempPathId);
			m_shared.m_tempPathId = ~0u;
		}

		UBA_ASSERT(m_refCount);

		if (m_exitTime)
			m_processStats.exitTime += GetTime() - m_exitTime;

		// Must be done last to make sure shutdown is not racing
		m_session.ProcessExited(*this, m_processStats.wallTime);

		#if UBA_DEBUG_TRACK_PROCESS
		m_session.m_debugLogger->Info(TC("ProcessExitedDone  (%u)"), m_id);
		#endif
	}

	void ProcessImpl::CallExitedFunc()
	{
		if (!m_startInfo.exitedFunc)
			return;
		auto exitedFunc = m_startInfo.exitedFunc;
		auto userData = m_userDataOverride ? m_userDataOverride : m_startInfo.userData;
		m_userDataOverride = nullptr;
		m_startInfo.exitedFunc = nullptr;
		ProcessHandle h;
		h.m_process = this;
		ProcessExitedResponse exitedResponse = ProcessExitedResponse_None;
		exitedFunc(userData, h, exitedResponse);
		h.m_process = nullptr;
	}

	bool ProcessImpl::HandleSpecialApplication()
	{
	#if PLATFORM_WINDOWS
		if (!Equals(m_startInfo.application, TC("ubacopy")))
			return false;
		const tchar* fromFileBegin = m_startInfo.arguments;
		if (*fromFileBegin == '\"')
			++fromFileBegin;
		const tchar* fromFileEnd = TStrchr(fromFileBegin, '\"');
		UBA_ASSERT(fromFileEnd);
		if (!fromFileEnd)
			return false;
		const tchar* toFileBegin = TStrchr(fromFileEnd + 1, '\"');
		UBA_ASSERT(toFileBegin);
		if (!toFileBegin)
			return false;
		++toFileBegin;
		const tchar* toFileEnd = TStrchr(toFileBegin, '\"');
		UBA_ASSERT(toFileEnd);
		if (!toFileEnd)
			return false;

		m_processStats.wallTime = GetTime() - m_startTime;

		StringBuffer<> workDir(m_startInfo.workingDir);
		workDir.EnsureEndsWithSlash();

		StringBuffer<> fromName;
		StringBuffer<> toName;

		StringBuffer<> temp;
		temp.Append(fromFileBegin, fromFileEnd - fromFileBegin);
		FixPath(temp.data, workDir.data, workDir.count, fromName);
		temp.Clear().Append(toFileBegin, toFileEnd - toFileBegin);
		FixPath(temp.data, workDir.data, workDir.count, toName);

		UBA_ASSERTF(m_session.m_shouldWriteToDisk, TC("NotImplemented - Copy code path does not support keeping files in memory. (%s -> %s)"), fromName.data, toName.data);

		DWORD oldAttributes = GetFileAttributes(toName.data);
		if (oldAttributes != INVALID_FILE_ATTRIBUTES && (oldAttributes & FILE_ATTRIBUTE_READONLY))
			SetFileAttributes(toName.data, oldAttributes & (~FILE_ATTRIBUTE_READONLY));

		StringKey toKey = ToStringKeyLower(toName);

		#if UBA_DEBUG_LOG_ENABLED
		FileAccessor logFile(m_session.m_logger, m_startInfo.logFile);
		bool logSuccess = *m_startInfo.logFile && logFile.CreateWrite();
		auto writeLogLine = [&](const StringBufferBase& line)
			{
				if (!logSuccess)
					return;
				char buf[2*1024];
				if (u32 written = line.Parse(buf, sizeof_array(buf)))
					logFile.Write(buf, written - 1);
				logFile.Write("\n", 1);
			};
		writeLogLine(StringBuffer<>().Appendf(TC("ProcessId: %u"), m_id));
		writeLogLine(StringBuffer<>().Appendf(TC("CmdLine: %s %s"), m_startInfo.application, m_startInfo.arguments));
		auto logGuard = MakeGuard([&]() { if (logSuccess) logFile.Close(); });
		#endif

		// We need to report before writing to flush out potential deferred cas creations
		m_session.m_storage.ReportFileWrite(toKey, toName.data);

		bool copySuccess = CopyFileW(fromName.data, toName.data, false);

		#if UBA_DEBUG_LOG_ENABLED
		writeLogLine(StringBuffer<>().Appendf(TC("T      CopyFileW %s to %s -> %s"), fromName.data, toName.data, (copySuccess?TC("true"):TC("true"))));
		#endif

		if (!copySuccess)
		{
			m_exitCode = GetLastError();
			temp.Clear().Appendf(TC("Failed to copy %s to %s (%s)"), fromName.data, toName.data, LastErrorToText(m_exitCode).data);
			m_logLines.push_back({ temp.ToString(), LogEntryType_Error });
			return true;
		}
		
		bool attributesSuccess = SetFileAttributes(toName.data, DefaultAttributes());(void)attributesSuccess;

		#if UBA_DEBUG_LOG_ENABLED
		writeLogLine(StringBuffer<>().Appendf(TC("T      SetFileAttributes %s -> %s"), toName.data, (attributesSuccess?TC("true"):TC("true"))));
		#endif

		m_session.m_storage.InvalidateCachedFileInfo(toKey);

		m_session.RegisterCreateFileForWrite(toKey, toName, 0, 0, DefaultAttributes(), true, true);

		//m_writtenFiles.try_emplace(name);
		//WrittenFile& writtenFile = m_writtenFiles[toName.data];
		//writtenFile.key = toKey;
		//writtenFile.name = toName.data;
		//writtenFile.ownerId = GetId();
		//writtenFile.attributes = DefaultAttributes();

		StackBinaryWriter<1024> trackedInputs;
		trackedInputs.WriteString(fromName);
		m_trackedInputs.resize(trackedInputs.GetPosition());
		memcpy(m_trackedInputs.data(), trackedInputs.GetData(), trackedInputs.GetPosition());

		StackBinaryWriter<1024> trackedOutputs;
		trackedOutputs.WriteString(toName);
		m_trackedOutputs.resize(trackedOutputs.GetPosition());
		memcpy(m_trackedOutputs.data(), trackedOutputs.GetData(), trackedOutputs.GetPosition());
		
		m_exitCode = 0;
		return true;
	#else
		return false;
	#endif
	}


	bool ProcessImpl::HandleMessage(BinaryReader& reader, BinaryWriter& writer)
	{
		MessageType messageType = (MessageType)reader.ReadByte();
		switch (messageType)
		{
			#define UBA_PROCESS_MESSAGE(type) case MessageType_##type: return Handle##type(reader, writer);
			UBA_PROCESS_MESSAGES
			#undef UBA_PROCESS_MESSAGE
		}
		return CancelWithError(m_session.m_logger.Error(TC("Unknown message type %u when communicating with detoured process %s (%u). (Message index %u after %s)"), messageType, m_startInfo.GetDescription(), m_id, m_messageCount, TimeToText(GetTime() - m_startTime).str));
	}

	void ProcessImpl::CallSession(MessageType type, bool success)
	{
		if (!success && m_firstMessageError == MessageType_None)
			m_firstMessageError = type;
	}

	void ProcessImpl::LogLine(bool printInSession, TString&& line, LogEntryType logType)
	{
		if (IsCancelled())// || m_startInfo.stdoutHandle != INVALID_HANDLE_VALUE)
			return;
		if (printInSession)
			m_session.m_logger.Log(LogEntryType_Warning, line.c_str(), u32(line.size()));
		if (m_startInfo.logLineFunc)
			m_startInfo.logLineFunc(m_startInfo.logLineUserData, line.c_str(), u32(line.size()), logType);

		#if !PLATFORM_WINDOWS
		for (auto& c : line) // Strip non-ASCII so Mac/Linux log entries render on Windows.
			if ((u8)c > 127)
				c = ' ';
		#endif

		SCOPED_FUTEX(m_logLinesLock, l);
		m_logLines.push_back({ std::move(line), logType });
	}

	bool ProcessImpl::HandleInit(BinaryReader& reader, BinaryWriter& writer)
	{
		InitMessage msg { *this };
		InitResponse response;

		CallSession(MessageType_Init, m_session.GetInitResponse(response, msg));
		writer.WriteU32(m_id);
		writer.WriteBool(m_parentProcess != nullptr);
		writer.WriteBool(m_startInfo.trackInputs);
		writer.WriteString(m_startInfo.application);
		writer.WriteString(m_startInfo.workingDir);
		writer.WriteU64(response.sharedMemoryAllocatorHandle.ToU64());
		writer.WriteU64(response.permanentFilesHandle.ToU64());
		writer.WriteU64(response.directoryTableHandle.ToU64());
		writer.WriteU64(ToU64(response.directoryTableSize));
		writer.WriteU32(response.directoryTableCount);
		writer.WriteU64(response.mappedFileTableHandle.ToU64());
		writer.WriteU32(response.mappedFileTableSize);
		writer.WriteU32(response.mappedFileTableCount);

		u64 snapshotHandle = 0;
		u64 snapshotAddress = 0;
		u32 snapshotSize = DirTableSnapshotMaxSize;

		m_dirTableSnapshot = m_session.ReferenceDirTableSnapshot();
		if (m_dirTableSnapshot)
		{
			snapshotHandle = m_dirTableSnapshot->handle.ToU64();
			snapshotAddress = (u64)m_dirTableSnapshot->mappedMemory;
		}
		
		writer.WriteU64(snapshotHandle);
		writer.WriteU64(snapshotAddress);
		writer.WriteU32(snapshotSize);

		writer.WriteU64(m_shared.overlay.handle.ToU64());
		// TODO: We should also send overlay count to make a good prediction of size.. we never want to rehash

		if (!m_startInfo.rootsHandle)
		{
			writer.WriteU16(0);
			return true;
		}

		auto rootsEntry = m_session.GetRootsEntry(m_startInfo.rootsHandle);
		if (!rootsEntry)
			return CancelWithError();

		UBA_ASSERTF(rootsEntry->memory.size() <= u64(~u16(0)), TC("HandleInit: rootsEntry memory size %llu exceeds u16 wire cap; would truncate"), (u64)rootsEntry->memory.size());
		writer.WriteU16(u16(rootsEntry->memory.size()));
		writer.WriteBytes(rootsEntry->memory.data(), rootsEntry->memory.size());
		return true;
	}
	
	bool ProcessImpl::HandleSnapshotDuplicated(BinaryReader& reader, BinaryWriter& writer)
	{
		m_session.ReleaseDirTableSnapshot(m_dirTableSnapshot);
		m_dirTableSnapshot = nullptr;
		return true;
	}

	bool ProcessImpl::HandleCreateFile(BinaryReader& reader, BinaryWriter& writer)
	{
		CreateFileMessage msg { *this };
		reader.ReadString(msg.fileName);
		msg.fileNameKey = reader.ReadStringKey();
		msg.access = (FileAccess)reader.ReadByte();
		UBA_ASSERT(msg.fileName.count);

		CreateFileResponse response;
		CallSession(MessageType_CreateFile, m_session.CreateFile(response, msg));
		writer.WriteString(response.fileName);
		writer.WriteU64(response.size);
		writer.WriteU32(response.closeId);
		writer.WriteU32(response.mappedFileTableSize);
		writer.WriteU64(ToU64(response.directoryTableSize));
		return true;
	}

	bool ProcessImpl::HandleRegisterFileForWrite(BinaryReader& reader, BinaryWriter& writer)
	{
		StringBuffer<> fileName;
		reader.ReadString(fileName);
		StringKey fileNameKey = reader.ReadStringKey();
		DirTableOffset offset;
		{
			OverlayEditScope scope(m_shared.overlay);
			m_session.AddOverlayFile(offset, scope, fileName);

			m_shared.movedFiles.erase(fileName.data);
			m_shared.deletedFiles.erase(fileName.data); // TODO: Maybe we need to track if a delete existed in case there will be rename while file is open?
		}
		writer.WriteU32(offset.internal);
		writer.WriteU64(ToU64(m_session.GetDirectoryTableSize(&m_shared.overlay)));
		return true;
	}

	bool ProcessImpl::HandleGetFullFileName(BinaryReader& reader, BinaryWriter& writer)
	{
		GetFullFileNameMessage msg { *this };
		reader.ReadString(msg.fileName);
		msg.fileNameKey = reader.ReadStringKey();
		msg.loaderPathsSize = reader.ReadU16();
		msg.loaderPaths = reader.GetPositionData();
		
		GetFullFileNameResponse response;
		CallSession(MessageType_GetFullFileName, m_session.GetFullFileName(response, msg));
		writer.WriteString(response.fileName);
		writer.WriteString(response.virtualFileName);
		writer.WriteU32(response.mappedFileTableSize);
		return true;
	}

	bool ProcessImpl::HandleGetLongPathName(BinaryReader& reader, BinaryWriter& writer)
	{
		GetLongPathNameMessage msg { *this };
		reader.ReadString(msg.fileName);
		GetLongPathNameResponse response;
		CallSession(MessageType_GetLongPathName, m_session.GetLongPathName(response, msg));
		writer.WriteU32(response.errorCode);
		writer.WriteString(response.fileName);
		return true;
	}

	bool ProcessImpl::HandleCloseFile(BinaryReader& reader, BinaryWriter& writer)
	{
		CloseFileMessage msg { *this };
		reader.ReadString(msg.fileName);
		msg.closeId = reader.ReadU32();
		msg.deleteOnClose = reader.ReadBool();
		msg.success = reader.ReadBool();
		msg.memoryHandle = SharedMemoryHandle::FromU64(reader.ReadU64());
		msg.fileSize = reader.ReadU64();
		msg.lastWriteTime = reader.ReadU64();
		msg.attributes = reader.ReadU32();
		msg.newNameKey = reader.ReadStringKey();
		if (msg.newNameKey != StringKeyZero)
			reader.ReadString(msg.newName);
		CloseFileResponse response;
		CallSession(MessageType_CloseFile, m_session.CloseFile(response, msg));
		writer.WriteU64(ToU64(response.directoryTableSize));
		return true;
	}

	bool ProcessImpl::HandleDeleteFile(BinaryReader& reader, BinaryWriter& writer)
	{
		// TODO: Should be removed and handled by overlay
		DeleteFileMessage msg { *this };
		reader.ReadString(msg.fileName);
		msg.fileNameKey = reader.ReadStringKey();
		msg.closeId = reader.ReadU32();
		DeleteFileResponse response;
		CallSession(MessageType_DeleteFile, m_session.DeleteFile(response, msg));
		writer.WriteBool(response.result);
		writer.WriteU32(response.errorCode);
		writer.WriteU64(ToU64(response.directoryTableSize));
		return true;
	}

	bool ProcessImpl::HandleCopyFile(BinaryReader& reader, BinaryWriter& writer)
	{
		// TODO: Should be removed and handled by overlay
		CopyFileMessage msg{ *this };
		msg.fromKey = reader.ReadStringKey();
		reader.ReadString(msg.fromName);
		msg.fileSize = reader.ReadU64();
		msg.lastWriteTime = reader.ReadU64();
		msg.fileAttributes = reader.ReadU32();

		msg.toKey = reader.ReadStringKey();
		reader.ReadString(msg.toName);
		CopyFileResponse response;
		CallSession(MessageType_CopyFile, m_session.CopyFile(response, msg));
		writer.WriteString(response.fromName);
		writer.WriteString(response.toName);
		writer.WriteU32(response.closeId);
		writer.WriteU32(response.errorCode);
		writer.WriteU64(ToU64(response.directoryTableSize));
		return true;
	}

	bool ProcessImpl::HandleMoveFile(BinaryReader& reader, BinaryWriter& writer)
	{
		// TODO: Should be removed and handled by overlay
		MoveFileMessage msg { *this };
		msg.fromKey = reader.ReadStringKey();
		reader.ReadString(msg.fromName);
		msg.fileSize = reader.ReadU64();
		msg.lastWriteTime = reader.ReadU64();
		msg.fileAttributes = reader.ReadU32();
		msg.toKey = reader.ReadStringKey();
		reader.ReadString(msg.toName);
		msg.flags = reader.ReadU32();
		if (IsSymlink(msg.fileAttributes))
			reader.ReadString(msg.symlinkContent);

		UBA_ASSERT(msg.fromName.count);
		UBA_ASSERT(msg.toName.count);
		MoveFileResponse response;
		CallSession(MessageType_MoveFile, m_session.MoveFile(response, msg));
		writer.WriteBool(response.result);
		writer.WriteU32(response.errorCode);
		writer.WriteU64(ToU64(response.directoryTableSize));
		return true;
	}

	bool ProcessImpl::HandleChmod(BinaryReader& reader, BinaryWriter& writer)
	{
		// TODO: Should be removed and handled by overlay
		ChmodMessage msg { *this };
		msg.fileNameKey = reader.ReadStringKey();
		reader.ReadString(msg.fileName);
		msg.fileMode = reader.ReadU32();
		ChmodResponse response;
		CallSession(MessageType_Chmod, m_session.Chmod(response, msg));
		writer.WriteU32(response.errorCode);
		return true;
	}

	bool ProcessImpl::HandleSymlink(BinaryReader& reader, BinaryWriter& writer)
	{
		SymlinkMessage msg { *this };
		msg.filePathKey = reader.ReadStringKey();
		reader.ReadString(msg.symlinkFilePath);
		msg.contentPathKey = reader.ReadStringKey();
		reader.ReadString(msg.symlinkContentPath);
		reader.ReadString(msg.symlinkContent);
		msg.fileSize = reader.ReadFileSize();
		msg.lastWriteTime = reader.ReadFileTime();
		msg.fileAttributes = reader.ReadFileAttributes();
		SymlinkResponse response;
		CallSession(MessageType_Symlink, m_session.Symlink(response, msg));
		writer.WriteBool(response.success);
		writer.WriteU32(response.errorCode);
		writer.WriteU64(ToU64(response.directoryTableSize));
		return true;
	}

	bool ProcessImpl::HandleCreateDirectory(BinaryReader& reader, BinaryWriter& writer)
	{
		// TODO: Should be removed and handled by overlay
		CreateDirectoryMessage msg { *this };
		msg.nameKey = reader.ReadStringKey();
		reader.ReadString(msg.name);
		CreateDirectoryResponse response;
		CallSession(MessageType_CreateDirectory, m_session.CreateDirectory(response, msg));
		writer.WriteU64(ToU64(response.directoryTableSize));
		return true;
	}

	bool ProcessImpl::HandleRemoveDirectory(BinaryReader& reader, BinaryWriter& writer)
	{
		// TODO: Should be removed and handled by overlay
		RemoveDirectoryMessage msg { *this };
		msg.nameKey = reader.ReadStringKey();
		reader.ReadString(msg.name);
		RemoveDirectoryResponse response;
		CallSession(MessageType_RemoveDirectory, m_session.RemoveDirectory(response, msg));
		writer.WriteU64(ToU64(response.directoryTableSize));
		return true;
	}

	bool ProcessImpl::HandleListDirectory(BinaryReader& reader, BinaryWriter& writer)
	{
		ListDirectoryMessage msg;
		reader.ReadString(msg.directoryName);
		msg.directoryNameKey = reader.ReadStringKey();
		ListDirectoryResponse response;
		CallSession(MessageType_ListDirectory, m_session.GetListDirectoryInfo(response, msg.directoryName, msg.directoryNameKey, &m_shared.overlay));
		writer.WriteU64(ToU64(response.directoryTableSize));
		writer.WriteU32(response.tableOffset.internal);
		return true;
	}

	bool ProcessImpl::HandleUpdateTables(BinaryReader& reader, BinaryWriter& writer)
	{
		bool isInit = reader.ReadBool();
		writer.WriteU64(ToU64(m_session.GetDirectoryTableSize(&m_shared.overlay)));
		writer.WriteU32(m_session.GetFileMappingSize());

#if PLATFORM_LINUX
		return true;
#else
		UBA_ASSERT(!m_firstOverflowMessage); // LD_PRELOAD drains under g_communicationLock so this should be empty; defensive drain below for the stub path.
		while (auto msg = m_firstOverflowMessage)
		{
			m_firstOverflowMessage = msg->next;
			delete msg;
		}
		return HandleGetWrittenFiles(isInit, writer);
#endif
	}

	bool ProcessImpl::HandleGetWrittenFiles(BinaryReader& reader, BinaryWriter& writer)
	{
		bool isInit = reader.ReadBool();
		return HandleGetWrittenFiles(isInit, writer);
	}

	bool ProcessImpl::HandleGetWrittenFiles(bool isInit, BinaryWriter& writer)
	{
		if (auto msg = m_firstOverflowMessage)
		{
			writer.WriteBytes(msg->buffer, msg->written);
			m_firstOverflowMessage = msg->next;
			delete msg;
			return true;
		}

		auto writtenCount = (u32*)writer.AllocWrite(4);
		auto overflowed = (u8*)writer.AllocWrite(1);
		*overflowed = 0;

		BinaryWriter* w = &writer;
		
		BinaryWriter tempWriter(0, 0, 0);
		OverflowedWrittenFilesMessage* activeOverflowMessage = nullptr;

		u32 count = 0;
		SCOPED_FUTEX_READ(m_shared.writtenFilesLock, lock);
		for (auto& kv : m_shared.writtenFiles)
		{
			WrittenFile& wf = kv.second;

			if (!isInit)
			{
				// If file is owned by ancestor we also skip re-fetching it.. since we expect ancestor
				// not to write to file while child process exists. if this changes we need to change approach
				// and track write file stamp or something on the file and track what child processes have
				bool ownedByThisOrAncestor = false;
				for (ProcessImpl* it = this; it && !ownedByThisOrAncestor; it=it->m_parentProcess)
					ownedByThisOrAncestor = wf.ownerId == it->m_id;
				if (ownedByThisOrAncestor)
					continue;
			}

			u64 fileSize;
			if (wf.memoryHandle.IsValid())
				fileSize = wf.memoryWritten;
			else if (!FileExists(m_session.m_logger, wf.backedName.c_str(), &fileSize))
				continue;
			else if (wf.name == wf.backedName)
				continue;

			if (w->GetCapacityLeft() < 1024)
			{
				// Close previous
				*writtenCount = count;
				*overflowed = 1;
				count = 0;

				// Create new message
				auto newMsg = new OverflowedWrittenFilesMessage();

				// Update active message if there is one
				if (auto oldMsg = activeOverflowMessage)
				{
					oldMsg->written = w->GetPosition();
					oldMsg->next = newMsg;
				}
				else
				{
					w = &tempWriter;
					m_firstOverflowMessage = newMsg;
				}
				activeOverflowMessage = newMsg;

				// Setup writer and variables
				tempWriter.Reset(newMsg->buffer, 0, sizeof_array(OverflowedWrittenFilesMessage::buffer));
				writtenCount = (u32*)w->AllocWrite(4);
				overflowed = (u8*)w->AllocWrite(1);
				*overflowed = 0;
			}

			// This is an attempt to reduce size because we hit message max size (65536). Will need to split up in multiple messages eventually
			StringView fileName(wf.name);
			bool isInTemp = fileName.StartsWith(m_shared.m_tempPath);
			if (isInTemp)
				fileName = fileName.Skip(u32(m_shared.m_tempPath.size()));
			w->WriteStringKey(kv.first);
			w->WriteBool(isInTemp);
			w->WriteString(fileName);
			w->WriteString(wf.backedName);
			w->Write7BitEncoded(wf.memoryHandle.ToU64());
			w->Write7BitEncoded(fileSize);
			++count;
		}

		if (activeOverflowMessage)
			activeOverflowMessage->written = w->GetPosition();

		*writtenCount = count;
		return true;
	}

	#if PLATFORM_LINUX
	// This process will become the parent of a process if it becomes orphaned
	void SetSubReaper() { static bool subreaper = []() { prctl(PR_SET_CHILD_SUBREAPER, 1, 0, 0, 0); return true; }(); }
	#endif

	bool ProcessImpl::WriteCreateProcessResponse(ProcessHandle h, BinaryWriter& writer, bool isChild)
	{
		if (!h.m_process)
		{
			// TODO: Do we need to log something here? This failure is most likely a cause of something else so error logging might show somewhere else
			writer.WriteU32(0); // childProcessId
			return true;
		}

		u32 childProcessId = ~0u;
		if (isChild)
		{
			m_childProcesses.push_back(h);
			childProcessId = u32(m_childProcesses.size());
		}

		auto& process = *(ProcessImpl*)h.m_process;

		writer.WriteU32(childProcessId);
		writer.WriteU32(process.m_startInfo.rules->index);

	#if PLATFORM_WINDOWS
		auto& detoursLib = m_session.m_detoursLibrary[IsLinux][process.m_isArmBinary];
		u32 detoursLibLen = u32(detoursLib.size());
		UBA_ASSERT(detoursLibLen < 1024);
		writer.WriteU32(detoursLibLen);
		writer.WriteBytes(detoursLib.data(), detoursLibLen);
	
		if (m_session.m_sharedMemory.GetFileMappingBackend().linuxBackend)
		{
			StringView ubaWineModulePath = GetUbaWineModulePath();
			char ubaWineModule[256];
			ubaWineModulePath.Parse(ubaWineModule, 256);
			writer.WriteU32(ubaWineModulePath.count);
			writer.WriteBytes(ubaWineModule, ubaWineModulePath.count);
		}
		writer.WriteU32(0);
	#else
		auto& t = m_session.m_detoursLibrary[IsLinux][IsArmBinary];
		writer.WriteString(StringView(t.data(), u32(t.size())).GetPath()); // detours lib
		writer.WriteString(process.m_startInfo.applicationStr); // If shebang we need to provide this
	#endif
		
		writer.WriteString(m_realWorkingDir);

		writer.WriteString(process.m_realApplication);

		#if UBA_USE_NATIVE_MAC_SEMAPHORES
		writer.WriteU32(process.m_id);
		#endif

		#if !PLATFORM_WINDOWS
		writer.WriteU64(process.m_comMemory.handle.ToU64());
		writer.WriteU32(process.m_comMemory.offset);
		writer.WriteString(process.m_startInfo.logFile);
		#endif
		
		return true;
	}

	bool ProcessImpl::HandleCreateProcess(BinaryReader& reader, BinaryWriter& writer)
	{
		#if PLATFORM_LINUX
		SetSubReaper();
		#endif

		TString applicationStr = reader.ReadString();
		TString commandLineWithoutApplication = reader.ReadLongString();

		UBA_ASSERTF(!applicationStr.empty() && (applicationStr[0] != '\"' || applicationStr[1] != '\"'), TC("Invalid application name: %s  (Length %llu)"), applicationStr.c_str(), applicationStr.size());

		StringBuffer<> currentDir;
		reader.ReadString(currentDir);
		if (currentDir.IsEmpty())
			currentDir.Append(m_startInfo.workingDir);
		else if (!IsAbsolutePath(currentDir.data))
		{
			TString temp(currentDir.ToString());
			FixPath(temp.c_str(), m_startInfo.workingDir, m_startInfo.workingDirStr.size(), currentDir.Clear());
		}
		bool startSuspended = reader.ReadBool();
		bool isChild = reader.ReadBool();

		StringBuffer<> temp;
		ProcessStartInfo info;
		info.application = applicationStr.c_str();
		info.arguments = commandLineWithoutApplication.c_str();
		info.workingDir = currentDir.data;
		info.logFile = InternalGetChildLogFile(temp);
		info.priorityClass = m_startInfo.priorityClass;
		info.logLineUserData = this;
		info.logLineFunc = [](void* userData, const tchar* line, u32 length, LogEntryType type) { ((ProcessImpl*)userData)->LogLine(false, TString(line, length), type); };
		info.startSuspended = startSuspended;
		info.rootsHandle = m_startInfo.rootsHandle;
		info.trackInputs = m_startInfo.trackInputs;

		ProcessImpl* parent = isChild ? this : nullptr;
		ProcessHandle h = m_session.InternalRunProcess(info, true, parent, true, nullptr);

		if (!WriteCreateProcessResponse(h, writer, isChild))
			return false;

		#if UBA_DEBUG_TRACK_PROCESS
		m_session.m_debugLogger->Info(TC("CreateChildProcess (%u creating child %u at index %u) %s %s (%s)"), m_id, process.m_id, childProcessId-1, process.m_realApplication.c_str(), commandLineWithoutApplication.c_str(), info.logFile);
		#endif
		return true;
	}

	bool ProcessImpl::HandleStartProcess(BinaryReader& reader, BinaryWriter& writer)
	{
		u32 processId = reader.ReadU32();
		UBA_ASSERTF(processId > 0 && processId <= m_childProcesses.size(), TC("Got process id %u for child, expected number lower than %u"), processId, u32(m_childProcesses.size()));
		auto& process = *(ProcessImpl*)m_childProcesses[processId - 1].m_process;
		bool result = reader.ReadBool();
		u32 lastError = reader.ReadU32();

		auto setWaitForParent = MakeGuard([&process]() { process.m_waitForParent.Set(); });

		if (!result)
		{
			#if PLATFORM_WINDOWS
			if (lastError == ERROR_FILENAME_EXCED_RANGE) // This is a command line issue so we dont' want uba to be blamed just because it outputs below logging entry
				return true;
			#endif

			m_session.m_logger.Logf(LogEntryType_Info, TC("Detoured process failed to start child process - %s. %s (Working dir: %s)"), LastErrorToText(lastError).data, process.m_realApplication.c_str(), process.m_realWorkingDir);
			return true;
		}

#if PLATFORM_WINDOWS
		HANDLE nativeProcessHandle = (HANDLE)reader.ReadU64();
		u32 nativeProcessId = reader.ReadU32();
		HANDLE nativeThreadHandle = (HANDLE)reader.ReadU64();

		if (nativeProcessHandle)
		{
			if (!DuplicateHandle((HANDLE)m_nativeProcessHandle, nativeProcessHandle, GetCurrentProcess(), (HANDLE*)&process.m_nativeProcessHandle, 0, false, DUPLICATE_SAME_ACCESS))
				return CancelWithError(m_session.m_logger.Error(TC("Failed to duplicate handle for child process")));
			if (!process.m_nativeProcessHandle || process.m_nativeProcessHandle == InvalidProcHandle)
				return CancelWithError(m_session.m_logger.Error(TC("Failed to duplicate handle for child process")));
			if (!DuplicateHandle((HANDLE)m_nativeProcessHandle, nativeThreadHandle, GetCurrentProcess(), &process.m_nativeThreadHandle, 0, false, DUPLICATE_SAME_ACCESS))
				return CancelWithError(m_session.m_logger.Error(TC("Failed to duplicate handle for child thread %llu in process %llu (%s)"), nativeThreadHandle, m_nativeProcessHandle, LastErrorToText().data));
			if (!process.m_nativeThreadHandle || process.m_nativeThreadHandle == INVALID_HANDLE_VALUE)
				return CancelWithError(m_session.m_logger.Error(TC("Failed to duplicate handle for child thread")));
			process.m_nativeProcessId = nativeProcessId;
		}
#else
		u64 nativeProcessHandle = reader.ReadU64();
		u32 nativeProcessId = reader.ReadU32();
		u64 nativeThreadHandle = reader.ReadU64();
		process.m_nativeProcessHandle = (ProcHandle)nativeProcessHandle;
		process.m_nativeProcessId = nativeProcessId;
#endif
		
		#if UBA_DEBUG_TRACK_PROCESS
		m_session.m_debugLogger->Info(TC("WaitForChildProcessReady (%u waiting for child %u at index %u)"), m_id, process.m_id, processId-1);
		#endif

		setWaitForParent.Execute();

		#if UBA_DEBUG_TRACK_PROCESS
		m_session.m_debugLogger->Info(TC("WaitForChildProcessReadyDone (%u waiting for child %u at index %u)"), m_id, process.m_id, processId-1);
		#endif

		#if PLATFORM_WINDOWS
		// TOOD: This is ugly, should use event or something instead.. right now we make sure to wait and not return until we know payload has been uploaded etc
		// It is a very uncommon usecase that processes starts suspended.. some process in ninja/cmake does it when building clang/llvm
		if (process.m_startInfo.startSuspended)
			while (process.m_nativeThreadHandle)
				Sleep(1);
		#endif

		return true;
	}

	bool ProcessImpl::HandleForkProcess(BinaryReader& reader, BinaryWriter& writer)
	{
	#if PLATFORM_WINDOWS
		return false;
	#else
		#if PLATFORM_LINUX
		SetSubReaper();
		#endif

		StringBuffer<> temp;
		ProcessStartInfo info;
		info = m_startInfo;
		info.logFile = InternalGetChildLogFile(temp);
		info.exitedFunc = nullptr;

		ProcessHandle h = m_session.InternalRunProcess(info, true, this, true, nullptr);
		if (!h.m_process)
		{
			// TODO: Do we need to log something here? This failure is most likely a cause of something else so error logging might show somewhere else
			writer.WriteU32(0); // childProcessId
			return true;
		}

		{
			SCOPED_FUTEX(m_shared.activeFilesLock, lock);
			while (u32 closeId = reader.ReadU32())
			{
				auto findIt = m_shared.activeFiles.find(closeId);
				UBA_ASSERTF(findIt != m_shared.activeFiles.end(), TC("Can't find file with closeId %u. Can this be a race condition?"), closeId);
				++findIt->second.refCount;
			}
		}

		m_childProcesses.push_back(h);
		u32 childProcessId = u32(m_childProcesses.size());
		writer.WriteU32(childProcessId);

		auto& process = *(ProcessImpl*)h.m_process;
		writer.WriteU64(process.m_comMemory.handle.ToU64());
		writer.WriteU32(process.m_comMemory.offset);
		writer.WriteString(info.logFile);
		return true;
	#endif
	}

	bool ProcessImpl::HandleExecProcess(BinaryReader& reader, BinaryWriter& writer)
	{
	#if PLATFORM_WINDOWS
		return false;
	#else
		TString applicationStr = reader.ReadString();
		TString commandLineWithoutApplication = reader.ReadLongString();

		UBA_ASSERTF(!applicationStr.empty() && (applicationStr[0] != '\"' || applicationStr[1] != '\"'), TC("Invalid application name: %s  (Length %llu)"), applicationStr.c_str(), applicationStr.size());

		StringBuffer<> currentDir;
		reader.ReadString(currentDir);
		if (currentDir.IsEmpty())
			currentDir.Append(m_startInfo.workingDir);
		else if (!IsAbsolutePath(currentDir.data))
		{
			TString temp(currentDir.ToString());
			FixPath(temp.c_str(), m_startInfo.workingDir, m_startInfo.workingDirStr.size(), currentDir.Clear());
		}

		m_startInfo.applicationStr = applicationStr.c_str();
		m_startInfo.argumentsStr = commandLineWithoutApplication.c_str();
		m_startInfo.workingDirStr = currentDir.data;

		m_startInfo.application = m_startInfo.applicationStr.c_str();
		m_startInfo.arguments = m_startInfo.argumentsStr.c_str();
		m_startInfo.workingDir = m_startInfo.workingDirStr.c_str();

		m_startInfo.rules = m_session.GetRules(m_startInfo);

		// TODO: Lots of fixes here... m_startInfo should be fixed up here to be the correct (Kind of like HandleCreateProcess)
		StringBuffer<> realApplication(m_startInfo.application);
		const tchar* realWorkingDir = m_startInfo.workingDir;
		bool res = m_session.PrepareProcess(*this, m_parentProcess != nullptr, realApplication, realWorkingDir);

		// Python's PATH-walking fork_exec retries on ENOENT, so don't kill the worker on first miss. Signal ENOENT via processId=0 and let the loop continue.
		if (!res)
		{
			m_session.m_logger.Logf(LogEntryType_Detail, TC("ExecProcess: binary not preparable: %s — returning ENOENT to detoured client"), m_startInfo.application);
			writer.WriteU32(0);
			return true;
		}
		m_realApplication = realApplication.data;
		m_realWorkingDir = realWorkingDir;
		if (realWorkingDir == m_startInfo.workingDir)
			m_realWorkingDir = m_startInfo.workingDir;

		ProcessHandle h(this);
		return WriteCreateProcessResponse(h, writer, false); // isChild is false because we have already registered the child when for was created
	#endif
	}

	bool ProcessImpl::HandleExitChildProcess(BinaryReader& reader, BinaryWriter& writer)
	{
		// TODO: Should this traverse down? This is used for when child processes are terminated through job being closed. used by node.exe (building chrome)
		u32 nativeProcessId = reader.ReadU32();
		u32 exitCode = reader.ReadU32();
		for (auto& child : m_childProcesses)
		{
			auto& process = *(ProcessImpl*)child.m_process;
			if (process.m_nativeProcessId != nativeProcessId)
				continue;
			//m_session.m_logger.Info(TC("CHILD TERMINATED. Exit code %u"), exitCode);
			process.Cancel();
			process.m_exitCode = exitCode;
			return true;
		}
		return true;
	}

	bool ProcessImpl::HandleInheritFiles(BinaryReader& reader, BinaryWriter& writer)
	{
		SCOPED_FUTEX(m_shared.activeFilesLock, lock);

		while (true)
		{
			u32 closeId = reader.ReadU32();
			if (!closeId)
				break;
			auto findIt = m_shared.activeFiles.find(closeId);
			UBA_ASSERT(findIt != m_shared.activeFiles.end()); // Race condition?
			++findIt->second.refCount;
		}
		return true;
	}

	bool ProcessImpl::HandleVirtualAllocFailed(BinaryReader& reader, BinaryWriter& writer)
	{
		StringBuffer<> allocType;
		reader.ReadString(allocType);
		u32 error = reader.ReadU32();
		m_session.AllocFailed(*this, allocType.data, error);
		return true;
	}

	bool ProcessImpl::HandleLog(BinaryReader& reader, BinaryWriter& writer)
	{
		bool printInSession = reader.ReadBool();
		bool isError = reader.ReadBool();
		TString line = reader.ReadString();
		LogEntryType entryType = isError ? LogEntryType_Error : LogEntryType_Info;

		if (!m_session.LogLine(*this, line.c_str(), entryType))
			return CancelWithError();

		InternalLogLine(printInSession, std::move(line), entryType);
		return true;
	}

	bool ProcessImpl::HandleInputDependencies(BinaryReader& reader, BinaryWriter& writer)
	{
		UBA_ASSERTF(m_startInfo.trackInputs, TC("Got InputDependencies message even though track input dependencies was not set (%u)"), m_id);

		if (u64 reserveSize = reader.Read7BitEncoded())
			m_trackedInputs.reserve(m_trackedInputs.size() + reserveSize);

		u32 toRead = reader.ReadU32();
		u8* pos = m_trackedInputs.data() + m_trackedInputs.size();
		m_trackedInputs.resize(m_trackedInputs.size() + toRead);
		reader.ReadBytes(pos, toRead);
		return true;
	}

	bool ProcessImpl::HandleExit(BinaryReader& reader, BinaryWriter& writer)
	{
		m_gotExitMessage = true;
		m_nativeProcessExitCode = reader.ReadU32();

		StringBuffer<> logName;
		reader.ReadString(logName);

		ProcessStats stats;
		stats.Read(reader, ~0u);

		KernelStats kernelStats;
		kernelStats.Read(reader, ~0u);

		m_processStats.Add(stats);
		m_kernelStats.Add(kernelStats);

		AtomicMax(m_processStats.detoursMemory, stats.detoursMemory.load());
		AtomicMax(m_processStats.peakMemory, stats.peakMemory.load());

		return false;
	}

	bool ProcessImpl::HandleFlushWrittenFiles(BinaryReader& reader, BinaryWriter& writer)
	{
		WriteFilesToDisk();
		bool result = m_session.FlushWrittenFiles(*this);
		writer.WriteBool(result);
		return true;
	}

	bool ProcessImpl::HandleUpdateEnvironment(BinaryReader& reader, BinaryWriter& writer)
	{
		StringBuffer<> reason;
		reader.ReadString(reason);
		bool resetStats = reader.ReadBool();
		bool result = m_session.UpdateEnvironment(*this, reason, resetStats);
		writer.WriteBool(result);
		return true;
	}

	bool ProcessImpl::HandleGetNextProcess(BinaryReader& reader, BinaryWriter& writer)
	{
		u32 prevExitCode = reader.ReadU32();
		u32 timeoutMs = reader.ReadU32();
		NextProcessInfo nextProcess;

		StackBinaryWriter<16 * 1024> statsWriter;
					
		ProcessStats processStats;
		processStats.Read(reader, TraceVersion);

		processStats.startupTime = m_processStats.startupTime;
		processStats.wallTime = GetTime() - m_startTime;
		processStats.cpuTime = 0;
		processStats.hostTotalTime = m_processStats.hostTotalTime;
		
		KernelStats kernelStats;
		kernelStats.Read(reader, TraceVersion);
		
		// Transfer stats from detoured process to local
		m_processStats.Add(processStats);
		m_kernelStats.Add(kernelStats);

		// Transfer child times and clear
		WaitForChildrenExit();

		for (auto& child : m_childProcesses)
		{
			auto& process = *(ProcessImpl*)child.m_process;
			m_sessionStats.Add(process.m_sessionStats);
			m_storageStats.Add(process.m_storageStats);
			m_processStats.Add(process.m_processStats);
			m_kernelStats.Add(process.m_kernelStats);
			
			processStats.Add(process.m_processStats);
			kernelStats.Add(process.m_kernelStats);
			
			process.m_sessionStats = {};
			process.m_storageStats = {};
			process.m_processStats = {};
			process.m_kernelStats = {};
		}
		m_childProcesses.clear();


		processStats.Write(statsWriter);
		if (m_runningRemote)
			m_sessionStats.Write(statsWriter);
		m_storageStats.Write(statsWriter);
		kernelStats.Write(statsWriter);
		BinaryReader statsReader(statsWriter.GetData(), 0, statsWriter.GetPosition());

		WriteFilesToDisk();

		bool newProcess = false;
		bool shouldExit = false;
		m_exitCode = prevExitCode;
		CallSession(MessageType_GetNextProcess, m_session.GetNextProcess(*this, newProcess, shouldExit, nextProcess, prevExitCode, statsReader, timeoutMs));
		writer.WriteBool(newProcess);
		writer.WriteBool(shouldExit);

		m_exitCode = ~0u;
		CleanupWrittenFiles();

		m_startInfo.arguments = nullptr;

		if (!newProcess)
			return true;

		m_processStats = {};
		m_sessionStats = {};
		m_storageStats = {};
		m_kernelStats = {};

		m_startInfo.argumentsStr = nextProcess.arguments;
		m_startInfo.descriptionStr = nextProcess.description;
		m_startInfo.logFileStr = nextProcess.logFile;

		m_startInfo.arguments = m_startInfo.argumentsStr.c_str();
		m_startInfo.description = m_startInfo.descriptionStr.c_str();
		m_startInfo.logFile = m_startInfo.logFileStr.c_str();

		m_childProcesses.clear();
		m_logLines.clear();
		m_trackedInputs.clear();
		m_trackedOutputs.clear();
		m_shared.writtenFiles.clear();

		m_startTime = GetTime();
		++m_session.m_reuseCount;

		// Clean up temp, since the reset overlay will forget about all temp files anyway there is no point keeping them
		// And this will also prevent collisions of files.
		m_session.ClearTempDirectory(m_shared.m_tempPathId);

		// Rewrite overlay.. and let reused process recreate dir table
		{
			OverlayEditScope scope(m_shared.overlay);
			m_session.ResetOverlay(scope.overlay, nextProcess.overlayFiles.data(), nextProcess.overlayFiles.size());
			m_session.RegisterNonRootedDir(scope, ToView(m_session.m_tempPath).RemoveSuffix(1));
			m_session.RegisterNonRootedDir(scope, ToView(m_shared.m_tempPath).RemoveSuffix(1));
			m_shared.createdDirectories.clear();
			m_shared.removedDirectories.clear();
			m_shared.deletedFiles.clear();
			m_shared.movedFiles.clear();

			#if !PLATFORM_WINDOWS
			m_shared.symlinks.clear();
			#endif
		}
		writer.WriteString(nextProcess.arguments);
		writer.WriteString(nextProcess.workingDir);
		writer.WriteString(nextProcess.description);
		writer.WriteString(nextProcess.logFile);
		writer.WriteU64(ToU64(m_session.GetDirectoryTableSize(&m_shared.overlay)));
		return true;
	}

	bool ProcessImpl::HandleCustom(BinaryReader& reader, BinaryWriter& writer)
	{
		m_session.CustomMessage(*this, reader, writer);
		return true;
	}

	bool ProcessImpl::HandleSHGetKnownFolderPath(BinaryReader& reader, BinaryWriter& writer)
	{
		m_session.SHGetKnownFolderPath(*this, reader, writer);
		return true;
	}

	bool ProcessImpl::HandleRpcCommunication(BinaryReader& reader, BinaryWriter& writer)
	{
		//m_session.RpcCommunication(*this, reader, writer);
		return true;
	}

	bool ProcessImpl::HandleHostRun(BinaryReader& reader, BinaryWriter& writer)
	{
		u16 size = reader.ReadU16();
		BinaryReader reader2(reader.GetPositionData(), 0, size);
		m_session.HostRun(reader2, writer);
		return true;
	}

	bool ProcessImpl::HandleResolveCallstack(BinaryReader& reader, BinaryWriter& writer)
	{
		m_session.GetSymbols(m_startInfo.application, IsLinux, m_isArmBinary, reader, writer);
		return true;
	}

	bool ProcessImpl::HandleCheckRemapping(BinaryReader& reader, BinaryWriter& writer)
	{
		m_session.CheckRemapping(*this, reader, writer);
		return true;
	}

	bool ProcessImpl::HandleCheckRemapping2(BinaryReader& reader, BinaryWriter& writer)
	{
		StringKey fileNameKey = reader.ReadStringKey();

		SCOPED_FUTEX(m_shared.writtenFilesLock, lock);
		auto findIt = m_shared.writtenFiles.find(fileNameKey);
		if (findIt != m_shared.writtenFiles.end())
			writer.WriteString(findIt->second.backedName);
		else
			writer.WriteString(StringView());
		return true;
	}

	bool ProcessImpl::HandleRunSpecialProgram(BinaryReader& reader, BinaryWriter& writer)
	{
		m_session.RunSpecialProgram(*this, reader, writer);
		return true;
	}

	bool ProcessImpl::HandleCreateSharedMemory(BinaryReader& reader, BinaryWriter& writer)
	{
		u64 initialSize = reader.ReadU64();
		StringBuffer<> hint;
		reader.ReadString(hint);
		SharedMemoryHandle h = m_session.m_sharedMemory.CreateHandle(m_session.m_logger, hint.data);
		writer.Write7BitEncoded(h.ToU64());
		if (initialSize)
			m_session.m_sharedMemory.ExtendMemory(writer, h, initialSize, TC("HandleCreate"), false); // memset happens in detoured process

		SCOPED_FUTEX(m_shared.writtenFilesLock, lock);
		m_shared.sharedMemoryHandles.insert(h);
		return true;
	}

	bool ProcessImpl::HandleCommitSharedMemory(BinaryReader& reader, BinaryWriter& writer)
	{
		SharedMemoryHandle h = SharedMemoryHandle::FromU64(reader.ReadU64());
		u64 size = reader.ReadU64();
		m_session.m_sharedMemory.ExtendMemory(writer, h, size, TC("HandleCommit"), false); // memset happens in detoured process
		return true;
	}

	bool ProcessImpl::HandleGetSharedMemory(BinaryReader& reader, BinaryWriter& writer)
	{
		SharedMemoryHandle h = SharedMemoryHandle::FromU64(reader.ReadU64());
		u64 offset = reader.ReadU64();
		u64 size = reader.ReadU64();
		if (!size)
		{
			size = m_session.m_sharedMemory.GetCommitted(h);
			writer.Write7BitEncoded(size);
		}
		m_session.m_sharedMemory.WriteViewSlices(writer, h, offset, size);
		return true;
	}

	bool ProcessImpl::WriteFilesToDisk()
	{
		auto& rules = *m_startInfo.rules;

		if (!m_runningRemote && !m_parentProcess)
		{
			SCOPED_READ_LOCK(m_shared.overlay.lock, overlayLock);

			for (auto& removedDir : m_shared.removedDirectories)
			{
				if (ToView(removedDir).StartsWith(m_shared.m_tempPath))
					continue;
				m_session.RegisterDeleteFile(StringKeyZero, removedDir);
			}

			for (auto& createdDir : m_shared.createdDirectories)
			{
				if (ToView(createdDir).StartsWith(m_shared.m_tempPath))
					continue;
				if (rules.IsThrowAway(ToView(createdDir), true))
					continue;
				StringKey createdDirKey = CaseInsensitiveFs ? ToStringKeyLower(createdDir) : ToStringKey(createdDir);
				StringKey dirKey;
				const tchar* lastSlash;
				StringBuffer<> dirName;
				GetDirKey(dirKey, dirName, lastSlash, createdDir);
				m_session.RegisterCreateFileForWrite(createdDirKey, createdDir, 0, 0, DefaultDirAttributes(), true, true);
				m_session.WriteDirectoryEntries(dirKey, dirName);
			}

			for (auto& move : m_shared.movedFiles)
			{
				UBA_ASSERT(move.second.name.empty());
				auto& toFile = move.first;
				if (!ToView(toFile).StartsWith(m_shared.m_tempPath))
					m_session.RegisterCreateFileForWrite(StringKeyZero, toFile, 0, 0, DefaultAttributes(), true, true);
			}
			for (auto& deletedFile : m_shared.deletedFiles)
				if (!ToView(deletedFile).StartsWith(m_shared.m_tempPath))
					m_session.RegisterDeleteFile(StringKeyZero, deletedFile);

			#if !PLATFORM_WINDOWS
			for (auto& symlink : m_shared.symlinks)
				if (!ToView(symlink.first).StartsWith(m_shared.m_tempPath))
					m_session.RegisterCreateFileForWrite(StringKeyZero, symlink.first, 0, 0, DefaultAttributes(), true, true);
			#endif
		}

		Vector<WrittenFile*> files;
		TimerScope ts(m_processStats.writeFiles);
		SCOPED_FUTEX(m_shared.writtenFilesLock, lock);
		for (auto& kv : m_shared.writtenFiles)
		{
			// TODO: This requires some more logic. In some cases we have long living batch files as parent and then maybe we want to write out files to disk
			// But for now we only let the root process write out all output files
			if (m_parentProcess)
				continue;
			kv.second.ownerId = 0;
			if (kv.second.memoryHandle.IsValid())
			{
				files.push_back(&kv.second);
				continue;
			}
			if (m_runningRemote)
				continue;
			StringView file(kv.second.name);
			if (file.StartsWith(m_shared.m_tempPath) && !rules.KeepTempOutputFile(file))
				continue;
			if (rules.IsInvisible(file))
				continue;
			if (rules.IsThrowAway(file, m_runningRemote))
				continue;
			m_session.RegisterCreateFileForWrite(StringKeyZero, file, kv.second.memoryWritten, kv.second.lastWriteTime, kv.second.attributes, true, true); // Files that were written directly to disk like Default.rc2.res.obj
		}

		// We want file count to match number of files.. and it is actually fine that count is 0
		m_processStats.writeFiles.count += u32(files.size() - 1);

		if (!m_session.WriteFilesToDisk(*this, files.data(), u32(files.size())))
			return false;

		if (m_startInfo.trackInputs)
		{
			u64 totalBytes = 0;
			for (auto& kv : m_shared.writtenFiles)
				totalBytes += GetStringWriteSize(kv.second.name.c_str(), kv.second.name.size());
			m_trackedOutputs.resize(totalBytes);
			BinaryWriter writer(m_trackedOutputs.data(), 0, totalBytes);
			for (auto& kv : m_shared.writtenFiles)
				writer.WriteString(kv.second.name);
		}
		return true;
	}

	void ProcessImpl::CleanupWrittenFiles()
	{
		for (auto& kv : m_shared.writtenFiles)
			if (auto entry = kv.second.entryToReduceRefCount)
			{
				kv.second.entryToReduceRefCount = nullptr;
				SCOPED_FUTEX(entry->lock, entryLock);
				--entry->refCount;
			}
	}

	const tchar* ProcessImpl::InternalGetChildLogFile(StringBufferBase& temp)
	{
		if (!*m_startInfo.logFile)
			return TC("");
		temp.Append(m_startInfo.logFile);
		if (TStrcmp(temp.data + temp.count - 4, TC(".log")) == 0)
			temp.Resize(temp.count - 4);
		temp.Appendf(TC("_CHILD%03u.log"), u32(m_childProcesses.size()));
		return temp.data;
	}

#if PLATFORM_WINDOWS
	/** Disables Power Throttling in the provided process, to ensure P-Cores are preferred over E Cores on hybrid architecture intel platforms */
	void DisableProcessPowerThrottling(HANDLE ProcessHandle)
	{
		PROCESS_POWER_THROTTLING_STATE PowerThrottling;
		RtlZeroMemory(&PowerThrottling, sizeof(PowerThrottling));

		// Enable PowerThrottling policies for the process
		// and disable power throttling by setting the state mask to 0
		PowerThrottling.Version = PROCESS_POWER_THROTTLING_CURRENT_VERSION;
		PowerThrottling.ControlMask = PROCESS_POWER_THROTTLING_EXECUTION_SPEED;
		PowerThrottling.StateMask = 0;

		SetProcessInformation(ProcessHandle, ProcessPowerThrottling, &PowerThrottling, sizeof(PowerThrottling));
	}
#endif


	u32 ProcessImpl::InternalCreateProcess(Vector<tchar>& environment, FileMappingHandle communicationHandle, u64 communicationOffset)
	{
		SCOPED_FUTEX(m_initLock, initLock);
		Logger& logger = m_session.m_logger;

#if PLATFORM_WINDOWS

		HANDLE readPipe = INVALID_HANDLE_VALUE;
		auto readPipeGuard = MakeGuard([&]() { CloseHandle(readPipe); });

		bool allowCustomAllocator = true;

		if (!m_parentProcess)
		{
			// We want to make sure no crash dialogs pop up when failing to spawn processes since that will stall everything forever until someone clicks the dialog
			static bool once = []()
				{
					u32 old = SetErrorMode(SEM_FAILCRITICALERRORS | SEM_NOOPENFILEERRORBOX);
					SetErrorMode(old | SEM_FAILCRITICALERRORS | SEM_NOOPENFILEERRORBOX);
					return true;
				}();(void)once;


			if (IsRunningArm())
			{
				SCOPED_FUTEX(m_session.m_isX64ApplicationLock, lock);
				auto insres = m_session.m_isX64Application.try_emplace(m_realApplication);
				if (insres.second)
				{
					ImageInfo imageInfo;
					if (!GetImageInfo(imageInfo, m_session.m_logger, m_realApplication.c_str(), false))
						return UBA_EXIT_CODE(19);
					insres.first->second = imageInfo.isX64;
				}
				m_isArmBinary = !insres.first->second;

				if (!m_isArmBinary)
					allowCustomAllocator = false;
			}

			const char* detoursLib = m_session.m_detoursLibrary[IsLinux][m_isArmBinary].c_str();
			if (!*detoursLib)
				detoursLib = UBA_DETOURS_LIBRARY_ANSI;

			StringBuffer<> application(m_startInfo.applicationStr);
			m_session.VirtualizePath(application, m_startInfo.rootsHandle);

			TString commandLine;
			if (application.EndsWith(TCV(".bat")) && !IsRunningWine()) // If there are quotes around arguments used by batch file we need to quote the entire thing on windows
				commandLine.append(TC("\"\"")).append(application.data).append(TC("\" ")).append(m_startInfo.arguments).append(TC("\""));
			else
				commandLine.append(TC("\"")).append(application.data).append(TC("\" ")).append(m_startInfo.arguments);

			if (m_extractExportsVersion)
			{
				const tchar* pos = nullptr;
				if (Contains(commandLine.c_str(), g_extractExportsStr, true, &pos))
				{
					u32 len = sizeof_array(g_extractExportsStr) - 1; // "/extractexports" + any trailing version digits
					while (pos[len] >= '0' && pos[len] <= '9')
						++len;
					commandLine.erase(pos - commandLine.c_str(), len);
				}
			}

			LPCSTR dlls[2];
			u32 dllCount = 0;
			dlls[dllCount++] = detoursLib;

			char ubaWineModule[256];
			if (m_session.m_sharedMemory.GetFileMappingBackend().linuxBackend)
			{
				GetUbaWineModulePath().Parse(ubaWineModule, 256);
				dlls[dllCount++] = ubaWineModule;
			}

			STARTUPINFOEX siex;
			STARTUPINFO& si = siex.StartupInfo;
			ZeroMemory(&siex, sizeof(STARTUPINFOEX));
			si.cb = sizeof(STARTUPINFOEX);

			PROCESS_INFORMATION processInfo;
			ZeroMemory(&processInfo, sizeof(processInfo));

			DWORD creationFlags = CREATE_DEFAULT_ERROR_MODE | CREATE_SUSPENDED | CREATE_UNICODE_ENVIRONMENT | CREATE_NEW_PROCESS_GROUP | m_startInfo.priorityClass;
			BOOL inheritHandles = false;


			SIZE_T attributesBufferSize = 0;
			::InitializeProcThreadAttributeList(nullptr, 1, 0, &attributesBufferSize);

			u8 attributesBuffer[128];
			if (sizeof(attributesBuffer) < attributesBufferSize)
			{
				logger.Error(TC("Attributes buffer is too small, needs to be at least %llu"), u64(attributesBufferSize));
				return UBA_EXIT_CODE(2);
			}

			PPROC_THREAD_ATTRIBUTE_LIST attributes = reinterpret_cast<PPROC_THREAD_ATTRIBUTE_LIST>(attributesBuffer);
			if (!::InitializeProcThreadAttributeList(attributes, 1, 0, &attributesBufferSize))
			{
				logger.Error(TC("InitializeProcThreadAttributeList failed (%s)"), LastErrorToText().data);
				return UBA_EXIT_CODE(3);
			}

			auto destroyAttr = MakeGuard([&]() { ::DeleteProcThreadAttributeList(attributes); });


			siex.lpAttributeList = attributes;
			creationFlags |= EXTENDED_STARTUPINFO_PRESENT;


			SCOPED_FUTEX_READ(m_session.m_processJobObjectLock, jobObjectLock);
			if (!m_session.m_processJobObject)
			{
				m_cancelled = true;
				return ProcessCancelExitCode;
			}

			bool isDetachedProcess = m_startInfo.rules->AllowDetach() && m_detourEnabled;

			HANDLE hJob = CreateJobObject(nullptr, nullptr);
			JOBOBJECT_EXTENDED_LIMIT_INFORMATION info = { };
			info.BasicLimitInformation.LimitFlags = JOB_OBJECT_LIMIT_BREAKAWAY_OK;
			SetInformationJobObject(hJob, JobObjectExtendedLimitInformation, &info, sizeof(info));
			m_accountingJobObject = hJob;

			HANDLE jobs[] = { m_session.m_processJobObject, m_accountingJobObject };

			if (!::UpdateProcThreadAttribute(attributes, 0, PROC_THREAD_ATTRIBUTE_JOB_LIST, jobs, sizeof(jobs), nullptr, nullptr))
			{
				logger.Error(TC("UpdateProcThreadAttribute failed when setting job list (%s)"), LastErrorToText().data);
				return UBA_EXIT_CODE(4);
			}
			
			if (isDetachedProcess)
				creationFlags |= DETACHED_PROCESS;
			else
				creationFlags |= CREATE_NO_WINDOW;

			// Create environment specifically for this process

			u64 additionalSize = m_shared.m_tempPath.size()*2 + 4 + 5 + 3; // Two paths plus TEMP=/TMP= and 3 terminators
			u64 oldEnd = environment.size(); // Skip last double termination
			Vector<tchar> newEnvironment;
			newEnvironment.reserve(oldEnd + additionalSize);
			bool firstTempAdd = true;
			const tchar* nextCopyBegin = environment.data();
			tchar* writePos = newEnvironment.data();
			for (StringView sv = ToView(environment.data());sv.count;)
			{
				StringView toAdd = firstTempAdd ? TCV("TEMP=") : TCV("TMP=");
				if (_wcsnicmp(sv.data, toAdd.data, toAdd.count) < 0)
				{
					sv = ToView(sv.data+sv.count+1);
					continue;
				}
				UBA_ASSERT(!sv.StartsWith(toAdd));

				u64 toCopy = sv.data - nextCopyBegin;
				memcpy(writePos, nextCopyBegin, toCopy*sizeof(tchar));
				writePos += toCopy;
				nextCopyBegin = sv.data;
				StringBuffer<> temp;
				temp.Append(toAdd).Append(m_shared.m_tempPath).Append(tchar(0));
				memcpy(writePos, temp.data, temp.count*sizeof(tchar));
				writePos += temp.count;
				if (!firstTempAdd)
				{
					toCopy = environment.data() + environment.size() - nextCopyBegin;
					memcpy(writePos, nextCopyBegin, toCopy*sizeof(tchar));
					writePos += toCopy;
					UBA_ASSERT(u32(writePos - newEnvironment.data()) <= newEnvironment.capacity());
					break;
				}
				firstTempAdd = false;
			}

			u32 retryCount = 0;
			while (true)
			{
				if (IsCancelled())
					return ProcessCancelExitCode;

				LPCWSTR workingDir = *m_realWorkingDir ? m_realWorkingDir : NULL;

				if (m_detourEnabled)
				{
					//u64 createStartTime = GetTime();
					if (DetourCreateProcessWithDlls(m_realApplication.c_str(), (tchar*)commandLine.c_str(), NULL, NULL, inheritHandles, creationFlags, newEnvironment.data(), workingDir, &si, &processInfo, dllCount, dlls, NULL))
					{
						//m_startTime += GetTime() - createStartTime;

						DisableProcessPowerThrottling(processInfo.hProcess);
						break;
					}
				}
				else
				{
					SECURITY_ATTRIBUTES saAttr;
					saAttr.nLength = sizeof(SECURITY_ATTRIBUTES);
					saAttr.bInheritHandle = TRUE;
					saAttr.lpSecurityDescriptor = NULL;

					HANDLE writePipe;
					if (!CreatePipe(&readPipe, &writePipe, &saAttr, 0))
					{
						logger.Error(TC("CreatePipe failed"));
						return UBA_EXIT_CODE(18);
					}

					auto writePipeGuard = MakeGuard([&]() { CloseHandle(writePipe); });

					if (!SetHandleInformation(readPipe, HANDLE_FLAG_INHERIT, 0))
					{
						logger.Error(TC("SetHandleInformation failed"));
						return UBA_EXIT_CODE(18);
					}

					si.hStdInput = GetStdHandle(STD_INPUT_HANDLE);
					si.hStdError = writePipe;
					si.hStdOutput = writePipe;
					si.dwFlags |= STARTF_USESTDHANDLES;

					if (CreateProcessW(m_realApplication.c_str(), (tchar*)commandLine.c_str(), NULL, NULL, TRUE, creationFlags, newEnvironment.data(), workingDir, &si, &processInfo))
					{
						DisableProcessPowerThrottling(processInfo.hProcess);

						#if UBA_DEBUG && UBA_DEBUG_LOG_ENABLED
						if (*m_startInfo.logFile)
						{
							Logger* processLogger = StartDebugLogger(logger, m_startInfo.logFile);
							processLogger->Info(TC("NOT DETOURED"));
							processLogger->Log(LogEntryType_Info, commandLine.c_str(), u32(commandLine.size()));
							StopDebugLogger(processLogger);
						}
						#endif
						break;
					}
				}

				DWORD error = GetLastError();

				if (error == ERROR_ACCESS_DENIED || error == ERROR_INTERNAL_ERROR)
				{
					// We have no idea why this is happening.. but it seems to recover when retrying.
					// Could it be related to two process spawning at the exact same time or something?
					// It happens extremely rarely and can happen on both host and remotes
					bool retry = retryCount++ < 5;
					const tchar* errorText = error == ERROR_ACCESS_DENIED ? TC("access denied") : TC("internal error");
					logger.Logf(retry ? LogEntryType_Info : LogEntryType_Error, TC("DetourCreateProcessWithDllEx failed with %s, retrying %s (Working dir: %s)"), errorText, commandLine.c_str(), workingDir);
					if (!retry)
						return UBA_EXIT_CODE(5);
					Sleep(100 + (rand() % 200)); // We have no idea
					ZeroMemory(&processInfo, sizeof(processInfo));
					continue;
				}
				else if (error == ERROR_WRITE_PROTECT) // AWS shutting down
				{
					m_cancelEvent.Set();
					return ProcessCancelExitCode;
				}
				else if (error == ERROR_NOT_ENOUGH_MEMORY) // Have no explanation why this can happen but have seen it on own machine when helping out compressing staging data
				{
					bool retry = retryCount++ < 5;
					if (retry)
					{
						Sleep(100 + (rand() % 200));
						ZeroMemory(&processInfo, sizeof(processInfo));
						continue;
					}
				}
				else if (error == ERROR_WAIT_1 && IsRunningWine()) // Wine quirk: child died exit=1 during init; NtCreateUserProcess returns reply->exit_code as NTSTATUS -> STATUS_WAIT_1 -> 731
				{
					bool retry = retryCount++ < 5;
					logger.Logf(retry ? LogEntryType_Info : LogEntryType_Error, TC("DetourCreateProcessWithDllEx failed with wine error 731 (child died exit=1 during init), retrying %s (Working dir: %s)"), commandLine.c_str(), workingDir);
					if (retry)
					{
						Sleep(100 + (rand() % 200));
						ZeroMemory(&processInfo, sizeof(processInfo));
						continue;
					}
				}

				LastErrorToText lett(error);
				const tchar* errorText = lett.data;
				if (error == ERROR_INVALID_HANDLE)
					errorText = TC("Can't detour a 32-bit target process from a 64-bit parent process.");

				if (!IsCancelled())
				{
					if (error == ERROR_DIRECTORY)
						logger.Error(TC("HOW CAN THIS HAPPEN? '%s'"), workingDir);

					m_startInfo.logFile = TC("");

					logger.Error(TC("DetourCreateProcessWithDllEx failed: %s (Working dir: %s). Exit code: %u - %s"), commandLine.c_str(), workingDir, error, errorText);
				}
				return UBA_EXIT_CODE(6);
			}

			destroyAttr.Execute();

			m_nativeProcessHandle = (ProcHandle)(u64)processInfo.hProcess;
			m_nativeProcessId = processInfo.dwProcessId;
			m_nativeThreadHandle = processInfo.hThread;

			if (m_nativeProcessHandle == InvalidProcHandle) 
				logger.Error(TC("Seems like DetourCreateProcessWithDllEx succeeded starting %s but returned a process handle that is invalid (0)?"), commandLine.c_str());

			#if UBA_DEBUG
			USHORT processMachine = 0;
			USHORT nativeMachine = 0;
			if (IsWow64Process2(processInfo.hProcess, &processMachine, &nativeMachine))
			{
				bool is64Bit = processMachine == IMAGE_FILE_MACHINE_UNKNOWN;
				UBA_ASSERTF(is64Bit, TC("Can't detour 32-bit application %s"), m_realApplication.c_str());
			}
			#endif

			FILETIME dummy;
			if (processInfo.hProcess && !GetProcessTimes(processInfo.hProcess, (FILETIME*)&m_nativeProcessCreationTime, &dummy, &dummy, &dummy))
				logger.Error(TC("GetProcessTimes failed (%s)"), LastErrorToText().data);

			#if UBA_DEBUG_TRACK_PROCESS
			m_session.m_debugLogger->Info(TC("CreateRealProcess  (%u) %s %s"), m_id, m_realApplication.c_str(), m_startInfo.arguments);
			#endif
		}
		else
		{
			#if UBA_DEBUG_TRACK_PROCESS
			m_session.m_debugLogger->Info(TC("WaitingForParentReady (%u)"), m_id);
			#endif

			// We kind of want to catch childrens DetourCreateProcess here which can be insanely expensive
			//u64 startWaitTime = GetTime();
			WaitForParent();
			//m_startTime += GetTime() - startWaitTime;

			if (m_nativeProcessHandle == InvalidProcHandle) // Failed to create the child process
				return UBA_EXIT_CODE(7);

			#if UBA_DEBUG_TRACK_PROCESS
			m_session.m_debugLogger->Info(TC("WaitingForParentReadyDone (%u)"), m_id);
			#endif

			m_extractExportsVersion = m_parentProcess->m_extractExportsVersion;
		}

		auto closeThreadHandle = MakeGuard([&]() { CloseHandle(m_nativeThreadHandle); m_nativeThreadHandle = 0; });

		if (m_detourEnabled)
		{
			HANDLE hostProcess;
			HANDLE currentProcess = GetCurrentProcess();
			if (!DuplicateHandle(currentProcess, currentProcess, (HANDLE)m_nativeProcessHandle, &hostProcess, 0, FALSE, DUPLICATE_SAME_ACCESS))
			{
				if (!IsCancelled())
					logger.Error(TC("Failed to duplicate host process handle for process (%s)"), LastErrorToText().data);
				return UBA_EXIT_CODE(8);
			}

			DetoursPayload payload;
			payload.hostProcess = hostProcess;
			payload.cancelEvent = m_cancelEvent.GetHandle();
			payload.writeEvent = m_writeEvent.GetHandle();
			payload.readEvent = m_readEvent.GetHandle();
			payload.communicationHandle = communicationHandle.ToU64();
			payload.communicationOffset = communicationOffset;
			payload.rulesIndex = m_startInfo.rules->index;
			payload.version = ProcessMessageVersion;
			payload.runningRemote = m_runningRemote;
			payload.allowKeepFilesInMemory = m_session.m_allowKeepFilesInMemory;
			payload.allowOutputFiles = m_session.m_allowOutputFiles;
			payload.allowDiscardVirtualMemory = m_session.m_allowDiscardVirtualMemory;
			payload.suppressLogging = m_session.m_suppressLogging;
			payload.isChild = m_parentProcess != nullptr;
			payload.useCustomAllocator = allowCustomAllocator && m_startInfo.useCustomAllocator && m_startInfo.rules->AllowMiMalloc();
			payload.allowLazyCopyOnWrite = m_session.m_allowLazyCopyOnWrite;
			payload.reportAllExceptions = m_startInfo.reportAllExceptions || m_startInfo.rules->ReportAllExceptions();
			payload.isRunningWine = IsRunningWine();
			payload.readIntermediateFilesCompressed = m_session.m_readIntermediateFilesCompressed;
			payload.uiLanguage = m_startInfo.uiLanguage;
			if (*m_startInfo.logFile)
			{
				#if !UBA_DEBUG_LOG_ENABLED
				static bool runOnce = [&]() { logger.Warning(TC("Build has log files disabled so no logs will be produced")); return false; }();
				#endif
				payload.logFile.Append(m_startInfo.logFile);
			}

			if (!DetourCopyPayloadToProcess((HANDLE)m_nativeProcessHandle, DetoursPayloadGuid, &payload, sizeof(payload)))
			{
				if (!m_cancelled)
					logger.Error(TC("Process with id %u failed to copy payload to %sprocess %llu - %s (%s)"), m_id, m_parentProcess?TC("child "):TC(""), m_nativeProcessHandle, m_realApplication.c_str(), LastErrorToText().data);
				return UBA_EXIT_CODE(9);
			}
		}

		// Best-effort spread across processor groups; silently no-op on single-group / pre-Win10-1607.
		DistributeProcessSoft((HANDLE)m_nativeProcessHandle, m_nativeThreadHandle);

		m_processStats.startupTime = GetTime() - m_startTime;

		if (!m_startInfo.startSuspended && ResumeThread(m_nativeThreadHandle) == -1)
		{
			LogEntryType logType = m_cancelled ? LogEntryType_Info : LogEntryType_Error;
			logger.Logf(logType, TC("Failed to resume thread %llu for processid %u - %s. (%s)"), m_nativeThreadHandle, m_id, m_realApplication.c_str(), LastErrorToText().data);
			return UBA_EXIT_CODE(11);
		}

		closeThreadHandle.Execute();

		if (!m_detourEnabled)
		{
			PipeReader pipeReader(*this, LogEntryType_Info);
			TString currentString;
			while (true)
			{
				DWORD exitCode = STILL_ACTIVE;
				GetExitCodeProcess((HANDLE)m_nativeProcessHandle, &exitCode);
				
				while (true)
				{
					DWORD totalBytesAvailable = 0;
					if (!PeekNamedPipe(readPipe, NULL, 0, NULL, &totalBytesAvailable, NULL))
						break;
					if (totalBytesAvailable == 0)
						break;
					char buf[4096];
					DWORD readCount = 0;
					if (!::ReadFile(readPipe, buf, sizeof(buf) - 1, &readCount, NULL))
						break;
					buf[readCount] = 0;
					pipeReader.ReadData(buf, readCount);
				}

				if (exitCode != STILL_ACTIVE)
					break;
				WaitForSingleObject((HANDLE)m_nativeProcessHandle, 1000);
			}
		}

#else // #if PLATFORM_WINDOWS

		if (!m_parentProcess)
		{
			StringView realApplication = m_realApplication;
			StringBuffer<> tempApplication;
			if (m_realApplication.find(' ') != TString::npos)
			{
				tempApplication.Append('\"').Append(m_realApplication).Append('\"');
				realApplication = tempApplication;
			}

			Vector<TString> arguments;
			if (!ParseArguments(m_startInfo.arguments, [&](const tchar* arg, u32 argLen)
				{
					if (StartsWith(arg, g_extractExportsStr, true)) // strip the /extractexports<N> token
						return;
					arguments.push_back(TString(arg, argLen));
				}))
			{
				logger.Error("Failed to parse arguments: %s", m_startInfo.arguments);
				return UBA_EXIT_CODE(16);
			}
			Vector<const char*> arguments2;
			arguments2.reserve(arguments.size() + 2);

			StringBuffer<512> application(m_startInfo.application);
			m_session.VirtualizePath(application, m_startInfo.rootsHandle);

			// Shebang (local only): PrepareProcess rewrote m_realApplication to the resolved interpreter; replicate the kernel-shebang argv
			// shape so argv[0]=interpreter, argv[1]=script, argv[2..]=user args. Remote agents already prepend the script in SessionClient.
			#if !PLATFORM_WINDOWS
			if (!m_runningRemote)
			{
				StringView realApplicationView = ToView(m_realApplication.c_str());
				StringView applicationView = ToView(application.data);
				bool isShebang = !realApplicationView.GetFileName().Equals(applicationView.GetFileName()) && !realApplicationView.Contains("/1111/");
				if (isShebang)
					arguments2.push_back(m_realApplication.c_str());
			}
			#endif

			arguments2.push_back(application.data);

			for (auto& s : arguments)
				arguments2.push_back(s.data());
			arguments2.push_back(nullptr);
			auto argsArray = arguments2.data();

			posix_spawnattr_t attr;
			int res = posix_spawnattr_init(&attr);
			UBA_ASSERTF(res == 0, TC("posix_spawnattr_init (%s)"), strerror(errno));
			auto attrGuard = MakeGuard([&]() { posix_spawnattr_destroy(&attr); });

			// Reset signal mask because UBA threads block SIGINT, and posix_spawn inherits the calling
			// thread's signal mask - meaning spawned children would never receive SIGINT from ctrl-c.
			sigset_t emptymask;
			sigemptyset(&emptymask);
			res = posix_spawnattr_setsigmask(&attr, &emptymask);
			UBA_ASSERTF(res == 0, TC("posix_spawnattr_setsigmask (%s)"), strerror(errno));
			short spawnFlags = POSIX_SPAWN_SETSIGMASK;

			#if PLATFORM_MAC
			// POSIX_SPAWN_CLOEXEC_DEFAULT closes all fds in the child that aren't explicitly
			// set up via fileActions. Prevents a Mac race where another thread can fork between
			// pipe() and fcntl(FD_CLOEXEC), causing the child to inherit stray pipe fds and the
			// parent to hang forever in poll() waiting for an EOF that never comes.
			spawnFlags |= POSIX_SPAWN_CLOEXEC_DEFAULT;
			#endif

			spawnFlags |= POSIX_SPAWN_SETSID; // Detach from host TTY; otherwise /dev/tty writers (nsjail+sed pipelines) deadlock when the root process pty drains slowly.

			res = posix_spawnattr_setflags(&attr, spawnFlags);
			UBA_ASSERTF(res == 0, TC("posix_spawnattr_setflags (%s)"), strerror(errno));

			posix_spawn_file_actions_t fileActions;
			res = posix_spawn_file_actions_init(&fileActions);
			UBA_ASSERTF(res == 0, TC("posix_spawn_file_actions_init (%s)"), strerror(errno));
			auto actionsGuard = MakeGuard([&]() { posix_spawn_file_actions_destroy(&fileActions); });
			
			if (*m_realWorkingDir)
			{
				// addchdir_np needs glibc >= 2.29; on older systems the child inherits parent CWD.
				#if PLATFORM_MAC || (defined(__GLIBC__) && (__GLIBC__ > 2 || (__GLIBC__ == 2 && __GLIBC_MINOR__ >= 29)))
				posix_spawn_file_actions_addchdir_np(&fileActions, m_realWorkingDir);
				#endif
			}

			#if !PLATFORM_MAC && (defined(__GLIBC__) && (__GLIBC__ > 2 || (__GLIBC__ == 2 && __GLIBC_MINOR__ >= 34)))
			posix_spawn_file_actions_addclosefrom_np(&fileActions, 3);
			#endif

			TString comIdVar;
			TString workingDir;
			TString rulesStr;
			TString logFile;
			TString ldLibraryPath;
			TString detoursVar;
			TString processVar;
			TString tempVar;
			TString xdgRuntimeDirVar;

			#if UBA_USE_NATIVE_MAC_SEMAPHORES
			TString sessionId;
			TString pidVar;
			#endif

			Vector<const char*> envvars;

			const tchar* it = (const tchar*)environment.data();
			while (*it)
			{
				const char* s = it;
				envvars.push_back(s);
				it += TStrlen(s) + 1;
			}

			StringBuffer<1024> temp; // Reused over and over again, make sure to clear after use

			tempVar = temp.Append("TMPDIR=").Append(m_shared.m_tempPath).ToStringAndClear();
			envvars.push_back(tempVar.data());

			#if PLATFORM_LINUX
			// Point XDG_RUNTIME_DIR at our per-edge temp; AL2025 helpers have no /run/user/<faked-uid> and nsjail needs it for its chroot scratch root.
			// Linux-only: Apple clang on macOS gets confused by XDG_RUNTIME_DIR and falls into a code path that posix_spawn's cc1 with empty path.
			xdgRuntimeDirVar = temp.Append("XDG_RUNTIME_DIR=").Append(m_shared.m_tempPath).ToStringAndClear();
			envvars.push_back(xdgRuntimeDirVar.data());
			#endif

			int outPipe[2] = { -1, -1 };
			int errPipe[2] = { -1, -1 };
			auto pipeGuard0 = MakeGuard([&]() { if (outPipe[0] != -1) close(outPipe[0]); if (errPipe[0] != -1) close(errPipe[0]); });
			auto pipeGuard1 = MakeGuard([&]() { if (outPipe[1] != -1) close(outPipe[1]); if (errPipe[1] != -1) close(errPipe[1]); });

			if (m_detourEnabled)
			{
				auto& t = m_session.m_detoursLibrary[IsLinux][IsArmBinary];
				StringView detoursLib(t.data(), u32(t.size()));

				if (detoursLib.count)
				{
#if PLATFORM_LINUX
					StringView detoursLibPath = detoursLib.GetPath();
					if (detoursLibPath.count)
					{
						temp.Append("LD_LIBRARY_PATH=").Append(detoursLib.GetPath());
						detoursLib = detoursLib.Skip(detoursLibPath.count+1);
					}
#endif
				}
				else
					detoursLib = TCV("./" UBA_DETOURS_LIBRARY);

				if (m_runningRemote)
				{
					constexpr auto ldLibraryPathName = IsLinux ? TCV("LD_LIBRARY_PATH") : TCV("DYLD_LIBRARY_PATH");
					if (temp.count)
						temp.Append(':');
					else
						temp.Append(ldLibraryPathName).Append('=');
					temp.Append(realApplication.GetPath());
				}

				ldLibraryPath = temp.ToStringAndClear();
#if PLATFORM_LINUX
				detoursVar = temp.Append("LD_PRELOAD=").Append(detoursLib).ToStringAndClear();
#else
				detoursVar = temp.Append("DYLD_INSERT_LIBRARIES=").Append(detoursLib).ToStringAndClear();
#endif

				processVar = temp.Append("UBA_SESSION_PROCESS=").AppendValue(getpid()).ToStringAndClear();


				comIdVar = temp.Append("UBA_COMID=").AppendValue(communicationHandle.internal).Append('+').AppendValue(communicationOffset).ToStringAndClear();

				#if UBA_USE_NATIVE_MAC_SEMAPHORES
				sessionId = temp.Append("UBA_SESSION_ID=").AppendValue(m_session.GetSessionId()).ToStringAndClear;
				pidVar = temp.Append("UBA_PID=").AppendValue(m_id).ToStringAndClear();
				#endif

				workingDir = temp.Append("UBA_CWD=").Append(m_realWorkingDir).ToStringAndClear();
				rulesStr = temp.Append("UBA_RULES=").AppendValue(m_startInfo.rules->index).ToStringAndClear();

				if (*m_startInfo.logFile)
				{
#if !UBA_DEBUG_LOG_ENABLED
					static bool runOnce = [&]() { logger.Warning(TC("Build has log files disabled so no logs will be produced")); return false; }();
#endif
					logFile = temp.Append("UBA_LOGFILE=").Append(m_startInfo.logFile).ToStringAndClear();
				}

				//envvars.push_back("LD_DEBUG=bindings");

				if (!ldLibraryPath.empty())
					envvars.push_back(ldLibraryPath.data());
				envvars.push_back(detoursVar.data());
				envvars.push_back(processVar.data());
				envvars.push_back(comIdVar.data());
				envvars.push_back(workingDir.data());
				envvars.push_back(rulesStr.data());

				#if UBA_USE_NATIVE_MAC_SEMAPHORES
				envvars.push_back(sessionId.data);
				envvars.push_back(pidVar.data);
				#endif

				if (m_runningRemote)
					envvars.push_back("UBA_REMOTE=1");
				if (!logFile.empty())
					envvars.push_back(logFile.data());
			}

			envvars.push_back(nullptr);

			if (true)
			{
				#if PLATFORM_MAC
				if (pipe(outPipe) || pipe(errPipe))
				#else
				if (pipe2(outPipe, O_CLOEXEC) || pipe2(errPipe, O_CLOEXEC))
				#endif
				{
					logger.Error("pipe failed (%s)", strerror(errno));
					return UBA_EXIT_CODE(18);
				}
				
				res = posix_spawn_file_actions_addclose(&fileActions, outPipe[0]);
				UBA_ASSERTF(!res, "posix_spawn_file_actions_addclose outPipe[0] failed: %i", res);

				res = posix_spawn_file_actions_addclose(&fileActions, errPipe[0]);
				UBA_ASSERTF(!res, "posix_spawn_file_actions_addclose errPipe[0] failed: %i", res);

				res = posix_spawn_file_actions_adddup2(&fileActions, outPipe[1], 1);
				UBA_ASSERTF(!res, "posix_spawn_file_actions_adddup2 outPipe[1] failed: %i", res);

				res = posix_spawn_file_actions_adddup2(&fileActions, errPipe[1], 2);
				UBA_ASSERTF(!res, "posix_spawn_file_actions_adddup2 errPipe[1] failed: %i", res);

				res = posix_spawn_file_actions_addclose(&fileActions, outPipe[1]);
				UBA_ASSERTF(!res, "posix_spawn_file_actions_addclose outPipe[1] failed: %i", res);

				res = posix_spawn_file_actions_addclose(&fileActions, errPipe[1]);
				UBA_ASSERTF(!res, "posix_spawn_file_actions_addclose errPipe[1] failed: %i", res);
			}

			u32 retryCount = 0;
			bool shebangRetry = false;
			pid_t processID;
			while (true)
			{
				res = posix_spawnp(&processID, m_realApplication.c_str(), &fileActions, &attr, (char**)argsArray, (char**)envvars.data());
				if (res == 0)
					break;

				// posix_spawn doesn't do execvp's shebang fallback; retry as /bin/sh for shebang-less scripts (Soong rule_builder).
				if (res == ENOEXEC && !shebangRetry)
				{
					Vector<const char*> shArgs;
					shArgs.reserve(arguments2.size() + 1);
					shArgs.push_back("/bin/sh");
					for (auto a : arguments2) shArgs.push_back(a);
					res = posix_spawnp(&processID, "/bin/sh", &fileActions, &attr, (char**)shArgs.data(), (char**)envvars.data());
					shebangRetry = true;
					if (res == 0)
						break;
				}

				if (errno == ETXTBSY && retryCount < 5)
				{
					logger.Warning(TC("posix_spawn failed with ETXTBSY, will retry %s %s (Working dir: %s)"), m_realApplication.c_str(), m_startInfo.arguments, m_realWorkingDir);
					Sleep(2000);
					++retryCount;
					continue;
				}

				logger.Error(TC("posix_spawn failed: %s %s (Working dir: %s) -> %i (%s)"), m_realApplication.c_str(), m_startInfo.arguments, m_realWorkingDir, res, strerror(errno));
				return UBA_EXIT_CODE(12);
			}
			
			errno = 0;

			if (m_startInfo.priorityClass != ProcessPriority_Normal)
			{
				UBA_ASSERT(m_startInfo.priorityClass == ProcessPriority_BelowNormal);
				int newPrio = 10;
				if (setpriority(PRIO_PROCESS, processID, newPrio) == -1)
				{
					UBA_ASSERTF(errno == ESRCH || errno == EPERM, TC("setpriority failed: %s. pid: %i prio: %i (%s)"), m_realApplication.c_str(), processID, newPrio, strerror(errno));
				}
			}

			m_processStats.startupTime = GetTime() - m_startTime;

			m_nativeProcessHandle = (ProcHandle)1;
			m_nativeProcessId = u32(processID);
		#if PLATFORM_LINUX
			// Add wrapper to session cgroup so the host-exit watchdog's cgroup.kill reaches every descendant atomically.
			if (const char* cgroup = GetOrphanReaperCgroup(); cgroup && *cgroup)
			{
				char path[700];
				int n = snprintf(path, sizeof(path), "%s/cgroup.procs", cgroup);
				if (n > 0 && n < (int)sizeof(path))
				{
					int fd = ::open(path, O_WRONLY | O_CLOEXEC);
					if (fd >= 0)
					{
						char buf[16];
						int m = snprintf(buf, sizeof(buf), "%d\n", (int)processID);
						if (m > 0) (void)::write(fd, buf, size_t(m));
						::close(fd);
					}
				}
			}
		#endif

			pipeGuard1.Execute();

			m_stdOutPipe = outPipe[0];
			m_stdErrPipe = errPipe[0];
			pipeGuard0.Cancel();

			#if UBA_DEBUG_TRACK_PROCESS
			m_session.m_debugLogger->Info(TC("CreateRealProcess  (%u) %s %.100s"), m_id, m_realApplication.c_str(), m_startInfo.arguments);
			#endif
		}
		else
		{
			#if UBA_DEBUG_TRACK_PROCESS
			m_session.m_debugLogger->Info(TC("WaitingForParent (%u) %.100s"), m_id, m_realApplication.c_str());
			#endif

			//logger.Info("Waiting for parent");
			WaitForParent();

			//logger.Info("DONE waiting on parent");

			if (m_nativeProcessHandle == InvalidProcHandle) // Failed to create the child process
				return UBA_EXIT_CODE(7);
		}
#endif
		return 0;
	}

#if !PLATFORM_WINDOWS
	#if PLATFORM_LINUX && !defined(__USE_MISC)
	// glibc gates wait4() behind __USE_MISC (_DEFAULT_SOURCE), which UE's strict -std=c++20 disables; the symbol is always exported.
	extern "C" pid_t wait4(pid_t, int*, int, struct rusage*) noexcept;
	#endif

	// Memory + CPU for a non-detoured process from its reaped rusage (ru_maxrss is kB on Linux, bytes on Mac).
	static void CaptureRusageStats(ProcessStats& stats, const rusage& ru)
	{
		#if PLATFORM_LINUX
		u64 memBytes = u64(ru.ru_maxrss) * 1024;
		#else
		u64 memBytes = u64(ru.ru_maxrss);
		#endif
		AtomicMax(stats.peakMemory, memBytes);
		u64 cpuUs = u64(ru.ru_utime.tv_sec) * 1000000 + u64(ru.ru_utime.tv_usec)
		          + u64(ru.ru_stime.tv_sec) * 1000000 + u64(ru.ru_stime.tv_usec);
		AtomicMax(stats.cpuTime, cpuUs * (GetFrequency() / 1000000));
	}
#endif

	u32 ProcessImpl::InternalExitProcess(bool cancel)
	{
		SCOPED_FUTEX(m_initLock, lock);
		Logger& logger = m_session.m_logger;

		ProcHandle procHandle = m_nativeProcessHandle;
		if (procHandle == InvalidProcHandle)
			return ~0u;

		if (m_parentProcess)
			WaitForParent();
		m_nativeProcessHandle = InvalidProcHandle;

#if PLATFORM_WINDOWS
		HANDLE handle = (HANDLE)procHandle;
		auto closeHandleGuard = MakeGuard([&]()
			{
				IO_COUNTERS ioCounters;
				if (GetProcessIoCounters(handle, &ioCounters))
				{
					m_processStats.iopsRead = ioCounters.ReadOperationCount;
					m_processStats.iopsWrite = ioCounters.WriteOperationCount;
					m_processStats.iopsOther = ioCounters.OtherOperationCount;
				}

				CloseHandle(handle);
			});

		bool hadTimeout = false;
		if (cancel)
		{
			if (!TerminateProcess(handle, ProcessCancelExitCode))
				if (GetLastError() != ERROR_ACCESS_DENIED)
					logger.Info(TC("TerminateProcess for process %llu (%s) failed when cancelling (%s)"), u64(handle), m_startInfo.GetDescription(), LastErrorToText().data);
			DWORD res = WaitForSingleObject(handle, 1000); // On windows this should return in milliseconds. On wine it might take a little while. We need to wait to make sure process is reaped
			if (res == WAIT_FAILED)
				logger.Info(TC("WaitForSingleObject for process %llu (%s) failed when cancelling (%s)"), u64(handle), m_startInfo.GetDescription(), LastErrorToText().data);
		}
		else
		{	
			while (true)
			{
				DWORD res = WaitForSingleObject(handle, 120 * 1000);
				if (res == WAIT_OBJECT_0)
				{
					break;
				}

				if (res == WAIT_TIMEOUT)
				{
					if (!hadTimeout && m_nativeProcessExitCode != STILL_ACTIVE)
					{
						hadTimeout = true;
						const tchar* gotMessage = m_gotExitMessage ? TC("Got") : TC("Did not get");
						logger.Info(TC("WaitForSingleObject timed out after 120 seconds waiting for process %llu (%s) to exit (Exit code %u, %s ExitMessage and wrote %u files. Runtime: %s). Will terminate and wait again"), u64(handle), m_startInfo.GetDescription(), m_nativeProcessExitCode, gotMessage, u32(m_shared.writtenFiles.size()), TimeToText(GetTime() - m_startTime).str);
						if (!TerminateProcess(handle, m_nativeProcessExitCode))
							logger.Info(TC("TerminateProcess for process %llu (%s) failed when waiting timed out (%s)"), u64(handle), m_startInfo.GetDescription(), LastErrorToText().data);
						continue;
					}
					logger.Error(TC("WaitForSingleObject failed while waiting for process %llu (%s) to exit even after terminating it (%s)"), u64(handle), m_startInfo.GetDescription(), LastErrorToText().data);
				}
				else if (res == WAIT_FAILED)
					logger.Error(TC("WaitForSingleObject failed while waiting for process to exit (%s)"), LastErrorToText().data);
				else if (res == WAIT_ABANDONED)
					logger.Error(TC("Abandoned, this should never happen"));
				TerminateProcess(handle, UBA_EXIT_CODE(13));
				WaitForSingleObject(handle, 1000);
				return UBA_EXIT_CODE(13);
			}
		}

		bool res = true;
		if (!hadTimeout)
		{
			DWORD nativeExitCode = 0;
			res = GetExitCodeProcess(handle, (DWORD*)&nativeExitCode);
			if (!res && GetLastError() == ERROR_INVALID_HANDLE) // Was already terminated
				return ~0u;
			if (m_gotExitMessage || !m_detourEnabled)
				m_nativeProcessExitCode = nativeExitCode;
		}

		if (res || cancel)
			return m_nativeProcessExitCode;
		logger.Warning(TC("GetExitCodeProcess failed (%s)"), LastErrorToText().data);
		return UBA_EXIT_CODE(14);
#else

		auto g = MakeGuard([this]() { m_nativeProcessId = 0; });

		if (cancel)
		{
			// Cancel SIGKILLs the whole subtree directly — no SIGTERM first because JVM shutdown hooks can drain for minutes and keep UbaNinja alive.
			if (m_parentProcess == nullptr && m_nativeProcessId)
			{
				pid_t leaderPid = (pid_t)m_nativeProcessId;
				CancelProcessTree((int)leaderPid);

				// Bounded reap on the explicit cancel path; the leader is still a zombie until waited. FlushDeadProcesses picks up late-exiting descendants.
				int status = 0;
				rusage ru = {};
				bool reaped = false;
				bool reapedByUs = false;
				for (int i = 0; i < 50 && !reaped; ++i) // ~500ms
				{
					pid_t r = wait4(leaderPid, &status, WNOHANG, &ru);
					if (r == leaderPid)
						reaped = reapedByUs = true;
					else if (r == -1)
					{
						if (errno == ECHILD) // already reaped by drainer
							reaped = true;
						break;
					}
					else
						Sleep(10);
				}
				if (reaped)
				{
					if (WIFEXITED(status))
						m_nativeProcessExitCode = WEXITSTATUS(status);
					else if (WIFSIGNALED(status))
						m_nativeProcessExitCode = WTERMSIG(status);
					if (reapedByUs && !m_detourEnabled)
						CaptureRusageStats(m_processStats, ru);
				}
			}
			return m_nativeProcessExitCode;
		}


		if (m_parentProcess != nullptr) // We can't wait for grandchildren.. if we got here the parent reported the child as exited
			return 0;

		// Process should have been waited on here because of IsActive
		int status = 0;
		rusage ru = {};
		while (m_nativeProcessId)
		{
			int res = wait4((pid_t)m_nativeProcessId, &status, 0, &ru);
			if (res == -1)
			{
				if (errno == ECHILD) // Already reaped by drainer; trust the earlier-captured m_nativeProcessExitCode.
				{
					m_nativeProcessId = 0;
					break;
				}
				logger.Error(TC("waitpid failed on %u (%s)"), m_nativeProcessId, strerror(errno));
				return UBA_EXIT_CODE(15);
			}
			if (WIFEXITED(status))
			{
				m_nativeProcessExitCode = WEXITSTATUS(status);
				if (!m_detourEnabled)
					CaptureRusageStats(m_processStats, ru);
				break;
			}
			if (WIFSIGNALED(status))
			{
				//logger.Info(TC("SIGNALED"));
				m_nativeProcessExitCode = WTERMSIG(status);
				if (!m_detourEnabled)
					CaptureRusageStats(m_processStats, ru);
				break;
			}
			Sleep(1);
		}

		return m_nativeProcessExitCode;
#endif
	}

	void ProcessImpl::InternalLogLine(bool printInSession, TString&& line, LogEntryType logType)
	{
		m_session.DevirtualizeString(line, m_startInfo.rootsHandle, true, TC("LogLine"));
		LogLine(printInSession, std::move(line), logType);
	}

	void ProcessImpl::InternalDestroyEvents()
	{
		#if !PLATFORM_WINDOWS
		if (UseSemaphoreForSharedEvents())
		{
			delete &m_cancelEvent;
			delete &m_writeEvent;
			delete &m_readEvent;
		}
		else
		{
			m_cancelEvent.~SharedEvent();
			m_writeEvent.~SharedEvent();
			m_readEvent.~SharedEvent();
		}
		#endif
	}

#if !PLATFORM_WINDOWS
	bool ProcessImpl::PollStdPipes(PipeReader& outReader, PipeReader& errReader, int timeoutMs)
	{
		if (m_stdOutPipe == -1 && m_stdErrPipe == -1)
			return false;

		auto pipeGuard = MakeGuard([&]()
		{
			if (m_stdOutPipe != -1) { close(m_stdOutPipe); m_stdOutPipe = -1; }
			if (m_stdErrPipe != -1) { close(m_stdErrPipe); m_stdErrPipe = -1; }
		});

		bool outClosed = (m_stdOutPipe == -1);
		bool errClosed = (m_stdErrPipe == -1);

		PipeReader* pipeReaders[] = { &outReader, &errReader };

		while (!outClosed || !errClosed)
		{
			pollfd plist[2];
			int nfds = 0;

			if (!outClosed)
			{
				plist[nfds].fd = m_stdOutPipe;
				plist[nfds].events = POLLIN | POLLHUP;
				plist[nfds].revents = 0;
				++nfds;
			}
			if (!errClosed)
			{
				plist[nfds].fd = m_stdErrPipe;
				plist[nfds].events = POLLIN | POLLHUP;
				plist[nfds].revents = 0;
				++nfds;
			}

			int rval = poll(plist, nfds, timeoutMs);
			if (rval < 0)
			{
				if (errno == EINTR)
					continue;
				#if PLATFORM_MAC
				m_session.m_logger.Error(TC("pipe polling error (%s)"), strerror(errno));
				#endif
				return false;
			}

			if (rval == 0)
				break; // timeout

			for (int i = 0; i != nfds; ++i)
			{
				pollfd& p = plist[i];

				if (p.revents & POLLERR)
					return m_session.m_logger.Error(TC("pipe polling error"));

				if (!(p.revents & (POLLIN | POLLHUP)))
					continue;

				char buffer[1024];
				int bytesRead = read(p.fd, buffer, sizeof_array(buffer) - 1);

				if (bytesRead > 0)
				{
					buffer[bytesRead] = 0;

					if (p.fd == m_stdOutPipe)
						outReader.ReadData(buffer, bytesRead);
					else
						errReader.ReadData(buffer, bytesRead);
				}
				else if (bytesRead == 0)
				{
					// EOF
					if (p.fd == m_stdOutPipe)
					{
						close(m_stdOutPipe);
						m_stdOutPipe = -1;
						outClosed = true;
					}
					else
					{
						close(m_stdErrPipe);
						m_stdErrPipe = -1;
						errClosed = true;
					}
				}
				else
				{
					if (errno != EINTR && errno != EAGAIN)
					{
						m_session.m_logger.Error(TC("PollStdPipes read error (%s)"), strerror(errno));
						return false;
					}
				}
			}
		}

		pipeGuard.Cancel();
		return !outClosed || !errClosed;
	}
#endif

	void ProcessImpl::WaitForParent()
	{
		u64 startTime = GetTime();
		while (!m_waitForParent.IsSet(500) && !IsCancelled())
		{
			if (TimeToMs(GetTime() - startTime) > 120 * 1000) // 
			{
				startTime = GetTime();
				m_session.m_logger.Error(TC("Waiting for parent process in createprocess has now taken more than 120 seconds."));
			}
		}
	}

	void ProcessImpl::WaitForChildrenExit()
	{
		if (IsCancelled())
			for (auto& child : m_childProcesses)
				child.Cancel();

		for (auto& child : m_childProcesses)
			((ProcessImpl*)child.m_process)->m_waitForParent.Set();

		for (auto& child : m_childProcesses)
		{
			auto& childProcess = *(ProcessImpl*)child.m_process;
			while (!childProcess.m_messageThreadDone.IsSet(10000))
			{
				m_session.m_logger.Error(TC("Process %u (pid %i) timed out waiting for its child process %u (pid %i) to finish (%s), INVESTIGATE WHAT IS STUCK!!!"), m_id, m_nativeProcessId, childProcess.m_id, childProcess.m_nativeProcessId, m_startInfo.GetDescription());
				#if !UBA_DEBUG
				UBA_SHIPPING_ASSERTF(false, TC("Have to hard exit, this might crash the build if not"));
				#endif
			}
		}
	}

	bool ProcessImpl::CancelWithError(bool dummy)
	{
		Cancel();
		return false;
	}

	bool ProcessImpl::IsCalledFromThis() const
	{
		return m_messageThread && m_messageThread->IsInside();
	}
}
