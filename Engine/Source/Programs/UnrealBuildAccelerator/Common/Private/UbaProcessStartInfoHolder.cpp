// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaProcessStartInfoHolder.h"
#include "UbaConfig.h"
#include "UbaProcessUtils.h"

namespace uba
{
	#if !PLATFORM_WINDOWS
	// Shell-wrap detection helpers (Linux). Callers pass the raw command line
	// — i.e. <application> <arguments> joined — so the first-token checks see
	// what execv would actually get.
	static bool IsShellKeywordFirstToken(const tchar* cmd, u32 len)
	{
		// Skip leading whitespace.
		u32 i = 0;
		while (i < len && (cmd[i] == TC(' ') || cmd[i] == TC('\t'))) ++i;
		if (i >= len) return false;

		// Subshell: a leading '(' means the whole thing is shell syntax.
		if (cmd[i] == TC('('))
			return true;

		u32 tokStart = i;
		while (i < len && cmd[i] != TC(' ') && cmd[i] != TC('\t') && cmd[i] != TC('\n')) ++i;
		u32 tokLen = i - tokStart;
		if (tokLen == 0) return false;
		const tchar* tok = cmd + tokStart;

		// Env-var prefix: IDENT=...  (e.g. PWD=/proc/self/cwd clang++ -c ...).
		// First char must be [A-Za-z_]; remaining identifier chars up to '='.
		if ((tok[0] >= TC('A') && tok[0] <= TC('Z')) ||
			(tok[0] >= TC('a') && tok[0] <= TC('z')) ||
			tok[0] == TC('_'))
		{
			for (u32 k = 1; k < tokLen; ++k)
			{
				tchar c = tok[k];
				if (c == TC('='))
					return true;
				bool ident = (c >= TC('A') && c <= TC('Z')) ||
				             (c >= TC('a') && c <= TC('z')) ||
				             (c >= TC('0') && c <= TC('9')) ||
				             c == TC('_');
				if (!ident) break;
			}
		}

		// Shell keywords / builtins. Matched against tokLen to avoid matching
		// a binary that *starts* with e.g. "if_something".
		struct KW { const tchar* s; u32 n; };
		static const KW kw[] = {
			{ TC("if"),2 }, { TC("for"),3 }, { TC("while"),5 }, { TC("case"),4 },
			{ TC("do"),2 }, { TC("done"),4 }, { TC("fi"),2 }, { TC("then"),4 },
			{ TC("else"),4 }, { TC("elif"),4 }, { TC("{"),1 }, { TC("!"),1 },
			{ TC("true"),4 }, { TC("false"),5 }, { TC(":"),1 },
			{ TC("cd"),2 }, { TC("export"),6 }, { TC("set"),3 }, { TC("unset"),5 },
			{ TC("source"),6 }, { TC("."),1 }, { TC("eval"),4 }, { TC("exec"),4 },
		};
		for (const KW& k : kw)
			if (tokLen == k.n && memcmp(tok, k.s, k.n * sizeof(tchar)) == 0)
				return true;

		return false;
	}

	// Catches fused shell metachars (`'...'>OUT`, `cmd|foo`, `2>&1`) that ParseArguments misses as standalone tokens.
	// Idempotent for `sh -c "..."` wraps (honours `\"`, `\\` inside double quotes).
	static bool ContainsShellMetaOutsideQuotes(const tchar* cmd, u32 len)
	{
		bool inSingle = false, inDouble = false;
		bool prevWasWhitespace = true; // Start-of-string counts as a word boundary.
		for (u32 i = 0; i < len; ++i)
		{
			tchar c = cmd[i];
			if (inDouble && c == TC('\\') && i + 1 < len)
			{
				// Skip any escaped char inside "..." (covers \", \\, \$ etc.).
				++i;
				prevWasWhitespace = false;
				continue;
			}
			if (c == TC('\'') && !inDouble) { inSingle = !inSingle; prevWasWhitespace = false; continue; }
			if (c == TC('"')  && !inSingle) { inDouble = !inDouble; prevWasWhitespace = false; continue; }
			if (inSingle || inDouble) { prevWasWhitespace = false; continue; }
			if (c == TC('`')) return true;
			if (c == TC('$') && i + 1 < len && cmd[i + 1] == TC('(')) return true;
			if (c == TC('>') || c == TC('<') || c == TC('|') || c == TC(';')) return true;
			if (c == TC('&')) return true;
			if (c == TC('#') && prevWasWhitespace) return true; // `#` is a shell comment only at word boundary — guards ninja's trailing `# hash of inputs`.
			prevWasWhitespace = (c == TC(' ') || c == TC('\t') || c == TC('\n'));
		}
		return false;
	}
	#endif

	bool ProcessStartInfoHolder::Expand()
	{
		if (expanded)
			return false;
		expanded = true;

	#if PLATFORM_WINDOWS
		// Caller may supply an empty application with the full command line in
		// arguments. Promote the first token before the existing cmd.exe logic
		// runs — that logic assumes `application` identifies the real exe.
		if (applicationStr.empty())
		{
			// Copy argumentsStr into a local before we start writing back to
			// it — std::string::assign doesn't define behavior when the source
			// range overlaps the destination.
			TString src;
			src.swap(argumentsStr);
			const tchar* p = src.c_str();
			const tchar* end = p + src.size();
			while (p < end && (*p == TC(' ') || *p == TC('\t'))) ++p;
			const tchar* firstStart = p;
			if (p < end && *p == TC('"'))
			{
				++p; firstStart = p;
				while (p < end && *p != TC('"')) ++p;
				u32 n = u32(p - firstStart);
				if (p < end) ++p;
				applicationStr.assign(firstStart, n);
			}
			else
			{
				while (p < end && *p != TC(' ') && *p != TC('\t')) ++p;
				applicationStr.assign(firstStart, u32(p - firstStart));
			}
			application = applicationStr.c_str();
			while (p < end && (*p == TC(' ') || *p == TC('\t'))) ++p;
			argumentsStr.assign(p, u32(end - p));
			arguments = argumentsStr.c_str();
		}

		// Special handling to avoid calling cmd.exe if not needed
		if (!Contains(application, TC("cmd.exe")))
		{
			// We also need to check if we need to add cmd.exe around the arguments
			bool requiresShell = false;
			ParseArguments(argumentsStr.c_str(), argumentsStr.size(), [&](const tchar* arg, u32 argLen)
				{
					StringView view(arg, argLen);
					requiresShell |= argLen == 1 && (*arg == '<' || *arg == '>' || *arg == '|' || *arg == '^');
					requiresShell |= argLen == 2 && (view.Equals(TCV(">>")) || view.Equals(TCV("&&")) || view.Equals(TCV("||")));
				});
			if (requiresShell)
			{
				argumentsStr = TC("/C \"") + applicationStr + TC(" ") + argumentsStr + TC("\"");
				arguments = argumentsStr.c_str();
				applicationStr = TC("cmd.exe");
				application = applicationStr.c_str();
			}
			return false;
		}

		const tchar* argsBegin = argumentsStr.c_str();

		// Check if application is repeated as first argument, in that case consume it
		const tchar* firstArgBegin = argsBegin;
		const tchar* firstArgEnd = nullptr;
		if (*firstArgBegin == '\"')
		{
			++firstArgBegin;
			firstArgEnd = TStrchr(firstArgBegin, '\"');
		}
		else
			firstArgEnd = TStrchr(firstArgBegin, ' ');
		if (!firstArgEnd)
			return false;
		return InternalExpand(firstArgBegin, firstArgEnd);

	  #else // PLATFORM_WINDOWS end

		// The caller may supply an empty application and put the full command
		// line in arguments. We build fullCmd = application + " " + arguments
		// (or just arguments if application is empty) and run shell detection
		// against that — it's what the shell would see anyway.
		bool appEmpty = applicationStr.empty();
		TString fullCmd;
		if (appEmpty)
			fullCmd = argumentsStr;
		else
			fullCmd = applicationStr + TC(" ") + argumentsStr;

		bool requiresShell = false;

		// Wrap any .sh script (may lack execute bit or have a virtualised shebang).
		// Only applicable when an application was provided up-front.
		if (!appEmpty)
		{
			u32 appLen = u32(applicationStr.size());
			if (appLen >= 3 && applicationStr[appLen-3] == TC('.') && applicationStr[appLen-2] == TC('s') && applicationStr[appLen-1] == TC('h'))
				requiresShell = true;
		}

		// Only shell-parse a bare command line (empty application). With an explicit application the
		// arguments are literal argv, so metachars in them must not pull in /bin/sh.
		if (appEmpty)
		{
			// First-token checks: env-var prefix (PWD=... cmd), shell keywords
			// (if/for/while/cd/...), leading subshell '('.
			if (!requiresShell)
				requiresShell = IsShellKeywordFirstToken(fullCmd.c_str(), u32(fullCmd.size()));

			// Metachars exposed as tokens by ParseArguments: |, ;, <, >, &&, ||.
			// Also backslash-escaped glob chars (\*, \?, \[) which indicate the
			// caller relies on the shell to strip the escape.
			if (!requiresShell)
			{
				ParseArguments(argumentsStr.c_str(), argumentsStr.size(), [&](const tchar* arg, u32 argLen)
				{
					StringView view(arg, argLen);
					requiresShell |= argLen == 1 && (*arg == '<' || *arg == '>' || *arg == '|' || *arg == ';');
					requiresShell |= argLen == 2 && (view.Equals(TCV("&&")) || view.Equals(TCV("||")));
					if (!requiresShell)
						for (u32 i = 0; i + 1 < argLen; ++i)
							if (arg[i] == TC('\\') && (arg[i+1] == TC('*') || arg[i+1] == TC('?') || arg[i+1] == TC('[')))
								{ requiresShell = true; break; }
				});
			}

			// Metachars ParseArguments doesn't model distinctly: $(...), backticks,
			// and ';' when fused to adjacent tokens (e.g. "fi;"). Scan the raw
			// fullCmd respecting quotes.
			if (!requiresShell)
				requiresShell = ContainsShellMetaOutsideQuotes(fullCmd.c_str(), u32(fullCmd.size()));
		}

		if (requiresShell)
		{
			// Build sh -c '<escaped-command>' using SINGLE quotes. Inside `'...'`
			// every byte except `'` is literal — no `\` or `"` interactions, no
			// `\\n`/`\"` ambiguity. The single-level encoding is: each `'` in
			// the original becomes `'\''` (close, escaped `'`, reopen). Real
			// ninja passes argv[2] raw and our outer ParseArguments inner-loop
			// strips the `'...'` markers and unescapes `'\''` back to `'`, so
			// argv[2] reaches sh byte-for-byte identical to fullCmd.
			//
			// Why not double quotes: a body like `\"llvmorg-1\"` (literal `\"`
			// from ninja's shell-escape) becomes `\\"...\\"` after escaping
			// the `"`s, which sh inside `"..."` halves to `\` + closing-`"` →
			// unterminated quoted string. Single quotes sidestep this entire
			// escape-pair-vs-quote conflict.
			TString wrapped = TC("-c '");
			wrapped.reserve(fullCmd.size() + 8);
			for (tchar c : fullCmd)
			{
				if (c == TC('\'')) wrapped += TC("'\\''");
				else               wrapped += c;
			}
			wrapped += TC('\'');
			argumentsStr = std::move(wrapped);
			arguments = argumentsStr.c_str();
			// Bare "sh" — Session::PrepareProcess resolves via SearchPathForFile,
			// so the user's PATH decides which sh is used (busybox, bash, etc.).
			applicationStr = TC("sh");
			application = applicationStr.c_str();
			return true;
		}

		// No shell needed. If the caller didn't supply an application, promote
		// the first tokenized argument to be the application and keep the rest
		// as arguments.
		if (!appEmpty)
			return true;

		// Find the end of the first argument as ParseArguments would emit it,
		// respecting quotes, but we also need to know where it ended in the
		// *source* string so the remainder becomes argumentsStr verbatim.
		const tchar* p = fullCmd.c_str();
		const tchar* end = p + fullCmd.size();
		while (p < end && (*p == TC(' ') || *p == TC('\t'))) ++p;
		const tchar* firstStart = p;

		// Tokenize just enough to find the end of argv[0]. Honor single and
		// double quotes like ParseArguments does.
		bool inSingle = false, inDouble = false;
		while (p < end)
		{
			tchar c = *p;
			if (c == TC('\'') && !inDouble) { inSingle = !inSingle; ++p; continue; }
			if (c == TC('"')  && !inSingle) { inDouble = !inDouble; ++p; continue; }
			if (!inSingle && !inDouble && (c == TC(' ') || c == TC('\t')))
				break;
			++p;
		}

		// Reparse just argv[0] through ParseArguments so quotes/backslashes
		// are handled the same as the rest of UBA.
		TString firstArg;
		ParseArguments(firstStart, u32(p - firstStart), [&](const tchar* a, u32 n)
		{
			if (firstArg.empty())
				firstArg.assign(a, n);
		});

		while (p < end && (*p == TC(' ') || *p == TC('\t'))) ++p;

		applicationStr = std::move(firstArg);
		application = applicationStr.c_str();
		argumentsStr.assign(p, u32(end - p));
		arguments = argumentsStr.c_str();

		return true;
	#endif
	}

	UBA_NOINLINE bool ProcessStartInfoHolder::InternalExpand(const tchar* firstArgBegin, const tchar* firstArgEnd)
	{
		// Separate function to hide away large stack object
	#if PLATFORM_WINDOWS
		const tchar* argsBegin = argumentsStr.c_str();
		const tchar* argsEnd = argsBegin + argumentsStr.size();

		StringBuffer<32*1024> commands;
		commands.Append(firstArgBegin, firstArgEnd - firstArgBegin);
		if (commands.Contains(application))
			argsBegin = firstArgEnd;
		while (argsBegin && *argsBegin == ' ')
			++argsBegin;
		commands.Clear();


		// Parse switches... only supported is /C right now
		if (!StartsWith(argsBegin, TC("/C ")))
			return false;
		argsBegin += 3;
		while (argsBegin && *argsBegin == ' ')
			++argsBegin;

		if (argsBegin && *argsBegin == '/') // Unknown switch, don't try to expand cmd
			return false;

		if (argsBegin && *argsBegin == '\"')
		{
			++argsBegin;
			--argsEnd;
		}
		while (argsBegin && *argsBegin == ' ')
			++argsBegin;

		commands.Append(argsBegin, argsEnd - argsBegin);

		// It could be that there is just a chain of commands to set working dir, in that case we strip out cmd.exe
		const tchar* andPos = nullptr;
		if (commands.Contains(TC(" && "), true, &andPos))
		{
			if (Contains(andPos + 4, TC(" && "))) // If more than one && we don't try to expand cmd.exe
				return false;
			if (!commands.StartsWith(TC("cd /D"))) // First command is not cd, don't try to expand cmd.exe
				return false;
			const tchar* workDirStart = commands.data + 6;
			workingDirStr.assign(workDirStart, andPos - workDirStart);

			StringBuffer<> fixed;
			FixPath(workingDirStr.data(), nullptr, 0, fixed);
			workingDirStr = fixed.data;
			workingDir = workingDirStr.c_str();

			const tchar* commandLine = andPos + 4;
			while (*commandLine && *commandLine == ' ')
				++commandLine;
			const tchar* applicationBegin = commandLine;
			const tchar* applicationEnd = nullptr;
			if (*applicationBegin == '\"')
			{
				++applicationBegin;
				applicationEnd = TStrchr(applicationBegin, '\"');
			}
			else
				applicationEnd = TStrchr(applicationBegin, ' ');
			if (!applicationEnd)
				applicationEnd = applicationBegin + TStrlen(applicationBegin);
			applicationStr.assign(applicationBegin, applicationEnd - applicationBegin);
			FixPath(applicationStr.data(), nullptr, 0, fixed.Clear());
			applicationStr = fixed.data;
			application = applicationStr.c_str();
			argsBegin = *applicationEnd == '\"' ? applicationEnd + 1 : applicationEnd;
			while (*argsBegin && *argsBegin == ' ')
				++argsBegin;
			argumentsStr = argsBegin;
			arguments = argumentsStr.c_str();
			return true;
		}
		else
		{
			// TODO: This is super hacky... but we don't want to spawn the cmd.exe just to copy a file since the overhead can be half a second
			// "C:\WINDOWS\system32\cmd.exe" /C "copy /Y "E:\dev\fn\Engine\Source\Runtime\RenderCore\RenderCore.natvis" "E:\dev\fn\Engine\Intermediate\Build\Win64\x64\UnrealPak\Development\RenderCore\RenderCore.natvis" 1>nul"
			if (!commands.StartsWith(TC("copy /Y \"")))
				return false;

			const tchar* fromFileBegin = commands.data + 8;
			const tchar* fromFileEnd = TStrchr(fromFileBegin+1, '\"');
			if (!fromFileEnd)
				return false;
			const tchar* toFileBegin = TStrchr(fromFileEnd + 1, '\"');
			if (!toFileBegin)
				return false;
			++toFileBegin;
			const tchar* toFileEnd = TStrchr(toFileBegin, '\"');
			if (!toFileEnd)
				return false;

			applicationStr = TC("ubacopy");
			application = applicationStr.c_str();
			argumentsStr = fromFileBegin;
			arguments = argumentsStr.c_str();
			return true;
		}
	  #else
		  return false;
	  #endif
	}

	void ProcessStartInfoHolder::Apply(const Config& config, const uba::tchar* configTable)
	{
		const ConfigTable* tablePtr = config.GetTable(configTable);
		if (!tablePtr)
			return;
		const ConfigTable& table = *tablePtr;
		if (table.GetValueAsString(applicationStr, TC("Application")))
			application = applicationStr.c_str();
		if (table.GetValueAsString(argumentsStr, TC("Arguments")))
			arguments = argumentsStr.c_str();
		if (table.GetValueAsString(workingDirStr, TC("WorkingDir")))
			workingDir = workingDirStr.c_str();
		if (table.GetValueAsString(descriptionStr, TC("Description")))
			description = descriptionStr.c_str();
		if (table.GetValueAsString(logFileStr, TC("LogFile")))
			logFile = logFileStr.c_str();
		if (table.GetValueAsString(breadcrumbsStr, TC("Breadcrumbs")))
			breadcrumbs = breadcrumbsStr.c_str();
		if (table.GetValueAsString(overlayFilesStr, TC("OverlayFiles")))
			overlayFiles = overlayFilesStr.c_str();

		table.GetValueAsU32(priorityClass, TC("PriorityClass"));
		table.GetValueAsBool(trackInputs, TC("TrackInputs"));
		table.GetValueAsBool(useCustomAllocator, TC("UseCustomAllocator"));
		table.GetValueAsBool(writeOutputFilesOnFail, TC("WriteOutputFilesOnFail"));
		table.GetValueAsBool(startSuspended, TC("StartSuspended"));
		table.GetValueAsU64(rootsHandle, TC("RootsHandle"));
	}
}
