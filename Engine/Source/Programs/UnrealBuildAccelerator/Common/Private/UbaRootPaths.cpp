// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaRootPaths.h"
#include "UbaFileAccessor.h"

#if PLATFORM_WINDOWS
#include <shlobj_core.h>
#endif

namespace uba
{
	bool RootPaths::RegisterRoot(Logger& logger, const tchar* rootPath, bool includeInKey, u8 id)
	{
		return InternalRegisterRoot(logger, m_roots, rootPath, includeInKey, id);
	}

	bool RootPaths::RegisterSystemRoots(Logger& logger, u8 startId)
	{
#if PLATFORM_WINDOWS

		static StringBuffer<64> systemDir;
		static StringBuffer<64> programW6432;
		static StringBuffer<64> programFiles86;
		static StringBuffer<64> programData;

		static bool init = []()
			{
				systemDir.count = GetSystemDirectory(systemDir.data, systemDir.capacity);
				systemDir.EnsureEndsWithSlash();
				programW6432.count = GetEnvironmentVariable(TC("ProgramW6432"), programW6432.data, programW6432.capacity);
				programW6432.EnsureEndsWithSlash();
				programFiles86.count = GetEnvironmentVariable(TC("ProgramFiles(x86)"), programFiles86.data, programFiles86.capacity);
				programFiles86.EnsureEndsWithSlash();

				PWSTR path;
				if (!SUCCEEDED(SHGetKnownFolderPath(FOLDERID_ProgramData, 0, NULL, &path)))
					return false;
				programData.Append(path).EnsureEndsWithSlash();
				CoTaskMemFree(path);
				return true;
			}();

		if (!init)
			return false;

		u8 id = startId;
		auto GetId = [&]() { u8 res = id; if (id) id += 2; return res; };
		RegisterRoot(logger, systemDir.data, false, GetId()); // Ignore files from here.. we do expect them not to affect the output of a process
		RegisterRoot(logger, programW6432.data, true, GetId());
		RegisterRoot(logger, programFiles86.data, true, GetId());
		RegisterRoot(logger, programData.data, true, GetId());

#else
		// no system roots
#endif
		return true;
	}

	bool RootPaths::RegisterIgnoredRoot(Logger& logger, const tchar* rootPath)
	{
		return InternalRegisterRoot(logger, m_ignoredRoots, rootPath, false, 0);
	}

	bool RootPaths::IsEmpty() const
	{
		return m_roots.empty();
	}

	const RootPaths::Root* RootPaths::FindRoot(const StringView& path) const
	{
		return InternalFindRoot(m_roots, path);
	}

	static TString& EmptyString = *new TString(); // Need to leak to prevent shutdown hangs when running in managed process

	const TString& RootPaths::GetRoot(u32 index) const
	{
		const Roots& roots = m_roots;
		if (roots.size() <= index)
			return EmptyString;
		return roots[index].path;
	}

	TString RootPaths::GetAllRoots() const
	{
		TString res;
		bool isFirst = true;
		for (auto& root : m_roots)
		{
			if (!isFirst)
				res += ' ';
			isFirst = false;
			res += root.path;
		}
		return res;
	}

	CasKey RootPaths::NormalizeAndHashFile(Logger& logger, const tchar* filename, bool warnOnFileNotFound) const
	{
		FileAccessor file(logger, filename);
		if (!file.OpenMemoryRead(0, false))
		{
			if (warnOnFileNotFound)
				logger.Warning(TC("NormalizeAndHashFile Can't find file %s"), filename);
			return CasKeyZero;
		}
		bool wasNormalized = false;
		CasKeyHasher hasher;
		auto hashString = [&](const char* str, u64 strLen, u32 rootPos)
			{
				wasNormalized |= rootPos != ~0u;
				hasher.Update(str, strLen);
			};
		if (!NormalizeString<char>(logger, (const char*)file.GetData(), file.GetSize(), hashString, false, filename))
			return CasKeyZero;

		return AsNormalized(ToCasKey(hasher, false), wasNormalized);
	}

	bool RootPaths::InternalRegisterRoot(Logger& logger, Roots& roots, const tchar* rootPath, bool includeInKey, u8 id)
	{
		// Register rootPath both with single path separators and double path separators on windows because text files store them with double path separators
		#if PLATFORM_WINDOWS
		u8 index = id * 4;
		StringBuffer<512> forwardSlash;
		StringBuffer<512> backwardSlash;
		StringBuffer<512> doubleBackwardSlash;
		StringBuffer<512> spaceEscapedBackwardSlash;
		bool hasSpace = false;
		for (const tchar* it=rootPath; *it; ++it)
		{
			tchar c = *it;

			if (c == '/')
			{
				forwardSlash.Append('/');
				backwardSlash.Append('\\');
				doubleBackwardSlash.Append(TCV("\\\\"));
				spaceEscapedBackwardSlash.Append('\\');
			}
			else if (c == '\\')
			{
				forwardSlash.Append('/');
				backwardSlash.Append('\\');
				doubleBackwardSlash.Append("\\\\");
				spaceEscapedBackwardSlash.Append('\\');
			}
			else
			{
				if (c == ' ')
				{
					hasSpace = true;
					spaceEscapedBackwardSlash.Append('\\');
				}
				forwardSlash.Append(c);
				backwardSlash.Append(c);
				doubleBackwardSlash.Append(c);
				spaceEscapedBackwardSlash.Append(c);
			}
		}

		if (!hasSpace)
			spaceEscapedBackwardSlash.Clear();

		const tchar* rootPaths[] = { forwardSlash.data, backwardSlash.data, doubleBackwardSlash.data, spaceEscapedBackwardSlash.data };
		#else
		u8 index = id;
		const tchar* rootPaths[] = { rootPath };
		#endif

		for (const tchar* rp : rootPaths)
		{
			if (!InternalRegisterRoot2(logger, roots, rp, includeInKey, index))
				return false;
			if (id)
				++index;
		}
		return true;
	}

	bool RootPaths::InternalRegisterRoot2(Logger& logger, Roots& roots, const tchar* rootPath, bool includeInKey, u8 index)
	{
		if (index == 0)
			index = u8(roots.size());
		if (index == '~' - ' ') // This is not really true.. as long as value is under 256 we're good
			return logger.Error(TC("Too many roots added (%u)"), index);

		if (index >= roots.size())
			roots.resize(index+1);

		if (!*rootPath)
			return true;

		auto& root = roots[index];
		if (!root.path.empty())
			return logger.Error(TC("Root at index %u already added (existing as %s, added as %s)"), index, root.path.c_str(), rootPath);

		root.index = index;
		root.path = rootPath;

		if (CaseInsensitiveFs)
			ToLower(root.path.data());

		root.includeInKey = includeInKey;

		roots.longestRoot = Max(u32(root.path.size()), roots.longestRoot);

		if (!roots.shortestRoot || root.path.size() < roots.shortestRoot)
		{
			roots.shortestRoot = u32(root.path.size());
			for (auto& r : roots)
				r.shortestPathKey = ToStringKeyNoCheck(r.path.data(), roots.shortestRoot);
		}
		else
			root.shortestPathKey = ToStringKeyNoCheck(root.path.data(), roots.shortestRoot);
		return true;
	}

	const RootPaths::Root* RootPaths::InternalFindRoot(const Roots& roots, const StringView& path) const
	{
		if (path.count < roots.shortestRoot)
			return nullptr;

		StringBuffer<MaxPath> shortPath;
		shortPath.Append(path.data, roots.shortestRoot);
		if (CaseInsensitiveFs)
			shortPath.MakeLower();

		StringKey key = ToStringKeyNoCheck(shortPath.data, roots.shortestRoot);
		for (u32 i=0, e=u32(roots.size()); i!=e; ++i)
		{
			auto& root = roots[i];
			if (key != root.shortestPathKey)
				continue;
			if (!path.StartsWith(root.path.c_str()))
				continue;
			return &roots[i];
		}
		return nullptr;
	}
}