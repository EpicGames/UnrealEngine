// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaScheduler.h"
#include "UbaApplicationRules.h"
#include "UbaCacheClient.h"
#include "UbaConfig.h"
#include "UbaEnvironment.h"
#include "UbaFileAccessor.h"
#include "UbaNetworkMessage.h"
#include "UbaProcess.h"
#include "UbaProcessStartInfoHolder.h"
#include "UbaRootPaths.h"
#include "UbaSessionServer.h"
#include "UbaStringBuffer.h"

#if PLATFORM_WINDOWS
#include <winternl.h>
#endif

#define UBA_SCHEDULER_PROCESS_QUEUES \
			UBA_SCHEDULER_PROCESS_QUEUE(TestCache) \
			UBA_SCHEDULER_PROCESS_QUEUE(DownloadCache) \
			UBA_SCHEDULER_PROCESS_QUEUE(CompleteFromCache) \
			UBA_SCHEDULER_PROCESS_QUEUE(RunLocal) \
			UBA_SCHEDULER_PROCESS_QUEUE(RunRemote) \
			UBA_SCHEDULER_PROCESS_QUEUE(RunCrossArchitecture) \
			UBA_SCHEDULER_PROCESS_QUEUE(RunCrossPlatform) \
			UBA_SCHEDULER_PROCESS_QUEUE(RunCrossArchAndPlatform) \
			UBA_SCHEDULER_PROCESS_QUEUE(Skip) \

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	static inline void AtomicStatsAdd(u32& field, int delta)
	{
		std::atomic_ref<u32>(field).fetch_add(u32(delta), std::memory_order_relaxed);
	}
	static inline void AtomicStatsAdd(float& field, float delta)
	{
		if (delta == 0.0f)
			return;
		std::atomic_ref<float> ref(field);
		float expected = ref.load(std::memory_order_relaxed);
		while (!ref.compare_exchange_weak(expected, expected + delta, std::memory_order_relaxed, std::memory_order_relaxed))
			;
	}
	static inline u32   AtomicStatsLoad(const u32& field)   { return std::atomic_ref<u32>(const_cast<u32&>(field)).load(std::memory_order_relaxed); }
	static inline float AtomicStatsLoad(const float& field) { return std::atomic_ref<float>(const_cast<float&>(field)).load(std::memory_order_relaxed); }

	#if PLATFORM_WINDOWS
	constexpr u32 CreateTimeOffset = 32;
	constexpr u32 PidOffset = 80;
	constexpr u32 ParentPidOffset = 88;
	constexpr u32 PagefileUsageOffset = 184;
	static_assert(offsetof(SYSTEM_PROCESS_INFORMATION, UniqueProcessId) == PidOffset);
	//static_assert(offsetof(SYSTEM_PROCESS_INFORMATION, InheritedFromUniqueProcessId) == ParentPidOffset);
	//static_assert(offsetof(SYSTEM_PROCESS_INFORMATION, CreateTime) == CreateTimeOffset);
	//static_assert(offsetof(SYSTEM_PROCESS_INFORMATION, PagefileUsage) == PagefileUsageOffset);
	#endif

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct Scheduler::CacheFetchInfo : CacheClient::FetchContext
	{
		CacheFetchInfo(const ProcessStartInfo& i, Session& s, RootsHandle rh, MessagePriority mp) : CacheClient::FetchContext(i, s, rh, mp) {}
		CacheResult result;
		CacheClient* client;
		u64 speculativeOutputId = 0;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct Scheduler::ProcessStartInfo2 : ProcessStartInfoHolder
	{
		ProcessStartInfo2(const ProcessStartInfo& si, const u8* ki, u32 kic, const u8* ko, u32 koc)
		:	ProcessStartInfoHolder(si)
		, knownInputs(ki)
		, knownOutputs(ko)
		, knownInputsCount(kic)
		, knownOutputsCount(koc)
		{
		}

		~ProcessStartInfo2() { delete[] knownInputs; delete[] knownOutputs; }

		CacheFetchInfo* cacheInfo = nullptr;

		u64 predictedMemoryUsage = 0;
		const u8* knownInputs;
		const u8* knownOutputs;
		u32 knownInputsCount;
		u32 knownOutputsCount;
		float weight = 1.0f;
		u32 cacheBucketId = 0; // Zero means this process will not check cache
		u64 memoryGroupId = 0; // Zero means no memory group so this process will not be taken into account in memory allocations
		bool skipAsSuccessWhenScheduled = false;
		bool hasUpdatedPredictedMemoryUsage = false;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct Scheduler::ExitProcessInfo
	{
		Scheduler* scheduler = nullptr;
		ProcessStartInfo2* startInfo;
		Atomic<u32> refCount = 1;
		u32 processIndex = ~0u;
		u32 processCount = 1;
		float weight = 1.0f;
		bool wasReturned = false;
		bool hasCheckedWasReturned = false;
		Category runningCategory = Category_Local;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct Scheduler::MemGroupStats
	{
		u64 baseline = 0;
		u64 average = 0;
		u64 history[10] = { 0 };
		u32 historyCounter = 0;
		u32 activeProcessCount = 0;
	};

	struct Scheduler::MemGroupEntry
	{
		u64 baseline = 0;
		u64 average = 0;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class SkippedProcess : public Process
	{
	public:
		SkippedProcess(const ProcessStartInfo& i, u32 exitCode) : m_startInfo(i), m_exitCode(exitCode) {}
		virtual u32 GetExitCode() override { return m_exitCode; }
		virtual bool HasExited() override { return true; }
		virtual bool WaitForExit(u32 millisecondsTimeout) override{ return true; }
		virtual const ProcessStartInfo& GetStartInfo() const override { return m_startInfo; }
		virtual const Vector<ProcessLogLine>& GetLogLines() const override { static Vector<ProcessLogLine> v{ProcessLogLine{TC("Skipped"), LogEntryType_Warning}}; return v; }
		virtual const Vector<u8>& GetTrackedInputs() const override { static Vector<u8> v; return v;}
		virtual const Vector<u8>& GetTrackedOutputs() const override { static Vector<u8> v; return v;}
		virtual u64 GetTotalWallTime() const override { return 0; }
		virtual bool IsRemote() const override { return false; }
		virtual ProcessExecutionType GetExecutionType() const override { return ProcessExecutionType_Skipped; }
		ProcessStartInfoHolder m_startInfo;
		u32 m_exitCode;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class CachedProcess : public Process
	{
	public:
		CachedProcess(const ProcessStartInfo& i) : startInfo(i) {}
		virtual u32 GetExitCode() override { return 0; }
		virtual bool HasExited() override { return true; }
		virtual bool WaitForExit(u32 millisecondsTimeout) override{ return true; }
		virtual const ProcessStartInfo& GetStartInfo() const override { return startInfo; }
		virtual const Vector<ProcessLogLine>& GetLogLines() const override { return logLines; }
		virtual const Vector<u8>& GetTrackedInputs() const override { static Vector<u8> v; return v;}
		virtual const Vector<u8>& GetTrackedOutputs() const override { static Vector<u8> v; return v;}
		virtual bool IsRemote() const override { return false; }
		virtual ProcessExecutionType GetExecutionType() const override { return ProcessExecutionType_FromCache; }
		ProcessStartInfoHolder startInfo;
		Vector<ProcessLogLine> logLines;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void SchedulerCreateInfo::Apply(const Config& config)
	{
		if (const ConfigTable* table = config.GetTable(TC("Scheduler")))
		{
			table->GetValueAsBool(enableProcessReuse, TC("EnableProcessReuse"));
			table->GetValueAsBool(forceRemote, TC("ForceRemote"));
			table->GetValueAsBool(forceNative, TC("ForceNative"));
			table->GetValueAsU32(maxLocalProcessors, TC("MaxLocalProcessors"));
			table->GetValueAsU32(maxParallelCacheQueries, TC("MaxParallelCacheQueries"));
			table->GetValueAsBool(useThreadToExitRemoteProcess, TC("UseThreadToExitRemoteProcess"));
			table->GetValueAsU32(maxRacingPercent, TC("MaxRacingPercent"));

			table->GetValueAsBool(memWatchdog, TC("MemTrack")); // Legacy

			table->GetValueAsBool(memWatchdog, TC("MemWatchdog"));

			table->GetValueAsBool(writeTrackedInputs, TC("WriteTrackedInputs"));
			table->GetValueAsBool(deferRemoteCapableProcesses, TC("DeferRemoteCapableProcesses"));
			table->GetValueAsBool(enableSpeculativeCacheTest, TC("EnableSpeculativeCacheTest"));
			table->GetValueAsBool(traceDetails, TC("TraceDetails"));
			table->GetValueAsBool(leakMemoryAtShutdown, TC("LeakMemoryAtShutdown"));

			u32 temp = 0;
			if (table->GetValueAsU32(temp, TC("MemTracingLevel")))
				memTracingLevel = u8(temp);
			if (table->GetValueAsU32(temp, TC("MemStartWaitPercent")))
				memStartWaitPercent = u8(temp);
			if (table->GetValueAsU32(temp, TC("MemStartKillPercent")))
				memStartKillPercent = u8(temp);
			table->GetValueAsU32(memPagefileActivityThreshold, TC("MemPagefileActivityThreshold"));
		}
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct Scheduler::ProcessLink
	{
		ProcessLink* next;
		u32 entryIndex;

		static void DeleteAll(ProcessLink* first)
		{
			ProcessLink* it = first;
			while (it)
			{
				ProcessLink* next = it->next;
				delete it;
				it = next;
			}
		}

		template<typename Func>
		static void Traverse(ProcessLink* first, const Func& func)
		{
			for (ProcessLink* it = first; it; it = it->next)
				func(it->entryIndex);
		}
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	Scheduler::Scheduler(const SchedulerCreateInfo& info)
	:	m_session(info.session)
	,	m_maxLocalProcessors(info.maxLocalProcessors != ~0u ? info.maxLocalProcessors : GetLogicalProcessorCount())
	,	m_maxRacingWeight(float(info.maxRacingPercent)*float(m_maxLocalProcessors)/100.0f)
	,	m_maxParallelCacheQueries(Max(info.maxParallelCacheQueries, 4u))
	,	m_updateThreadWakeup(EventResetType_Auto)
	,	m_enableProcessReuse(info.enableProcessReuse)
	,	m_forceRemote(info.forceRemote)
	,	m_forceNative(info.forceNative)
	,	m_useThreadToExitRemoteProcess(info.useThreadToExitRemoteProcess)
	,	m_writeTrackedInputs(info.writeTrackedInputs)
	,	m_deferRemoteCapableProcesses(info.deferRemoteCapableProcesses)
	,	m_enableSpeculativeCacheTest(info.enableSpeculativeCacheTest)
	,	m_leakMemoryAtShutdown(info.leakMemoryAtShutdown)
	,	m_traceDetails(info.traceDetails)
	,	m_processConfigs(info.processConfigs)
	,	m_writeToCache(info.writeToCache && info.cacheClientCount)
	{
		m_cacheClients.insert(m_cacheClients.end(), info.cacheClients, info.cacheClients + info.cacheClientCount);
		m_session.RegisterGetNextProcess([this](Process& process, NextProcessInfo& outNextProcess, u32 prevExitCode, BinaryReader& prevStats, u32 timeoutMs, bool& outShouldExit)
			{
				if (HandleGetNextProcess(process, outNextProcess, prevExitCode, prevStats, timeoutMs))
					return true;
				outShouldExit = m_cancelled;
				return false;
			});

		if (info.memWatchdog && !IsLinux && !IsRunningWine()) // Only implemented for normal windows for now
		{
			m_memWatchdog = true;
			m_memTracingLevel = info.memTracingLevel;
			m_memStartWaitPercent = info.memStartWaitPercent;
			m_memStartKillPercent = info.memStartKillPercent;
			m_memPagefileActivityThreshold = info.memPagefileActivityThreshold;
		}

		if (m_writeTrackedInputs)
			DirectoryCache().CreateDirectory(m_session.GetLogger(), StringBuffer<>(m_session.GetSessionDir()).Append(TCV("inputs")).data);

		m_session.SetOuterScheduler(this);

		if (m_maxRacingWeight != 0)
		{
			if (!m_session.AllowKeepFilesInMemory())
			{
				m_maxRacingWeight = 0;
				m_session.GetLogger().Warning(TC("Session has keepFilesInMemoryDisabled. Racing will not work and has been disabled"));
			}
			if (m_forceRemote) // We don't want to race if we are also forcing remote
				m_maxRacingWeight = 0;
		}
	}

	Scheduler::~Scheduler()
	{
		Stop();
		for (auto rt : m_rootPaths)
			delete rt;
	}

	void Scheduler::Start()
	{
		m_session.SetRemoteProcessReturnedEvent([this](Process& process) { ProcessReturned(process, false); });
		m_session.SetRemoteProcessSlotAvailableEvent([this](bool isCrossArchitecture, bool isCrossPlatform) { RemoteSlotAvailable(isCrossArchitecture, isCrossPlatform); });

		if (!m_cacheClients.empty())
			m_cacheWorkManager = new WorkManagerImpl(m_maxParallelCacheQueries, TC("UbaSchedCach"));

		if (m_memWatchdog)
		{
			LoadMemTrackTable();
			m_session.SetNativeProcessCreatedFunc([this](ProcessImpl& process) { NativeProcessCreated(process); });
			m_memThreadEvent.Create(EventResetType_Manual);
			m_memThread.Start([this]() { ThreadMemoryCheckLoop();  return 0; }, TC("UbaMemTrackLoop"));

			// Make sure we have an estimate before starting to spawn processes
			while (!m_memThread.Wait(1) && !m_memEstimatedUsage)
				;
		}
		else if (m_maxRacingWeight != 0)
		{
			m_memThreadEvent.Create(EventResetType_Manual);
			m_memThread.Start([this]() { ThreadMemoryCheckLoop();  return 0; }, TC("UbaMemTrackLoop"));
		}

		m_loop = true;
		m_updateThread.Start([this]() { ThreadLoop(); return 0; }, TC("UbaSchedLoop"));
	}

	void Scheduler::Stop()
	{
		m_loop = false;
		m_cancelled = true;

		m_updateThreadWakeup.Set();
		m_updateThread.Wait();
		if (m_cacheWorkManager)
		{
			m_cacheWorkManager->FlushWork(60*1000);
			delete m_cacheWorkManager;
			m_cacheWorkManager = nullptr;
		}
		m_session.WaitOnAllTasks();
		SkipAllQueued();

		if (m_session.GetRemoteTraceEnabled())
		{
			m_session.DisableRemoteExecution();
			m_session.WaitOnAllClients();
		}

		m_session.SetRemoteProcessReturnedEvent({});
		m_session.SetRemoteProcessSlotAvailableEvent({});

		if (m_memWatchdog)
		{
			m_session.SetNativeProcessCreatedFunc({});
			m_memThreadEvent.Set();
			m_memThread.Wait();
			SaveMemTrackTable();
		}
		else if (m_maxRacingWeight != 0)
		{
			m_memThreadEvent.Set();
			m_memThread.Wait();
		}

		{
			SCOPED_FUTEX(m_processEntriesLock, lock);
			for (auto& entry : m_processEntries)
			{
				UBA_ASSERTF(entry.runStatus > RunStatus_Running, TC("Found process in queue/running state when stopping scheduler. (%u)"), u32(entry.runStatus));
				if (!m_leakMemoryAtShutdown)
				{
					ProcessLink::DeleteAll(entry.firstDependent);
					delete entry.info;
				}
			}
			m_processEntries.clear();
			m_session.SetOuterScheduler(nullptr);
		}
	}

	void Scheduler::Cancel()
	{
		m_cancelled = true;
		m_enableProcessReuse = false;
		SkipAllQueued();
		m_session.CancelAllProcesses();
	}

	Scheduler::ProcessEntry Scheduler::CreateProcessEntry(const EnqueueProcessInfo& info)
	{
		u8* ki = nullptr;
		if (info.knownInputsCount)
		{
			ki = new u8[info.knownInputsBytes];
			memcpy(ki, info.knownInputs, info.knownInputsBytes);
		}
		u8* ko = nullptr;
		if (info.knownOutputsCount)
		{
			ko = new u8[info.knownOutputsBytes];
			memcpy(ko, info.knownOutputs, info.knownOutputsBytes);
		}

		auto info2 = new ProcessStartInfo2(info.info, ki, info.knownInputsCount, ko, info.knownOutputsCount);
		info2->Expand();
		info2->weight = info.weight;
		info2->cacheBucketId = info.cacheBucketId;
		info2->memoryGroupId = info.memoryGroupId;
		info2->skipAsSuccessWhenScheduled = info.skipAsSuccessWhenScheduled;
		info2->predictedMemoryUsage = info.predictedMemoryUsage;

		const ApplicationRules* rules = m_session.GetRules(*info2);
		info2->rules = rules;

		bool useCache = info.cacheBucketId && !m_cacheClients.empty() && rules->IsCacheable();
		bool canDetour = info.canDetour;
		bool canExecuteRemotely = info.canExecuteRemotely && info.canDetour;
		bool canCrossArchitecture = info.canCrossArchitecture;
		bool canCrossPlatform = info.canCrossPlatform;

		info2->trackInputs |= m_writeTrackedInputs;

		if (m_processConfigs)
		{
			auto name = info2->application;
			if (auto lastSeparator = TStrrchr(name, PathSeparator))
				name = lastSeparator + 1;
			StringBuffer<128> lower(name);
			lower.MakeLower();
			lower.Replace('.', '_');
			if (const ConfigTable* processConfig = m_processConfigs->GetTable(lower.data))
			{
				processConfig->GetValueAsBool(canExecuteRemotely, TC("CanExecuteRemotely"));
				processConfig->GetValueAsBool(canDetour, TC("CanDetour"));
			}
		}

		auto GetCapabilityCategory = [](bool canExecuteRemotely, bool canCrossArchitecture, bool canCrossPlatform) -> Category
		{
			if (!canExecuteRemotely) return Category_Local;
			if (canCrossArchitecture && canCrossPlatform) return Category_RemoteCrossArchAndPlatform;
			if (canCrossArchitecture) return Category_RemoteCrossArch;
			if (canCrossPlatform) return Category_RemoteCrossPlatform;
			return Category_Remote;
		};


		ProcessEntry entry;
		entry.info = info2;
		entry.firstDependent = nullptr;
		entry.dependencyCount = 0;
		entry.speculativeDependencyCount = 0;
		entry.runStatus = RunStatus_QueuedForRun;
		entry.cacheStatus = useCache ? CacheStatus_QueuedForTest : CacheStatus_Failed;
		entry.statsState = StatsState_None;
		entry.category = GetCapabilityCategory(canExecuteRemotely, canCrossArchitecture, canCrossPlatform);
		entry.canDetour = canDetour;
		entry.shouldSkip = false;
		return entry;
	}

	Scheduler::ProcessQueue& Scheduler::GetReadyToRunQueue(ProcessEntry& entry)
	{
		if (!entry.canDetour || entry.category == Category_Local)
			return m_readyToRunLocal;
		switch (entry.category)
		{
		case Category_RemoteCrossArchAndPlatform:
			return m_readyToRunCrossArchAndPlatform;
		case Category_RemoteCrossPlatform:
			return m_readyToRunCrossPlatform;
		case Category_RemoteCrossArch:
			return m_readyToRunCrossArchitecture;
		default:
			return m_readyToRunRemote;
		}
	}

	void Scheduler::AddToQueue(ProcessQueue& queue, u32 entryIndex)
	{
		SCOPED_FUTEX(queue.lock, l);
		UBA_FOR_ASSERT(auto res =) queue.insert(entryIndex);
		UBA_ASSERT(res.second);
	}

	u32 Scheduler::PopFromQueue(ProcessQueue& queue)
	{
		SCOPED_FUTEX(queue.lock, l);
		if (queue.empty())
			return ~0u;
		auto it = queue.begin();
		u32 entryIndex = *it;
		queue.erase(it);
		return entryIndex;
	}

	Scheduler::ProcessEntry* Scheduler::PeekFromQueue(ProcessQueue& queue)
	{
		SCOPED_FUTEX(queue.lock, l);
		if (queue.empty())
			return nullptr;
		int idx = *queue.begin();
		l.Leave();
		return &m_processEntries[idx];
	}

	void Scheduler::EraseFromQueue(ProcessQueue& queue, u32 entryIndex)
	{
		SCOPED_FUTEX(queue.lock, l);
		queue.erase(entryIndex);
	}

	void Scheduler::ClearQueue(ProcessQueue& queue)
	{
		SCOPED_FUTEX(queue.lock, l);
		queue.clear();
	}

	u32 Scheduler::GetQueueCount(ProcessQueue& queue)
	{
		SCOPED_FUTEX_READ(queue.lock, l);
		return u32(queue.size());
	}

	void Scheduler::SetupDependents(ProcessEntry& entry, u32 entryIndex, const EnqueueProcessInfo& info)
	{
		// Go through all the processes and see if we have dependencies and if we should just skip this
		u32 dependencyCount = 0;
		u32 speculativeDependencyCount = 0;
		for (const u32* i=info.dependencies, *e=i+info.dependencyCount; i!=e; ++i)
		{
			auto& dependency = m_processEntries[*i];
			if (dependency.runStatus == RunStatus_Success)
				continue;
			if (dependency.runStatus == RunStatus_Failed || dependency.runStatus == RunStatus_Skipped || m_cancelled)
			{
				entry.cacheStatus = CacheStatus_Failed;
				AddToQueue(m_readyToSkip, entryIndex);
				return; // Not counted in stats — will be skipped immediately
			}
			++dependencyCount;
			// Speculative count only includes deps that haven't yet confirmed a cache hit.
			// Deps at QueuedForDownload or beyond already fired UpdateSpeculativeDependentsOnCacheHit.
			if (dependency.cacheStatus < CacheStatus_QueuedForDownload)
				++speculativeDependencyCount;
		}

		float weight = entry.info->weight;

		if (!dependencyCount)
		{
			if (entry.cacheStatus == CacheStatus_QueuedForTest)
			{
				// Cache outcome not yet known — enters pending state until miss confirmed
				TransitionStats(entry, StatsState_Pending, weight);
				AddToQueue(m_readyToTestCache, entryIndex);
			}
			else
			{
				// No cache query — ready immediately
				TransitionStats(entry, StatsState_Ready, weight);
				AddToQueue(GetReadyToRunQueue(entry), entryIndex);
			}
			return;
		}

		// Has unsatisfied deps — process enters the left state
		TransitionStats(entry, StatsState_Pending, weight);
		entry.dependencyCount = dependencyCount;
		entry.speculativeDependencyCount = speculativeDependencyCount;

		for (const u32* i=info.dependencies, *e=i+info.dependencyCount; i!=e; ++i)
		{
			auto& dependency = m_processEntries[*i];
			if (dependency.runStatus > RunStatus_Running)
				continue;
			dependency.firstDependent = new ProcessLink { dependency.firstDependent, entryIndex };
		}

		// If all deps already confirmed cache hits, speculatively test now even though deps are still downloading.
		if (m_enableSpeculativeCacheTest && !speculativeDependencyCount && entry.cacheStatus == CacheStatus_QueuedForTest)
		{
			AddToQueue(m_readyToTestCache, entryIndex);
			entry.cacheStatus = CacheStatus_QueuedForTestSpeculative;
		}
	}

	void Scheduler::UpdateDependents(RunStatus newRunStatus, ProcessLink* firstDependent, bool hadCacheHit)
	{
		bool shouldSkip = newRunStatus == RunStatus_Failed || newRunStatus == RunStatus_Skipped;
		for (ProcessLink* it = firstDependent; it; it = it->next)
		{
			u32 dependentIndex = it->entryIndex;
			bool specHitZero = false;

			SCOPED_FUTEX_READ(m_processEntriesLock, lock);
			auto& dependent = m_processEntries[dependentIndex];

			if (shouldSkip)
				std::atomic_ref<bool>(dependent.shouldSkip).store(true, std::memory_order_release);

			// Speculative count: count may already be 0 (entry had no speculative deps, or
			// they all already completed). CAS-decrement-if-positive avoids underflow.
			if (m_enableSpeculativeCacheTest && !hadCacheHit)
			{
				std::atomic_ref<u32> specRef(dependent.speculativeDependencyCount);
				u32 expected = specRef.load(std::memory_order_relaxed);
				while (expected > 0)
				{
					if (!specRef.compare_exchange_weak(expected, expected - 1, std::memory_order_acq_rel, std::memory_order_relaxed))
						continue;
					if (expected == 1)
						specHitZero = true;
					break;
					// CAS failed; expected updated, retry.
				}
			}

			// Main count: state machine guarantees > 0 here (each dep's setup gave us a slot).
			u32 oldCount = std::atomic_ref<u32>(dependent.dependencyCount).fetch_sub(1, std::memory_order_acq_rel);
			bool mainHitZero = (oldCount == 1);
				
			if (!specHitZero && !mainHitZero)
				continue; // common case: no transition.

			// One or both counters hit zero — do the state transition under READ lock too. Atomicity for
			// THIS dependent is provided by the state-machine guarantee that only one thread brought the
			// counter to zero (the unique handler). Stats are atomic. AddToQueue uses per-queue lock.
			// cacheStatus / runStatus are plain fields, but only this thread will mutate them at this
			// state-machine point — other paths handle different states (Running, Testing, Downloading)
			// and can't be concurrently mutating this dependent.
			bool dependentShouldSkip = std::atomic_ref<bool>(dependent.shouldSkip).load(std::memory_order_acquire);

			if (specHitZero && dependent.cacheStatus == CacheStatus_QueuedForTest && !dependentShouldSkip && !shouldSkip)
			{
				AddToQueue(m_readyToTestCache, dependentIndex);
				dependent.cacheStatus = CacheStatus_QueuedForTestSpeculative;
			}

			if (!mainHitZero)
				continue;

			float weight = dependent.info ? dependent.info->weight : 0.0f;

			if (!shouldSkip && !dependentShouldSkip && newRunStatus == RunStatus_Success)
			{
				switch (dependent.cacheStatus)
				{
				case CacheStatus_QueuedForTest:
					AddToQueue(m_readyToTestCache, dependentIndex); // stays pending
					break;
				case CacheStatus_QueuedForTestSpeculative:
				case CacheStatus_Testing:
				case CacheStatus_QueuedForDownload:
				case CacheStatus_Downloading:
					break; // in-flight, stays pending
				case CacheStatus_PendingCompletion:
					AddToQueue(m_readyToCompleteFromCache, dependentIndex); // stays pending
					break;
				default: // CacheStatus_Suspended (non-cacheable) or CacheStatus_Failed (cache miss confirmed)
					TransitionStats(dependent, StatsState_Ready, weight); // pending → ready
					AddToQueue(GetReadyToRunQueue(dependent), dependentIndex);
					break;
				}
				continue;
			}

			// Dep failed/skipped, or this dependent itself should be skipped.
			if (dependent.cacheStatus == CacheStatus_Testing
				|| dependent.cacheStatus == CacheStatus_QueuedForDownload
				|| dependent.cacheStatus == CacheStatus_Downloading)
			{
				TransitionStats(dependent, StatsState_None, weight); // worker will handle skip
				std::atomic_ref<bool>(dependent.shouldSkip).store(true, std::memory_order_release);
			}
			else if (dependent.cacheStatus == CacheStatus_QueuedForTestSpeculative)
			{
				TransitionStats(dependent, StatsState_None, weight);
				EraseFromQueue(m_readyToTestCache, dependentIndex);
				dependent.cacheStatus = CacheStatus_Failed;
				AddToQueue(m_readyToSkip, dependentIndex);
			}
			else if (dependent.runStatus == RunStatus_QueuedForRun)
			{
				TransitionStats(dependent, StatsState_None, weight);
				dependent.cacheStatus = CacheStatus_Failed;
				AddToQueue(m_readyToSkip, dependentIndex);
			}
			else if (dependent.cacheStatus == CacheStatus_PendingCompletion)
			{
				TransitionStats(dependent, StatsState_None, weight);
				dependent.runStatus = RunStatus_Skipped;
				AddToQueue(m_readyToSkip, dependentIndex);
			}
		}
	}

	void Scheduler::UpdateSpeculativeDependentsOnCacheHit(ProcessLink* firstDependent)
	{
		// Called under m_processEntriesLock when an entry confirms a cache hit (-> QueuedForDownload).
		// Decrements speculativeDependencyCount for each dependent; promotes to cache test when it hits zero.
		// Caller holds m_processEntriesLock (write). Use atomic_ref on speculativeDependencyCount /
		// shouldSkip for consistency with UpdateDependents — both touch these fields, and even
		// though they're mutually exclusive via the lock, mixed atomic + plain access to the same
		// memory is technically UB per the C++ standard.
		for (ProcessLink* it = firstDependent; it; it = it->next)
		{
			u32 dependentIndex = it->entryIndex;
			auto& dependent = m_processEntries[dependentIndex];
			if (dependent.runStatus >= RunStatus_Success)
				continue; // already terminal

			std::atomic_ref<u32> specRef(dependent.speculativeDependencyCount);
			u32 expected = specRef.load(std::memory_order_relaxed);
			if (expected == 0)
				continue; // already at zero (already promoted, or never had speculative deps)

			bool hitZero = false;
			while (expected > 0)
				if (specRef.compare_exchange_weak(expected, expected - 1, std::memory_order_acq_rel, std::memory_order_relaxed))
				{
					if (expected == 1)
						hitZero = true;
					break;
				}

			if (m_enableSpeculativeCacheTest && hitZero && dependent.cacheStatus == CacheStatus_QueuedForTest && !std::atomic_ref<bool>(dependent.shouldSkip).load(std::memory_order_acquire))
			{
				AddToQueue(m_readyToTestCache, dependentIndex);
				dependent.cacheStatus = CacheStatus_QueuedForTestSpeculative;
			}
		}
	}

	void Scheduler::SkipAllQueued()
	{
		Vector<ProcessStartInfo2*> skipped;
		SCOPED_FUTEX(m_processEntriesLock, lock);
		for (auto& entry : m_processEntries)
		{
			if (entry.runStatus != RunStatus_QueuedForRun)
				continue;

			if (entry.cacheStatus == CacheStatus_Testing
				|| entry.cacheStatus == CacheStatus_Downloading
				|| entry.cacheStatus == CacheStatus_QueuedForDownload)
			{
				// Worker thread is in-flight. Set shouldSkip so the worker detects it on completion.
				entry.shouldSkip = true;
				continue;
			}

			if (entry.cacheStatus == CacheStatus_PendingCompletion)
			{
				UBA_ASSERT(entry.info->cacheInfo);
				delete entry.info->cacheInfo;
				entry.info->cacheInfo = nullptr;
			}

			TransitionStats(entry, StatsState_None, entry.info ? entry.info->weight : 0.0f);
			entry.cacheStatus = CacheStatus_Failed;
			entry.runStatus = RunStatus_Skipped;
			// UpdateDependentsNoLock(RunStatus_Skipped, firstDependent); // No need to do this, all following entries are cancelled
			skipped.push_back(entry.info);
		}
		ClearQueue(m_readyToTestCache);
		ClearQueue(m_readyToDownloadCache);
		ClearQueue(m_readyToRunLocal);
		ClearQueue(m_readyToRunRemote);
		ClearQueue(m_readyToRunCrossArchitecture);
		ClearQueue(m_readyToRunCrossPlatform);
		ClearQueue(m_readyToRunCrossArchAndPlatform);
		ClearQueue(m_readyToSkip);
		ClearQueue(m_readyToCompleteFromCache);

		lock.Leave();
		for (auto pi : skipped)
			SkipProcess(*pi);
	}

	void Scheduler::SetMaxLocalProcessors(u32 maxLocalProcessors)
	{
		m_maxLocalProcessors = maxLocalProcessors;
		m_updateThreadWakeup.Set();
	}

	void Scheduler::SetAllowDisableRemoteExecution(bool allow)
	{
		m_allowDisableRemoteExecution = allow;
	}

	u32 Scheduler::EnqueueProcess(const EnqueueProcessInfo& info)
	{
		ProcessEntry entry = CreateProcessEntry(info);

		SCOPED_FUTEX(m_processEntriesLock, lock);
		u32 index = u32(m_processEntries.size());
		SetupDependents(m_processEntries.emplace_back(entry), index, info);
		lock.Leave();
		++m_totalProcesses;
		UpdateQueueCounter();
		m_updateThreadWakeup.Set();
		return index;
	}

	u32 Scheduler::EnqueueSuspendedProcess()
	{
		SCOPED_FUTEX(m_processEntriesLock, lock);
		UBA_ASSERT(!m_cancelled);
		u32 index = u32(m_processEntries.size());
		auto& entry = m_processEntries.emplace_back();
		entry.dependencyCount = 0;
		entry.speculativeDependencyCount = 0;
		entry.firstDependent = nullptr;
		entry.runStatus = RunStatus_Suspended;
		entry.cacheStatus = CacheStatus_Suspended;
		++m_totalProcesses;
		UpdateQueueCounter();
		return index;
	}

	void Scheduler::ResumeQueuedProcess(u32 index, const EnqueueProcessInfo& info)
	{
		auto newEntry = CreateProcessEntry(info);
		SCOPED_FUTEX(m_processEntriesLock, lock);
		auto& entry = m_processEntries[index];
		ProcessLink* firstDependent = entry.firstDependent;
		UBA_ASSERT(entry.dependencyCount == 0);
		entry = newEntry;
		entry.firstDependent = firstDependent;

		if (m_cancelled)
		{
			entry.runStatus = RunStatus_Skipped;
			entry.firstDependent = nullptr;
			auto pi = entry.info;
			entry.info = nullptr;
			lock.Leave();
			UpdateDependents(RunStatus_Skipped, firstDependent, false); // walk outside lock; per-dep re-acquire
			ProcessLink::DeleteAll(firstDependent);
			SkipProcess(*pi);
			delete pi;
			return;
		}

		SetupDependents(entry, index, info);
		m_updateThreadWakeup.Set();
	}

	u32 Scheduler::GetTotalProcesses()
	{
		return m_totalProcesses;
	}

	void Scheduler::GetStats(u32& outQueued, u32& outActiveLocal, u32& outActiveRemote, u32& outFinished)
	{
		// Stats are atomic — slightly inconsistent snapshot acceptable for telemetry. Read lock
		// kept for m_finishedProcesses / m_totalProcesses coherence with each other.
		SCOPED_FUTEX_READ(m_processEntriesLock, lock);
		outActiveLocal  = AtomicStatsLoad(m_stats.runningLocalCount);
		outActiveRemote = AtomicStatsLoad(m_stats.runningRemoteCount) + AtomicStatsLoad(m_stats.runningRemoteCrossArchCount) + AtomicStatsLoad(m_stats.runningRemoteCrossPlatformCount) + AtomicStatsLoad(m_stats.runningRemoteCrossArchAndPlatformCount);
		outFinished     = m_finishedProcesses;
		outQueued       = m_totalProcesses - outFinished - outActiveLocal - outActiveRemote;
	}

	void Scheduler::GetResults(u32& outSucceeded, u32& outFailed, u32& outSkipped)
	{
		outSucceeded = 0;
		outFailed = 0;
		outSkipped = 0;
		SCOPED_FUTEX_READ(m_processEntriesLock, lock);
		for (auto& entry : m_processEntries)
		{
			outSucceeded += entry.runStatus == RunStatus_Success;
			outFailed += entry.runStatus == RunStatus_Failed;
			outSkipped += entry.runStatus == RunStatus_Skipped;
		}
	}

	bool Scheduler::IsEmpty()
	{
		SCOPED_FUTEX_READ(m_processEntriesLock, lock);
		bool isEmpty = m_processEntries.size() <= m_finishedProcesses;
		if (isEmpty)
			return true;
			
		#if UBA_DEBUG
		if (m_cancelled)
		{
			u32 index = 0;
			for (auto& entry : m_processEntries)
			{
				if (entry.runStatus != RunStatus_Skipped && entry.runStatus != RunStatus_Failed && entry.runStatus != RunStatus_Success)
				{
					m_session.GetLogger().Error(TC("Entry %u: Run: %u Cache: %u"), index, u32(entry.runStatus), u32(entry.cacheStatus));
					break;
				}
				++index;
			}
		}
		#endif
		
		return false;
	}

	void Scheduler::SetProcessFinishedCallback(const Function<void(const ProcessHandle&)>& processFinished)
	{
		m_processFinished = processFinished;
	}

	void Scheduler::SetPredictedMemoryCallback(const Function<u64(void* userData)>& predictedMemory)
	{
		m_predictedMemoryFunc = predictedMemory;
	}

	void Scheduler::GetStats(SchedulerStats& outStats)
	{
		// Stats fields are atomically updated; struct copy gives a per-field-atomic snapshot
		// (fields may be slightly inconsistent across each other — acceptable for telemetry).
		outStats = m_stats;
	}

	u32 Scheduler::GetProcessCountThatCanRunRemotelyNow()
	{
		if (m_session.IsRemoteExecutionDisabled())
			return 0;
		// Stats are atomic — no lock needed.
		return	AtomicStatsLoad(m_stats.activeRemoteCount) + AtomicStatsLoad(m_stats.activeRemoteCrossArchCount) + AtomicStatsLoad(m_stats.activeRemoteCrossPlatformCount) + AtomicStatsLoad(m_stats.activeRemoteCrossArchAndPlatformCount) +
				AtomicStatsLoad(m_stats.readyRemoteCount)  + AtomicStatsLoad(m_stats.readyRemoteCrossArchCount)  + AtomicStatsLoad(m_stats.readyRemoteCrossPlatformCount)  + AtomicStatsLoad(m_stats.readyRemoteCrossArchAndPlatformCount);
	}

	void Scheduler::GetProcessWeightThatCanRunRemotelyNow(float& outTotal, float& outCrossArchitecture, float& outCrossPlatform)
	{
		outTotal = 0;
		outCrossArchitecture = 0;
		outCrossPlatform = 0;
		if (m_session.IsRemoteExecutionDisabled())
			return;
		// Stats are atomic — no lock needed.
		float archAndPlatformWeight = AtomicStatsLoad(m_stats.activeRemoteCrossArchAndPlatformWeight) + AtomicStatsLoad(m_stats.readyRemoteCrossArchAndPlatformWeight);
		outCrossArchitecture = AtomicStatsLoad(m_stats.activeRemoteCrossArchWeight)     + AtomicStatsLoad(m_stats.readyRemoteCrossArchWeight);
		outCrossPlatform     = AtomicStatsLoad(m_stats.activeRemoteCrossPlatformWeight) + AtomicStatsLoad(m_stats.readyRemoteCrossPlatformWeight);
		outTotal             = AtomicStatsLoad(m_stats.activeRemoteWeight) + AtomicStatsLoad(m_stats.readyRemoteWeight) + outCrossArchitecture + outCrossPlatform
		                     + archAndPlatformWeight;
		outCrossArchitecture += archAndPlatformWeight;
		outCrossPlatform     += archAndPlatformWeight;
	}

	void Scheduler::ThreadLoop()
	{
		auto& trace = m_session.GetTrace();
		#define UBA_SCHEDULER_PROCESS_QUEUE(queue) u32 queue##Len = 0;
		UBA_SCHEDULER_PROCESS_QUEUES
		#undef UBA_SCHEDULER_PROCESS_QUEUE
		SchedulerStats lastStats = {};
		bool queuesChanged = true;
		const u32 startRow = 20;

		if (m_traceDetails)
		{
			u32 row = startRow;
			trace.StatusUpdate(row++, 1, TCV("Running"),  LogEntryType_Info);
			trace.StatusUpdate(row++, 1, TCV("Queues"),  LogEntryType_Info);
			trace.StatusUpdate(row, 1, TCV("Category"),  LogEntryType_Info);
			trace.StatusUpdate(row, 13, TCV("Pending"),  LogEntryType_Info);
			trace.StatusUpdate(row, 17, TCV("Ready"),  LogEntryType_Info);
			trace.StatusUpdate(row, 21, TCV("Active"),  LogEntryType_Info);
			row++;
			#define UBA_PROCESS_CATEGORY(cat) trace.StatusUpdate(row++, 1, TCV("  " #cat),  LogEntryType_Info);
			UBA_PROCESS_CATEGORIES
			#undef UBA_PROCESS_CATEGORY
		}
		while (m_loop)
		{
			if (!m_updateThreadWakeup.IsSet())
				break;

			m_session.FlushDeadProcesses(true);
			m_session.UpdateDirTableSnapshot(false);

			while (DrainCompleteFromCache())
				;

			while (RunCacheQuery())
				;

			while (RunQueuedProcess(true, false, false))
				;

			if (m_traceDetails)
			{
				// Telemetry — per-field atomic loads of stats accept slight inconsistency across fields.
				// Queue sizes use the per-queue lock via GetQueueCount.
				SchedulerStats stats = m_stats; // struct copy; fields atomically updated by writers (relaxed)
				#define UBA_SCHEDULER_PROCESS_QUEUE(queue) u32 queue##LenNew = GetQueueCount(m_readyTo##queue); queuesChanged |= queue##LenNew != queue##Len; queue##Len = queue##LenNew;
				UBA_SCHEDULER_PROCESS_QUEUES
				#undef UBA_SCHEDULER_PROCESS_QUEUE
				StringBuffer<> temp;
				if (queuesChanged)
				{
					queuesChanged = false;
					#define UBA_SCHEDULER_PROCESS_QUEUE(queue) temp.Append(TC("") #queue ": ").AppendValue(queue##Len).Append(TCV("  "));
					UBA_SCHEDULER_PROCESS_QUEUES
					#undef UBA_SCHEDULER_PROCESS_QUEUE
					trace.StatusUpdate(startRow+1, 6, temp, LogEntryType_Info);
				}

				if (memcmp(&stats, &lastStats, sizeof(stats)) != 0)
				{
					u32 row = startRow;
					lastStats = stats;

					temp.Clear();
					temp.Appendf(TC("TestCache: %u "), stats.activeCacheQueries - stats.activeCacheDownloads);
					temp.Appendf(TC("DownloadCache: %u "), stats.activeCacheDownloads);
					#define UBA_PROCESS_CATEGORY(cat) temp.Appendf(TC(#cat ": %u "), stats.running##cat##Count);
					UBA_PROCESS_CATEGORIES
					#undef UBA_PROCESS_CATEGORY
					trace.StatusUpdate(row,  6, temp, LogEntryType_Info);
					row += 3;

					auto traceValue = [&](u32 row, u32 col, u32 value) { temp.Clear().AppendValue(value); trace.StatusUpdate(row,  13 + col*4, temp, LogEntryType_Info); };
					#define UBA_PROCESS_CATEGORY(cat) traceValue(row, 0, stats.pending##cat##Count);traceValue(row, 1, stats.ready##cat##Count);traceValue(row, 2, stats.active##cat##Count); ++row;
					UBA_PROCESS_CATEGORIES
					#undef UBA_PROCESS_CATEGORY
				}
			}				

		}
	}

	bool Scheduler::ProcessReturned(Process& process, bool isLocal)
	{
		auto& si = process.GetStartInfo();

		if (!si.userData)
		{
			if (!isLocal)
				process.Cancel(); // This can happen when racing process won.. then remote process callback already called so userData is nullptr
			return false;
		}
		
		auto& ei = *(ExitProcessInfo*)si.userData;
		float processWeight = 0;
		u32 processIndex;

		++ei.refCount;
		auto eig = MakeGuard([&]() { UBA_ASSERT(ei.refCount); if (!--ei.refCount) delete &ei; });

		// We can end up here at the same time for both remote and racing process pointing to the same ExitProcessInfo
		// The last one of them needs to update the process entries lock, the first needs to early exit
		// Note that local process does not affect processCount until ProcessExit which is called in parallel between Cancel and WaitForExit

		// Remote processes can only end up here from disconnect or network message. so a remote process can't finish and return at the same time

		// Local processes can only end up here from race/memtrack killing them. They can also not finish at the same time since the Cancel call
		// will fail if process is about to finish.


		if (isLocal)
		{
			processWeight = ei.weight;
			processIndex = ei.processIndex; // Must be fetched before cancel

			SCOPED_CRITICAL_SECTION(m_wasReturnedLock, lock);
			if (ei.hasCheckedWasReturned)
				return false;
			if (!process.Cancel())
				return false;
			ei.wasReturned = true;
		}
		else
		{
			processIndex = ei.processIndex; // Must be fetched before cancel
			ei.wasReturned = true;

			bool res = process.Cancel(); // Cancel will call ProcessExited
			UBA_ASSERTF(res, TC("Failed to cancel remote process?"));(void)res;
		}
		
		if (processIndex == ~0u)
		{
			UBA_ASSERT(!isLocal);
			return false;
		}

		if (isLocal)
		{
			if (!process.WaitForExit(8*60*60*1000)) // 8 hours, hopefully someone will come and get me so I can debug :)
			{
				m_updateThreadWakeup.Set();
				return m_session.GetLogger().Error(TC("Took more than 8 hours to wait for process to exit (%s)"), si.GetDescription());
			}
		}

		// We let the process that matches ei.runningCategory do the return of process
		// .. if remote process cancels first it will give over ownership to local race process
		// .. if race process cancels first ei.runningCategory will still be remote and remote will clean up
		if ((ei.runningCategory == Category_Local) != isLocal)
			return true;

		UBA_ASSERT(ei.processCount == 0);

		SCOPED_FUTEX(m_processEntriesLock, lock);

		if (isLocal)
			m_activeLocalProcessWeight -= processWeight;

		ProcessEntry& entry = m_processEntries[processIndex];
		if (entry.runStatus != RunStatus_Running)
			return m_session.GetLogger().Error(TC("This should not happen (%u)"), entry.runStatus);

		StatsRunning(m_stats, ei.runningCategory, -1, ei.weight);

		ProcessLink* firstDependent = entry.firstDependent;
		if (!m_cancelled)
		{
			entry.runStatus = RunStatus_QueuedForRun;
			TransitionStats(entry, StatsState_Ready, ei.weight);
			AddToQueue(GetReadyToRunQueue(entry), processIndex);
			lock.Leave();
		}
		else
		{
			m_session.GetLogger().Error(TC("Here to check this path works"));
			entry.runStatus = RunStatus_Skipped;
			TransitionStats(entry, StatsState_None, ei.weight);
			ProcessHandle ph(&process);
			FinishProcess(ph);

			entry.firstDependent = nullptr;
			lock.Leave();
			UpdateDependents(RunStatus_Skipped, firstDependent, false); // walk outside lock; per-dep re-acquire
			ProcessLink::DeleteAll(firstDependent);
		}

		m_updateThreadWakeup.Set();
		return true;
	}

	void Scheduler::HandleCacheMissed(u32 processIndex)
	{
		SCOPED_FUTEX(m_processEntriesLock, lock);
		auto& entry = m_processEntries[processIndex];
		UBA_ASSERTF((entry.cacheStatus == CacheStatus_Testing || entry.cacheStatus == CacheStatus_Downloading) && entry.runStatus == RunStatus_QueuedForRun, TC("Unexpected entry state. Run: %u Cache: %u"), u32(entry.runStatus), u32(entry.cacheStatus));

		if (entry.cacheStatus == CacheStatus_Downloading)
			AtomicStatsAdd(m_stats.activeCacheDownloads, -1);

		AtomicStatsAdd(m_stats.activeCacheQueries, -1);

		// Capture before overwrite: Testing->Failed = unconfirmed (must decrement speculative);
		// Downloading→Failed = confirmed (UpdateSpeculativeDependentsOnCacheHit already ran).
		bool hadCacheHit = (entry.cacheStatus == CacheStatus_Downloading);

		entry.cacheStatus = CacheStatus_Failed;

		if (m_cancelled || entry.shouldSkip)
		{
			entry.runStatus = RunStatus_Skipped;
			auto pi = entry.info;
			TransitionStats(entry, StatsState_None, pi->weight);
			entry.info = nullptr;
			ProcessLink* firstDependent = entry.firstDependent;
			entry.firstDependent = nullptr;
			lock.Leave();
			UpdateDependents(RunStatus_Skipped, firstDependent, hadCacheHit); // walk outside lock; per-dep re-acquire
			ProcessLink::DeleteAll(firstDependent);
			SkipProcess(*pi);
			delete pi;
			return;
		}

		UBA_ASSERT(entry.runStatus == RunStatus_QueuedForRun);

		// Only add to run queue if all dependencies have completed.
		// With speculative cache testing, dependencyCount can be > 0 here; UpdateDependents
		// will promote to the run queue when the last dep completes (cacheStatus == Failed case).
		if (entry.dependencyCount == 0)
		{
			TransitionStats(entry, StatsState_Ready, entry.info->weight); // pending → ready (cache miss confirmed)
			AddToQueue(GetReadyToRunQueue(entry), processIndex);
		}

		lock.Leave();

		m_updateThreadWakeup.Set();
	}

	void Scheduler::RemoteSlotAvailable(bool isCrossArchitecture, bool isCrossPlatform)
	{
		if (RunQueuedProcess(false, isCrossArchitecture, isCrossPlatform))
			return;
		if (!m_allowDisableRemoteExecution)
			return;
		if (m_session.IsRemoteExecutionDisabled())
			return;
		// Stats are atomic — no lock needed.
		u32 count =
				AtomicStatsLoad(m_stats.activeRemoteCount)                        + AtomicStatsLoad(m_stats.readyRemoteCount)                        + AtomicStatsLoad(m_stats.pendingRemoteCount) +
				AtomicStatsLoad(m_stats.activeRemoteCrossArchCount)               + AtomicStatsLoad(m_stats.readyRemoteCrossArchCount)               + AtomicStatsLoad(m_stats.pendingRemoteCrossArchCount) +
				AtomicStatsLoad(m_stats.activeRemoteCrossPlatformCount)           + AtomicStatsLoad(m_stats.readyRemoteCrossPlatformCount)           + AtomicStatsLoad(m_stats.pendingRemoteCrossPlatformCount) +
				AtomicStatsLoad(m_stats.activeRemoteCrossArchAndPlatformCount)    + AtomicStatsLoad(m_stats.readyRemoteCrossArchAndPlatformCount)    + AtomicStatsLoad(m_stats.pendingRemoteCrossArchAndPlatformCount);
		if (count < m_maxLocalProcessors)
			m_session.DisableRemoteExecution();
		else
			m_session.SetMaxRemoteProcessCount(count);
	}

	void Scheduler::ProcessExited(ExitProcessInfo* info, const ProcessHandle& handle)
	{
		auto ig = MakeGuard([info]() { UBA_ASSERT(info->refCount);if (!--info->refCount) delete info; });

		auto si = info->startInfo;

		if (!handle.IsRemote() && si)
			LocalProcessExit(*si, handle);

		u32 processCount = info->processCount--;

		if (processCount == 2)
		{
			UBA_ASSERT(handle.GetExitCode() == ProcessCancelExitCode);

			UBA_ASSERT(si);
			float weight = info->weight;

			if (handle.IsRemote()) // Racing process won. Let's make it a "real" local process
			{
				// handle.HasExited() returns true before we get in here.. so might not found racing process
				SCOPED_FUTEX(m_racingProcessesLock, lock);
				for (auto i=m_racingProcesses.begin(),e=m_racingProcesses.end(); i!=e; ++i)
				{
					auto& rp = *(ProcessImpl*)i->m_process;
					if (rp.m_startInfo.userData != info)
						continue;
					m_racingProcesses.erase(i);
					break;
				}

				// Turn it into real process adding weight
				const Category prevRunningCategory = info->runningCategory;
				info->runningCategory = Category_Local;

				m_activeLocalProcessWeight += weight;

				SCOPED_FUTEX(m_processEntriesLock, entriesLock);
				StatsRunning(m_stats, prevRunningCategory, -1, weight);
				StatsRunning(m_stats, Category_Local, 1, weight);
			}

			m_activeRacingProcessWeight -= weight;

			SCOPED_CRITICAL_SECTION(m_wasReturnedLock, lock);
			info->wasReturned = false;
			info->hasCheckedWasReturned = false;
			return;
		}

		{
			SCOPED_CRITICAL_SECTION(m_wasReturnedLock, lock);
			if (info->wasReturned)
				return;
			info->hasCheckedWasReturned = true;
		}

		if (!si) // Can be a remote process or a failed reuse
		{
			UBA_ASSERT(info->processIndex == ~0u);
			
			// Have to return the weight here if local. This means that this is a failed reuse and process has already been "exited" but we've kept process weight alive
			if (info->runningCategory == Category_Local)
			{
				m_activeLocalProcessWeight -= info->weight;
				UBA_ASSERTF(m_activeLocalProcessWeight >= 0, TC("Active process weight is below zero (%f)"), m_activeLocalProcessWeight.load());
				info->weight = 0;
				m_updateThreadWakeup.Set();
			}
			return;
		}

		if (!m_useThreadToExitRemoteProcess || info->runningCategory == Category_Local)
		{
			#if 0
			static int counter = 0;
			StringBuffer<> buf;
			buf.Appendf("/Users/henrik.karlsson/access/action%u.txt", counter++);
			FileAccessor f(m_session.GetLogger(), buf.data);
			if (f.CreateWrite())
			{
				Set<TString> files;
				auto& inputs = handle.m_process->GetTrackedInputs();
				BinaryReader reader(inputs.data(), 0, inputs.size());
				while (reader.GetLeft())
					files.insert(reader.ReadString());

				for (auto& input : files)
				{
					f.Write(input.data(), input.size());
					f.Write("\n", 1);
				}
				f.Close();
			}
			#endif

			ExitProcess(*info, *handle.m_process, handle.m_process->GetExitCode(), false, false);
			return;
		}

		// If process is remote we want to return as soon as possible to return worker thread
		ig.Cancel();

		m_finishedRemoteProcessThreads.emplace_back([this, handle = handle, info]()
			{
				ExitProcess(*info, *handle.m_process, handle.m_process->GetExitCode(), false, false);
				UBA_ASSERT(info->refCount);
				if (!--info->refCount)
					delete info;
				return 0;

			}, TC("UbaScheRemFin"));
	}

	u32 Scheduler::PopProcessNoLock(bool isLocal, bool isCrossArchitecture, bool isCrossPlatform, RunStatus& outPrevStatus, bool forReuse, bool& outShouldSleep, ProcessLink*& outSkipFirstDependent)
	{
		outShouldSleep = false;
		outSkipFirstDependent = nullptr;

		u32 skipEntry = PopFromQueue(m_readyToSkip);
		if (skipEntry != ~0u)
		{
			auto& entry = m_processEntries[skipEntry];
			outPrevStatus = entry.runStatus;
			entry.runStatus = RunStatus_Skipped;
			// Detach chain; caller walks it (and deletes) AFTER releasing m_processEntriesLock.
			outSkipFirstDependent = entry.firstDependent;
			entry.firstDependent = nullptr;
			return skipEntry;
		}

		if (m_cancelled)
			return ~0u;

		auto returnEntry = [&](u32 index)
			{
				if (index == ~0u)
					return index;
				auto& entry = m_processEntries[index];
				UBA_ASSERT(entry.cacheStatus == CacheStatus_Failed);
				outPrevStatus = entry.runStatus;
				UBA_ASSERT(!m_cancelled);
				entry.runStatus = RunStatus_Running;
				return index;
			};

		if (!isLocal)
		{
			if (isCrossArchitecture && isCrossPlatform)
				return returnEntry(PopFromQueue(m_readyToRunCrossArchAndPlatform));

			if (isCrossArchitecture)
			{
				u32 index = returnEntry(PopFromQueue(m_readyToRunCrossArchitecture));
				if (index != ~0u) return index;
				return returnEntry(PopFromQueue(m_readyToRunCrossArchAndPlatform));
			}

			if (isCrossPlatform)
			{
				u32 index = returnEntry(PopFromQueue(m_readyToRunCrossPlatform));
				if (index != ~0u) return index;
				return returnEntry(PopFromQueue(m_readyToRunCrossArchAndPlatform));
			}

			u32 index = returnEntry(PopFromQueue(m_readyToRunRemote));
			if (index != ~0u) return index;
			index = returnEntry(PopFromQueue(m_readyToRunCrossArchitecture));
			if (index != ~0u) return index;
			index = returnEntry(PopFromQueue(m_readyToRunCrossPlatform));
			if (index != ~0u) return index;
			return returnEntry(PopFromQueue(m_readyToRunCrossArchAndPlatform));
		}

		if (!m_maxLocalProcessors)
			return ~0u;

		// A candidate is only runnable if both its weight fits in the remaining budget and, when throttling,
		// its predicted memory usage keeps us under the memory threshold.
		auto getEntry = [&](float weightLeft, bool applyThrottle, bool& outMemoryBlocked)
			{
				outMemoryBlocked = false;
				auto canRun = [&](ProcessEntry* entry)
					{
						if (entry->info->weight > weightLeft)
							return false;

						UpdatePredictedMemoryUsage(*entry->info);

						u64 predictedMemoryUsage = entry->info->predictedMemoryUsage;
						if (IsWindows && m_memWatchdog && entry->info->memoryGroupId)
						{
							SCOPED_FUTEX_READ(m_memGroupLookupLock, statsLock);
							auto findIt = m_memGroupLookup.find(entry->info->memoryGroupId);
							if (findIt != m_memGroupLookup.end())
								predictedMemoryUsage = Max(predictedMemoryUsage, findIt->second.average);
							else if (auto dbIt = m_memGroupDb.find(entry->info->memoryGroupId); dbIt != m_memGroupDb.end())
								predictedMemoryUsage = Max(predictedMemoryUsage, dbIt->second.average);
						}

						if (applyThrottle && !LocalProcessMemoryOkToSpawn(AtomicStatsLoad(m_stats.runningLocalCount), predictedMemoryUsage))
						{
							outMemoryBlocked = true;
							return false;
						}
						return true;
					};

				if (ProcessEntry* entry = PeekFromQueue(m_readyToRunLocal)) // Check local only first
					if (canRun(entry))
						return PopFromQueue(m_readyToRunLocal);
				if (!m_forceRemote || (!m_hadMatchingSystemRequest && m_hadCrossSystemRequest)) // If force remote but we've had cross system requests we let local machine pop non-cross system queue
					if (ProcessEntry* entry = PeekFromQueue(m_readyToRunRemote))
						if (canRun(entry))
							return PopFromQueue(m_readyToRunRemote);
				if (m_forceRemote && (m_hadMatchingSystemRequest || m_hadCrossSystemRequest))
					return ~0u;
				if (ProcessEntry* entry = PeekFromQueue(m_readyToRunCrossArchitecture))
					if (canRun(entry))
						return PopFromQueue(m_readyToRunCrossArchitecture);
				if (ProcessEntry* entry = PeekFromQueue(m_readyToRunCrossPlatform))
					if (canRun(entry))
						return PopFromQueue(m_readyToRunCrossPlatform);
				if (ProcessEntry* entry = PeekFromQueue(m_readyToRunCrossArchAndPlatform))
					if (canRun(entry))
						return PopFromQueue(m_readyToRunCrossArchAndPlatform);
				return ~0u;
			};

		if (forReuse)
		{
			bool memoryBlocked;
			return returnEntry(getEntry(100000.0f, false, memoryBlocked)); // Don't care about max weight or memory
		}

		float activeWeight = m_activeLocalProcessWeight;
		float maxWeight = float(m_maxLocalProcessors) + 0.9f;

		// Only start throttling once we already have one process running.
		// Below that we always allow a process to run so the build keeps making progress.
		bool applyThrottle = activeWeight > 0.0f;
		if (applyThrottle && activeWeight >= maxWeight)
			return ~0u;
		if (activeWeight == 0)
			maxWeight = 100000.0f; // If we are not running anything we must pick up the last process regardless of weight

		// If there are ready getnext, then we try to hand over running ready processes
		// Setting the event will wake up the getnext thread.. and If the getnext thread is timing out we will retry
		if (m_getNextQueueCount)
		{
			m_updateThreadWakeup.Set();
			outShouldSleep = true;
			return ~0u;
		}

		bool memoryBlocked = false;
		u32 entry = getEntry(maxWeight - activeWeight, applyThrottle, memoryBlocked);

		// Update the spawn-delay throttle once, after all queues were considered. We only delay if a candidate could have
		// run but was held back by memory, and we didn't end up picking anything else.
		SetMemDelaySpawning(entry == ~0u && memoryBlocked);

		if (entry != ~0u)
			m_activeLocalProcessWeight += m_processEntries[entry].info->weight;
		return returnEntry(entry);
	}

	bool Scheduler::DrainCompleteFromCache()
	{
		SCOPED_FUTEX(m_processEntriesLock, entriesLock);
		u32 idx = PopFromQueue(m_readyToCompleteFromCache);
		if (idx == ~0u)
			return false;

		auto& entry = m_processEntries[idx];
		UBA_ASSERT(entry.cacheStatus == CacheStatus_PendingCompletion);
		UBA_ASSERT(entry.runStatus == RunStatus_QueuedForRun);
		UBA_ASSERT(entry.dependencyCount == 0);
		UBA_ASSERT(entry.info->cacheInfo);

		entry.cacheStatus = CacheStatus_Success;
		entry.runStatus = RunStatus_Success;
		ProcessLink* firstDependent = entry.firstDependent;
		entry.firstDependent = nullptr;
		ProcessStartInfo2* si = entry.info;
		TransitionStats(entry, StatsState_None, si->weight);
		entry.info = nullptr;

		entriesLock.Leave();

		auto cacheInfo = si->cacheInfo;
		si->cacheInfo = nullptr;
		auto process = new CachedProcess(*si);
		process->logLines.swap(cacheInfo->result.logLines);
		delete cacheInfo;
		ProcessHandle ph(process);
		ProcessExitedResponse exitedResponse = ProcessExitedResponse_None;
		if (auto func = si->exitedFunc)
			func(si->userData, ph, exitedResponse);
		UBA_ASSERT(exitedResponse == ProcessExitedResponse_None);

		if (!m_cancelled)
			UpdateDependents(RunStatus_Success, firstDependent, true); // hadCacheHit=true; per-dep re-acquire

		ProcessLink::DeleteAll(firstDependent);
		m_updateThreadWakeup.Set();

		if (m_writeTrackedInputs)
		{
			Vector<u8> inputs;
			Vector<u8> outputs;
			CreateCacheInputsAndOutputs(inputs, outputs, *si, *process);
			WriteTrackedInputs(inputs,outputs, *process);
		}

		FinishProcess(ph);
		delete si;
		return true;
	}

	bool Scheduler::RunCacheQuery()
	{
		SCOPED_FUTEX(m_processEntriesLock, lock);

		if (AtomicStatsLoad(m_stats.activeCacheQueries) >= m_maxParallelCacheQueries)
			return false;

		u32 entryToProcess = ~0u;

		// Some hard coded constants here.
		// We want to some downloading while testing ahead.. so we always try to use half of the slots for downloading
		if (AtomicStatsLoad(m_stats.activeCacheDownloads) < m_maxParallelCacheQueries/2)
			entryToProcess = PopFromQueue(m_readyToDownloadCache);

		if (entryToProcess == ~0u)
		{
			entryToProcess = PopFromQueue(m_readyToTestCache);
			if (entryToProcess == ~0u)
			{
				// We always reserve two entries in the worker for testing when speculative testing is enabled. Because sometimes there are bottlenecks where testing will happen sequentially
				// and if we don't reserve these slots we might catch up with testing which we don't want in order to keep bandwidth maxed out
				if (m_enableSpeculativeCacheTest && AtomicStatsLoad(m_stats.activeCacheDownloads) >= m_maxParallelCacheQueries - 2)
					return false;
				entryToProcess = PopFromQueue(m_readyToDownloadCache);
			}
			if (entryToProcess == ~0u)
				return false;
		}

		auto& entry = m_processEntries[entryToProcess];
		auto info = entry.info;

		if (entry.cacheStatus == CacheStatus_QueuedForTest || entry.cacheStatus == CacheStatus_QueuedForTestSpeculative)
		{
			entry.cacheStatus = CacheStatus_Testing;
			AtomicStatsAdd(m_stats.activeCacheQueries, 1);
			lock.Leave();

			m_cacheWorkManager->AddWork([this, info, entryToProcess](const WorkContext&)
				{
					ProcessStartInfo2& si = *info;

					MessagePriority priority = AtomicStatsLoad(m_stats.activeCacheDownloads) != 0 ? HasPriority : NormalPriority;
					auto cacheInfo = new CacheFetchInfo(si, m_session, si.rootsHandle, priority);

					for (auto cacheClient : m_cacheClients)
					{
						if (!cacheClient->FetchEntryFromCache(cacheInfo->result, *cacheInfo, si.cacheBucketId) || !cacheInfo->result.hit)
							continue;

						cacheInfo->client = cacheClient;
						info->cacheInfo = cacheInfo;

						if (m_cancelled)
							break;

						SCOPED_FUTEX(m_processEntriesLock, lock);
						auto& hitEntry = m_processEntries[entryToProcess];
						hitEntry.cacheStatus = CacheStatus_QueuedForDownload;
						AddToQueue(m_readyToDownloadCache, entryToProcess);
						AtomicStatsAdd(m_stats.activeCacheQueries, -1);
						// Register A's expected output CAS keys so speculative dependents
						// see them instead of stale on-disk values during their cache test.
						if (m_enableSpeculativeCacheTest)
							cacheInfo->speculativeOutputId = cacheInfo->client->RegisterSpeculativeOutputs(*cacheInfo);
						// Allow dependents to start speculative cache tests now that this entry has a confirmed hit.
						UpdateSpeculativeDependentsOnCacheHit(hitEntry.firstDependent);
						lock.Leave();

						m_updateThreadWakeup.Set();
						return;
					}

					delete cacheInfo;
					HandleCacheMissed(entryToProcess);
					++m_cacheMissCount;
					UpdateStatusHitMissCount();

				}, 1, TC("UbaSchedTest"), ColorWork, WorkPriority_High);
		}
		else if (entry.cacheStatus == CacheStatus_QueuedForDownload)
		{
			entry.cacheStatus = CacheStatus_Downloading;
			AtomicStatsAdd(m_stats.activeCacheQueries, 1);
			AtomicStatsAdd(m_stats.activeCacheDownloads, 1);
			lock.Leave();

			m_cacheWorkManager->AddWork([this, info, entryToProcess](const WorkContext&)
				{
					auto cacheInfo = info->cacheInfo;

					ProcessStartInfo2& si = *info;

					bool success = cacheInfo->client->FetchFilesFromCache(cacheInfo->result, *cacheInfo, m_session.WritePlaceholders());
					if (cacheInfo->speculativeOutputId)
						cacheInfo->client->UnregisterSpeculativeOutputs(cacheInfo->speculativeOutputId);

					if (success)
					{
						SCOPED_FUTEX(m_processEntriesLock, lock);
						auto& entry = m_processEntries[entryToProcess];

						AtomicStatsAdd(m_stats.activeCacheDownloads, -1);

						if (m_cancelled || entry.shouldSkip)
						{
							// Build is being cancelled or a dep failed; skip the process.
							AtomicStatsAdd(m_stats.activeCacheQueries, -1);
							entry.cacheStatus = CacheStatus_Success;
							entry.runStatus = RunStatus_Skipped;
							ProcessLink* firstDependent = entry.firstDependent;
							entry.firstDependent = nullptr;
							TransitionStats(entry, StatsState_None, entry.info->weight);
							entry.info = nullptr;
							lock.Leave();
							UpdateDependents(RunStatus_Skipped, firstDependent, true); // walk outside lock; per-dep re-acquire
							ProcessLink::DeleteAll(firstDependent);
							SkipProcess(*info);
							delete cacheInfo;
							delete info;
							++m_cacheHitCount;
							UpdateStatusHitMissCount();
							return;
						}

						if (entry.dependencyCount > 0)
						{
							// Speculative path: dependencies are still in flight.
							// Park cacheInfo (which holds the log lines) in info; DrainCompleteFromCache
							// will create the CachedProcess and clean up when the last dep finishes.
							AtomicStatsAdd(m_stats.activeCacheQueries, -1);
							entry.cacheStatus = CacheStatus_PendingCompletion;
							lock.Leave();
							m_updateThreadWakeup.Set();
							++m_cacheHitCount;
							UpdateStatusHitMissCount();
							return;
						}

						// dependencyCount == 0: fall through to the existing ExitProcess path.
						// ExitProcess will handle cacheStatus, m_stats.activeCacheQueries, and completion.
						lock.Leave();

						auto process = new CachedProcess(si);
						process->logLines.swap(cacheInfo->result.logLines);
						ProcessHandle ph(process);
						ExitProcessInfo exitInfo;
						exitInfo.scheduler = this;
						exitInfo.startInfo = info;
						exitInfo.processIndex = entryToProcess;
						ExitProcess(exitInfo, *process, 0, true, false);
						++m_cacheHitCount;
					}
					else
					{
						HandleCacheMissed(entryToProcess);
						++m_cacheMissCount;
					}

					delete cacheInfo;
					UpdateStatusHitMissCount();

				}, 1, TC("UbaSchedDlwd"));
		}
		else
		{
			UBA_ASSERT(entry.cacheStatus == CacheStatus_Failed);
			UBA_ASSERTF(entry.runStatus == RunStatus_Skipped, TC("Expected process to be skipped: %u"), u32(entry.runStatus));
			UBA_ASSERT(!info->cacheInfo);
			lock.Leave();
			SkipProcess(*info);
		}

		return true;
	}

	bool Scheduler::RunQueuedProcess(bool isLocal, bool isCrossArchitecture, bool isCrossPlatform)
	{
		while (true)
		{
			RunStatus prevStatus;
			SCOPED_FUTEX(m_processEntriesLock, lock);

			if (!isLocal)
			{
				if (isCrossArchitecture || isCrossPlatform)
					m_hadCrossSystemRequest = true;
				else
					m_hadMatchingSystemRequest = true;
			}
			bool shouldSleep;
			ProcessLink* skipFirstDependent = nullptr;
			u32 indexToRun = PopProcessNoLock(isLocal, isCrossArchitecture, isCrossPlatform, prevStatus, false, shouldSleep, skipFirstDependent);
			if (indexToRun == ~0u)
			{
				lock.Leave();

				if (shouldSleep)
					Sleep(1); // We want to sleep a little bit in case this PopProcess failed because there is a process sitting on reuse request

				if (!isLocal)
					return false;
				return RaceRemoteProcess();
			}

			auto GetRunningCategory = [](bool isLocal, bool isCrossArchitecture, bool isCrossPlatform)
			{
				if (isLocal) return Category_Local;
				if (isCrossArchitecture && isCrossPlatform) return Category_RemoteCrossArchAndPlatform;
				if (isCrossArchitecture) return Category_RemoteCrossArch;
				if (isCrossPlatform) return Category_RemoteCrossPlatform;
				return Category_Remote;
			};


			auto& processEntry = m_processEntries[indexToRun];
			auto info = processEntry.info;
			bool canDetour = processEntry.canDetour && !m_forceNative;
			bool wasSkipped = processEntry.runStatus == RunStatus_Skipped;
			UBA_ASSERTF(processEntry.cacheStatus == CacheStatus_Failed, TC("cacheStatus: %u, runStatus: %u"), u32(processEntry.cacheStatus), u32(processEntry.runStatus));
			const Category runningCategory = GetRunningCategory(isLocal, isCrossArchitecture, isCrossPlatform);

			if (!wasSkipped && !m_cancelled)
			{
				TransitionStats(processEntry, StatsState_Active, info->weight);
				StatsRunning(m_stats, runningCategory, 1, info->weight);
			}
			else
			{
				TransitionStats(processEntry, StatsState_None, info->weight);
			}
			lock.Leave();

			// If we popped a skip-entry, walk its dependents OUTSIDE the lock now.
			if (skipFirstDependent)
			{
				UpdateDependents(RunStatus_Skipped, skipFirstDependent, false);
				ProcessLink::DeleteAll(skipFirstDependent);
			}

			if (wasSkipped || m_cancelled)
			{
				SkipProcess(*info);
				continue;
			}

			if (isLocal)
				LocalProcessStart(*info);

			auto exitInfo = new ExitProcessInfo();
			exitInfo->scheduler = this;
			exitInfo->startInfo = info;
			exitInfo->runningCategory = runningCategory;
			exitInfo->processIndex = indexToRun;
			exitInfo->weight = info->weight;

			ProcessStartInfo si = *info;
			si.userData = exitInfo;
			//si.trackInputs = m_writeToCache && info->cacheBucketId && !m_cacheClients.empty() && info->rules->IsCacheable();
			si.exitedFunc = [](void* userData, const ProcessHandle& handle, ProcessExitedResponse& response)
				{
					auto ei = (ExitProcessInfo*)userData;
					ei->scheduler->ProcessExited(ei, handle);
				};
			UBA_ASSERT(si.rules);

			if (info->skipAsSuccessWhenScheduled)
			{
				m_session.GetWorkManager().AddWork([si](const WorkContext& context)
				{
					ProcessHandle ph(new SkippedProcess(si, 0));
					ProcessExitedResponse exitedResponse = ProcessExitedResponse_None;
					if (auto func = si.exitedFunc)
						func(si.userData, ph, exitedResponse);
				}, 1, TC("Skip"), ColorWork);
				return true;
			}

			if (isLocal)
				m_session.RunProcess(si, true, canDetour);
			else
			{
				bool canCrossArch = processEntry.category == Category_RemoteCrossArch || processEntry.category == Category_RemoteCrossArchAndPlatform;
				bool canCrossPlatform = processEntry.category == Category_RemoteCrossPlatform || processEntry.category == Category_RemoteCrossArchAndPlatform;
				m_session.RunProcessRemote(si, info->weight, info->knownInputs, info->knownInputsCount, canCrossArch, canCrossPlatform);
			}
			return true;
		}
	}

	bool Scheduler::RaceRemoteProcess()
	{
		if (m_maxRacingWeight == 0)
			return false;
		// Stats are atomic — no lock needed.
		if (!AtomicStatsLoad(m_stats.runningRemoteCount) && !AtomicStatsLoad(m_stats.runningRemoteCrossArchCount) && !AtomicStatsLoad(m_stats.runningRemoteCrossPlatformCount) && !AtomicStatsLoad(m_stats.runningRemoteCrossArchAndPlatformCount))
			return false;
		u32 runningLocalCount = AtomicStatsLoad(m_stats.runningLocalCount);

		float totalWeight = m_activeLocalProcessWeight + m_activeRacingProcessWeight;
		if (totalWeight >= m_maxRacingWeight)
			return false;

		if (!LocalProcessOkToSpawn(runningLocalCount))
			return false;

		SCOPED_FUTEX(m_racingProcessesLock, lock);
		lock.Leave(); // Have to leave to prevent entrance inversion with m_remoteProcessAndSessionLock

		ProcessHandle handle = m_session.RunProcessRacing([&](ProcessHandle& remoteProcess, ProcessStartInfo& psi)
			{
				auto ei = (ExitProcessInfo*)remoteProcess.GetStartInfo().userData;
				float weight = ei->weight;
				if (totalWeight + weight > m_maxRacingWeight)
					return false;
				m_activeRacingProcessWeight += weight;
				psi.userData = ei;
				psi.exitedFunc = [](void* userData, const ProcessHandle& handle, ProcessExitedResponse& response)
					{
						auto ei = (ExitProcessInfo*)userData;
						ei->scheduler->ProcessExited(ei, handle);
					};

				LocalProcessStart(*ei->startInfo);

				lock.Enter(); // Enter here instead

				++ei->refCount;
				++ei->processCount;
				return true;
			});

		lock.Enter(); // In case we didn't create a process

		for (auto i=m_racingProcesses.begin();i!=m_racingProcesses.end();)
		{
			if (i->HasExited())
				i = m_racingProcesses.erase(i);
			else
				++i;
		}

		if (!handle.IsValid())
			return false;

		m_racingProcesses.emplace_back(handle);
		return true;
	}

	bool Scheduler::HandleGetNextProcess(Process& process, NextProcessInfo& outNextProcess, u32 prevExitCode, BinaryReader& prevStats, u32 timeoutMs)
	{
		if (!m_enableProcessReuse)
			return false;

		auto& currentStartInfo = process.GetStartInfo();
		auto ei = (ExitProcessInfo*)currentStartInfo.userData;
		if (!ei) // If null, process has already exited from some other thread
			return false;

		bool isLocal = !process.IsRemote();

		bool forReuse = true;

		ExitProcess(*ei, process, prevExitCode, false, forReuse);

		ei->startInfo = nullptr;
		ei->processIndex = ~0u;

		{
			SCOPED_CRITICAL_SECTION(m_wasReturnedLock, lock);
			if (ei->wasReturned)
				return false;
			ei->hasCheckedWasReturned = true;
		}

		// TODO: handle timeout.. when local process we can sit on this thread until a new process pops up
		// or scheduler wants to exit.
		u64 startTime = GetTime();
		bool firstWait = true;

		// We always want to use the getnext processes before spawning new ones.
		// This logic makes sure that the update thread does not pick up actions if getNextQueueCount > 0
		++m_getNextQueueCount;
		auto queueGuard = MakeGuard([&]() { --m_getNextQueueCount; m_updateThreadWakeup.Set(); });

		while (true)
		{
			bool shouldSleep;
			RunStatus prevStatus;
			ProcessLink* skipFirstDependent = nullptr;
			SCOPED_FUTEX(m_processEntriesLock, lock);
			u32 indexToRun = PopProcessNoLock(isLocal, false, false, prevStatus, forReuse, shouldSleep, skipFirstDependent);
			if (indexToRun == ~0u)
			{
				u64 usedTimeMs = TimeToMs(GetTime() - startTime);
				if (usedTimeMs >= timeoutMs || !isLocal || m_cancelled)
					return false;
				lock.Leave();
				if (firstWait)
				{
					m_session.SetProcessAsIdle((ProcessImpl&)process, prevStats, prevExitCode);
					//m_session.GetTrace().ProcessEnvironmentUpdated(process.GetId(), TCV("IDLE"), prevStats.GetPositionData(), prevStats.GetLeft(), TCV(""));
					firstWait = false;
				}

				// Piggyback on event that is triggered when queue changes.
				// And set it again if we got it since there might be more things happening that the update thread should handle
				u32 timeToWaitMs = u32(timeoutMs - usedTimeMs);
				if (m_updateThreadWakeup.IsSet(timeToWaitMs))
					m_updateThreadWakeup.Set();
				continue;
			}

			queueGuard.Execute();

			//UBA_ASSERT(prevStatus != RunStatus_QueuedForCache);
			auto& processEntry = m_processEntries[indexToRun];
			auto newInfo = processEntry.info;
			bool wasSkipped = processEntry.runStatus == RunStatus_Skipped;
			if (!wasSkipped)
			{
				TransitionStats(processEntry, StatsState_Active, newInfo->weight);
				StatsRunning(m_stats, ei->runningCategory, 1, newInfo->weight);
			}
			lock.Leave();

			// If we popped a skip-entry, walk its dependents OUTSIDE the lock now.
			if (skipFirstDependent)
			{
				UpdateDependents(RunStatus_Skipped, skipFirstDependent, false);
				ProcessLink::DeleteAll(skipFirstDependent);
			}

			if (wasSkipped)
			{
				SkipProcess(*newInfo);
				continue;
			}

			auto& si = *newInfo;

			UBA_ASSERT(ei->weight == si.weight); // TOOD: Support different weights?

			ei->startInfo = newInfo;
			ei->processIndex = indexToRun;
			ei->weight = si.weight;

			StringBuffer<> temp;
			const tchar* logFile = m_session.FixLogFile(si.logFile, temp, si.arguments, process.GetId());

			outNextProcess.arguments = si.arguments;
			outNextProcess.workingDir = si.workingDir;
			outNextProcess.description = si.description;
			outNextProcess.logFile = logFile;
			outNextProcess.breadcrumbs = si.breadcrumbs;

			if (*si.overlayFiles)
			{
				outNextProcess.overlayFiles.resize(16*1024);
				DirectoryTableOverlay overlay;
				overlay.committed = u32(outNextProcess.overlayFiles.size());
				overlay.memory = outNextProcess.overlayFiles.data();
				if (!m_session.PopulateOverlayFiles(overlay, si.overlayFilesStr))
					m_session.GetLogger().Error(TC("Failed to populate overlay for next process (%s) — overlay files: %s"), si.GetDescription(), si.overlayFilesStr.c_str());
				outNextProcess.overlayFiles.resize(overlay.size);
			}

			#if UBA_DEBUG
			auto PrepPath = [this](StringBufferBase& out, const ProcessStartInfo& psi)
				{
					if (IsAbsolutePath(psi.application))
						FixPath(psi.application, nullptr, 0, out);
					else
						m_session.GetSearchPathCache().SearchPathForFile(m_session.GetLogger(), out, ToView(psi.application), ToView(psi.workingDir), {});
				};
			StringBuffer<> temp1;
			StringBuffer<> temp2;
			PrepPath(temp1, currentStartInfo);
			PrepPath(temp2, si);
			UBA_ASSERTF(temp1.Equals(temp2.data), TC("%s vs %s"), temp1.data, temp2.data);
			#endif
			return true;
		}
	}

	void Scheduler::ExitProcess(ExitProcessInfo& info, Process& process, u32 exitCode, bool fromCache, bool forReuse)
	{
		auto si = info.startInfo;
		if (!si)
			return;

		ProcessHandle ph;
		ph.m_process = &process;

		ProcessExitedResponse exitedResponse = ProcessExitedResponse_None;
		if (auto func = si->exitedFunc)
			func(si->userData, ph, exitedResponse);

		bool isDone = exitedResponse == ProcessExitedResponse_None;

		// Capture under lock; call UpdateDependents on these AFTER lock.Leave so the long
		// per-dependent walk doesn't hold this lock — the bulk of scheduler contention.
		ProcessLink* firstDependent = nullptr;
		RunStatus depRunStatus = RunStatus_QueuedForRun;
		bool depHadCacheHit = false;

		{
			SCOPED_FUTEX(m_processEntriesLock, lock);
			u32 entryIndex = info.processIndex;
			auto& entry = m_processEntries[entryIndex];
			if (isDone)
			{
				// hadCacheHit: if cacheStatus is Downloading or beyond, the hit was already confirmed and
				// UpdateSpeculativeDependentsOnCacheHit already fired — don't double-decrement.
				depHadCacheHit = (entry.cacheStatus >= CacheStatus_QueuedForDownload);
				entry.runStatus = exitCode == 0 ? RunStatus_Success : RunStatus_Failed;
				firstDependent = entry.firstDependent;
				entry.firstDependent = nullptr;
				depRunStatus = entry.runStatus;
				if (!fromCache)
					StatsRunning(m_stats, info.runningCategory, -1, si->weight);
				TransitionStats(entry, StatsState_None, si->weight);
				entry.info = nullptr;
			}
			else
			{
				UBA_ASSERT(!fromCache);
				UBA_ASSERT(entry.cacheStatus == CacheStatus_Failed);

				entry.canDetour = exitedResponse != ProcessExitedResponse_RerunNative;
				entry.runStatus = RunStatus_QueuedForRun;
				AddToQueue(GetReadyToRunQueue(entry), entryIndex);
				StatsRunning(m_stats, info.runningCategory, -1, si->weight);
				TransitionStats(entry, StatsState_None, si->weight);  // Active → None (old category)
				entry.category = Category_Local;               // now can only run locally
				TransitionStats(entry, StatsState_Ready, si->weight); // None → Queued (new category)
			}

			if (info.runningCategory == Category_Local)
			{
				if (fromCache)
				{
					UBA_ASSERT(isDone);
					entry.cacheStatus = CacheStatus_Success;
					AtomicStatsAdd(m_stats.activeCacheQueries, -1);
				}
				else if (!forReuse)
				{
					m_activeLocalProcessWeight -= info.weight;
					info.weight = 0;
				}
			}
		} // lock released — entry reference is invalid after this since vector can realloc

		// Walk dependents OUTSIDE the lock. Per-dep re-acquire keeps each transition atomic
		// while letting other threads make progress between dependents.
		if (firstDependent)
			UpdateDependents(depRunStatus, firstDependent, depHadCacheHit);

		m_updateThreadWakeup.Set();

		if (isDone)
		{
			ProcessLink::DeleteAll(firstDependent);

			if (exitCode != 0)
			{
				++m_errorCount;
				UpdateQueueCounter();
			}

			bool writeTracked = m_writeTrackedInputs;
			bool writeToCache = m_writeToCache && exitCode == 0 && process.GetExecutionType() != ProcessExecutionType_FromCache;

			if (writeTracked || writeToCache)
			{
				Vector<u8> inputs;
				Vector<u8> outputs;
				CreateCacheInputsAndOutputs(inputs, outputs, *si, process);

				if (writeTracked)
					WriteTrackedInputs(inputs,outputs, process);

				// This does only work if build step actually always write. Many tools read and check if file looks ok and in that case don't write
				// So this is not a reliable approach..
				if (writeToCache)
				{
					CacheClient& cacheClient = *m_cacheClients[0];
					if (!cacheClient.WriteToCache(si->cacheBucketId, *si, inputs.data(), inputs.size(), outputs.data(), outputs.size(), nullptr, 0, process.GetId()))
						m_session.GetLogger().Info(TC("Failed to write %s to cache"), si->GetDescription());
				}
			}
			// Need to be called after cache write in order to prevent IsEmpty from returning true
			FinishProcess(ph);

			delete si;
		}

		ph.m_process = nullptr;
	}

	void Scheduler::SkipProcess(ProcessStartInfo2& info)
	{
		ProcessHandle ph(new SkippedProcess(info, ProcessCancelExitCode));
		ProcessExitedResponse exitedResponse = ProcessExitedResponse_None;
		if (auto func = info.exitedFunc)
			func(info.userData, ph, exitedResponse);
		UBA_ASSERT(exitedResponse == ProcessExitedResponse_None);
		FinishProcess(ph);
		m_updateThreadWakeup.Set();
	}

	void Scheduler::UpdateStatusHitMissCount()
	{
		StringBuffer<> str;
		str.Appendf(TC("Hits %u Misses %u"), m_cacheHitCount.load(), m_cacheMissCount.load());
		m_session.GetTrace().StatusUpdate(1, 6, str, LogEntryType_Info);
	}

	void Scheduler::UpdateQueueCounter()
	{
		m_session.UpdateProgress(m_totalProcesses, m_finishedProcesses, m_errorCount);
	}

	#define UBA_STATS_SWITCH(field) \
		switch (cat) { \
		case Category_Local:                      AtomicStatsAdd(s.field##LocalCount, d);                       AtomicStatsAdd(s.field##LocalWeight, w);                       return; \
		case Category_Remote:                     AtomicStatsAdd(s.field##RemoteCount, d);                      AtomicStatsAdd(s.field##RemoteWeight, w);                      return; \
		case Category_RemoteCrossArch:            AtomicStatsAdd(s.field##RemoteCrossArchCount, d);             AtomicStatsAdd(s.field##RemoteCrossArchWeight, w);             return; \
		case Category_RemoteCrossPlatform:        AtomicStatsAdd(s.field##RemoteCrossPlatformCount, d);         AtomicStatsAdd(s.field##RemoteCrossPlatformWeight, w);         return; \
		case Category_RemoteCrossArchAndPlatform: AtomicStatsAdd(s.field##RemoteCrossArchAndPlatformCount, d);  AtomicStatsAdd(s.field##RemoteCrossArchAndPlatformWeight, w);  return; \
		default: return; }

	void Scheduler::StatsActive(SchedulerStats& s, Category cat, int d, float w) { UBA_STATS_SWITCH(active)   }
	void Scheduler::StatsReady(SchedulerStats& s, Category cat, int d, float w) { UBA_STATS_SWITCH(ready)    }
	void Scheduler::StatsPending(SchedulerStats& s, Category cat, int d, float w) { UBA_STATS_SWITCH(pending) }
	void Scheduler::StatsRunning(SchedulerStats& s, Category cat, int d, float w) { UBA_STATS_SWITCH(running)  }
	#undef UBA_STATS_SWITCH


	void Scheduler::TransitionStats(ProcessEntry& entry, StatsState newState, float weight)
	{
		switch (entry.statsState)
		{
		case StatsState_Pending: StatsPending(m_stats, entry.category, -1, -weight); break;
		case StatsState_Ready:   StatsReady  (m_stats, entry.category, -1, -weight); break;
		case StatsState_Active:  StatsActive (m_stats, entry.category, -1, -weight); break;
		default: break;
		}
		entry.statsState = newState;
		switch (newState)
		{
		case StatsState_Pending: StatsPending(m_stats, entry.category, 1, weight); break;
		case StatsState_Ready:   StatsReady  (m_stats, entry.category, 1, weight); break;
		case StatsState_Active:  StatsActive (m_stats, entry.category, 1, weight); break;
		default: break;
		}
	}

	void Scheduler::FinishProcess(const ProcessHandle& handle)
	{
		++m_finishedProcesses;
		UpdateQueueCounter();
		if (m_processFinished)
			m_processFinished(handle);
	}

	void Scheduler::CreateCacheInputsAndOutputs(Vector<u8>& outInputs, Vector<u8>& outOutputs, ProcessStartInfo2& info, Process& process)
	{
		auto writeString = [](Vector<u8>& v, u64& p, StringView str)
			{
				if (p + 1024 > v.size())
					v.resize(p + 1024);
				BinaryWriter w(v.data(), p, v.size());
				w.WriteString(str);
				p = w.GetPosition();
			};

		u64 writePosOutputs = 0;
		Set<TString> handledFiles;
		auto writeOutput = [&](TString&& str)
			{
				auto res = handledFiles.emplace(std::move(str));
				if (!res.second)
					return;
				writeString(outOutputs, writePosOutputs, *res.first);
			};

		auto& outputs = process.GetTrackedOutputs();
		BinaryReader outputsReader(outputs.data(), 0, outputs.size());
		while (outputsReader.GetLeft())
			writeOutput(outputsReader.ReadString());

		if (auto ko = (const tchar*)info.knownOutputs)
			while (*ko)
			{
				StringView kov = ToView(ko);
				writeOutput(kov.ToString());
				ko += kov.count + 1;
			}

		outOutputs.resize(writePosOutputs);

		// Adding inputs. Note, inputs can't also be outputs.
		u64 writePosInputs = 0;
		auto writeInput = [&](TString&& str)
			{
				auto res = handledFiles.emplace(std::move(str));
				if (!res.second)
					return;
				#if PLATFORM_WINDOWS
				if (StringView(*res.first).Contains(TCV("api-ms-win-")) || StringView(*res.first).Contains(TCV("tzres.dll")))
					return;
				#endif
				writeString(outInputs, writePosInputs, *res.first);
			};

		auto& inputs = process.GetTrackedInputs();
		outInputs.reserve(inputs.size());
		BinaryReader inputsReader(inputs.data(), 0, inputs.size());
		while (inputsReader.GetLeft())
			writeInput(inputsReader.ReadString());

		if (auto ki = (const tchar*)info.knownInputs)
			while (*ki)
			{
				StringView kiv = ToView(ki);
				writeInput(kiv.ToString());
				ki += kiv.count + 1;
			}

		outInputs.resize(writePosInputs);
	}

	void Scheduler::WriteTrackedInputs(const Vector<u8>& inputs, const Vector<u8>& outputs, Process& process)
	{
		static int counter = 0;
		StringBuffer<> inputsFile;
		inputsFile.Append(m_session.GetSessionDir()).Append(TCV("inputs")).EnsureEndsWithSlash();
		static u32 processId = 1; // TODO: This should be done in a better way.. or not at all?
		GenerateNameForProcess(inputsFile, process.GetStartInfo().arguments, ++processId);
		inputsFile.Append(TCV(".log"));
		FileAccessor f(m_session.GetLogger(), inputsFile.data);
		auto writeLine = [&](StringView str)
			{
				u8 buffer[1024];
				BinaryWriter writer(buffer, 0, sizeof(buffer));
				writer.WriteUtf8String(str.data, str.count);
				writer.WriteByte('\n');
				f.Write(buffer, writer.GetPosition());
			};
		auto writeLines = [&](const Vector<u8>& files)
			{
				BinaryReader intputReader(files.data(), 0, files.size());
				while (intputReader.GetLeft())
				{
					StringBuffer<> file;
					intputReader.ReadString(file);
					writeLine(file);
				}
			};
		if (f.CreateWrite())
		{
			writeLine(TCV("Inputs:"));
			writeLines(inputs);
			writeLine(TCV(""));
			writeLine(TCV("Outputs:"));
			writeLines(outputs);
			writeLine(TCV(""));

			f.Close();
		}
	}

	void Scheduler::KillNewestLocalProcess(StringView reason)
	{
		ProcessHandle ph;

		// Kill racing processes first and then newest
		
		SCOPED_FUTEX(m_racingProcessesLock, lock);
		for (auto i=m_racingProcesses.rbegin();i!=m_racingProcesses.rend(); ++i)
		{
			if (i->HasExited())
				continue;
			ph = *i;
			break;
		}
		
		if (!ph.IsValid())
		{
			ph = m_session.GetNewestLocalProcess();
			if (!ph.IsValid())
				return;
		}

		m_session.GetTrace().SchedulerKillProcess(ph.GetId(), reason);

		u64 startTime = GetTime();
		if (ProcessReturned(*ph.m_process, true))
			m_session.GetLogger().Info(TC("Killed process %s (%s) - %s"), ph.GetStartInfo().GetDescription(), TimeToText(GetTime() - startTime).str, reason.data);
	}

	void Scheduler::NativeProcessCreated(ProcessImpl& process)
	{
		if (!m_memWatchdog)
			return;

		if (!IsWindows) // Memory prediction is not implemented for non-windows
			return;

		auto& impl = (ProcessImpl&)process;
		auto ei = (ExitProcessInfo*)impl.m_startInfo.userData;
		
		// Child process, we still need to track
		if (!ei)
		{
			impl.m_startInfo.userData = this;
			impl.m_startInfo.exitedFunc = [](void* userData, const ProcessHandle& handle, ProcessExitedResponse& response)
				{
					auto& impl = *(ProcessImpl*)handle.m_process;
					auto& scheduler = *(Scheduler*)userData;
					StringKey key(impl.m_nativeProcessId, impl.m_nativeProcessCreationTime);
					SCOPED_FUTEX(scheduler.m_processesLock, l);
					bool res = scheduler.m_processes.erase(key) == 1;
					UBA_ASSERT(res);(void)res;
				};
		}
		else
			++ei->refCount;

		StringKey key(impl.m_nativeProcessId, impl.m_nativeProcessCreationTime);
		SCOPED_FUTEX(m_processesLock, l);
		m_processes.try_emplace(key, ProcessRecord{ process, ei });
	}

	void Scheduler::LocalProcessStart(ProcessStartInfo2& info)
	{
		if (!IsWindows) // Memory prediction is not implemented for non-windows
			return;
			
		if (!m_memWatchdog || !info.memoryGroupId)
			return;
		
		SCOPED_FUTEX(m_memGroupLookupLock, statsLock);
		auto insres = m_memGroupLookup.try_emplace(info.memoryGroupId);
		MemGroupStats& stats = insres.first->second;
		if (insres.second)
		{
			auto findIt = m_memGroupDb.find(info.memoryGroupId);
			if (findIt != m_memGroupDb.end())
			{
				stats.baseline = findIt->second.baseline;
				stats.average = findIt->second.average;
				stats.history[0] = stats.average;
				stats.historyCounter = 1;
			}
		}

		// Use the predicted memory as a baseline floor so it's reflected in m_memEstimatedUsage immediately.
		// Overwrite (don't Max) so a one-off high estimate can't permanently poison the persisted DB; a later
		// lower estimate lowers the floor and lets the rolling average converge back down.
		if (info.predictedMemoryUsage > 0)
		{
			stats.baseline = info.predictedMemoryUsage;
			if (stats.average < stats.baseline)
				stats.average = stats.baseline;
		}

		m_memEstimatedUsage += stats.average;
		++stats.activeProcessCount;
	}

	void Scheduler::UpdatePredictedMemoryUsage(ProcessStartInfo2& info)
	{
		if (info.hasUpdatedPredictedMemoryUsage)
			return;

		// Prefer the callback so callers that don't know the estimate at enqueue time can supply it lazily.
		if (m_predictedMemoryFunc)
			if (u64 predicted = m_predictedMemoryFunc(info.userData))
				info.predictedMemoryUsage = predicted;

		info.hasUpdatedPredictedMemoryUsage = true;
	}

	bool Scheduler::LocalProcessMemoryOkToSpawn(u32 runningLocalCount, u64 predictedMemoryUsage)
	{
		if (!m_memWaitThreshold)
			return true;

		u64 estimatedUsage = predictedMemoryUsage + m_memEstimatedUsage.load();

		if (estimatedUsage < m_memWaitThreshold && !m_lastKillTime)
			return true;

		if (!runningLocalCount) // Always let one run so it can finish eventually
			return true;

		return false;
	}

	void Scheduler::SetMemDelaySpawning(bool delaySpawning)
	{
		if (m_memDelaySpawning == delaySpawning)
			return;
		m_memDelaySpawning = delaySpawning;
		m_session.GetTrace().SchedulerUpdate(delaySpawning);
	}

	bool Scheduler::LocalProcessOkToSpawn(u32 runningLocalCount, u64 predictedMemoryUsage)
	{
		bool okToSpawn = LocalProcessMemoryOkToSpawn(runningLocalCount, predictedMemoryUsage);
		SetMemDelaySpawning(!okToSpawn);
		if (okToSpawn)
			return true;

#if 0//PLATFORM_WINDOWS
		static bool hasBeenRunOnce;
		if (!hasBeenRunOnce)
		{
			hasBeenRunOnce = true;
			auto& logger = m_session.GetLogger();
			logger.BeginScope();
			logger.Info(TC("NOTE - To mitigate this spawn delay it is recommended to make page file larger until you don't see these messages again (Or reduce number of max parallel processes)"));
			logger.Info(TC("       Set max page file to a large number (like 128gb). It will not use disk space unless you actually start using that amount of committed memory"));
			logger.Info(TC("       Also note, this is \"committed\" memory. Not memory in use. So you necessarily don't need more physical memory"));
			MEMORYSTATUSEX memStatus = { sizeof(memStatus) };
			GlobalMemoryStatusEx(&memStatus);
			logger.Info(TC("  TotalPhys: %s"), BytesToText(memStatus.ullTotalPhys));
			logger.Info(TC("  AvailPhys: %s"), BytesToText(memStatus.ullAvailPhys));
			logger.Info(TC("  TotalPage: %s"), BytesToText(memStatus.ullTotalPageFile));
			logger.Info(TC("  AvailPage: %s"), BytesToText(memStatus.ullAvailPageFile));
			logger.EndScope();
		}
#endif
		return false;
	}

	void Scheduler::LocalProcessExit(ProcessStartInfo2& info, const ProcessHandle& handle)
	{
		if (!m_memWatchdog)
			return;

		if (!IsWindows) // Not implemented for non windows
			return;

		auto& impl = *(ProcessImpl*)handle.m_process;

		if (info.memoryGroupId)
		{
			u64 processPeakMemory = impl.m_processStats.peakMemory;

			SCOPED_FUTEX(m_memGroupLookupLock, statsLock);
			MemGroupStats& stats = m_memGroupLookup[info.memoryGroupId];

			if (!stats.historyCounter) // First process in memory group could be pch
			{
				for (auto& kv : impl.m_shared.writtenFiles)
				{
					auto& writtenFile = kv.second;
					if (StringView(writtenFile.name).EndsWith(TC(".pch")))
					{
						stats.baseline = writtenFile.memoryWritten;
						processPeakMemory = stats.baseline;
					}
				}
			}			

			UBA_ASSERT(stats.activeProcessCount);
			--stats.activeProcessCount;

			if (processPeakMemory >= stats.baseline) // Don't know if this test is too aggressive.. but 
			{
				u64& peakMemory = stats.history[stats.historyCounter % sizeof_array(stats.history)];
				peakMemory = processPeakMemory;
				++stats.historyCounter;

				u32 count = Min(u32(sizeof_array(stats.history)), stats.historyCounter);
				u64 average = 0;
				for (u32 i=0;i!=count;++i)
					average += stats.history[i];
				average /= count;

				if (average < stats.baseline)
					average = stats.baseline;
				stats.average = average;
			}

			m_memEstimatedUsage -= stats.average;
		}

		if (impl.m_nativeProcessCreationTime)
		{
			StringKey key(impl.m_nativeProcessId, impl.m_nativeProcessCreationTime);
			SCOPED_FUTEX(m_processesLock, l);
			auto findIt = m_processes.find(key);
			UBA_ASSERTF(findIt != m_processes.end(), TC("Tried to erase local process from process list but can't find it (%s)"), info.GetDescription());
			if (findIt != m_processes.end())
			{
				if (auto ep = findIt->second.ep)
				{
					UBA_ASSERT(ep->refCount);
					if (!--ep->refCount)
						delete ep;
				}
				m_processes.erase(findIt);
			}
		}
	}

	void Scheduler::LoadMemTrackTable()
	{
		if (!IsWindows) // Not implemented for non windows
			return;
		StringBuffer<> fileName;
		fileName.Append(m_session.GetRootDir()).Append("memgroups");
		FileAccessor file(m_session.GetLogger(), fileName.data);
		if (!file.OpenMemoryRead(0, false) || !file.GetSize())
			return;
		BinaryReader reader(file.GetData(), 0, file.GetSize());
		
		u16 version = reader.ReadU16();
		if (version != 2)
			return;

		while (reader.GetLeft() >= 24)
		{
			u64 key = reader.ReadU64();
			u64 baseline = reader.ReadU64();
			u64 average = reader.ReadU64();
			m_memGroupDb[key] = MemGroupEntry{ baseline, average };
		}
	}

	void Scheduler::SaveMemTrackTable()
	{
		if (!IsWindows) // Not implemented for non windows
			return;

		for (auto& kv : m_memGroupLookup)
			m_memGroupDb[kv.first] = MemGroupEntry{ kv.second.baseline, kv.second.average };

		StringBuffer<> fileName;
		fileName.Append(m_session.GetRootDir()).Append("memgroups");
		FileAccessor file(m_session.GetLogger(), fileName.data);

		u64 fileSize = 2 + m_memGroupDb.size()*24;

		if (!file.CreateMemoryWrite(false, DefaultAttributes(), fileSize))
			return;

		BinaryWriter writer(file.GetData(), 0, fileSize);

		writer.WriteU16(2); // Version

		for (auto& kv : m_memGroupDb)
		{
			writer.WriteU64(kv.first);
			writer.WriteU64(kv.second.baseline);
			writer.WriteU64(kv.second.average);
		}

		file.Close();
	}

	void Scheduler::ThreadMemoryCheckLoop()
	{
		auto& logger = m_session.GetLogger();

		//u64 randomKillTime = GetTime() + MsToTime(5000);

		Vector<u8> queryBuffer(1024 * 1024);

		#if PLATFORM_WINDOWS
		struct UBA_SYSTEM_PERFORMANCE_INFORMATION
		{
			LARGE_INTEGER IdleTime;
			LARGE_INTEGER ReadTransferCount;
			LARGE_INTEGER WriteTransferCount;
			LARGE_INTEGER OtherTransferCount;
			ULONG ReadOperationCount;
			ULONG WriteOperationCount;
			ULONG OtherOperationCount;
			ULONG AvailablePages;
			ULONG TotalCommittedPages;
			ULONG TotalCommitLimit;
			ULONG PeakCommitment;
			ULONG PageFaults;
			ULONG WriteCopyFaults;
			ULONG TransitionFaults;
			ULONG CacheTransitionFaults;
			ULONG DemandZeroFaults;
			ULONG PagesRead;
			ULONG PageReadIos;
			ULONG CacheReads;
			ULONG CacheIos;
			ULONG PagefilePagesWritten;
			// ...
		};
		static_assert(sizeof(UBA_SYSTEM_PERFORMANCE_INFORMATION) <= sizeof(SYSTEM_PERFORMANCE_INFORMATION));
		UBA_SYSTEM_PERFORMANCE_INFORMATION prevPerfInfo = {};
		bool firstPerfInfo = true;
		#endif

		constexpr u32 StartRow = 20;

		Trace& trace = m_session.GetTrace();

		if (m_memTracingLevel >= 1)
		{
			trace.StatusUpdate(StartRow, 1, TCV("MemTot"), LogEntryType_Info);
			trace.StatusUpdate(StartRow+1, 1, TCV("MemTot Est"), LogEntryType_Info);
			trace.StatusUpdate(StartRow, 10, TCV("MemProc"), LogEntryType_Info);
			trace.StatusUpdate(StartRow+1, 10, TCV("MemProc Est"), LogEntryType_Info);
			trace.StatusUpdate(StartRow+2, 1, TCV("MemBaseline"), LogEntryType_Info);
			trace.StatusUpdate(StartRow+3, 1, TCV("MemUntrack"), LogEntryType_Info);
			trace.StatusUpdate(StartRow+4, 1, TCV("MemStartWait"), LogEntryType_Info);
			trace.StatusUpdate(StartRow+5, 1, TCV("MemStartKill"), LogEntryType_Info);
			trace.StatusUpdate(StartRow+6, 1, TCV("PagefileRead"), LogEntryType_Info);
			trace.StatusUpdate(StartRow+7, 1, TCV("PagefileWrite"), LogEntryType_Info);
		}

		u64 memAvail;
		u64 memTotal;
		u64 maxPageFile;
		if (!GetMemoryInfo(logger, memAvail, memTotal, &maxPageFile))
		{
			m_memWatchdog = false;
			logger.Warning(TC("GetMemoryInfo failed. Memory tracking disabled."));
			m_updateThreadWakeup.Set();
			return;
		}
		u64 memPhys = memTotal - maxPageFile;

		UnorderedMap<u64, u32> memoryGroupToIndex;

		u64 memKillThreshold = 0;
		

		u8 memStartWaitPercent = m_memStartWaitPercent;
		u8 memStartWaitPercentMin = u8(double(memPhys)/double(memTotal) * 100.0f);
		u64 lastPageFileWriteTimeMs = 0;

		u32 timeoutMs = 0;

		StringBuffer<> str;

		while (!m_memThreadEvent.IsSet(timeoutMs))
		{
			timeoutMs = 1000;

			GetMemoryInfo(logger, memAvail, memTotal, &maxPageFile);

			m_memWaitThreshold = u64(double(memTotal) * double(memStartWaitPercent) / 100.0);
			memKillThreshold = u64(double(memTotal) * double(m_memStartKillPercent) / 100.0);

			// Kill racing processes if we have run over racing process weight
			float totalWeight = m_activeLocalProcessWeight + m_activeRacingProcessWeight;
			if (totalWeight > m_maxRacingWeight)
			{
				SCOPED_FUTEX(m_racingProcessesLock, lock);
				for (auto i=m_racingProcesses.rbegin();i!=m_racingProcesses.rend(); ++i)
				{
					if (i->HasExited())
						continue;
					ProcessHandle ph = *i;
					m_session.GetTrace().SchedulerKillProcess(ph.GetId(), TCV("race"));
					u64 startTime = GetTime();
					if (!ProcessReturned(*ph.m_process, true))
						continue;
					m_session.GetLogger().Debug(TC("Killed process %s (%s) - Too many racing processes"), ph.GetStartInfo().GetDescription(), TimeToText(GetTime() - startTime).str);
					timeoutMs = 5;
					break;
				}
			}


			if (!m_memWatchdog)
				continue;

			u64 untrackedMemory = 0;
			u64 processMemTotalPageCount = 0;

			#if PLATFORM_WINDOWS
			ULONG returnedSize;
			while (true)
			{
				NTSTATUS res = NtQuerySystemInformation(SystemProcessInformation, queryBuffer.data(), (u32)queryBuffer.size(), &returnedSize);
				if (res == STATUS_SUCCESS)
					break;
				if (res == STATUS_INFO_LENGTH_MISMATCH)
				{
					queryBuffer.resize(returnedSize + (10*PagefileUsageOffset));
					continue;
				}

				m_memWatchdog = false;
				logger.Warning(TC("NtQuerySystemInformation failed. Memory tracking disabled."));
				m_updateThreadWakeup.Set();
				break;
			}

			u8* it = queryBuffer.data();
			ULONG nextEntryOffset = *(ULONG*)it;
			if (nextEntryOffset < PagefileUsageOffset + 8)
			{
				m_memWatchdog = false;
				logger.Warning(TC("NtQuerySystemInformation does not contain PageFileUsage. Memory tracking disabled."));
				m_updateThreadWakeup.Set();
				break;
			}

			u32 activeProcessCount;
			{
				SCOPED_FUTEX(m_processesLock, l);
				activeProcessCount = u32(m_processes.size());

				while (nextEntryOffset)
				{
					u64 createTime = *(u64*)(it + CreateTimeOffset);
					u32 pid = *(u32*)(it + PidOffset);
					auto procIt = m_processes.find(StringKey(pid, createTime));
					if (procIt != m_processes.end())
					{
						ProcessRecord rec = procIt->second;
						u64 procMem = *(u64*)(it + PagefileUsageOffset);
						if (rec.ep)
							if (rec.ep->startInfo->memoryGroupId == 0)
								untrackedMemory += procMem;
						AtomicMax(rec.process.m_processStats.peakMemory, procMem);
						processMemTotalPageCount += procMem;
					}

					nextEntryOffset = *(ULONG*)it;
					it += nextEntryOffset;
				}
			}
			#endif

			u64 memUsed = memTotal - memAvail;
			u64 memBaseLine = memUsed - processMemTotalPageCount;
			u64 estimatedProcessMemoryUsage = 0;
			u64 estimatedMemoryUsage = 0;

			{
				SCOPED_FUTEX(m_memGroupLookupLock, statsLock);

				for (auto& kv : m_memGroupLookup)
				{
					MemGroupStats& stats = kv.second;
					estimatedProcessMemoryUsage += stats.activeProcessCount * stats.average;

					if (m_memTracingLevel >= 2)
					{
						constexpr u32 colIndex = 25;
						auto res = memoryGroupToIndex.emplace(kv.first, 0u);
						u32& index = res.first->second;
						if (res.second)
						{
							index = StartRow + u32(memoryGroupToIndex.size() - 1);
							str.Clear().AppendValue(kv.first);
							trace.StatusUpdate(index, colIndex, str, LogEntryType_Info);
						}

						str.Clear().Append(BytesToText(stats.average).str);
						trace.StatusUpdate(index, colIndex + 5, str, LogEntryType_Info);
					}
				}
				estimatedMemoryUsage = memBaseLine + untrackedMemory + estimatedProcessMemoryUsage;
				m_memEstimatedUsage = Max(memUsed, estimatedMemoryUsage);
			}

			if (m_memTracingLevel >= 1)
			{
				trace.StatusUpdate(StartRow, 6, BytesToText(memUsed), LogEntryType_Info);
				trace.StatusUpdate(StartRow+1, 6, BytesToText(estimatedMemoryUsage), LogEntryType_Info);
				trace.StatusUpdate(StartRow, 15, BytesToText(processMemTotalPageCount), LogEntryType_Info);
				trace.StatusUpdate(StartRow+1, 15, BytesToText(estimatedProcessMemoryUsage), LogEntryType_Info);
				trace.StatusUpdate(StartRow+2, 6, BytesToText(memBaseLine), LogEntryType_Info);
				trace.StatusUpdate(StartRow+3, 6, BytesToText(untrackedMemory), LogEntryType_Info);
				trace.StatusUpdate(StartRow+4, 6, BytesToText(m_memWaitThreshold), LogEntryType_Info);
				trace.StatusUpdate(StartRow+5, 6, BytesToText(memKillThreshold), LogEntryType_Info);
			}

			if (memKillThreshold)
			{
				if (m_lastKillTime && m_lastKillTime + MsToTime(3000) < GetTime())
				{
					m_lastKillTime = 0;
					m_updateThreadWakeup.Set();
				}

				while (memUsed > memKillThreshold)
				{
					// Stats are atomic — no lock needed.
					if (AtomicStatsLoad(m_stats.runningLocalCount) <= 1)
						break;
					m_lastKillTime = GetTime();

					str.Clear().Appendf(TC("Low on memory (%s/%s). Kill threshold is %s"), BytesToText(memUsed).str, BytesToText(memTotal).str, BytesToText(memKillThreshold).str);

					KillNewestLocalProcess(str);
					if (!GetMemoryInfo(logger, memAvail, memTotal, &maxPageFile))
						break;
					memUsed = memTotal - memAvail;
				}
			}

			#if PLATFORM_WINDOWS
			if (m_memPagefileActivityThreshold)
			{
				auto& currPerfInfo = *(UBA_SYSTEM_PERFORMANCE_INFORMATION*)queryBuffer.data();
				if (NtQuerySystemInformation(SystemPerformanceInformation, &currPerfInfo, sizeof(SYSTEM_PERFORMANCE_INFORMATION), NULL) == STATUS_SUCCESS && !firstPerfInfo)
				{
					ULONG reads = currPerfInfo.PagesRead - prevPerfInfo.PagesRead;
					ULONG writes = currPerfInfo.PagefilePagesWritten - prevPerfInfo.PagefilePagesWritten;

					if (m_memTracingLevel >= 1)
					{
						trace.StatusUpdate(StartRow+6, 6, CountToText(reads), LogEntryType_Info);
						trace.StatusUpdate(StartRow+7, 6, CountToText(writes), LogEntryType_Info);
					}

					if (memUsed > memPhys && activeProcessCount > 6 && writes > m_memPagefileActivityThreshold)
					{
						memStartWaitPercent = u8(double(memUsed)/double(memTotal) * 100.0f);
						str.Clear().Appendf(TC("Pagefile activity high (%lu writes). MemUsed %s, MemPhys %s, MemTot %s"), writes, BytesToText(memUsed).str, BytesToText(memPhys).str, BytesToText(memTotal).str);
						KillNewestLocalProcess(str);
						lastPageFileWriteTimeMs = TimeToMs(GetTime());
					}
					else if (writes > 0)
					{
						if (memStartWaitPercent > memStartWaitPercentMin)
							--memStartWaitPercent;
						lastPageFileWriteTimeMs = TimeToMs(GetTime());
					}
					else if (TimeToMs(GetTime()) - lastPageFileWriteTimeMs > 5*1000)
					{
						if (memStartWaitPercent < m_memStartWaitPercent)
						{
							++memStartWaitPercent;
							if (memStartWaitPercent == m_memStartWaitPercent)
								m_updateThreadWakeup.Set(); // Just in case.. this should never happen in practice
						}
					}
				}
				firstPerfInfo = false;
				prevPerfInfo = currPerfInfo;
			}
			#endif

			//if (randomKillTime < GetTime())
			//{
			//	randomKillTime = GetTime() + MsToTime(500);
			//	if (m_activeLocalProcesses > 4)
			//		KillNewestLocalProcess();
			//}
		}
	}

	bool Scheduler::EnqueueFromFile(const tchar* yamlFilename, const Function<void(EnqueueProcessInfo&)>& enqueued)
	{
		auto& logger = m_session.GetLogger();

		TString app;
		TString arg;
		TString dir;
		TString desc;
		bool allowDetour = true;
		bool allowRemote = true;
		float weight = 1.0f;
		u32 cacheBucket = 0;
		u32 memoryGroup = 0;
		Vector<u32> deps;
		RootsHandle rootsHandle = 0;

		ProcessStartInfo si;

		auto enqueueProcess = [&]()
			{
				si.application = app.c_str();
				si.arguments = arg.c_str();
				si.workingDir = dir.c_str();
				si.description = desc.c_str();
				si.rootsHandle = rootsHandle;

				#if UBA_DEBUG
				StringBuffer<> logFile;
				if (true)
				{
					static u32 processId = 1; // TODO: This should be done in a better way.. or not at all?
					GenerateNameForProcess(logFile, si.arguments, ++processId);
					logFile.Append(TCV(".log"));
					si.logFile = logFile.data;
				};
				#endif

				EnqueueProcessInfo info { si };
				info.dependencies = deps.data();
				info.dependencyCount = u32(deps.size());
				info.canDetour = allowDetour;
				info.canExecuteRemotely = allowRemote;
				info.weight = weight;
				info.cacheBucketId = cacheBucket;
				info.memoryGroupId = memoryGroup;
				if (enqueued)
					enqueued(info);
				EnqueueProcess(info);
				app.clear();
				arg.clear();
				dir.clear();
				desc.clear();
				deps.clear();
				allowDetour = true;
				allowRemote = true;
				weight = 1.0f;
				cacheBucket = 0;
				memoryGroup = 0;
				rootsHandle = 0;
			};

		enum InsideArray
		{
			InsideArray_None,
			InsideArray_CacheRoots,
			InsideArray_Processes,
		};

		InsideArray insideArray = InsideArray_None;

		auto readLine = [&](const TString& line)
			{
				const tchar* keyStart = line.c_str();
				while (*keyStart && *keyStart == ' ')
					++keyStart;
				if (!*keyStart)
					return true;
				u32 indentation = u32(keyStart - line.c_str());

				if (insideArray != InsideArray_None && !indentation)
					insideArray = InsideArray_None;

				StringBuffer<32> key;
				const tchar* valueStart = nullptr;

				if (*keyStart == '-')
				{
					UBA_ASSERT(insideArray != InsideArray_None);
					valueStart = keyStart + 2;
				}
				else
				{
					const tchar* colon = TStrchr(keyStart, ':');
					if (!colon)
						return false;
					key.Append(keyStart, colon - keyStart);
					valueStart = colon + 1;
					while (*valueStart && *valueStart == ' ')
						++valueStart;
				}

				switch (insideArray)
				{
				case InsideArray_None:
				{
					if (key.Equals(TCV("environment")))
					{
						#if PLATFORM_WINDOWS
						SetEnvironmentVariable(TC("PATH"), valueStart);
						#endif
						return true;
					}
					if (key.Equals(TCV("cacheroots")))
					{
						insideArray = InsideArray_CacheRoots;
						return true;
					}
					if (key.Equals(TCV("processes")))
					{
						insideArray = InsideArray_Processes;
						return true;
					}
					return true;
				}
				case InsideArray_CacheRoots:
				{
					auto& rootPaths = *m_rootPaths.emplace_back(new RootPaths());
					if (Equals(valueStart, TC("SystemRoots")))
						rootPaths.RegisterSystemRoots(logger);
					else
						rootPaths.RegisterRoot(logger, valueStart);
					return true;
				}
				case InsideArray_Processes:
				{
					if (*keyStart == '-')
					{
						keyStart += 2;
						if (!app.empty())
							enqueueProcess();
					}

					if (key.Equals(TCV("app")))
						app = valueStart;
					else if (key.Equals(TCV("arg")))
						arg = valueStart;
					else if (key.Equals(TCV("dir")))
						dir = valueStart;
					else if (key.Equals(TCV("desc")))
						desc = valueStart;
					else if (key.Equals(TCV("detour")))
						allowDetour = !Equals(valueStart, TC("false"));
					else if (key.Equals(TCV("remote")))
						allowRemote = !Equals(valueStart, TC("false"));
					else if (key.Equals(TCV("weight")))
						StringBuffer<32>(valueStart).Parse(weight);
					else if (key.Equals(TCV("cache")))
						StringBuffer<32>(valueStart).Parse(cacheBucket);
					else if (key.Equals(TCV("memgroup")))
						StringBuffer<32>(valueStart).Parse(memoryGroup);
					else if (key.Equals(TCV("dep")))
					{
						const tchar* depStart = TStrchr(valueStart, '[');
						if (!depStart)
							return false;
						++depStart;
						StringBuffer<32> depStr;
						for (const tchar* it = depStart; *it; ++it)
						{
							if (*it != ']' && *it != ',')
							{
								if (*it != ' ')
									depStr.Append(*it);
								continue;
							}
							u32 depIndex;
							if (!depStr.Parse(depIndex))
								return false;
							depStr.Clear();
							deps.push_back(depIndex);

							if (!*it)
								break;
							depStart = it + 1;
						}
					}
					else if (key.Equals(TCV("vfs")))
					{
						const tchar* depStart = TStrchr(valueStart, '[');
						if (!depStart)
							return false;
						const tchar* it = depStart + 1;
						StackBinaryWriter<8*1024> buffer;
						while (true)
						{
							const tchar* vfsStart = it;
							while (*it != ';')
								++it;
							buffer.WriteByte(0);
							buffer.WriteString(vfsStart, it - vfsStart);
							++it;
							const tchar* localStart = it;
							while (*it != ',' && *it != ']')
								++it;
							buffer.WriteString(localStart, it - localStart);
							if (*it == ']')
								break;
							++it;
							if (*it == ' ')
								++it;
						}
						rootsHandle = m_session.RegisterRoots(buffer.GetData(), buffer.GetPosition());
					}
					return true;
				}
				}
				return true;
			};

		if (!ReadLines(logger, yamlFilename, readLine))
			return false;

		if (!app.empty())
			enqueueProcess();

		return true;
	}

	bool Scheduler::EnqueueFromSpecialJson(const tchar* jsonFilename, const tchar* workingDir, const tchar* description, RootsHandle rootsHandle, void* userData)
	{
		Logger& logger = m_session.GetLogger();
		FileAccessor fa(logger, jsonFilename);
		if (!fa.OpenMemoryRead())
			return false;

		auto data = (const char*)fa.GetData();
		u64 dataLen = fa.GetSize();
		auto i = data;
		auto e = data + dataLen;
		u32 scope = 0;
		const char* stringStart = nullptr;
		std::string lastString;
		char lastChar = 0;

		struct Command { TString application; TString arguments; };
		Vector<Command> commands;

		while (i != e)
		{
			if (!stringStart)
			{
				if (*i == '{')
				{
					++scope;
				}
				else if (*i == '}')
				{
					--scope;
				}
				else if (*i == '\"' && lastChar != '\\')
				{
					stringStart = i+1;
				}
			}
			else
			{
				if (*i == '\"' && lastChar != '\\')
				{
					if (lastString == "command")
					{
						Command& command = commands.emplace_back();
						StringBuffer<2048> args;
						ParseArguments(stringStart, int(i - stringStart), [&](char* arg, u32 argLen)
						{
							// Strip out double backslash
							char* readIt = arg;
							char* writeIt = arg;
							char last = 0;
							while (true)
							{
								char c = *readIt;
								*writeIt = c;
								if (!(c == '\\' && last == '\\'))
									++writeIt;
								if (c == 0)
									break;
								++readIt;
								last = c;
							};
							u32 newLen = u32(writeIt - arg);

							if (command.application.empty())
							{
								command.application.assign(arg, arg + newLen);
								return;
							}
							if (args.count)
								args.Append(' ');
							args.Append(arg, newLen);
						});
						command.arguments = args.ToString();
					}
					lastString.assign(stringStart, int(i - stringStart));
					stringStart = nullptr;
				}
			}
			lastChar = *i;
			++i;
		}
		UBA_ASSERT(scope == 0);

		float weight = 0;
		if (userData)
		{
			auto& ei = *(ExitProcessInfo*)userData;
			weight = ei.weight;
		}

		// Return weight while running these tasks
		SCOPED_FUTEX(m_processEntriesLock, lock);
		m_activeLocalProcessWeight -= weight;
		lock.Leave();

		Event done(EventResetType_Manual);
		struct Context
		{
			Logger& logger;
			Event& done;
			Atomic<u32> counter;
		} context { logger, done, 0 };

		auto exitedFunc = [](void* userData, const ProcessHandle& ph, ProcessExitedResponse&)
			{
				auto& context = *(Context*)userData;
				if (ph.GetExitCode() != 0 && ph.GetExecutionType() != ProcessExecutionType_Skipped)
					for (auto& line : ph.GetLogLines())
						context.logger.Log(LogEntryType_Error, line.text);

				if (!--context.counter)
					context.done.Set();
			};

		for (auto& command : commands)
		{
			StringBuffer<> application(command.application);
			m_session.DevirtualizePath(application, rootsHandle);
			//StringBuffer<> logFile;
			//logFile.Appendf(L"%s_LOG_FILE_%u.log", description, context.counter.load());
			++context.counter;
			ProcessStartInfo si;
			si.application = application.data;
			si.workingDir = workingDir;
			si.arguments = command.arguments.c_str();
			si.description = description;
			si.exitedFunc = exitedFunc;
			si.userData = &context;
			si.rootsHandle = rootsHandle;
			//si.logFile = logFile.data;
			EnqueueProcess({si});
		}

		m_session.ReenableRemoteExecution();

		if (!done.IsSet(2*60*60*1000))
			logger.Error(TC("Something went wrong waiting for %s"), description);

		// Take back weight.. TODO: Should this wait for available weight before returning?
		lock.Enter();
		m_activeLocalProcessWeight += weight;
		lock.Leave();

		return true;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
