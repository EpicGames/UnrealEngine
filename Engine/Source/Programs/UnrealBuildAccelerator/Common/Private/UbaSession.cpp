// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaSession.h"
#include "UbaBinaryParser.h"
#include "UbaCompressedFileHeader.h"
#include "UbaConfig.h"
#include "UbaEnvironment.h"
#include "UbaFileAccessor.h"
#include "UbaObjectFile.h"
#include "UbaProcess.h"
#include "UbaStaticPatcher.h"
#include "UbaStorage.h"
#include "UbaDirectoryIterator.h"
#include "UbaApplicationRules.h"
#include "UbaPathUtils.h"
#include "UbaStorageUtils.h"
#include "UbaTraceChannel.h"
#include "UbaWorkManager.h"
#include <span>

#if !PLATFORM_WINDOWS
extern char **environ;
#endif

#if !PLATFORM_LINUX
#include "UbaLinuxBinDependencyParser.h"
#endif

#if UBA_USE_NATIVE_MAC_SEMAPHORES
#include <mach/mach.h>
#include <mach/semaphore.h>
#include <servers/bootstrap.h>
#endif

#define UBA_DEBUG_TRACK_WRITES 0 // UBA_DEBUG_LOGGER

//////////////////////////////////////////////////////////////////////////////

namespace uba
{
	bool g_dummy;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	ProcessStartInfo::ProcessStartInfo() = default;
	ProcessStartInfo::~ProcessStartInfo() = default;
	ProcessStartInfo::ProcessStartInfo(const ProcessStartInfo&) = default;

	const tchar* ProcessStartInfo::GetDescription() const
	{
		if (description && *description)
			return description;
		const tchar* d = application;
		if (const tchar* lps = TStrrchr(d, PathSeparator))
			d = lps + 1;
		if (const tchar* lps2 = TStrrchr(d, NonPathSeparator))
			d = lps2 + 1;
		return d;
	}

	ProcessHandle::ProcessHandle()
	:	m_process(nullptr)
	{
	}
	ProcessHandle::~ProcessHandle()
	{
		if (m_process)
			m_process->Release();
	}

	ProcessHandle::ProcessHandle(const ProcessHandle& o)
	{
		m_process = o.m_process;
		if (m_process)
			m_process->AddRef();
	}

	ProcessHandle::ProcessHandle(ProcessHandle&& o) noexcept
	{
		m_process = o.m_process;
		o.m_process = nullptr;
	}

	ProcessHandle& ProcessHandle::operator=(const ProcessHandle& o)
	{
		if (&o == this)
			return *this;
		if (o.m_process)
			o.m_process->AddRef();
		if (m_process)
			m_process->Release();
		m_process = o.m_process;
		return *this;
	}

	ProcessHandle& ProcessHandle::operator=(ProcessHandle&& o) noexcept
	{
		if (&o == this)
			return *this;
		if (o.m_process == m_process)
		{
			if (o.m_process)
			{
				o.m_process->Release();
				o.m_process = nullptr;
			}
			return *this;
		}
		if (m_process)
			m_process->Release();
		m_process = o.m_process;
		o.m_process = nullptr;
		return *this;
	}

	bool ProcessHandle::IsValid() const
	{
		return m_process != nullptr;
	}

	const ProcessStartInfo& ProcessHandle::GetStartInfo() const
	{
		UBA_ASSERT(m_process);
		return m_process->GetStartInfo();
	}
	u32 ProcessHandle::GetId() const
	{
		UBA_ASSERT(m_process);
		return m_process->GetId();
	}
	u32 ProcessHandle::GetExitCode() const
	{
		UBA_ASSERT(m_process);
		return m_process->GetExitCode();
	}
	bool ProcessHandle::HasExited() const
	{
		UBA_ASSERT(m_process);
		return m_process->HasExited();
	}
	bool ProcessHandle::WaitForExit(u32 millisecondsTimeout) const
	{
		UBA_ASSERT(m_process);
		return m_process->WaitForExit(millisecondsTimeout);
	}
	const Vector<ProcessLogLine>& ProcessHandle::GetLogLines() const
	{
		UBA_ASSERT(m_process);
		return m_process->GetLogLines();
	}
	const Vector<u8>& ProcessHandle::GetTrackedInputs() const
	{
		UBA_ASSERT(m_process);
		return m_process->GetTrackedInputs();
	}
	const Vector<u8>& ProcessHandle::GetTrackedOutputs() const
	{
		UBA_ASSERT(m_process);
		return m_process->GetTrackedOutputs();
	}
	u64 ProcessHandle::GetTotalProcessorTime() const
	{
		UBA_ASSERT(m_process);
		return m_process->GetTotalProcessorTime();
	}
	u64 ProcessHandle::GetTotalWallTime() const
	{
		UBA_ASSERT(m_process);
		return m_process->GetTotalWallTime();
	}
	u64 ProcessHandle::GetPeakMemory() const
	{
		UBA_ASSERT(m_process);
		return m_process->GetPeakMemory();
	}
	bool ProcessHandle::Cancel() const
	{
		UBA_ASSERT(m_process);
		return m_process->Cancel();
	}
	const tchar* ProcessHandle::GetExecutingHost() const
	{
		UBA_ASSERT(m_process);
		return m_process->GetExecutingHost();
	}
	bool ProcessHandle::IsRemote() const
	{
		UBA_ASSERT(m_process);
		return m_process->IsRemote();
	}

	ProcessExecutionType ProcessHandle::GetExecutionType() const
	{
		UBA_ASSERT(m_process);
		return m_process->GetExecutionType();
	}
	void ProcessHandle::TraverseOutputFiles(const Function<void(StringView file)>& func) const
	{
		UBA_ASSERT(m_process);
		m_process->TraverseOutputFiles(func);
	}
	ProcessHandle::ProcessHandle(Process* process)
	{
		m_process = process;
		process->AddRef();
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void SessionCreateInfo::Apply(const Config& config)
	{
		const ConfigTable* tablePtr = config.GetTable(TC("Session"));
		if (!tablePtr)
			return;
		const ConfigTable& table = *tablePtr;
		table.GetValueAsString(rootDir, TC("RootDir"));
		table.GetValueAsString(traceName, TC("TraceName"));
		table.GetValueAsString(traceChannelName, TC("TraceChannel"));
		table.GetValueAsString(traceOutputFile, TC("TraceOutputFile"));
		table.GetValueAsString(extraInfo, TC("ExtraInfo"));
		table.GetValueAsBool(logToFile, TC("LogToFile"));
		table.GetValueAsBool(useUniqueId, TC("UseUniqueId"));
		table.GetValueAsBool(allowCustomAllocator, TC("AllowCustomAllocator"));
		table.GetValueAsBool(allowLazyCopyOnWrite, TC("AllowLazyCopyOnWrite"));
		table.GetValueAsBool(launchVisualizer, TC("LaunchVisualizer"));
		table.GetValueAsBool(allowMemoryMaps, TC("AllowMemoryMaps"));
		table.GetValueAsBool(allowKeepFilesInMemory, TC("AllowKeepFilesInMemory"));
		table.GetValueAsBool(allowOutputFiles, TC("AllowOutputFiles"));
		table.GetValueAsBool(allowDiscardVirtualMemory, TC("AllowDiscardVirtualMemory"));
		table.GetValueAsBool(allowSpecialApplications, TC("AllowSpecialApplications"));
		table.GetValueAsBool(suppressLogging, TC("SuppressLogging"));
		table.GetValueAsBool(shouldWriteToDisk, TC("ShouldWriteToDisk"));
		table.GetValueAsBool(writePlaceholders, TC("WritePlaceholders"));
		table.GetValueAsBool(traceEnabled, TC("TraceEnabled"));
		table.GetValueAsBool(detailedTrace, TC("DetailedTrace"));
		table.GetValueAsBool(traceChildProcesses, TC("TraceChildProcesses"));
		table.GetValueAsBool(traceWrittenFiles, TC("TraceWrittenFiles"));
		table.GetValueAsInt(linkCrawlerMaxParallel, TC("LinkCrawlerMaxParallel"));
		table.GetValueAsBool(storeIntermediateFilesCompressed, TC("StoreIntermediateFilesCompressed"));
		table.GetValueAsBool(allowLocalDetour, TC("AllowLocalDetour"));
		table.GetValueAsBool(extractObjFilesSymbols, TC("ExtractObjFilesSymbols"));
		table.GetValueAsBool(useFakeVolumeSerial, TC("UseFakeVolumeSerial"));
		table.GetValueAsBool(allowLinkDependencyCrawler, TC("AllowLinkDependencyCrawler"));
		table.GetValueAsBool(dirTableSnapshotEnabled, TC("DirTableSnapshotEnabled"));
		table.GetValueAsU32(traceReserveSizeMb, TC("TraceReserveSizeMB"));
		table.GetValueAsU32(writeFilesBottleneck, TC("WriteFilesBottleneck"));
		table.GetValueAsU32(writeFilesFileMapMaxMb, TC("WriteFilesFileMapMaxMB"));
		table.GetValueAsU32(writeFilesNoBufferingMinMb, TC("WriteFilesNoBufferingMinMB"));
		table.GetValueAsU32(traceIntervalMs, TC("TraceIntervalMs"));
		table.GetValueAsU64(keepOutputFileMemoryMapsThreshold, TC("KeepOutputFileMemoryMapsThreshold"));
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool Session::WriteFileToDisk(ProcessImpl& process, WrittenFile& file)
	{
		auto& rules = *process.m_startInfo.rules;

		if (StringView(file.name).StartsWith(m_tempPath) && !rules.KeepTempOutputFile(StringView(file.name)))
		{
			m_sharedMemory.CloseHandle(m_logger, file.memoryHandle, file.backedName.c_str());
			file.memoryHandle = {};
			#if UBA_DEBUG
			m_logger.Info(TC("Skipping writing temp file %s to disk"), file.name.c_str());
			#endif
			return true;
		}

		bool isMaybeReadAfterWritten = IsMaybeReadAfterWritten(process, file.name);
		bool isBigEnoughToKeep = file.memoryWritten >= m_keepOutputFileMemoryMapsThreshold;

		bool shouldKeepInMemory = isMaybeReadAfterWritten && isBigEnoughToKeep;
		bool tryKeepInMemory = true;

		u64 writtenSize = 0;
		bool shouldWriteToDisk = ShouldWriteToDisk(file.name);

		u64 fileSize = file.memoryWritten;
		u8* mem = nullptr;
		auto memClose = MakeGuard([&, memoryHandle = file.memoryHandle]()
			{
				if (mem)
					m_sharedMemory.UnmapView(memoryHandle, mem);
			});

		#if UBA_DEBUG_TRACK_WRITES
		static u64 firstFile = [](Logger* logger)
			{
				logger->Info(TC("[%5s %8s] %6s %9s %-13s File"), TC("Wall"), TC("Total"), TC("Time"), TC("Bytes"), TC("Mode"));
				return GetTime();
			}(m_debugLogger);
		u64 start = GetTime(); 
		StringBuffer<256> writeMode;
		static Atomic<u64> allFilesSize;
		#define UBA_WRITE_MODE_BUFFER &writeMode
		#else
		#define UBA_WRITE_MODE_BUFFER nullptr
		#endif

		if (shouldWriteToDisk)
		{
			RootsHandle rootsHandle = process.GetStartInfo().rootsHandle;

			bool storeCompressed = m_storeIntermediateFilesCompressed && g_globalRules.FileCanBeCompressed(file.name);
			bool shouldDevirtualize = false;
			bool escapeSpaces = false;
			if (!storeCompressed)
				shouldDevirtualize = HasVfs(rootsHandle) && rules.ShouldDevirtualizeFile(file.name, escapeSpaces);

			#if UBA_DEBUG_TRACK_MAPPING
			m_debugLogger->Info(TC("Writing written file with mapping 0x%llx for %s"), u64(file.memoryHandle.ToU64()), file.name.c_str());
			#endif

			u32 desiredAccess = FILE_MAP_READ;
			if (m_allowDiscardVirtualMemory)
				desiredAccess |= FILE_MAP_WRITE;

			mem = m_sharedMemory.MapView(file.memoryHandle, file.name.c_str(), SharedMemoryMapType_ReadOnly);
			if (!mem)
				return m_logger.Error(TC("Failed to map view of filehandle for read %s (%s)"), file.name.c_str(), LastErrorToText().data);

			bool shouldWritePlaceholder = m_writePlaceholders && rules.SupportsPlaceholder(file.name);

			if (storeCompressed)
			{
				UBA_ASSERT(!shouldWritePlaceholder);

				#if UBA_DEBUG_TRACK_WRITES
				writeMode = TCV("Compressed");
				#endif

				Storage::WriteResult res;
				CompressedFileHeader header { CalculateCasKey(mem, fileSize, true, &m_workManager, file.name.c_str()) };

				if (!m_storage.WriteCompressedFile(res, TC("MemoryMap"), InvalidFileHandle, mem, fileSize, file.name.c_str(), &header, sizeof(header), file.lastWriteTime))
					return false;

				writtenSize = res.storedSize;

				// Can't evict without properly update filemappingtable.. the file on disk does now not match what was registered for write
				tryKeepInMemory = shouldKeepInMemory;
				shouldKeepInMemory = true;
			}
			else
			{
				FileAccessor destinationFile(m_logger, file.name.c_str());

				if (shouldDevirtualize)
				{
					#if UBA_DEBUG_TRACK_WRITES
					writeMode = TCV("Devirtualize");
					#endif

					UBA_ASSERT(!shouldWritePlaceholder);

					// Need to turn paths back into local paths

					if (!destinationFile.CreateWrite(false, DefaultAttributes(), 0, m_tempPath.data))
						return false;

					MemoryBlock block(TC("WriteFileToDiskDevirtualize"), 5*1024*1024);
					if (!DevirtualizeDepsFile(rootsHandle, block, mem, fileSize, escapeSpaces, file.name.c_str()))
						return false;

					if (!destinationFile.Write(block.memory, block.writtenSize))
						return false;
					writtenSize = block.writtenSize;
				}
				else
				{
					u64 writeSize = fileSize;
					if (shouldWritePlaceholder)
					{
						writeSize = 0; // If writing placeholders we are ok with file size 0
						shouldKeepInMemory = true; // Must keep in memory in case we want to use the cache
					}

					if (!WriteMemoryToDisk(destinationFile, mem, writeSize, UBA_WRITE_MODE_BUFFER))
						return false;
					writtenSize = writeSize;
				}

				if (u64 time = file.lastWriteTime)
					if (!SetFileLastWriteTime(destinationFile.GetHandle(), time))
						return m_logger.Error(TC("Failed to set file time on filehandle for %s"), file.name.c_str());

				if (!destinationFile.Close(file.lastWriteTime ? nullptr : &file.lastWriteTime))
					return false;
			}

			// There are directory crawlers happening in parallel so we need to really make sure to invalidate this one since a crawler can actually
			// hit this file with information from a query before it was written.. and then it will turn it back to "verified" using old info
			m_storage.InvalidateCachedFileInfo(file.key);
		}
		else
		{
			#if UBA_DEBUG_TRACK_WRITES
			writeMode = TCV("Skipped");
			#endif

			// Delete existing file to make sure it is not picked up (since it is out of date)
			uba::DeleteFileW(file.name.c_str());

			// If shouldWriteToDisk is false we can't evict from memory because we can't re-read from disk if needed
			shouldKeepInMemory = true;
		}

		memClose.Execute();

		SharedMemoryHandle memoryHandle = file.memoryHandle;
		file.memoryHandle = {};
		file.originalMemoryHandle = {};

		if (shouldKeepInMemory)
		{
			SCOPED_FUTEX(m_fileMappingTableLookupLock, lookupLock);
			auto insres = m_fileMappingTableLookup.try_emplace(file.key);
			FileMappingEntry& entry = insres.first->second;
			lookupLock.Leave();

			SCOPED_FUTEX(entry.lock, entryCs);
			if (!insres.second && entry.canBeFreed) // It could be that this file has been read as input and that is fine.
				m_logger.Error(TC("Trying to write the same file twice (%s)"), file.name.c_str());
			UBA_ASSERT(memoryHandle.IsValid());

			if (!entry.handled && !tryKeepInMemory)
			{
				shouldKeepInMemory = false;
			}
			else
			{
				// Free up unused memory since we're keeping it around
				m_sharedMemory.TrimMemory(memoryHandle, file.memoryWritten);

				bool isInvisible = rules.IsInvisible(file.name);

				entry.handled = true;
				entry.memoryHandle = memoryHandle;
				entry.memoryOffset = 0;
				entry.contentSize = file.memoryWritten;
				entry.lastWriteTime = file.lastWriteTime;
				entry.isDir = false;
				entry.success = true;
				entry.canBeFreed = true;
				entry.usedCount = 0;
				entry.usedCountBeforeFree = g_globalRules.GetUsedCountBeforeFree(file.name);
				entry.storedSize = writtenSize;
				entry.isInvisible = isInvisible;

				//m_logger.Info(TC("KEEPING %s"), file.name.c_str());
				++m_independentMappingCreated;
				++m_independentMappingActive;

				if (shouldWriteToDisk)
				{
					// This is used to really make sure file stays in memory until process has called the exit callback
					// if file is invisible it should never be 
					file.entryToReduceRefCount = &entry;
					++entry.refCount;

					// If writing to disk we only keep file in memory for 3 seconds until we evict it
					SCOPED_FUTEX(m_fileMappingsThatCanBeFreedLock, lock);
					m_fileMappingsThatCanBeFreed[file.key] = GetTime() + MsToTime(3*1000);
				}

				#if UBA_DEBUG_TRACK_MAPPING
				entry.name = file.name;
				m_debugLogger->Info(TC("Mapping kept 0x%llx (%s) from detoured process (UsedCountBeforeFree: %u)"), u64(memoryHandle.ToU64()), entry.name.c_str(), u32(entry.usedCountBeforeFree));
				#endif

				if (!isInvisible)
				{
					StringBuffer<> name;
					GetMappingString(name, memoryHandle, 0, entry.canBeFreed);

					SCOPED_WRITE_LOCK(m_fileMappingTableMemLock, lock);
					BinaryWriter writer(m_fileMappingTableMem, m_fileMappingTableSize, FileMappingTableMaxSize);
					writer.WriteStringKey(file.key);
					writer.WriteString(name);
					writer.Write7BitEncoded(file.memoryWritten);
					u32 newSize = (u32)writer.GetPosition();
					m_fileMappingTableSize = (u32)newSize;
				}
			}
		}

		if (!shouldKeepInMemory)
		{
			//m_logger.Info(TC("EVICTING %s"), file.name.c_str());

			#if UBA_DEBUG_TRACK_MAPPING
			m_debugLogger->Info(TC("Mapping evicted 0x%llx (%s)"), u64(memoryHandle.ToU64()), file.name.c_str());
			#endif

			m_sharedMemory.CloseHandle(m_logger, memoryHandle, file.name.c_str());
		}

		if (shouldWriteToDisk)
			TraceWrittenFile(process.m_id, file.name, writtenSize);

		RegisterCreateFileForWrite(StringKeyZero, file.name, writtenSize, file.lastWriteTime, file.attributes, false, true);

		#if UBA_DEBUG_TRACK_WRITES
		allFilesSize += fileSize;
		m_debugLogger->Info(TC("[%5s %8s] %6s %9s %-13s %s"), TimeToText(start-firstFile), BytesToText(allFilesSize).str, TimeToText(GetTime() - start).str, BytesToText(fileSize).str, writeMode.data, StringView(file.name).GetFileName().data);
		#endif
		#undef UBA_WRITE_MODE_BUFFER
		return true;
	}

	bool Session::WriteMemoryToDisk(FileAccessor& destinationFile, const void* fileMem, u64 fileSize, StringBufferBase* outWriteMode)
	{
		// Seems like best combo (for windows at least) is to use writes with overlap and max 16 at the same time.
		// On one machine we get twice as fast without overlap if no bottleneck. On another machine (ntfs compression on) we get twice as slow without overlap
		// Both machines behaves well with overlap AND bottleneck. Both machine are 128 logical core thread rippers.
		bool useFileMapForWrite = fileSize && fileSize <= m_writeFilesFileMapMax; // ::CreateFileMappingW does not work for zero-length files
		bool useOverlap = !useFileMapForWrite && fileSize >= m_writeFilesNoBufferingMin && EventIsNative;//fileSize > 2*1024*1024;

		#if UBA_DEBUG_TRACK_WRITES
		if (outWriteMode)
		{
			if (useOverlap)
				outWriteMode->Append(TCV("NoBuffering"));
			else if (useFileMapForWrite)
				outWriteMode->Append(TCV("MemoryMap"));
			else
				outWriteMode->Append(TCV("Normal"));
		}
		#endif

		u32 attributes = DefaultAttributes();
		if (useOverlap)
			attributes |= (FILE_FLAG_OVERLAPPED | FILE_FLAG_NO_BUFFERING);

		if (useFileMapForWrite)
		{
			if (!destinationFile.CreateMemoryWrite(false, attributes, fileSize, m_tempPath.data))
				return false;
			FileMapping_CopyMem(destinationFile.GetData(), fileMem, fileSize);
		}
		else
		{
			if (!destinationFile.CreateWrite(false, attributes, fileSize, m_tempPath.data))
				return false;
			if (!destinationFile.Write(fileMem, fileSize, 0, true))
				return false;
		}
		return true;
	}

	bool Session::GetFileMemory(const Function<bool(const void*, u64)>& func, const StringKey& fileNameKey, StringView filePath, bool deleteInternalMapping)
	{
		SharedMemoryHandle mapping;
		u64 size;
		{
			SCOPED_FUTEX(m_fileMappingTableLookupLock, lookupLock);
			auto findIt = m_fileMappingTableLookup.find(fileNameKey);
			if (findIt == m_fileMappingTableLookup.end())
				return false;
			FileMappingEntry& entry = findIt->second;
			if (deleteInternalMapping && !entry.isInvisible)
				return m_logger.Error(TC("Trying to delete mapping that is not invisible (%s)"), filePath.data);
			mapping = entry.memoryHandle;
			size = entry.contentSize;
			if (deleteInternalMapping)
			{
				UBA_ASSERT(entry.refCount == 0);
				if (!entry.isInvisible)
					return m_logger.Error(TC("Trying to delete mapping that is not invisible (%s)"), filePath.data);
				m_fileMappingTableLookup.erase(findIt);
			}
		}

		u8* mem = m_sharedMemory.MapView(mapping, TC(""));
		if (!mem)
			return false;
		bool res = func(mem, size);
		m_sharedMemory.UnmapView(mapping, mem);
		if (deleteInternalMapping)
			m_sharedMemory.CloseHandle(m_logger, mapping, TC(""));
		return res;
	}

	void Session::AddEnvironmentVariableNoLock(StringView key, StringView value)
	{
		m_environmentVariables.insert(m_environmentVariables.end(), key.data, key.data + key.count);
		m_environmentVariables.push_back('=');
		m_environmentVariables.insert(m_environmentVariables.end(), value.data, value.data + value.count);
		m_environmentVariables.push_back(0);
	}

	u32 Session::AddFileMapping(StringKey fileNameKey, const tchar* fileName, const tchar* newFileName, u64 fileSize)
	{
		UBA_ASSERT(fileNameKey != StringKeyZero);
		SCOPED_FUTEX(m_fileMappingTableLookupLock, lookupLock);
		auto insres = m_fileMappingTableLookup.try_emplace(fileNameKey);
		FileMappingEntry& entry = insres.first->second;
		lookupLock.Leave();

		SCOPED_FUTEX(entry.lock, entryCs);

		if (entry.handled)
		{
			entryCs.Leave();
			SCOPED_READ_LOCK(m_fileMappingTableMemLock, lookupCs2);
			return entry.success ? m_fileMappingTableSize : 0;
		}

		entry.contentSize = fileSize;
		entry.isDir = false;
		entry.success = true;
		entry.memoryHandle = {};
		entry.handled = true;

		#if UBA_DEBUG_TRACK_MAPPING
		entry.name = newFileName;
		#endif

		SCOPED_WRITE_LOCK(m_fileMappingTableMemLock, lock);
		BinaryWriter writer(m_fileMappingTableMem, m_fileMappingTableSize, FileMappingTableMaxSize);
		writer.WriteStringKey(fileNameKey);
		writer.WriteString(newFileName);
		writer.Write7BitEncoded(fileSize);
		u32 newSize = (u32)writer.GetPosition();
		m_fileMappingTableSize = (u32)newSize;
		return newSize;
	}

	bool Session::GetOrCreateMemoryMapFromFile(MemoryMap& out, StringKey fileNameKey, const tchar* fileName, bool isCompressedCas, u64 alignment, const tchar* hint, ProcessImpl* requestingProcess, bool canBeFreed)
	{
		TimerScope ts(Stats().waitMmapFromFile);

		StringView fileNameView = ToView(fileName);

		SCOPED_FUTEX(m_fileMappingTableLookupLock, lookupLock);
		auto insres = m_fileMappingTableLookup.try_emplace(fileNameKey);
		FileMappingEntry& entry = insres.first->second;
		lookupLock.Leave();


		auto updateRequestingProcess = [&]()
			{
				if (requestingProcess && entry.canBeFreed && !m_runningRemote)
				{
					ProcessImpl::UsedFileMapping usedFileMapping { requestingProcess->m_startInfo.rules->CloseFileMappingAfterUse(fileNameView) };
					SCOPED_FUTEX(requestingProcess->m_usedFileMappingsLock, lock);
					if (!requestingProcess->m_hasExited && requestingProcess->m_usedFileMappings.insert({fileNameKey, usedFileMapping}).second)
						++entry.refCount;
				}
			};

		SCOPED_FUTEX(entry.lock, entryLock);

		if (requestingProcess && (requestingProcess->m_gotExitMessage || requestingProcess->m_cancelled)) // This is a crawler call that we can ignore
			return false;

		if (entry.handled)
		{
			if (!entry.success)
				return false;
			out.size = entry.contentSize;
			if (entry.memoryHandle.IsValid())
			{
				GetMappingString(out.name, entry.memoryHandle, entry.memoryOffset, entry.canBeFreed);
				updateRequestingProcess();
			}
			else
				out.name.Append(entry.isDir ? TC("$d") : TC("$f"));
			return true;
		}

		ts.Cancel();
		TimerScope ts2(Stats().createMmapFromFile);

		out.size = 0;

		entry.handled = true;

		bool isDir = false;
		u32 attributes = DefaultAttributes() | FILE_FLAG_SEQUENTIAL_SCAN;
		FileHandle fileHandle = uba::CreateFileW(fileName, GENERIC_READ, FILE_SHARE_READ, OPEN_EXISTING, attributes);
		if (fileHandle == InvalidFileHandle)
		{
			u32 error = GetLastError();
			if (error == ERROR_ACCESS_DENIED || error == ERROR_PATH_NOT_FOUND) // Probably directory? .. path not found can be returned if path is the drive ('e:\' etc)
			{
				fileHandle = uba::CreateFileW(fileName, 0, 0x00000007, 0x00000003, FILE_FLAG_BACKUP_SEMANTICS);
				if (fileHandle == InvalidFileHandle)
					return m_logger.Error(TC("Failed to open file %s (%s)"), fileName, LastErrorToText().data);

				isDir = true;
			}
			else
				return m_logger.Error(TC("Failed to open file %s (%s)"), fileName, LastErrorToText().data);
		}
		auto _ = MakeGuard([&](){ uba::CloseFile(fileName, fileHandle); });

		u64 storedSize = InvalidValue;
		u64 contentSize = 0;
		u64 lastWriteTime = 0;
		u64 fileStartOffset = 0;

		bool isCompressed = isCompressedCas;
		if (!isDir)
		{
			if (isCompressedCas)
			{
				if (!ReadFile(m_logger, fileName, fileHandle, &contentSize, sizeof(u64)))
					return m_logger.Error(TC("Failed to read first bytes from file %s (%s)"), fileName, LastErrorToText().data);

				if (contentSize > InvalidValue)
					return m_logger.Error(TC("Compressed cas has content size larger than %s. File %s is corrupt"), BytesToText(InvalidValue).str, fileName);
			}
			else
			{
				FileBasicInformation info;
				if (!GetFileBasicInformationByHandle(info, m_logger, fileName, fileHandle))
					return false;

				storedSize = info.size;
				contentSize = info.size;
				lastWriteTime = info.lastWriteTime;

				if (m_readIntermediateFilesCompressed && info.size > sizeof(CompressedFileHeader) && g_globalRules.FileCanBeCompressed(fileNameView))
				{
					CompressedFileHeader header(CasKeyZero);
					if (!ReadFile(m_logger, fileName, fileHandle, &header, sizeof(header)))
						return m_logger.Error(TC("Failed to read header of compressed file %s (%s)"), fileName, LastErrorToText().data);
					if (header.IsValid())
					{
						fileStartOffset = sizeof(CompressedFileHeader);
						isCompressed = true;
						if (!ReadFile(m_logger, fileName, fileHandle, &contentSize, sizeof(u64)))
							return m_logger.Error(TC("Failed to read first bytes from file %s (%s)"), fileName, LastErrorToText().data);
						if (contentSize > InvalidValue)
							return m_logger.Error(TC("Compressed cas has content size larger than %s. File %s is corrupt"), BytesToText(InvalidValue).str, fileName);
					}
					else
					{
						if (!SetFilePointer(m_logger, fileName, fileHandle, 0))
							return false;
					}
				}
			}
		}

		if (isDir)
			out.name.Append(TCV("$d"));
		else if (contentSize == 0)
			out.name.Append(TCV("$f"));
#if 0
		else if (isCompressed && fileNameView.EndsWith(TCV(".obj")) && m_sharedMemory.GetUsed() > 16ull*1024*1024*1024)
		{
			m_logger.Warning(TC("SharedMemory over threshold, off-loading memory mappings to disk (%s - %llu)"), fileName, contentSize);
			Guid g;
			CreateGuid(g);
			StringBuffer<> tempFileName(m_tempPath);
			tempFileName.Append(GuidToString(g));
			FileAccessor tempFile(m_logger, tempFileName.data);
			if (!tempFile.CreateMemoryWrite(false, FILE_ATTRIBUTE_TEMPORARY|FILE_FLAG_DELETE_ON_CLOSE, contentSize))
				return false;
			u8* tempFileMem = tempFile.GetData();
			if (!m_storage.DecompressFileToMemory(fileName, fileHandle, tempFileMem, contentSize, TC("GetOrCreateMemoryMapFromFile"), fileStartOffset))
				return false;
			
			FileHandle tempFileHandle;
			if (!DuplicateHandle(GetCurrentProcess(), (HANDLE)tempFile.GetHandle(), GetCurrentProcess(), (HANDLE*)&tempFileHandle, 0, false, DUPLICATE_SAME_ACCESS))
				return false;
			if (tempFileHandle == InvalidFileHandle)
				return false;
			if (!tempFile.Close())
				return false;
			//FileHandle tempFileHandle;
			//if (!OpenFileSequentialRead(m_logger, tempFileName.data, tempFileHandle))
			//	return false;
			FileMappingHandle mappingHandle = FileMapping_CreateFromFile(m_logger, tempFileHandle, PAGE_READONLY, contentSize, tempFileName.data);
			if (!mappingHandle.IsValid())
				return false;
			//uba::CloseFile(tempFileName.data, tempFileHandle); // it seems like this can't be closed because then the mapping might go away
			entry.memoryHandle = m_sharedMemory.RegisterExternalMapping(mappingHandle);
			GetMappingString(out.name, entry.memoryHandle, 0, false);
		}
#endif
		else
		{
			SharedMemoryHandle memoryHandle;
			u64 memoryOffset = 0;
			u8* memory = nullptr;

			MappedView mappedView;

			auto ownedGuard1 = MakeGuard([&]() { m_sharedMemory.CloseHandle(m_logger, memoryHandle, fileName); });
			auto ownedGuard2 = MakeGuard([&]() { m_sharedMemory.UnmapView(memoryHandle, memory); });
			auto viewGuard = MakeGuard([&]() { m_fileMappingBuffer.UnmapView(mappedView, fileName); });

			if (canBeFreed)
			{
				viewGuard.Cancel();

				memoryHandle = m_sharedMemory.CreateHandle(m_logger, fileName);
				if (!memoryHandle.IsValid())
					return false;
				m_sharedMemory.ExtendMemory(memoryHandle, contentSize, fileName, false);
				memory = m_sharedMemory.MapView(memoryHandle, fileName, SharedMemoryMapType_ReadWrite);
				if (!memory)
					return false;

				// TODO: THIS IS THE ALLOCATED obj files etc
				//m_logger.Info(TC("MAPPING %s"), fileName);
				++m_independentMappingCreated;
				++m_independentMappingActive;
			}
			else
			{
				UBA_ASSERTF(alignment, TC("No alignment set when creating memory map for %s (%s)"), fileName, hint);
				//UBA_ASSERTF(!entry.canBeFreed, TC("Found entry %s which is set as canBeFreed but is not opened as canBeFreed (%s)"), fileName, hint);
				ownedGuard1.Cancel();
				ownedGuard2.Cancel();
				mappedView = m_fileMappingBuffer.AllocAndMapView(contentSize, alignment, fileName);
				if (!mappedView.memory)
					return false;
				memoryHandle = mappedView.handle;
				memoryOffset = mappedView.offset;
				memory = mappedView.memory;
			}

			if (isCompressed)
			{
				if (!m_storage.DecompressFileToMemory(fileName, fileHandle, memory, contentSize, TC("GetOrCreateMemoryMapFromFile"), fileStartOffset))
					return false;
			}
			else
			{
				if (!ReadFile(m_logger, fileName, fileHandle, memory, contentSize))
					return false;
			}

			ownedGuard2.Execute();
			ownedGuard1.Cancel();
			viewGuard.Execute();

			entry.canBeFreed = canBeFreed;
			entry.memoryOffset = memoryOffset;
			entry.memoryHandle = memoryHandle;
			GetMappingString(out.name, memoryHandle, memoryOffset, entry.canBeFreed);

			if (canBeFreed)
			{
				UBA_ASSERT(!memoryHandle.IsValid() || m_sharedMemory.GetRefCount(memoryHandle));

				entry.usedCount = 0;
				entry.usedCountBeforeFree = g_globalRules.GetUsedCountBeforeFree(fileNameView);
			}

			updateRequestingProcess();
		}

		entry.success = true;

		{
			SCOPED_WRITE_LOCK(m_fileMappingTableMemLock, lock);
			BinaryWriter writer(m_fileMappingTableMem, m_fileMappingTableSize, FileMappingTableMaxSize);
			writer.WriteStringKey(fileNameKey);
			writer.WriteString(out.name);
			writer.Write7BitEncoded(contentSize);
			m_fileMappingTableSize = (u32)writer.GetPosition();
		}

		#if UBA_DEBUG_TRACK_MAPPING
		entry.name = fileName;
		m_debugLogger->Info(TC("Mapping created 0x%llx (%s) from file (%s) - %s"), u64(entry.memoryHandle.ToU64()), entry.name.c_str(), hint, TimeToText(GetTime() - ts2.start).str);
		#endif

		entry.isDir = isDir;
		entry.contentSize = contentSize;
		entry.storedSize = storedSize;
		entry.lastWriteTime = lastWriteTime;
		
		out.size = contentSize;
		return true;
	}

	bool Session::GetOrCreateMemoryMapFromStorage(MemoryMap& out, StringKey fileNameKey, const tchar* fileName, const CasKey& casKey, u64 alignment)
	{
		//StringBuffer<> workName;
		//u32 len = TStrlen(fileName);
		//workName.Append(TCV("MM:")).Append(len > 30 ? fileName + (len - 30) : fileName);
		//TrackWorkScope tws(m_workManager, workName.data);

		SCOPED_FUTEX(m_fileMappingTableLookupLock, lookupLock);
		auto insres = m_fileMappingTableLookup.try_emplace(fileNameKey);
		FileMappingEntry& entry = insres.first->second;
		lookupLock.Leave();

		SCOPED_FUTEX(entry.lock, entryCs);

		if (entry.handled)
		{
			entryCs.Leave();
			if (!entry.success)
				return false;
			out.size = entry.contentSize;
			if (entry.memoryHandle.IsValid())
				GetMappingString(out.name, entry.memoryHandle, entry.memoryOffset, entry.canBeFreed);
			else
				out.name.Append(entry.isDir ? TC("$d") : TC("$f"));
			return true;
		}

		out.size = 0;

		entry.handled = true;

		MappedView2 mappedViewRead = m_storage.MapView(casKey, fileName);
		if (!mappedViewRead.handle.IsValid())
			return false;

		u64 contentSize = InvalidValue;

		if (mappedViewRead.isCompressed)
		{
			auto mvrg = MakeGuard([&](){ m_storage.UnmapView(mappedViewRead, fileName); });
			const u8* readMemory = mappedViewRead.memory;
			contentSize = *(u64*)readMemory;
			readMemory += 8;

			if (contentSize == 0)
			{
				out.name.Append(TCV("$f"));
			}
			else
			{
				auto mappedViewWrite = m_fileMappingBuffer.AllocAndMapView(contentSize, alignment, fileName);
				if (!mappedViewWrite.memory)
					return false;
				auto unmapGuard = MakeGuard([&](){ m_fileMappingBuffer.UnmapView(mappedViewWrite, fileName); });

				if (!m_storage.DecompressMemoryToMemory(readMemory, mappedViewRead.size, mappedViewWrite.memory, contentSize, fileName, TC("TransientMapping")))
					return false;
				unmapGuard.Execute();

				entry.memoryOffset = mappedViewWrite.offset;
				entry.memoryHandle = mappedViewWrite.handle;
				GetMappingString(out.name, mappedViewWrite.handle, mappedViewWrite.offset, entry.canBeFreed);
			}
			mvrg.Execute();
		}
		else
		{
			UBA_ASSERT(mappedViewRead.memory == nullptr);
			entry.memoryOffset = mappedViewRead.offset;
			GetMappingString(out.name, mappedViewRead.handle, mappedViewRead.offset, entry.canBeFreed);
			entry.memoryHandle = mappedViewRead.handle;
			contentSize = mappedViewRead.size;
		}
		entry.success = true;

		{
			SCOPED_WRITE_LOCK(m_fileMappingTableMemLock, lock);
			BinaryWriter writer(m_fileMappingTableMem, m_fileMappingTableSize, FileMappingTableMaxSize);
			writer.WriteStringKey(fileNameKey);
			writer.WriteString(out.name);
			writer.Write7BitEncoded(contentSize);
			m_fileMappingTableSize = (u32)writer.GetPosition();
		}

		entry.isDir = false;
		entry.contentSize = contentSize;
		
		out.size = contentSize;

		#if UBA_DEBUG_TRACK_MAPPING
		entry.name = fileName;
		m_debugLogger->Info(TC("Mapping created 0x%llx (%s) from view"), u64(entry.memoryHandle.ToU64()), entry.name.c_str());
		#endif

		return true;
	}

	void Session::ThreadGcFileMappings()
	{
		while (!m_fileMappingGcDone.IsSet(1000))
		{
			u64 now = GetTime();
			SCOPED_FUTEX(m_fileMappingsThatCanBeFreedLock, lock);

			for (auto i=m_fileMappingsThatCanBeFreed.begin(), e=m_fileMappingsThatCanBeFreed.end(); i!=e;)
			{
				auto inc = MakeGuard([&]() { ++i;  });
				if (i->second > now)
					continue;

				SCOPED_FUTEX(m_fileMappingTableLookupLock, lookupLock);
				auto findIt = m_fileMappingTableLookup.find(i->first);
				UBA_ASSERT(findIt != m_fileMappingTableLookup.end());
				FileMappingEntry& entry = findIt->second;
				lookupLock.Leave();

				SCOPED_FUTEX(entry.lock, entryLock);
				if (entry.refCount)
					continue;

				if (entry.memoryHandle.IsValid() && entry.canBeFreed)
				{
					// There is a chance something might be evicted and re-added in between and when reloaded, loaded into non-freeable memory
					if (entry.canBeFreed)
					{
						m_sharedMemory.CloseHandle(m_logger, entry.memoryHandle, TC("GarbageCollect"));
						--m_independentMappingActive;
						entry.handled = false;
						entry.memoryHandle = {};
					}
				}

				inc.Cancel();

				i = m_fileMappingsThatCanBeFreed.erase(i);
				e = m_fileMappingsThatCanBeFreed.end();
			}
		}
	}

	bool Session::PopulateOverlayFiles(DirectoryTableOverlay& overlay, StringView overlayFiles)
	{
		#if PLATFORM_WINDOWS
		auto getFileInformation = [&](FileInformation& out, StringKey fileNameKey, StringView)
			{
				SCOPED_FUTEX(m_fileMappingTableLookupLock, mappingLookupLock);
				auto findIt = m_fileMappingTableLookup.find(fileNameKey);
				if (findIt == m_fileMappingTableLookup.end())
					return false;
				FileMappingEntry& entry = findIt->second;
				mappingLookupLock.Leave();
				out.size = entry.contentSize;
				out.lastWriteTime = entry.lastWriteTime;
				out.attributes = DefaultAttributes();
				out.volumeSerialNumber = 0;
				out.index = ++m_fileIndexCounter;
				return true;
			};
		#else
		// TODO: For now we don't support virtual files.... just use regular file system
		GetOverlayFileInformation getFileInformation = {};
		#endif

		tchar* it = const_cast<tchar*>(overlayFiles.data);
		tchar* end = it + overlayFiles.count;
		while (*it)
		{
			tchar* fileEnd = TStrchr(it, ';');
			if (fileEnd)
				*fileEnd = 0;
			else
				fileEnd = end;
			StringView filePath(it, u32(fileEnd - it));
			if (!PopulateOverlayFile(overlay, filePath, getFileInformation))
				return m_logger.Error(TC("PopulateOverlayFiles: file %.*s is not in the file mapping table - caller must RegisterVirtualFile/CreateVirtualFile before populating the overlay"), filePath.count, filePath.data);
			if (fileEnd == end)
				break;
			*fileEnd = ';';
			it = ++fileEnd;
		}
		return true;
	}

	RootsHandle Session::RegisterRoots(const void* rootsData, uba::u64 rootsDataSize)
	{
		CasKey key = ToCasKey(CasKeyHasher().Update(rootsData, rootsDataSize), false);
		RootsHandle rootsHandle = WithVfs(key.a, false);

		SCOPED_FUTEX(m_rootsLookupLock, rootsLock);
		RootsEntry& entry = m_rootsLookup.try_emplace(rootsHandle).first->second;
		rootsLock.Leave();

		SCOPED_FUTEX(entry.lock, entryLock);
		UBA_ASSERT(!entry.handled || memcmp(entry.memory.data(), rootsData, rootsDataSize) == 0);

		if (!entry.handled)
		{
			PopulateRootsEntry(entry, rootsData, rootsDataSize);
			entry.handled = true;
		}

		return WithVfs(rootsHandle, !entry.locals.empty());
	}

	bool Session::CopyImports(BinaryAndDeps& out, const tchar* library, tchar* applicationDir, tchar* applicationDirEnd, UnorderedSet<TString>& handledImports, const tchar* const* loaderPaths, bool isLinuxBinary, const tchar* hint)
	{
		if (!handledImports.insert(library).second)
			return true;
		TSprintf_s(applicationDirEnd, 512 - (applicationDirEnd - applicationDir), TC("%s"), library);
		const tchar* applicationName = applicationDir;
		u32 attr = GetFileAttributesW(applicationName); // TODO: Use attributes table
		tchar temp[512];
		tchar temp2[512];
		StringBuffer<512> temp3;
		bool result = true;

		if (attr == INVALID_FILE_ATTRIBUTES)
		{
#if PLATFORM_WINDOWS
			if (!SearchPathW(NULL, library, NULL, 512, temp, NULL))
				return true; // TODO: We have to return true here because there are scenarios where failing is actually ok (it seems it can return false on crt shim libraries such as api-ms-win-crt*)
#else
			u32 loaderPathCount = 0;
			auto handleLoaderPath = [&](StringView applicationPath, const char* loaderPath)
				{
					++loaderPathCount;
					StringBuffer<> absolutePath;
					if (loaderPath[0] == '/')
						absolutePath.Append(loaderPath);
					else
						absolutePath.Append(applicationPath).Append(loaderPath);
					absolutePath.EnsureEndsWithSlash().Append(library);
					FixPath(absolutePath.data, nullptr, 0, temp3.Clear());
					attr = GetFileAttributesW(temp3.data);
					if (attr == INVALID_FILE_ATTRIBUTES)
						return false;
					memcpy(temp, temp3.data, temp3.count+1);
					return true;
				};

			if (loaderPaths)
			{
				StringView applicationPath(applicationDir, u32(applicationDirEnd - applicationDir));
				for (auto it = loaderPaths; *it; ++it)
					if (handleLoaderPath(applicationPath, *it))
						break;
			}

			// TODO: Linux has hard coded paths. should probably parse /etc/ld.so.conf.d/*.conf and get the paths from there instead
			#if PLATFORM_LINUX
			const char* hardcodedPaths[] = { "/lib", "/usr/lib", "/lib64", "/usr/lib64", "/lib/x86_64-linux-gnu", "/usr/lib/x86_64-linux-gnu/" };
			if (attr == INVALID_FILE_ATTRIBUTES)
				for (auto hardcodedPath : hardcodedPaths)
					if (handleLoaderPath({}, hardcodedPath))
						break;
			#endif

			if (attr == INVALID_FILE_ATTRIBUTES)
			{
				LogEntryType logType = IsLinux ? LogEntryType_Warning : LogEntryType_Error;
				m_logger.Logf(logType, "CopyImports - Failed to find file %s requested by %s (%u loader paths)", applicationName, hint, loaderPathCount);
				return logType != LogEntryType_Error;
			}

			// TODO: We need to check if this is a symlink and resolve it in that case!!!
			// ... or will this resolve itself because whatever the symlink points to will be transferred
			// ... but in that case, will the "binary dir" be wrong?
			#if 0 
			struct stat st;
			if (lstat(outRealApplication.data, &st) == 0 && S_ISLNK(st.st_mode))
			{
				char target[1024];
				ssize_t n = readlink(outRealApplication.data, target, sizeof(target) - 1);
				if (n > 0)
					target[n] = 0;
				else
					strcpy(target, "?");
				m_logger.Logf(LogEntryType_Warning,
					TC("Remote-mode application '%s' is a symlink (-> '%s'). The detour returns the build-view path for /proc/self/exe in remote mode; tools that strip a basename to compute their install root (Java's libjava.so lookup, others) will see the symlink and likely fail. Fix in this case requires propagating the host's realpath() result through the Init RPC so the detour can synthesize the resolved path."),
					outRealApplication.data, target);
			}
			#endif
#endif

			applicationName = temp;
			attr = DefaultAttributes();

			tchar* lastSlash = TStrrchr(temp, PathSeparator);
			UBA_ASSERTF(lastSlash, TC("No slash found in path %s"), temp);
			u64 applicationDirLen = u64(lastSlash + 1 - temp);
			memcpy(temp2, temp, applicationDirLen * sizeof(tchar));
			applicationDir = temp2;
			applicationDirEnd = temp2 + applicationDirLen;
		}
		else
		{
			#if PLATFORM_WINDOWS
			attr = DefaultAttributes();
			#endif
		}

		FixPath(applicationName, nullptr, 0, temp3.Clear());

		bool isInSystemFolder = StartsWith(applicationName, m_systemPath.data);
		if (isInSystemFolder && IsKnownSystemFile(applicationName))
			return true;

		#if PLATFORM_WINDOWS
		if (isLinuxBinary)
			temp3.Replace(PathSeparator, NonPathSeparator);
		#endif

		out.modules.emplace_back(library, temp3.ToString(), attr, isInSystemFolder);

		// Statically-linked ELFs have no PT_DYNAMIC and no dependencies to
		// ship — skip ParseBinary entirely, which would fail with "Failed
		// to parse binary for imports". The static-detour patching is
		// handled later (Session::PrepareProcess locally; UbaSessionServer
		// on the agent-mode path before shipping the binary to a client).
		#if PLATFORM_LINUX
		{
			FileAccessor sourceFile(m_logger, temp3.data);
			if (!sourceFile.OpenMemoryRead())
				return false;
			if (BinaryRequiresPatching(m_logger, temp3, sourceFile.GetData(), sourceFile.GetSize()))
				return true;
		}
		#endif

		StringBuffer<> errorStr;
		BinaryInfo info;
		auto handleImport = [&](const tchar* importName, bool isKnown, const tchar* const* importLoaderPaths)
			{
				if (result && !isKnown)
					result = CopyImports(out, importName, applicationDir, applicationDirEnd, handledImports, importLoaderPaths, isLinuxBinary, hint);
			};

		bool parseRes;
		// $ORIGIN for RUNPATH: realpath the binary so symlinks resolve to lib parent (ld.so semantics).
		StringView originalDir(applicationDir, u32(applicationDirEnd - applicationDir));
		StringBuffer<512> realDir;
		#if PLATFORM_LINUX
		{
			char realBin[512];
			if (realpath(temp3.data, realBin))
			{
				if (const tchar* lastSlash = TStrrchr(realBin, PathSeparator))
				{
					u32 dirLen = u32((lastSlash - realBin) + 1);
					realDir.Append(realBin, dirLen);
					originalDir = StringView(realDir.data, realDir.count);
				}
			}
		}
		#endif
		#if !PLATFORM_LINUX
		if (isLinuxBinary)
			parseRes = linux::ParseBinary(temp3, originalDir, info, handleImport, errorStr);
		else
		#endif
			parseRes = ParseBinary(temp3, originalDir, info, handleImport, errorStr);

		if (!parseRes)
		{
			#if PLATFORM_LINUX
			if (out.modules.size() == 1)
			{
				// This can be a script using a shebang.. in that case we need to transfer the binaries it is using
				FileAccessor readFile(m_logger, temp3.data);
				if (readFile.OpenMemoryRead())
				{
					auto mem = (const u8*)readFile.GetData();
					u64 fileSize = readFile.GetSize();
					if (fileSize > 2 && mem[0] == '#' && mem[1] == '!')
					{
						u64 i=2;
						for (; i<fileSize && i<132; ++i)
							if (mem[i] == '\n')
								break;
						StringView shebang((const char*)mem+2, u32(i-2));
						if (shebang.StartsWith("/usr/bin/env"))
							shebang = shebang.Skip(13).Trim();
							
						// Strip args after interpreter path.
						u32 sp = 0;
						while (sp < shebang.count && shebang.data[sp] != ' ' && shebang.data[sp] != '\t')
							++sp;
						shebang = StringView(shebang.data, sp);

						out.modules.clear();
						out.isShebang = true;
						if (IsAbsolutePath(shebang))
							temp3.Clear().Append(shebang);
						else if (!m_searchPathCache.SearchPathForFile(m_logger, temp3.Clear(), shebang, StringView(applicationDir, u32(applicationDirEnd - applicationDir)), {}))
							return m_logger.Error(TC("Failed to find binary for shebang %.*s"), shebang.count, shebang.data);
						readFile.Close();
						handledImports.clear();
						return CopyImports(out, temp3.data, temp, temp, handledImports, nullptr, isLinuxBinary, hint);
					}
				}
			}
			#endif

			return m_logger.Error(TC("Failed to parse binary %s for imports%s"), temp3.data, isLinuxBinary ? TC(" (using linux binary parser)") : TC(""));
		}

		if (errorStr.count)
			return m_logger.Error(errorStr.data);

		#if PLATFORM_MAC
		UBA_ASSERT(!out.modules.empty());
		out.modules[0].minOsVersion = info.minVersion;
		#endif

		// This code is needed if application is compiled with tsan
		//strcpy(applicationDirEnd, "libclang_rt.tsan.so");
		//out.modules.push_back({ "libclang_rt.tsan.so", applicationDir, S_IRUSR | S_IWUSR });
		return result;
	}

	u32 GetCrawlerBottleneck(bool runningRemote, int linkCrawlerMaxParallel)
	{
		if (runningRemote)
			return ~0u;
		if (linkCrawlerMaxParallel <= 0)
			return u32(Max(0, int(GetLogicalProcessorCount()) + linkCrawlerMaxParallel));
		else
			return u32(Min(linkCrawlerMaxParallel, int(GetLogicalProcessorCount())));
	}

	Session::Session(const SessionCreateInfo& info, const tchar* logPrefix, bool runningRemote, WorkManager& workManager)
	:	DirectoryTableHolder(info.logWriter, logPrefix, runningRemote)
	,	m_storage(info.storage)
	,	m_sharedMemory(info.storage.GetSharedMemory())
	,	m_workManager(workManager)
	,	m_ownsTrace(info.trace == nullptr)
	,	m_fileMappingBuffer(m_logger, m_sharedMemory)
	,	m_nameToHashTableMem(TC("SessionNameToHashTable"))
	,	m_processCommunicationAllocator(m_logger, TC("CommunicationAllocator"))
	,	m_processMessageThreads(*new WorkManagerImpl(~0u, TC("UbaProcMsg")))
	,	m_trace(info.trace ? *info.trace : *new Trace(info.logWriter))
	,	m_writeFilesBottleneck(info.writeFilesBottleneck)
	,	m_writeFilesFileMapMax(u64(info.writeFilesFileMapMaxMb)*1024*1024)
	,	m_writeFilesNoBufferingMin(u64(info.writeFilesNoBufferingMinMb)*1024*1024)
	,	m_dependencyCrawler(m_logger, workManager, GetCrawlerBottleneck(runningRemote, info.linkCrawlerMaxParallel))
	{
		UBA_ASSERTF(info.rootDir && *info.rootDir, TC("No root dir set when creating session"));
		m_rootDir.count = GetFullPathNameW(info.rootDir, m_rootDir.capacity, m_rootDir.data, NULL);
		m_rootDir.Replace('/', PathSeparator).EnsureEndsWithSlash();

		m_sessionDir.Append(m_rootDir).Append(TCV("sessions")).Append(PathSeparator);
		if (info.useUniqueId)
		{
			u32 retryIndex = 0;
			while (true)
			{
				time_t rawtime;
				time(&rawtime);
				tm ti;
				localtime_s(&ti, &rawtime);
				m_id.Appendf(TC("%02i%02i%02i_%02i%02i%02i"), ti.tm_year - 100,ti.tm_mon+1,ti.tm_mday, ti.tm_hour, ti.tm_min, ti.tm_sec);
				if (retryIndex > 0)
					m_id.Append('_').AppendValue(retryIndex);
				m_sessionDir.Append(m_id);
				bool alreadyExists;
				if (m_storage.CreateDirectory(m_sessionDir.data, &alreadyExists) && !alreadyExists)
					break;
				m_sessionDir.Resize(m_sessionDir.count - m_id.count);
				m_id.Clear();
				++retryIndex;
			}
		}
		else
		{
			m_id.Append(TCV("Debug"));
			m_sessionDir.Append(m_id);
		}
		m_sessionDir.Append(PathSeparator);

		m_allowCustomAllocator = info.allowCustomAllocator;
		m_allowLazyCopyOnWrite = info.allowLazyCopyOnWrite;
		m_allowMemoryMaps = info.allowMemoryMaps && IsWindows;
		m_allowKeepFilesInMemory = info.allowKeepFilesInMemory && IsWindows;
		m_allowOutputFiles = info.allowOutputFiles && IsWindows;
		m_allowSpecialApplications = info.allowSpecialApplications;
		m_suppressLogging = info.suppressLogging;
		if (!info.allowMemoryMaps)
			m_keepOutputFileMemoryMapsThreshold = 0;
		else
			m_keepOutputFileMemoryMapsThreshold = info.keepOutputFileMemoryMapsThreshold;
		m_shouldWriteToDisk = info.shouldWriteToDisk;
		UBA_ASSERTF(m_shouldWriteToDisk || m_allowMemoryMaps, TC("Can't disable both should write to disk and allow memory maps"));
		m_writePlaceholders = info.writePlaceholders && m_allowMemoryMaps;

		m_storeIntermediateFilesCompressed = info.storeIntermediateFilesCompressed && IsWindows; // Non-windows not implemented (yet)
		m_readIntermediateFilesCompressed = (m_storeIntermediateFilesCompressed || (info.readIntermediateFilesCompressed && IsWindows)) && !runningRemote; // with remote we decompress the files into memory
		m_allowLocalDetour = info.allowLocalDetour;
		m_extractObjFilesSymbols = info.extractObjFilesSymbols;
		m_allowLinkDependencyCrawler = info.allowLinkDependencyCrawler && m_allowMemoryMaps;
		m_allowDiscardVirtualMemory = info.allowDiscardVirtualMemory;
		m_dirTableSnapshotEnabled = info.dirTableSnapshotEnabled;

		m_detailedTrace = info.detailedTrace;
		m_traceChildProcesses = info.traceChildProcesses;
		m_traceWrittenFiles = info.traceWrittenFiles;
		m_logToFile = info.logToFile;
		if (info.extraInfo)
			m_extraInfo = info.extraInfo;

		if (info.deleteSessionsOlderThanSeconds)
		{
			StringBuffer<> sessionsDir;
			sessionsDir.Append(m_rootDir).Append(TCV("sessions"));

			u64 systemTimeAsFileTime = GetSystemTimeAsFileTime();

			TraverseDir(m_logger, sessionsDir,
				[&](const DirectoryEntry& e)
				{
					u64 seconds = GetFileTimeAsSeconds(systemTimeAsFileTime - e.lastWritten);
					if (seconds <= info.deleteSessionsOlderThanSeconds)
						return;

					if (IsDirectory(e.attributes)) // on macos we get a ".ds_store" file created by the os
					{
						StringBuffer<> sessionDir(sessionsDir);
						sessionDir.EnsureEndsWithSlash().Append(e.name);
						DeleteAllFiles(m_logger, sessionDir.data);
					}
				});
		}

		m_sessionBinDir.Append(m_sessionDir).Append(TCV("bin"));
		m_sessionOutputDir.Append(m_sessionDir).Append(TCV("output"));
		m_sessionLogDir.Append(m_sessionDir).Append(TCV("log"));

		if (m_runningRemote)
		{
			m_storage.CreateDirectory(m_sessionBinDir.data);
			m_storage.CreateDirectory(m_sessionOutputDir.data);
		}

		m_tempPath.Append(m_sessionDir).Append(TCV("temp"));
		m_storage.CreateDirectory(m_tempPath.data);
		m_tempPath.EnsureEndsWithSlash();

		m_sessionBinDir.EnsureEndsWithSlash();
		m_sessionOutputDir.EnsureEndsWithSlash();

		m_storage.CreateDirectory(m_sessionLogDir.data);
		m_sessionLogDir.EnsureEndsWithSlash();

		if (info.treatTempDirAsEmpty)
			SetTreatTempDirAsEmpty();

		if (info.traceOutputFile)
			m_traceOutputFile = info.traceOutputFile;

		if (m_readIntermediateFilesCompressed && !m_runningRemote)
			m_dependencyCrawler.Init(
				[this](const StringView& fileName, u32& outAttr) { return FileExists(m_logger, fileName.data, nullptr, &outAttr); }, // FileExists
				[this](const StringView& path, const DependencyCrawler::FileFunc& fileFunc) {}, // TraverseFilesFunc
				false);

	}

	bool Session::Create(const SessionCreateInfo& info)
	{
		#if UBA_DEBUG_LOGGER
		CreateDebugLogger(StringBuffer<512>().Append(m_sessionDir).Append(TCV("SessionDebug.log")).data);
		#endif

		#if PLATFORM_WINDOWS
		m_systemPath.count = GetEnvironmentVariableW(TC("SystemRoot"), m_systemPath.data, m_systemPath.capacity);
		#else
		m_systemPath.Append(TCV("/nonexistingpath"));
		#endif

		m_fileMappingTableHandle = FileMapping_Create(m_logger, PAGE_READWRITE, FileMappingTableMaxSize, nullptr, TC("FileMappings"));
		UBA_ASSERT(m_fileMappingTableHandle.IsValid());
		if (!m_fileMappingTableHandle.IsValid())
			return false;

		m_fileMappingTableMem = FileMapping_Map(m_logger, m_fileMappingTableHandle, FILE_MAP_WRITE, 0, FileMappingTableMaxSize, m_allowDiscardVirtualMemory);
		LockMemory(m_logger, m_fileMappingTableMem, FileMappingTableMaxSize, TC("FileMappings"));
		UBA_ASSERT(m_fileMappingTableMem);
		m_fileMappingTableLookup.reserve(70000);

		u64 capacity = 92ull*1024*1024*1024;
		m_fileMappingBuffer.Init(TC("FileDataNoDelete"), capacity, 1ull*1024*1024); // 32 slots to be able to add multiple files in parallel

		u64 maxSlotCount = IsWindows ? 512ull : 4096ull; // 512 slots (64 MB on Linux) was too tight for big concurrent AOSP builds.
		
		if (!m_processCommunicationAllocator.Init(CommunicationMemSize, CommunicationMemSize * maxSlotCount))
		{
			m_allowLocalDetour = false;
			m_logger.Warning(TC("Failed to create process communication allocator.. local detouring will be disabled."));
		}
		if (!CreateProcessJobObject())
			return false;

		// Environment variables that should stay local when building remote (not replicated)
		#if PLATFORM_WINDOWS
		m_localEnvironmentVariables.insert(TC("SystemRoot"));
		m_localEnvironmentVariables.insert(TC("SystemDrive"));
		m_localEnvironmentVariables.insert(TC("NUMBER_OF_PROCESSORS"));
		m_localEnvironmentVariables.insert(TC("PROCESSOR_ARCHITECTURE"));
		m_localEnvironmentVariables.insert(TC("PROCESSOR_IDENTIFIER"));
		m_localEnvironmentVariables.insert(TC("PROCESSOR_LEVEL"));
		m_localEnvironmentVariables.insert(TC("PROCESSOR_REVISION"));
		#endif

		if (info.useFakeVolumeSerial && !m_runningRemote)
			if (!m_volumeCache.Init(m_logger))
				return false;

		if (m_ownsTrace)
		{
			StringBuffer<> traceName;
			if (info.traceName && *info.traceName)
				traceName.Append(info.traceName);
			else if (info.launchVisualizer || !m_traceOutputFile.empty() || info.traceEnabled)
			{
				traceName.Append(m_id);

				OwnerInfo ownerInfo = GetOwnerInfo();
				if (ownerInfo.pid)
					traceName.Appendf(TC("_%s%u"), ownerInfo.id, ownerInfo.pid);

				if (!info.useUniqueId)
				{
					Guid guid;
					CreateGuid(guid);
					traceName.Append(GuidToString(guid));
				}
			}

			if (!traceName.IsEmpty())
			{
				u64 traceReserveSize = info.traceReserveSizeMb * 1024 * 1024;
				if (m_detailedTrace)
					traceReserveSize *= 2;
				if (info.traceChannelName && *info.traceChannelName)
					m_trace.StartWriteAndThread(traceName.data, traceReserveSize, true, info.traceChannelName);
				else
					m_trace.StartWriteAndThread(traceName.data, traceReserveSize, true);
			}
		}

		if (m_trace.IsWriting())
		{
			StringBuffer<512> sessionInfo;
			GetSessionInfo(sessionInfo);
			m_trace.SessionInfo(0, sessionInfo);
		}

		#if PLATFORM_WINDOWS
		if (info.launchVisualizer)
		{
			bool spawnNew = true;
			uba::StringBuffer<> launchArg;

			if (info.traceChannelName && *info.traceChannelName)
			{
				TraceChannel channel(m_logger);
				if (channel.Init(info.traceChannelName) && channel.IsListenerActive())
					spawnNew = false;
				launchArg.Append(TCV(" -listen=")).Append(info.traceChannelName);
			}
			else
			{
				launchArg.Append(TCV(" -named=")).Append(m_trace.GetNamedTrace());
			}

			if (spawnNew)
			{
				HMODULE currentModule = GetModuleHandle(NULL);
				GetModuleHandleExW(GET_MODULE_HANDLE_EX_FLAG_FROM_ADDRESS, (LPCWSTR)&g_dummy, &currentModule);
				tchar fileName[512];
				GetModuleFileNameW(currentModule, fileName, 512);
				uba::StringBuffer<> launcherCmd;
				launcherCmd.Append(TCV("\""));
				launcherCmd.AppendDir(fileName);
				launcherCmd.Append(TCV("\\UbaVisualizer.exe\""));
				launcherCmd.Append(launchArg);
				STARTUPINFOW si;
				memset(&si, 0, sizeof(si));
				PROCESS_INFORMATION pi;
				m_logger.Info(TC("Starting visualizer: %s"), launcherCmd.data);
				DWORD creationFlags = DETACHED_PROCESS | CREATE_NEW_PROCESS_GROUP;
				CreateProcessW(NULL, launcherCmd.data, NULL, NULL, false, creationFlags, NULL, NULL, &si, &pi);
				CloseHandle(pi.hThread);
				CloseHandle(pi.hProcess);
			}
			else
			{
				m_logger.Info(TC("Reusing visualizer listening on channel '%s'"), info.traceChannelName);
			}
		}
		#endif

		m_storage.RegisterExternalFileMappingsProvider([this](Storage::ExternalFileMapping& out, StringKey fileNameKey, const tchar* fileName)
			{
				SCOPED_FUTEX_READ(m_fileMappingTableLookupLock, lookupLock);
				auto findIt = m_fileMappingTableLookup.find(fileNameKey);
				if (findIt == m_fileMappingTableLookup.end())
					return false;
				FileMappingEntry& entry = findIt->second;
				lookupLock.Leave();

				SCOPED_FUTEX_READ(entry.lock, entryLock);
				if (!entry.handled || !entry.success || !entry.memoryHandle.IsValid())
					return false;
				
				out.handle = m_sharedMemory.DuplicateHandle(entry.memoryHandle, TC("ExternalFileMapping"));
				out.offset = entry.memoryOffset;
				out.size = entry.contentSize;
				out.lastWriteTime = entry.lastWriteTime;
				out.storedSize = entry.storedSize;
				out.createIndependentMapping = entry.createIndependentMapping;
				out.dropCasAfterUse = entry.dropCasAfterUse; // TODO: Make this an option. This will throw away cas files after they have been used
				return true;
			});

		#if UBA_USE_NATIVE_MAC_SEMAPHORES
		if (!StartSemaphoreService())
			return false;
		#endif


		m_fileMappingGcDone.Create(EventResetType_Manual);
		m_fileMappingGcThread.Start([this]() { ThreadGcFileMappings();  return 0; }, TC("UbaMemGc"));

		return true;
	}

	Session::~Session()
	{
		m_fileMappingGcDone.Set();

		if (m_ownsTrace)
		{
			m_trace.StopThread();
			m_trace.StopWrite(m_traceOutputFile.c_str());
		}

		CancelAllProcessesAndWait();
		FlushDeadProcesses(true);

		#if UBA_USE_NATIVE_MAC_SEMAPHORES
		StopSemaphoreService();
		#endif

		delete &m_processMessageThreads;

		while (m_dirTableSnapshotInProgress)
			Sleep(1);
		ReleaseDirTableSnapshot(m_dirTableSnapshot);

		m_fileMappingGcThread.Wait();

		for (auto& kv : m_virtualSourceFiles)
		{
			VirtualSourceFile& file = kv.second;
			if (file.mappingHandle.IsValid())
				FileMapping_Close(m_logger, file.mappingHandle, TC("VirtualFile"));
			if (file.memoryHandle.IsValid())
				m_sharedMemory.CloseHandle(m_logger, file.memoryHandle, TC("VirtualFile"));
		}

		for (auto& i : m_fileMappingTableLookup)
			if (i.second.canBeFreed)
				m_sharedMemory.CloseHandle(m_logger, i.second.memoryHandle, TC("FileMappingKeptFromOutput"));

		FileMapping_Unmap(m_logger, m_fileMappingTableMem, FileMappingTableMaxSize, TC("FileMappingTable"), m_allowDiscardVirtualMemory);
		FileMapping_Close(m_logger, m_fileMappingTableHandle, TC("FileMappingTable"));

		//#if !UBA_DEBUG
		//u32 count;
		//DeleteAllFiles(m_logger, m_sessionDir.data, count);
		//#endif

		if (m_ownsTrace)
			delete &m_trace;
	}

	void Session::CancelAllProcessesAndWait(bool terminate)
	{
		bool isEmpty = false;
		bool isFirst = true;
		while (!isEmpty)
		{
			Vector<ProcessHandle> processes;
			{
				SCOPED_FUTEX(m_processesLock, lock);
				processes.reserve(m_processes.size());
				for (auto& pair : m_processes)
					if (pair.second.m_process && !pair.second.m_process->IsCalledFromThis())
						processes.push_back(pair.second);
				isEmpty = processes.empty();
			}

			if (isFirst)
			{
				isFirst = false;
				if (!isEmpty)
					m_logger.Info(TC("Cancelling %llu processes and wait for them to exit"), processes.size());
				++m_logger.isMuted;
			}

			for (auto& process : processes)
				process.Cancel();

			#if PLATFORM_WINDOWS
			if (m_processJobObject != NULL)
			{
				SCOPED_FUTEX(m_processJobObjectLock, lock);
				CloseHandle(m_processJobObject);
				m_processJobObject = NULL;
			}
			#endif

			for (auto& process : processes)
				process.WaitForExit(100000);
		}

		--m_logger.isMuted;
	}

	void Session::CancelAllProcesses()
	{
		Vector<ProcessHandle> processes;
		SCOPED_FUTEX(m_processesLock, lock);
		processes.reserve(m_processes.size());
		for (auto& pair : m_processes)
			processes.push_back(pair.second);
		lock.Leave();
		for (auto& process : processes)
			process.Cancel();
	}

	ProcessHandle Session::RunProcess(const ProcessStartInfo& startInfo, bool async, bool enableDetour)
	{
		FlushDeadProcesses(false);
		ValidateStartInfo(startInfo);
		enableDetour &= m_allowLocalDetour;
		return InternalRunProcess(startInfo, async, nullptr, enableDetour, startInfo.userData);
	}

	void Session::ValidateStartInfo(const ProcessStartInfo& startInfo)
	{
		UBA_ASSERTF(startInfo.workingDir && *startInfo.workingDir, TC("Working dir must be set when spawning process"));
		UBA_ASSERTF(!TStrchr(startInfo.workingDir, '~'), TC("WorkingDir path must use long name (%s)"), startInfo.workingDir);
		UBA_ASSERTF(!IsWindows || !TStrchr(startInfo.application, '~'), TC("Application path must use long name (%s)"), startInfo.application);
	}

	ProcessHandle Session::InternalRunProcess(const ProcessStartInfo& startInfo, bool async, ProcessImpl* parent, bool enableDetour, void* userData)
	{
		auto& si = const_cast<ProcessStartInfo&>(startInfo);
		const tchar* originalLogFile = si.logFile;
		
		u32 processId = CreateProcessId();

		StringBuffer<> temp;
		si.logFile = FixLogFile(si.logFile, temp, startInfo.arguments, processId);

		if (!si.rules)
			si.rules = GetRules(si);

		Vector<tchar>& env = GetProcessEnvironmentVariables();
		auto process = new ProcessImpl(*this, processId, parent, enableDetour);
		ProcessHandle h(process);
		if (!process->Start(startInfo, m_runningRemote, env, async, userData))
			h = {};

		si.logFile = originalLogFile;
		return h;
	}

	bool Session::RegisterVirtualFile(const tchar* filePath, const tchar* sourceFile, u64 sourceOffset, u64 sourceSize)
	{
		UBA_ASSERT(!m_runningRemote);
		StringBuffer<> fixedFilePath;
		auto fileNameKey = GetKeyAndFixedName(fixedFilePath, filePath);
		if (!RegisterVirtualFileInternal(fileNameKey, fixedFilePath, sourceFile, sourceOffset, sourceSize))
			return false;
		return RegisterCreateFileForWrite(fileNameKey, fixedFilePath, sourceSize, 0, DefaultAttributes(), false, false);
	}

	bool Session::CreateVirtualFile(const tchar* filePath, const void* memory, u64 memorySize, bool transient, bool registerToDir)
	{
		UBA_ASSERT(!m_runningRemote);
		StringBuffer<> fixedFilePath;
		auto fileNameKey = GetKeyAndFixedName(fixedFilePath, filePath);
		if (!CreateVirtualFileInternal(fileNameKey, fixedFilePath, memory, memorySize, transient))
			return false;
		if (!registerToDir)
			return true;
		return RegisterCreateFileForWrite(fileNameKey, fixedFilePath, memorySize, 0, DefaultAttributes(), false, false);
	}

	bool Session::DeleteVirtualFile(const tchar* filePath, bool registerToDir)
	{
		UBA_ASSERT(!m_runningRemote);
		StringBuffer<> fixedFilePath;
		auto key = GetKeyAndFixedName(fixedFilePath, filePath);
		if (registerToDir)
			RegisterDeleteFile(key, fixedFilePath);

		m_storage.DeleteExternalFileMapping(key, filePath);

		SCOPED_FUTEX(m_virtualSourceFilesLock, virtualSourceFilesLock);
		auto findIt = m_virtualSourceFiles.find(key); 
		if (findIt == m_virtualSourceFiles.end())
		{
			//m_logger.Warning(TC("Failed to delete virtual file %s (%s)"), filePath, KeyToString(key).data);
			return false;
		}
		VirtualSourceFile vsf = findIt->second;
		m_virtualSourceFiles.erase(findIt);
		virtualSourceFilesLock.Leave();

		SCOPED_FUTEX(m_fileMappingTableLookupLock, lookupLock);
		auto findIt2 = m_fileMappingTableLookup.find(key);
		if (findIt2 != m_fileMappingTableLookup.end())
		{
			UBA_ASSERTF(findIt2->second.refCount == 0, TC("Deleting virtual file %s which is in use"), filePath);
			m_fileMappingTableLookup.erase(findIt2);
		}
		lookupLock.Leave();

		m_sharedMemory.CloseHandle(m_logger, vsf.memoryHandle, TC("VirtualFile"));
		return true;
	}

	void Session::RegisterCustomService(CustomServiceFunction&& function)
	{
		m_customServiceFunction = function;
	}

	void Session::RegisterGetNextProcess(GetNextProcessFunction&& function)
	{
		m_getNextProcessFunction = function;
	}

	bool Session::AddEnvironmentVariable(const tchar* name, const tchar* value)
	{
		if (!m_environmentMemory.empty())
			return m_logger.Error(TC("AddEnvironmentVariable must be done earlier"));
		m_additionalEnvironmentVariables.emplace_back(name, value);
		return true;
	}

	bool Session::GetOutputFileSize(u64& outSize, const tchar* filePath)
	{
		if (m_shouldWriteToDisk)
			return m_logger.Error(TC("GetFileSize only implemented for path where ShouldWriteToDisk is false"));
		UBA_ASSERT(!m_runningRemote);
		StringBuffer<> fixedFilePath;
		auto key = GetKeyAndFixedName(fixedFilePath, filePath);
		return GetOutputFileSizeInternal(outSize, key, fixedFilePath);
	}

	bool Session::GetOutputFileData(void* outData, const tchar* filePath, bool deleteInternalMapping)
	{
		UBA_ASSERT(!m_runningRemote);
		StringBuffer<> fixedFilePath;
		auto key = GetKeyAndFixedName(fixedFilePath, filePath);
		return GetOutputFileDataInternal(outData, key, fixedFilePath, deleteInternalMapping);
	}

	bool Session::WriteOutputFile(const tchar* filePath, bool deleteInternalMapping)
	{
		UBA_ASSERT(!m_runningRemote);
		StringBuffer<> fixedFilePath;
		auto key = GetKeyAndFixedName(fixedFilePath, filePath);
		return WriteOutputFileInternal(key, fixedFilePath, deleteInternalMapping);
	}

	const tchar* Session::GetId() { return m_id.data; }
	Storage& Session::GetStorage() { return m_storage; }
	MutableLogger& Session::GetLogger() { return m_logger; }
	LogWriter& Session::GetLogWriter() { return m_logger.m_writer; }
	WorkManager& Session::GetWorkManager() { return m_workManager; }
	Trace& Session::GetTrace() { return m_trace; }
	SearchPathCache& Session::GetSearchPathCache() { return m_searchPathCache; } 


	const ApplicationRules* Session::GetRules(const ProcessStartInfo& si)
	{
		const tchar* exeNameStart = si.application;
		const tchar* exeNameEnd = exeNameStart + TStrlen(si.application);
		UBA_ASSERT(exeNameEnd - exeNameStart > 1);
		if (const tchar* lastSeparator = TStrrchr(exeNameStart, PathSeparator))
			exeNameStart = lastSeparator + 1;
		if (const tchar* lastSeparator2 = TStrrchr(exeNameStart, NonPathSeparator))
			exeNameStart = lastSeparator2 + 1;
		if (*exeNameStart == '"')
			++exeNameStart;
		if (exeNameEnd[-1] == '"')
			--exeNameEnd;
		StringBuffer<128> exeName;
		UBA_ASSERTF(exeNameStart < exeNameEnd, TC("Bad application string: %s"), si.application);
		exeName.Append(exeNameStart, exeNameEnd - exeNameStart);
		
		auto rules = GetApplicationRules();
		
		bool isDotnet = false;

		while (true)
		{
			exeName.MakeLower();
			u32 appHash = GetApplicationHash(exeName);

			for (u32 i = 1;; ++i)
			{
				u32 hash = rules[i].hash;
				if (!hash)
					break;
				if (appHash != hash)
					continue;
				return rules[i].rules;
			}

			if (!exeName.Equals(TCV("dotnet.exe")) && !exeName.Equals(TCV("dotnet")))
				return rules[isDotnet].rules;
			
			isDotnet = true;

			u32 firstArgumentStart = 0;
			u32 firstArgumentEnd = 0;
			bool quoted = false;
			for (u32 i = 0, e = TStrlen(si.arguments); i != e; ++i)
			{
				tchar c = si.arguments[i];
				if (firstArgumentEnd)
				{
					if (c == '\\')
						firstArgumentStart = i + 1;
					firstArgumentEnd = i + 1;
					if ((quoted && c != '"') || (!quoted && c != ' ' && c != '\t'))
						continue;
					firstArgumentEnd = i;
					break;
				}
				else
				{
					if (c == ' ' || c == '\t')
					{
						++firstArgumentStart;
						continue;
					}
					else if (c == '"')
					{
						++firstArgumentStart;
						quoted = true;
					}
					firstArgumentEnd = firstArgumentStart + 1;
				}
			}
			exeName.Clear().Append(si.arguments + firstArgumentStart, firstArgumentEnd - firstArgumentStart);
		}
	}

	const tchar* Session::GetTempPath()
	{
		return m_tempPath.data;
	}
	
	const tchar* Session::GetRootDir()
	{
		return m_rootDir.data;
	}

	const tchar* Session::GetSessionDir()
	{
		return m_sessionDir.data;
	}

	u32 Session::CreateProcessId()
	{
		return ++m_processIdCounter;
	}

	bool Session::VirtualizePath(StringBufferBase& inOut, RootsHandle rootsHandle)
	{
		if (!rootsHandle)
			return true;
		if (IsWindows ? inOut[1] != ':' : inOut[0] != '/')
			return true;
		auto rootsEntry = GetRootsEntry(rootsHandle);
		if (!rootsEntry)
			return false;
		if (rootsEntry->roots.IsEmpty())
			return true;
		auto& locals = rootsEntry->locals;
		auto& vfs = rootsEntry->vfs;
		for (u32 i=0, e=u32(locals.size()); i!=e; ++i)
		{
			if (!inOut.StartsWith(locals[i].c_str()))
				continue;
			StringBuffer<> temp;
			temp.Append(inOut.data + locals[i].size());
			inOut.Clear().Append(vfs[i]).Append(temp);
			return true;
		}

		return true;
	}

	bool Session::DevirtualizePath(StringBufferBase& inOut, RootsHandle rootsHandle, bool reportError)
	{
		if (!rootsHandle)
			return true;
		if (IsWindows ? inOut[1] != ':' : inOut[0] != '/')
			return true;
		auto rootsEntry = GetRootsEntry(rootsHandle);
		if (!rootsEntry)
			return false;
		if (rootsEntry->roots.IsEmpty())
			return true;
		auto root = rootsEntry->roots.FindRoot(inOut);
		if (!root)
			return reportError ? m_logger.Error(TC("Can't find root for path %s (Available roots: %s)"), inOut.data, rootsEntry->roots.GetAllRoots().c_str()) : false;

		auto& path = rootsEntry->locals[root->index/PathsPerRoot];
		StringBuffer<> temp;
		temp.Append(inOut.data + root->path.size());
		inOut.Clear().Append(path).Append(temp);
		return true;
	}

	bool Session::DevirtualizeString(TString& inOut, RootsHandle rootsHandle, bool allowPathsWithoutRoot, const tchar* hint)
	{
		if (!HasVfs(rootsHandle))
			return true;
		auto rootsEntry = GetRootsEntry(rootsHandle);
		if (!rootsEntry)
			return false;

		u64 newSize = 0;
		bool hasRoot = false;
		auto checkString = [&](const tchar* str, u64 strLen, u32 rootPos)
			{
				if (rootPos == ~0u)
				{
					newSize += strLen;
					return;
				}
				hasRoot = true;
				auto& path = rootsEntry->locals[(*str - RootPaths::RootStartByte)/PathsPerRoot];
				newSize += path.size();
				#if PLATFORM_WINDOWS
				u32 rootIndex = (*str - RootPaths::RootStartByte);
				u32 type = rootIndex % PathsPerRoot;
				if (type == 2) // DoubleForward
					for (tchar c : path)
						newSize += c == '\\';

				#endif
			};

		if (!rootsEntry->roots.NormalizeString<tchar>(m_logger, inOut.data(), inOut.size(), checkString, allowPathsWithoutRoot, hint))
			return false;

		if (!hasRoot)
			return false;

		TString newString;
		newString.resize(newSize);
		tchar* newStringPos = newString.data();

		auto handleString = [&](const tchar* str, u64 strLen, u32 rootPos)
			{
				if (rootPos == ~0u)
				{
					memcpy(newStringPos, str, strLen*sizeof(tchar));
					newStringPos += strLen;
					return;
				}
				u32 rootIndex = (*str - RootPaths::RootStartByte);
				u32 localsIndex = rootIndex/PathsPerRoot;
				auto& path = rootsEntry->locals[localsIndex];
				tchar* start = newStringPos;
				newStringPos += path.size();
				memcpy(start, path.data(), path.size()*sizeof(tchar));

				#if PLATFORM_WINDOWS
				u32 type = rootIndex % PathsPerRoot;
				if (type == 1) // just backslash
				{
				}
				else if (type == 0)
				{
					*newStringPos = 0;
					Replace(start, '\\', '/');
				}
				else if (type == 2) // Double forward slash
				{
					*newStringPos = 0;
					for (tchar* it=start; it!=newStringPos;++it)
						if (*it == '\\')
						{
							*it = '/';
							memmove(it+1, it, (newStringPos-it)*sizeof(tchar));
							++newStringPos;
						}
				}
				else
				{
					UBA_ASSERTF(false, TC("Not root path type %u not implemented (%s)"), type, hint); // We don't like double backslash or escaped space
				}
				#endif
			};

		rootsEntry->roots.NormalizeString<tchar>(m_logger, inOut.data(), inOut.size(), handleString, allowPathsWithoutRoot, hint);

		UBA_ASSERT(newStringPos == newString.data() + newString.size());
		inOut = std::move(newString);

		return true;
	}

	bool Session::PopulateLocalToIndexRoots(RootPaths& out, RootsHandle rootsHandle)
	{
		if (!rootsHandle)
			return true;
		auto rootsEntry = GetRootsEntry(rootsHandle);
		if (!rootsEntry)
			return false;

		BinaryReader reader((const u8*)rootsEntry->memory.data(), 0, rootsEntry->memory.size());
		while (reader.GetLeft())
		{
			u8 id = reader.ReadByte();
			reader.SkipString();
			StringBuffer<> rootPath;
			reader.ReadString(rootPath);
			if (!out.RegisterRoot(m_logger, rootPath.data, true, id))
				return false;
		}

		// TODO: Provide or calculate these
		#if PLATFORM_WINDOWS
		out.RegisterIgnoredRoot(m_logger, TC("z:/UEVFS"));
		#else
		out.RegisterIgnoredRoot(m_logger, TC("/UEVFS"));
		#endif

		return true;//out.RegisterSystemRoots(m_logger, 0);
	}

	void Session::ProcessAdded(Process& process, u32 sessionId)
	{
		u32 processId = process.GetId();

		auto& startInfo = process.GetStartInfo();
		if (!process.IsChild() || m_traceChildProcesses)
			m_trace.ProcessAdded(sessionId, processId, ToView(startInfo.GetDescription()), ToView(startInfo.breadcrumbs));

		SCOPED_FUTEX(m_processesLock, lock);
		bool success = m_processes.try_emplace(processId, ProcessHandle(&process)).second;
		UBA_ASSERTF(success, TC("Failed to add process with id %u. It already exists (%s)"), processId, startInfo.GetDescription());(void)success;
	}

	void Session::ProcessExited(ProcessImpl& process, u64 executionTime)
	{
		const tchar* application = process.GetStartInfo().application;
		StringBuffer<> applicationName;
		applicationName.AppendFileName(application);
		if (applicationName.count > 21)
			applicationName[21] = 0;

		u32 id = process.GetId();

		if (!process.IsChild() || m_traceChildProcesses)
		{
			StackBinaryWriter<1024> writer;
			process.m_processStats.Write(writer);
			process.m_sessionStats.Write(writer);
			process.m_storageStats.Write(writer);
			process.m_kernelStats.Write(writer);
			u32 exitCode = process.GetExitCode();
			Vector<ProcessLogLine> emptyLines;
			auto& logLines = (exitCode != 0 || m_detailedTrace) ? process.m_logLines : emptyLines;
			m_trace.ProcessExited(id, exitCode, writer.GetData(), writer.GetPosition(), logLines);
			SCOPED_FUTEX(m_processStatsLock, lock);
			m_processStats.Add(process.m_processStats);
			m_stats.Add(process.m_sessionStats);
		}

		SCOPED_FUTEX(process.m_usedFileMappingsLock, usedFileMappingsLock);
		auto usedFileMappings = std::move(process.m_usedFileMappings); // Might be that crawler is still working
		usedFileMappingsLock.Leave();

		for (auto& kv : usedFileMappings)
		{
			const StringKey& fileNameKey = kv.first;
			SCOPED_FUTEX(m_fileMappingTableLookupLock, lookupLock);
			auto findIt = m_fileMappingTableLookup.find(fileNameKey);
			UBA_ASSERT(findIt != m_fileMappingTableLookup.end());
			FileMappingEntry& entry = findIt->second;
			lookupLock.Leave();

			SCOPED_FUTEX(entry.lock, entryLock);

			if (entry.usedCount < entry.usedCountBeforeFree)
				++entry.usedCount;

			if (--entry.refCount)
				continue;

			auto& usedFileMapping = kv.second;
			if (!usedFileMapping.closeAfterWritten)
			{
				if (entry.usedCountBeforeFree == 255 || entry.usedCount < entry.usedCountBeforeFree)
					continue;
			}

			UBA_ASSERT(entry.canBeFreed);

			#if UBA_DEBUG_TRACK_MAPPING
			m_debugLogger->Info(TC("Mapping freed 0x%llx (%s)"), u64(entry.memoryHandle.ToU64()), entry.name.c_str());
			#endif

			m_workManager.AddWork([mh = entry.memoryHandle, fileSize = entry.contentSize, this](const WorkContext&)
				{
					//m_logger.Info(TC("RELEASING %s"), name.c_str());
					m_sharedMemory.CloseHandle(m_logger, mh, TC("UsedFileMapping"));
					--m_independentMappingActive;

				}, 1, TC("CloseFileMapping"));
			entry.handled = false;
			entry.memoryHandle = {};
		}



		SCOPED_FUTEX(m_processesLock, lock);
		m_deadProcesses.emplace_back(&process); // Here to prevent Process thread call trigger a delete of Process which causes a deadlock
		auto& stats = m_applicationStats[applicationName.data];
		stats.count++;
		stats.time += executionTime;
		auto count = m_processes.erase(id);
		UBA_ASSERT(count == 1);(void)count;
	}

	void Session::FlushDeadProcesses(bool reapZombies)
	{
		SCOPED_FUTEX(m_processesLock, lock);
		Vector<ProcessHandle> deadProcesses;
		deadProcesses.swap(m_deadProcesses);
		lock.Leave();

		#if PLATFORM_LINUX
		if (!reapZombies)
			return;
		for (auto& handle : deadProcesses)
		{
			auto& process = *(ProcessImpl*)handle.m_process;
			u32 pid = process.m_nativeProcessId;
			if (pid == 0) continue;
			int status = 0;
			waitpid(static_cast<pid_t>(pid), &status, WNOHANG);
		}
		#endif
	}

	bool Session::UpdateDirTableSnapshot(bool force)
	{
		if (!m_dirTableSnapshotEnabled)
			return true;

		if (force)
			return CreateSnapshot(GetDirectoryTableSize(nullptr).main);

		if (GetDirectoryTableSize(nullptr).main - m_dirTableSizeForSnapshot < 256*1024)
			return true;

		u32 expected = 0;
		if (!m_dirTableSnapshotInProgress.compare_exchange_strong(expected, 1))
			return true;

		m_workManager.AddWork([this](const WorkContext&)
			{
				CreateSnapshot(GetDirectoryTableSize(nullptr).main);
				m_dirTableSnapshotInProgress.exchange(0);
			}, 1, TC("DirTableSnapshot"));

		return true;
	}

	const tchar* Session::FixLogFile(const tchar* logFile, StringBufferBase& temp, const tchar* processArguments, u32 processId)
	{
		if (logFile && *logFile)
		{
			if (TStrchr(logFile, PathSeparator) == nullptr)
				return temp.Append(m_sessionLogDir).Append(logFile).data;
		}
		else if (m_logToFile)
		{
			temp.Append(m_sessionLogDir);
			GenerateNameForProcess(temp, processArguments, processId);
			return temp.Append(TCV(".log")).data;
		}
		return logFile;
	}

	void Session::SetProcessAsIdle(ProcessImpl& process, BinaryReader& prevStats, u32 prevExitCode)
	{
		m_trace.ProcessEnvironmentUpdated(process.GetId(), TCV("IDLE"), prevExitCode, prevStats.GetPositionData(), prevStats.GetLeft(), TCV(""));
		process.m_processStats = {};
		process.m_sessionStats = {};
		process.m_storageStats = {};
		process.m_kernelStats = {};
	}

	bool Session::ProcessThreadStart(ProcessImpl& process)
	{
		return true;
	}

	bool Session::ProcessNativeCreated(ProcessImpl& process)
	{
		return true;
	}

	bool Session::ProcessCancelled(ProcessImpl& process)
	{
		auto& shared = process.m_shared;
		SCOPED_FUTEX(shared.activeFilesLock, lock);
		auto closeIds(std::move(shared.activeFiles));
		lock.Leave();

		CloseFileResponse out;
		CloseFileMessage msg { process };

		for (auto& kv : closeIds)
		{
			// TODO: Should file be deleted from disk if it is there?
			msg.closeId = kv.first;
			msg.success = false;
			CloseFile(out, msg);
		}
		return true;
	}

	bool Session::GetInitResponse(InitResponse& out, const InitMessage& msg)
	{
		out.sharedMemoryAllocatorHandle = m_sharedMemory;
		out.permanentFilesHandle = m_fileMappingBuffer.GetMemoryHandle();

		out.directoryTableHandle = m_directoryTableHandle;
		out.directoryTableSize = GetDirectoryTableSize(&msg.process.m_shared.overlay);

		{
			SCOPED_READ_LOCK(m_directoryTable.m_lookupLock, l);
			out.directoryTableCount = (u32)m_directoryTable.m_lookup.size();
		}
		out.mappedFileTableHandle = m_fileMappingTableHandle;
		{
			SCOPED_READ_LOCK(m_fileMappingTableMemLock, l);
			out.mappedFileTableSize = m_fileMappingTableSize;
		}
		{
			SCOPED_FUTEX_READ(m_fileMappingTableLookupLock, l);
			out.mappedFileTableCount = (u32)m_fileMappingTableLookup.size();
		}
		return true;
	}

	u32 Session::GetFileMappingSize()
	{
		SCOPED_READ_LOCK(m_fileMappingTableMemLock, lock);
		return m_fileMappingTableSize;
	}

	SessionStats& Session::Stats()
	{
		if (SessionStats* s = SessionStats::GetCurrent())
			return *s;
		return m_stats;
	}

	u32 Session::GetActiveProcessCount()
	{
		SCOPED_FUTEX_READ(m_processesLock, cs);
		return u32(m_processes.size());
	}

	void Session::PrintProcessStats(ProcessStats& stats, const tchar* logName)
	{
		m_logger.Info(TC("  -- %s --"), logName);
		stats.Print(m_logger);
	}

	bool Session::SaveSnapshotOfTrace()
	{
		return m_trace.Write(m_traceOutputFile.c_str(), true);
	}

	void Session::PrintSummary(Logger& logger)
	{
		logger.BeginScope();
		logger.Info(TC("  ------- Detours stats summary -------"));
		m_processStats.Print(logger);
		logger.Info(TC(""));

		std::multimap<u64, std::pair<const TString*, u32>> sortedApps;
		for (auto& pair : m_applicationStats)
			sortedApps.insert({pair.second.time, {&pair.first, pair.second.count}});
		for (auto i=sortedApps.rbegin(), e=sortedApps.rend(); i!=e; ++i)
		{
			const TString& name = *i->second.first;
			u64 time = i->first;
			u32 count = i->second.second;
			logger.Info(TC("  %-21s %5u %9s"), name.c_str(), count, TimeToText(time).str);
		}
		logger.Info(TC(""));

		logger.Info(TC("  ------- Session stats summary -------"));

		PrintSessionStats(logger);
		logger.EndScope();
	}

	bool Session::GetBinaryModules(BinaryAndDeps& out, const tchar* application, bool isLinuxBinary)
	{
		const tchar* applicationName = application;

		if (tchar* lastSlash = TStrrchr((tchar*)application, PathSeparator))
			applicationName = lastSlash + 1;

		u64 applicationDirLen = u64(applicationName - application);
		tchar applicationDir[512];
		UBA_ASSERT(applicationDirLen < 512);
		memcpy(applicationDir, application, applicationDirLen * sizeof(tchar));
		tchar* applicationDirEnd = applicationDir + applicationDirLen;

		UnorderedSet<TString> handledImports;
		return CopyImports(out, applicationName, applicationDir, applicationDirEnd, handledImports, nullptr, isLinuxBinary, application);
	}

	void Session::Free(BinaryAndDeps& v)
	{
		v.modules.resize(0);
		v.modules.shrink_to_fit();
	}

	bool Session::IsRarelyRead(ProcessImpl& process, const StringView& fileName) const
	{
		return process.m_startInfo.rules->IsRarelyRead(fileName);
	}

	bool Session::IsMaybeReadAfterWritten(ProcessImpl& process, const StringView& fileName) const
	{
		return process.m_startInfo.rules->IsMaybeReadAfterWritten(fileName);
	}

	bool Session::IsKnownSystemFile(const tchar* applicationName)
	{
#if PLATFORM_WINDOWS
		return uba::IsKnownSystemFile(applicationName);
#else
		return false;
#endif
	}

	bool Session::IsInTemp(StringView path)
	{
		return path.StartsWith(m_tempPath);
	}

	bool Session::ShouldWriteToDisk(StringView fileName)
	{
		if (m_shouldWriteToDisk)
			return true;
		return fileName.EndsWith(TCV(".h"));
	}

	void Session::InvalidateCachedFileInfo(const StringKey& fileNameKey)
	{
		m_storage.InvalidateCachedFileInfo(fileNameKey);
	}

	bool Session::PrepareProcess(ProcessImpl& process, bool isChild, StringBufferBase& outRealApplication, const tchar*& outRealWorkingDir)
	{
		ProcessStartInfoHolder& startInfo = process.m_startInfo;
		if (StartsWith(startInfo.application, TC("ubacopy")))
			return true;

		if (!IsAbsolutePath(startInfo.application))
		{
			if (!m_searchPathCache.SearchPathForFile(m_logger, outRealApplication.Clear(), ToView(startInfo.application), ToView(startInfo.workingDir), {}))
				return m_logger.Error(TC("Failed to find path to application %s"), startInfo.application);
			startInfo.applicationStr = outRealApplication.data;
			startInfo.application = startInfo.applicationStr.c_str();
		}

		#if PLATFORM_LINUX
		// Shebang detection: if the application's first two bytes are "#!",
		// resolve the interpreter inline. The kernel's own shebang resolution
		// would otherwise chain `execve(script)` -> `execve(/usr/bin/env, ...)`
		// -> `execve(real interp)`, which produces a second ExecProcess RPC
		// on the same comm channel — and the protocol does not support more
		// than one ExecProcess per process, so the second one hangs forever.
		//
		// On a successful resolve we rewrite startInfo so the detour sees:
		//   - virtualApplication = interpreter (so argv[0] = interpreter,
		//     which Python uses to derive sys.executable / sys.prefix —
		//     argv[0] = script would make it look for stdlib next to the .py
		//     and fail with "encodings module not found")
		//   - argumentsStr = "<script> <orig args>" (so argv[1] = script)
		//   - outRealApplication = interpreter (what trueExecve will hit)
		// Mirrors SessionClient::PrepareProcess's remote shebang branch.
		//
		// Parsing mirrors Session::CopyImports (UbaSession.cpp:1217).
		{
			FileAccessor readFile(m_logger, outRealApplication.data);
			if (readFile.OpenMemoryRead())
			{
				auto mem = (const u8*)readFile.GetData();
				u64 fileSize = readFile.GetSize();
				if (fileSize > 2 && mem[0] == '#' && mem[1] == '!')
				{
					// Linux caps the shebang line at 127 chars; 132 matches
					// the existing CopyImports limit with a few bytes of slack.
					u64 i = 2;
					for (; i < fileSize && i < 132; ++i)
						if (mem[i] == '\n')
							break;
					StringView shebang((const char*)mem + 2, u32(i - 2));
					// `/usr/bin/env <prog>` indirection: drop the `/usr/bin/env`
					// prefix so SearchPathForFile resolves <prog> via PATH
					// instead of letting env do it (env's own resolution would
					// trigger another execve and re-introduce the double-exec).
					if (shebang.StartsWith("/usr/bin/env "))
						shebang = shebang.Skip(13);
					// Trim leading whitespace.
					while (shebang.count && (shebang.data[0] == ' ' || shebang.data[0] == '\t'))
						shebang = shebang.Skip(1);
					// Truncate at first whitespace — SearchPathForFile expects
					// a single token. Shebang args (e.g. `python3 -E`) are
					// dropped today; if a build hits a script that needs them,
					// extend this to forward them as additional argv entries.
					u32 binLen = 0;
					for (; binLen < shebang.count; ++binLen)
					{
						char c = shebang.data[binLen];
						if (c == ' ' || c == '\t' || c == '\r')
							break;
					}

					if (binLen > 0)
					{
						// Copy into a stack buffer because `interpName` would
						// otherwise alias the FileAccessor's mmap memory; we
						// must keep it valid past readFile.Close() and across
						// the SearchPathForFile call.
						StringBuffer<512> interpName;
						interpName.Append(shebang.data, binLen);
						readFile.Close();

						bool interpResolved = false;
						if (IsAbsolutePath(interpName.data))
						{
							// Shebang already names an absolute path (e.g.
							// `#!/bin/bash`). SearchPathForFile asserts on
							// absolute inputs — use the path directly.
							outRealApplication.Clear().Append(interpName);
							interpResolved = true;
						}
						else
						{
							StringBuffer<512> resolvedInterp;
							if (m_searchPathCache.SearchPathForFile(m_logger, resolvedInterp, ToView(interpName), ToView(startInfo.workingDir), {}))
							{
								outRealApplication.Clear().Append(resolvedInterp);
								interpResolved = true;
							}
							else
							{
								m_logger.Warning(TC("Shebang interpreter '%s' not found via PATH for %s — leaving outRealApplication as the script and letting the kernel handle it (may chain-exec)"),
									interpName.data, startInfo.application);
							}
						}

						if (interpResolved)
						{
							// argv[0] must be the interpreter, argv[1] the script.
							// The detour's shebang branch in UbaDetoursSharedPosix.cpp
							// only sets tempArgv[0] = virtualApplication; argv[1..]
							// comes from the original execve argv (script as argv[0]
							// in the source process). startInfo.argumentsStr is the
							// host's view used for logging/rules — keep it in sync.
							TString scriptPath = startInfo.applicationStr;
							startInfo.applicationStr = outRealApplication.data;
							startInfo.application = startInfo.applicationStr.c_str();
							startInfo.argumentsStr = scriptPath + " " + startInfo.argumentsStr;
							startInfo.arguments = startInfo.argumentsStr.c_str();
						}
					}
				}
			}
		}
		#endif

		// Statically-linked ELFs (Go static binaries, glibc-static clang, AOSP
		// build tools, etc.) can't be reached by LD_PRELOAD because they don't
		// go through ld.so. When detouring is requested and the application is
		// one of these, produce a patched `<app>.uba` copy via UbaStaticPatcher
		// (which injects a freestanding detour stub at a fresh PT_LOAD and
		// redirects e_entry) and run that instead. outRealApplication then
		// points at the patched file; startInfo.application stays as the
		// user's original path so description / logging / rules matching is
		// unaffected. Dynamic binaries fall through and take the normal
		// LD_PRELOAD path.
		#if PLATFORM_LINUX
		if (process.m_detourEnabled && !TryPatchBinary(outRealApplication, startInfo.applicationStr))
			return false;
		#endif

		if (!isChild && !m_runningRemote && m_readIntermediateFilesCompressed && m_allowLinkDependencyCrawler)
		{
			auto crawlerType = startInfo.rules->GetDependencyCrawlerType();
			if (crawlerType == DependencyCrawlerType_MsvcLinker || crawlerType == DependencyCrawlerType_ClangLinker)
				RunDependencyCrawler(process);
		}

		if (!m_allowCustomAllocator)
			startInfo.useCustomAllocator = false;

		return true;
	}

	u32 Session::GetMemoryMapAlignment(const StringView& fileName) const
	{
		return GetMemoryMapAlignment(fileName, m_runningRemote);
	}

	u32 Session::GetMemoryMapAlignment(const StringView& fileName, bool runningRemote) const
	{
		// It is not necessarily better to make mem maps of everything.. only things that are read more than once in the build.
		// Reason is because there is additional overhead to use memory mappings.
		// Upside is that all things that are memory mapped can be stored compressed in cas storage so it saves space.

		if (fileName.EndsWith(TCV(".h")) || fileName.EndsWith(TCV(".inl")) || fileName.EndsWith(TCV(".gch")) || fileName.EndsWith(TCV(".hpp")))
			return 4 * 1024; // clang seems to need 4k alignment? Is it a coincidence it works or what is happening inside the code? (msvc works with alignment 1byte here)
		if (fileName.EndsWith(TCV(".lib")))
			return 4 * 1024;

		if (runningRemote) // We store these compressed to save space
		{
			if (fileName.EndsWith(TCV(".obj")) || fileName.EndsWith(TCV(".o")))
				return 4 * 1024; // pch needs 64k alignment
			if (fileName.EndsWith(TCV(".pch")))
				return 64 * 1024; // pch needs 64k alignment
		}
		else
		{
			if (fileName.EndsWith(TCV(".h.obj")))
				return 4 * 1024;
		}
		return 0;
	}

	Vector<tchar>& Session::GetProcessEnvironmentVariables()
	{
		SCOPED_FUTEX(m_environmentVariablesLock, lock);
		if (!m_environmentVariables.empty())
			return m_environmentVariables;

#if PLATFORM_WINDOWS
		auto HandleEnvironmentVar = [&](const tchar* env)
		{
			StringBuffer<> varName;
			varName.Append(env, TStrchr(env, '=') - env);
			const tchar* varValue = env + varName.count + 1;

			if (m_runningRemote && varName.Equals(TCV("PATH")))
			{
				AddEnvironmentVariableNoLock(TCV("PATH"), TCV("c:\\noenvironment"));
				return;
			}
			if (varName.Equals(TCV("TEMP")) || varName.Equals(TCV("TMP"))) // These are added per-process
				return;
			if (varName.Equals(TCV("_CL_")) || varName.Equals(TCV("CL")))
				return;

			AddEnvironmentVariableNoLock(varName, ToView(varValue));
		};

		if (m_environmentMemory.empty())
		{
			auto strs = GetEnvironmentStringsW();
			for (auto env = strs; *env; env += TStrlen(env) + 1)
				HandleEnvironmentVar(env);
			FreeEnvironmentStrings(strs);
		}
		else
		{
			BinaryReader reader(m_environmentMemory.data(), 0, m_environmentMemory.size());
			while (reader.GetLeft())
				HandleEnvironmentVar(reader.ReadString().c_str());
		}

		for (auto& pair : m_additionalEnvironmentVariables)
			AddEnvironmentVariableNoLock(pair.first, pair.second);

		AddEnvironmentVariableNoLock(TCV("MSBUILDDISABLENODEREUSE"), TCV("1")); // msbuild will reuse existing helper nodes but since those are not detoured we can't let that happen
		AddEnvironmentVariableNoLock(TCV("DOTNET_CLI_USE_MSBUILD_SERVER"), TCV("0")); // Disable msbuild server
		AddEnvironmentVariableNoLock(TCV("DOTNET_CLI_TELEMETRY_OPTOUT"), TCV("1")); // Stop talking to telemetry service
#else
		auto HandleEnvironmentVar = [&](const tchar* env)
		{
			if (StartsWith(env, "TMPDIR="))
				return;

			if (!StartsWith(env, "PATH="))
			{
				m_environmentVariables.insert(m_environmentVariables.end(), env, env + TStrlen(env) + 1);
				return;
			}

			TString paths;

			const char* start = env + 5;
			const char* it = start;
			bool isLast = false;
			while (!isLast)
			{
				if (*it != ':')
				{
					if (*it)
					{
						++it;
						continue;
					}
					isLast = true;
				}

				const char* s = start;
				const char* e = it;
				start = ++it;

				if (StartsWith(s, "/mnt/"))
					continue;
				if (!paths.empty())
					paths += ":";
				paths.append(s, e - s);
			}
			AddEnvironmentVariableNoLock(TCV("PATH"), paths);
		};

		if (m_environmentMemory.empty())
		{
			int i = 0;
			while (char* env = environ[i++])
				HandleEnvironmentVar(env);
		}
		else
		{
			BinaryReader reader(m_environmentMemory.data(), 0, m_environmentMemory.size());
			while (reader.GetLeft())
				HandleEnvironmentVar(reader.ReadString().c_str());
		}
#endif

		AddEnvironmentVariableNoLock(TCV("UBA_DETOURED"), TCV("1"));

		// ResolveBacking's setenv doesn't reach children (this base env is a snapshot sent to helpers, and
		// wine has a separate native environ), so inject the backing choice explicitly.
		if (g_fileMappingBackingEnv[0])
			if (const char* eq = strchr(g_fileMappingBackingEnv, '='))
			{
				StringBuffer<64> name; name.Append(g_fileMappingBackingEnv, u32(eq - g_fileMappingBackingEnv));
				StringBuffer<512> value; value.Append(eq + 1);
				AddEnvironmentVariableNoLock(name, value);
			}

		// io_uring calls not detoured right now
		AddEnvironmentVariableNoLock(TCV("UV_USE_IO_URING"), TCV("0"));

		m_environmentVariables.push_back(0);
		return m_environmentVariables;
	}

	bool Session::CreateFile(CreateFileResponse& out, const CreateFileMessage& msg)
	{
		const StringBufferBase& fileName = msg.fileName;
		const StringKey& fileNameKey = msg.fileNameKey;
		auto& process = msg.process;
		auto& shared = process.m_shared;

		if ((msg.access & ~FileAccess_Read) == 0)
		{
			TrackWorkScope tws(m_workManager, AsView(TC("CreateFile")), ColorWork);
			tws.AddHint(msg.fileName);
			return CreateFileForRead(out, tws, fileName, fileNameKey, process, *process.m_startInfo.rules);
		}

		auto tableSizeGuard = MakeGuard([&]()
			{
				out.mappedFileTableSize = GetFileMappingSize();
				out.directoryTableSize = GetDirectoryTableSize(&shared.overlay);
			});

		// if ((message.Access & FileAccess.Write) != 0)
		m_storage.ReportFileWrite(fileNameKey, fileName.data);

		if (m_runningRemote)
		{
			SCOPED_FUTEX_READ(shared.writtenFilesLock, writtenLock);
			auto wfIt = shared.writtenFiles.find(fileNameKey);
			if (wfIt != shared.writtenFiles.end() && !wfIt->second.backedName.empty())
			{
				out.fileName.Append(wfIt->second.backedName);
			}
			else
			{
				if (fileName.StartsWith(shared.m_tempPath))
				{
					// We need to create fake names because the file could be under a sub directory under temp and we don't create dirs
					out.fileName.Append(shared.m_tempPath).Append(KeyToString(fileNameKey));
				}
				else
				{
					// It might be that the file we are opening for write exists in output already
					// but we can't overwrite since the output files are used as cas storage for UbaStorageClient.. so always create new
					Guid g;
					CreateGuid(g);
					out.fileName.Append(m_sessionOutputDir).Append(GuidToString(g).data);
				}
			}
		}
		else
		{
			out.fileName.Append(fileName);
		}

		u32 wantsOnCloseId = m_wantsOnCloseIdCounter++;

		UBA_ASSERT(fileNameKey != StringKeyZero);
		SCOPED_FUTEX(shared.activeFilesLock, lock);
		out.closeId = wantsOnCloseId;
		auto insres = shared.activeFiles.try_emplace(wantsOnCloseId);
		if (!insres.second)
			return m_logger.Error(TC("TRYING TO ADD %s twice!"), out.fileName.data);
		ActiveFile& activeFile = insres.first->second;
		activeFile.name = fileName.data;
		activeFile.nameKey = fileNameKey;
		activeFile.refCount = 1;

		// Wire invariant: absolute or marker-prefixed (detour passes this to kernel open via tempFileName).
		UBA_ASSERTF(out.fileName.count == 0
			|| IsAbsolutePath(out.fileName.data)
			|| out.fileName.data[0] == '#'
			|| out.fileName.data[0] == '$'
			|| out.fileName.data[0] == MemoryHandleChar
			|| out.fileName.data[0] == FreeableMemoryHandleChar
			|| out.fileName.data[0] == WrittenMemoryHandleChar,
			TC("Session::CreateFile: response path must be absolute or marker, got %s (sent %s)"),
			out.fileName.data, fileName.data);
		return true;
	}

	bool Session::CreateFileForRead(CreateFileResponse& out, TrackWorkScope& tws, StringView fileName, StringKey fileNameKey, ProcessImpl& process, const ApplicationRules& rules)
	{
		auto tableSizeGuard = MakeGuard([&]()
			{
				out.mappedFileTableSize = GetFileMappingSize();
				out.directoryTableSize = GetDirectoryTableSize(&process.m_shared.overlay);
			});

		// Wire-format invariant guard — runs on every return path.
		// out.fileName must be absolute (or one of the marker prefixes the
		// detour understands). Same rationale as Session::CreateFile.
		auto absolutePathGuard = MakeGuard([&]()
			{
				UBA_ASSERTF(out.fileName.count == 0
					|| IsAbsolutePath(out.fileName.data)
					|| out.fileName.data[0] == '#'
					|| out.fileName.data[0] == '$'
					|| out.fileName.data[0] == MemoryHandleChar
					|| out.fileName.data[0] == FreeableMemoryHandleChar
					|| out.fileName.data[0] == WrittenMemoryHandleChar,
					TC("Session::CreateFileForRead: response path must be absolute or marker, got %s (sent %s)"),
					out.fileName.data, fileName.data);
			});

		if constexpr (!IsWindows)
		{
			out.fileName.Append(fileName);
			return true;
		}
		
		if (fileName.EndsWith(TCV(".dll")) || fileName.EndsWith(TCV(".exe")))
		{
			UBA_ASSERTF(IsAbsolutePath(fileName.data), TC("Got bad filename from process (%s)"), fileName.data);
			AddFileMapping(fileNameKey, fileName.data, TC("#"));
			out.fileName.Append(TCV("#"));
			return true;
		}
			
		if (m_allowMemoryMaps)
		{
			u64 alignment = GetMemoryMapAlignment(fileName);
			bool canBeCompressed = m_readIntermediateFilesCompressed && g_globalRules.FileCanBeCompressed(fileName);
			bool useMemoryMap = alignment != 0 || canBeCompressed;
			if (useMemoryMap)
			{
				MemoryMap map;
				bool canBeFreed = canBeCompressed;
				if (GetOrCreateMemoryMapFromFile(map, fileNameKey, fileName.data, false, alignment, TC("CreateFile"), &process, canBeFreed))
				{
					UBA_ASSERT(!map.name.IsEmpty());
					out.size = map.size;
					out.fileName.Append(map.name);
				}
				else
				{
					out.fileName.Append(fileName);
				}
				return true;
			}
			else // Still need to check if file exists since it can be a virtual file
			{
				SCOPED_FUTEX(m_fileMappingTableLookupLock, lookupLock);
				auto insres = m_fileMappingTableLookup.try_emplace(fileNameKey);
				FileMappingEntry& entry = insres.first->second;
				lookupLock.Leave();
				SCOPED_FUTEX(entry.lock, entryLock);
				if (entry.handled && entry.success && entry.memoryHandle.IsValid())
				{
					UBA_ASSERT(entry.isInvisible);
					out.size = entry.contentSize;
					GetMappingString(out.fileName, entry.memoryHandle, entry.memoryOffset, entry.canBeFreed);
					return true;
				}
			}
		}

		if (!IsRarelyRead(process, fileName))
		{
			AddFileMapping(fileNameKey, fileName.data, TC("#"));
			out.fileName.Append(TCV("#"));
			return true;
		}

		out.fileName.Append(fileName);
		return true;
	}

	void Session::RemoveWrittenFile(ProcessImpl& process, const StringKey& fileKey)
	{
		SCOPED_FUTEX(process.m_shared.writtenFilesLock, writtenLock);
		auto& writtenFiles = process.m_shared.writtenFiles;
		auto findIt = writtenFiles.find(fileKey);
		if (findIt == writtenFiles.end())
			return;

		SharedMemoryHandle h = findIt->second.memoryHandle;
		TString name = findIt->second.name;
		writtenFiles.erase(findIt);
		writtenLock.Leave();

		if (!h.IsValid())
			return;

		#if UBA_DEBUG_TRACK_MAPPING
		m_debugLogger->Info(TC("Removed %s with handle 0x%llx"), name.c_str(), h.ToU64());
		#endif

		m_sharedMemory.CloseHandle(process.m_session.GetLogger(), h, name.c_str());
	}

	bool Session::CloseFile(CloseFileResponse& out, const CloseFileMessage& msg)
	{
		auto& process = msg.process;
		auto& shared = process.m_shared;

		SCOPED_FUTEX(shared.activeFilesLock, lock);
		auto findIt = shared.activeFiles.find(msg.closeId);
		if (findIt == shared.activeFiles.end())
			return process.m_cancelled ? false : m_logger.Error(TC("This should not happen. Got unknown closeId %u - %s"), msg.closeId, msg.fileName.data);
		if (--findIt->second.refCount)
			return true;
		ActiveFile activeFile = findIt->second;
		shared.activeFiles.erase(findIt);
		lock.Leave();

		if (!msg.success)
		{
			return true;
		}

		bool registerRealFile = true;
		u64 fileSize = msg.fileSize;
		u64 lastWriteTime = msg.lastWriteTime;
		u32 attributes = msg.attributes;

		ProcessStartInfo& startInfo = process.m_startInfo;
		auto& rules = *startInfo.rules;

		// open→unlink→close pattern (GNU tac mkstemp+unlink): an entry in deletedFiles here means unlink fired AFTER our open.
		// Treat the close as a delete — skip writtenFile registration, let the trailing overlay branch record the delete.
		bool unlinkedWhileOpen = false;
		if (!msg.deleteOnClose && msg.newName.IsEmpty())
		{
			OverlayEditScope scope(shared.overlay);
			auto delIt = shared.deletedFiles.find(activeFile.name.c_str());
			if (delIt != shared.deletedFiles.end())
			{
				shared.deletedFiles.erase(delIt);
				unlinkedWhileOpen = true;
			}
		}

		if (msg.deleteOnClose || unlinkedWhileOpen)
		{
			RemoveWrittenFile(process, activeFile.nameKey);
		}
		else
		{
			StringKey key = activeFile.nameKey;
			StringView name = activeFile.name;
			StringView msgName = msg.fileName;
			if (!msg.newName.IsEmpty())
			{
				UBA_ASSERT(!msg.deleteOnClose);
				name = msg.newName;
				key = msg.newNameKey;
				if (!m_runningRemote)
					msgName = msg.newName;
			}
			UBA_ASSERT(key != StringKeyZero);
			auto& writtenFiles = shared.writtenFiles;

			SCOPED_FUTEX(shared.writtenFilesLock, writtenLock);
			auto insres = writtenFiles.try_emplace(key);
			WrittenFile& writtenFile = insres.first->second;

			if (!msg.newName.IsEmpty()) // Transfer potential memory map from old file
			{
				auto oldFindIt = writtenFiles.find(activeFile.nameKey);
				if (oldFindIt != writtenFiles.end())
				{
					UBA_ASSERT(insres.second || !oldFindIt->second.memoryHandle.IsValid());
					writtenFile.memoryHandle = oldFindIt->second.memoryHandle;
					writtenFile.memoryWritten = oldFindIt->second.memoryWritten;
					writtenFiles.erase(oldFindIt);
				}
				else if (writtenFile.memoryHandle.IsValid())
				{
					UBA_ASSERT(shared.sharedMemoryHandles.find(writtenFile.memoryHandle) == shared.sharedMemoryHandles.end());
					UBA_ASSERT(writtenFile.memoryHandle != msg.memoryHandle);
					m_sharedMemory.CloseHandle(m_logger, writtenFile.memoryHandle, msg.newName.data);
					writtenFile.memoryHandle = {};
					writtenFile.originalMemoryHandle = {};
				}
			}

			if (m_allowOutputFiles && writtenFile.ownerId && writtenFile.ownerId != process.m_id)
			{
				// This can happen when library has /GL (whole program optimization) but target has not.. then link.exe will restart
				//UBA_ASSERTF(false, TC("File %s changed owner.. should not happen (OldOwner: %s New owner: %s)"), name, writtenFile.owner->m_realApplication.c_str(), process.m_realApplication.c_str());
			}

			writtenFile.attributes = msg.attributes;

			bool addMapping = true;
			if (insres.second)
			{
				writtenFile.name = name.ToString();
				writtenFile.key = key;
				writtenFile.backedName = msgName.ToString();
				writtenFile.ownerId = process.m_id;
			}
			else
			{
				if (writtenFile.backedName != msgName.data)
				{
					UBA_ASSERT(!msg.memoryHandle.IsValid() && !writtenFile.memoryHandle.IsValid());
					writtenFile.backedName = msgName.ToString();
				}

				if (!msg.memoryHandle.IsValid() || (msg.memoryHandle == writtenFile.originalMemoryHandle && writtenFile.ownerId == process.m_id))
				{
					if (msg.fileSize)
					{
						writtenFile.memoryWritten = msg.fileSize;
						writtenFile.lastWriteTime = GetSystemTimeAsFileTime();
					}
					addMapping = false;
				}
				else if (writtenFile.memoryHandle.IsValid())
				{
					#if UBA_DEBUG_TRACK_MAPPING
					m_debugLogger->Info(TC("Closing old mapping 0x%llx for %s"), u64(writtenFile.memoryHandle.ToU64()), writtenFile.name.c_str());
					#endif
					
					if (writtenFile.memoryHandle != msg.memoryHandle)
						m_sharedMemory.CloseHandle(m_logger, writtenFile.memoryHandle, msg.fileName.data);
					writtenFile.memoryHandle = {};
					writtenFile.originalMemoryHandle = {};
				}

				writtenFile.ownerId = process.m_id;
			}

			if (!m_runningRemote && HasVfs(startInfo.rootsHandle)) // For posix we write the dependency file directly to disk so we need to update it if vfs is enabled
			{
				bool escapeSpaces;
				if (!msg.memoryHandle.IsValid() && rules.ShouldDevirtualizeFile(activeFile.name, escapeSpaces))
				{
					// TODO: On linux we don't use file mappings for outputs yet.. so we have to open the file and change it
					FileAccessor readFile(m_logger, name.data);
					if (!readFile.OpenMemoryRead())
						return false;
					void* mem = readFile.GetData();
					fileSize = readFile.GetSize();
					MemoryBlock block(TC("DevirtualizeDepsFile"), 5*1024*1024);
					RootsHandle rootsHandle = startInfo.rootsHandle;
					if (!DevirtualizeDepsFile(rootsHandle, block, mem, fileSize, escapeSpaces, name.data))
						return false;
					if (!readFile.Close())
						return false;
					FileAccessor writeFile(m_logger, name.data);
					if (!writeFile.CreateWrite())
						return false;
					if (!writeFile.Write(block.memory, block.writtenSize))
						return false;
					if (!writeFile.Close(&lastWriteTime))
						return false;
					fileSize = block.writtenSize;
				}
			}

			{
				// We need to check if there are any memory maps for this file from before. If it is we need to update it (for example, build steps that writes .h files might be here)
				// TODO For now we just turn that mapping to a normal file but maybe we should recreate the file mapping with a new mapping
				SCOPED_FUTEX(m_fileMappingTableLookupLock, lookupLock);
				auto findMappingIt = m_fileMappingTableLookup.find(key);
				if (findMappingIt != m_fileMappingTableLookup.end())
				{
					FileMappingEntry& entry = findMappingIt->second;
					SCOPED_FUTEX(entry.lock, entryLock);
					if (entry.memoryHandle.IsValid()) // If there is a memory map we need to make sure to update it..
					{
						//m_logger.Warning(TC("MOVING %s to file"), name.data), 
						writtenFile.memoryHandle = {};
						UBA_ASSERT(!msg.memoryHandle.IsValid());
						entry.contentSize = InvalidValue;
						entry.isDir = false;
						entry.success = true;
						entry.memoryHandle = {};
						entry.handled = true;
						#if UBA_DEBUG_TRACK_MAPPING
						entry.name = TC("#");
						#endif
						SCOPED_WRITE_LOCK(m_fileMappingTableMemLock, memLock);
						BinaryWriter writer(m_fileMappingTableMem, m_fileMappingTableSize, FileMappingTableMaxSize);
						writer.WriteStringKey(key);
						if (m_runningRemote)
							writer.WriteString(writtenFile.backedName);
						else
							writer.WriteString(TCV("#"));
						writer.Write7BitEncoded(InvalidValue);
						m_fileMappingTableSize = (u32)writer.GetPosition();
						addMapping = false;
					}
					else if (m_runningRemote && writtenFile.backedName != writtenFile.name)
					{
						UBA_ASSERT(StringView(writtenFile.backedName).StartsWith(m_sessionOutputDir));
						SCOPED_WRITE_LOCK(m_fileMappingTableMemLock, memLock);
						BinaryWriter writer(m_fileMappingTableMem, m_fileMappingTableSize, FileMappingTableMaxSize);
						writer.WriteStringKey(key);
						writer.WriteString(writtenFile.backedName);
						writer.Write7BitEncoded(InvalidValue);
						m_fileMappingTableSize = (u32)writer.GetPosition();
					}
				}
			}

			if (addMapping)
			{
				shared.sharedMemoryHandles.erase(msg.memoryHandle); // memory handle transferred to written files
				writtenFile.memoryHandle = msg.memoryHandle;
				writtenFile.originalMemoryHandle = msg.memoryHandle;
				writtenFile.memoryWritten = msg.fileSize;
				writtenFile.lastWriteTime = GetSystemTimeAsFileTime();

				#if UBA_DEBUG_TRACK_MAPPING
				m_debugLogger->Info(TC("Adding written file with mapping 0x%llx (from 0x%llx) for %s"), u64(writtenFile.memoryHandle.ToU64()), u64(msg.memoryHandle.ToU64()), writtenFile.name.c_str());
				#endif
			}

			if (writtenFile.memoryHandle.IsValid())
			{
				registerRealFile = false;
				fileSize = writtenFile.memoryWritten;
				lastWriteTime = writtenFile.lastWriteTime;
			}
		}

		UBA_ASSERT(fileSize != InvalidValue);
		if (!msg.newName.IsEmpty())
		{
			OverlayEditScope scope(shared.overlay);
			RegisterOverlayDelete(scope, activeFile.nameKey, activeFile.name);
			RegisterOverlayCreateOrWrite(scope, msg.newNameKey, msg.newName, fileSize, lastWriteTime, attributes);

			// Atomic-replace (unlink A → open A.tmp → rename-while-open → close): clear any prior
			// delete/move on newName so finalize doesn't leak a stale DeleteFile alongside upload.
			shared.deletedFiles.erase(msg.newName.data);
			shared.movedFiles.erase(msg.newName.data);

			//TraceWrittenFile(process.m_id, msg.newName, fileSize);
		}
		else if (msg.deleteOnClose || unlinkedWhileOpen)
		{
			OverlayEditScope scope(shared.overlay);
			RegisterOverlayDelete(scope, activeFile.nameKey, activeFile.name);
		}
		else
		{
			OverlayEditScope scope(shared.overlay);
			RegisterOverlayCreateOrWrite(scope, activeFile.nameKey, activeFile.name, fileSize, lastWriteTime, attributes);
			//TraceWrittenFile(process.m_id, activeFile.name, fileSize);
		}

		out.directoryTableSize = GetDirectoryTableSize(&shared.overlay);
		return true;
	}

	bool Session::DeleteFile(DeleteFileResponse& out, const DeleteFileMessage& msg)
	{
		UBA_ASSERT(!m_runningRemote); // Only implemented for local
		auto& shared = msg.process.m_shared;

		out.result = true;
		bool deleteRealFile = true;

		if (msg.closeId != 0)
		{
			UBA_ASSERTF(false, TC("Deleting a file that has closeId %u. Should be closed first (%s)"), msg.closeId, msg.fileName.data);
			SCOPED_FUTEX(shared.activeFilesLock, lock);
			shared.activeFiles.erase(msg.closeId);
		}

		{
			SCOPED_FUTEX(shared.writtenFilesLock, lock);
			auto findIt = shared.writtenFiles.find(msg.fileNameKey);
			if (findIt != shared.writtenFiles.end())
			{
				WrittenFile& writtenFile = findIt->second;
				if (writtenFile.memoryHandle.IsValid())
				{
					m_sharedMemory.CloseHandle(m_logger, writtenFile.memoryHandle, writtenFile.name.c_str());
					deleteRealFile = false;
				}
				shared.writtenFiles.erase(findIt);
			}
		}

		RemoveWrittenFile(msg.process, msg.fileNameKey);

		if (deleteRealFile)
			out.result = uba::DeleteFileW(msg.fileName.data);
		out.errorCode = GetLastError();

		{
			OverlayEditScope scope(shared.overlay);
			RegisterOverlayDelete(scope, msg.fileNameKey, msg.fileName);

			#if !PLATFORM_WINDOWS
			shared.symlinks.erase(msg.fileName.data);
			#endif
			shared.movedFiles.erase(msg.fileName.data);
			shared.deletedFiles.emplace(msg.fileName.data);
		}

		out.directoryTableSize = GetDirectoryTableSize(&msg.process.m_shared.overlay);
		return true;
	}

	bool Session::CopyFile(CopyFileResponse& out, const CopyFileMessage& msg)
	{
		StringView fromName = msg.fromName;
		StringView toName = msg.toName;
		auto& process = msg.process;
		auto& shared = process.m_shared;
		auto& writtenFiles = shared.writtenFiles;(void)writtenFiles;

		{
			SCOPED_FUTEX(shared.writtenFilesLock, lock);
			UBA_ASSERTF(writtenFiles.find(msg.toKey) == writtenFiles.end(), TC("Not implemented, please fix (%s to %s)"), fromName.data, toName.data);
			auto findIt = writtenFiles.find(msg.fromKey);
			if (findIt != writtenFiles.end())
				out.fromName.Clear().Append(findIt->second.backedName);
		}
		if (!out.fromName.count)
		{
			CreateFileMessage readMsg { msg.process };
			readMsg.fileName.Append(msg.fromName);
			readMsg.fileNameKey = msg.fromKey;
			readMsg.access = FileAccess_Read;
			CreateFileResponse readOut;
			if (!CreateFile(readOut, readMsg))
				return false;
			out.fromName.Append(readOut.fileName);
		}

		CreateFileMessage writeMsg { msg.process };
		writeMsg.fileName.Append(msg.toName);
		writeMsg.fileNameKey = msg.toKey;
		writeMsg.access = FileAccess_Write;
		CreateFileResponse writeOut;
		if (!CreateFile(writeOut, writeMsg))
			return false;

		out.directoryTableSize = writeOut.directoryTableSize;
		out.toName.Append(writeOut.fileName);
		out.closeId = writeOut.closeId;
		return true;
	}

	bool Session::MoveFile(MoveFileResponse& out, const MoveFileMessage& msg)
	{
		// Local-only: disk + per-process overlay; main dirtable stays clean.
		UBA_ASSERT(!m_runningRemote);

		auto& process = msg.process;
		bool isMoved = false;
		auto& shared = process.m_shared;

		auto updateDirTable = [&]()
			{
				OverlayEditScope scope(shared.overlay);
				RegisterOverlayDelete(scope, msg.fromKey, msg.fromName);
				RegisterOverlayCreateOrWrite(scope, msg.toKey, msg.toName, msg.fileSize, msg.lastWriteTime, msg.fileAttributes, msg.symlinkContent);

				// Track end-state only (ops were applied to disk above).
				shared.movedFiles.erase(msg.fromName.data);
				shared.movedFiles.try_emplace(msg.toName.data);
				shared.deletedFiles.erase(msg.toName.data);
				shared.deletedFiles.emplace(msg.fromName.data);

				#if !PLATFORM_WINDOWS
				UBA_ASSERT(shared.symlinks.find(msg.toName.data) == shared.symlinks.end());
				shared.symlinks.erase(msg.fromName.data);
				#endif
			};


		{
			SCOPED_FUTEX(shared.writtenFilesLock, writtenLock);
			auto findIt = shared.writtenFiles.find(msg.fromKey);
			if (findIt != shared.writtenFiles.end())
			{
				auto& oldFile = findIt->second;
				bool isMapping = oldFile.memoryHandle.IsValid();
				if (!isMapping)
				{
					out.result = MoveFileExW(msg.fromName.data, msg.toName.data, msg.flags);
					if (!out.result)
					{
						out.errorCode = GetLastError();
						return true;
					}
					isMoved = true;
				}

				UBA_ASSERT(msg.toKey != StringKeyZero);
				auto insres = shared.writtenFiles.try_emplace(msg.toKey);
				WrittenFile& newFile = insres.first->second;
				newFile = oldFile;
				newFile.key = msg.toKey;
				newFile.name = msg.toName.data;
				if (!oldFile.memoryHandle.IsValid())
					newFile.backedName = newFile.name; // Is this right?
				shared.writtenFiles.erase(findIt);

				if (isMapping)
				{
					out.errorCode = ERROR_SUCCESS;
					out.result = true;
					updateDirTable();
					out.directoryTableSize = GetDirectoryTableSize(&shared.overlay);
					return true;
				}
			}
		}

		if (!isMoved)
		{
			out.result = MoveFileExW(msg.fromName.data, msg.toName.data, msg.flags);
			if (!out.result)
			{
				out.errorCode = GetLastError();
				return true;
			}
		}

		updateDirTable();

		out.errorCode = ERROR_SUCCESS;

		out.directoryTableSize = GetDirectoryTableSize(&shared.overlay);
		return true;
	}

	bool Session::Chmod(ChmodResponse& out, const ChmodMessage& msg)
	{
		#if PLATFORM_WINDOWS
		UBA_ASSERT(false); // This is not used
		#else
		
		// TODO: This entire function should use overlays
		out.errorCode = 0;
		if (chmod(msg.fileName.data, (mode_t)msg.fileMode) == 0)
		{
			auto& shared = msg.process.m_shared;
			// Session-temp paths are per-process scratch; RegisterCreateFileForWrite asserts !IsInTemp.
			if (!IsInTemp(msg.fileName))
				RegisterCreateFileForWrite(msg.fileNameKey, msg.fileName, 0, 0, 0, true, true);
			return true;
		}
		out.errorCode = errno;
		#endif
		return true;
	}

	bool Session::Symlink(SymlinkResponse& out, const SymlinkMessage& msg)
	{
		#if PLATFORM_WINDOWS
		UBA_ASSERT(false); // This is not used
		#else
		out.errorCode = 0;
		if (!m_runningRemote)
		{
			out.success = symlink(msg.symlinkContent.data, msg.symlinkFilePath.data) == 0;
			if (!out.success)
			{
				out.errorCode = errno;
				return true;
			}
		}
		else
		{
			// Should probably create output file, point it to the other file with a real symlink
			// Let's see how far we get by not doing anything except updating the dir table
			out.success = true;
		}

		auto& shared = msg.process.m_shared;
		auto& overlay = shared.overlay;

		// Register the symlink FILE (not its target); pass content so the entry stores S_IFLNK + target.
		{
			OverlayEditScope scope(overlay);
			if (!RegisterOverlayCreateOrWrite(scope, msg.filePathKey, msg.symlinkFilePath, msg.fileSize, msg.lastWriteTime, msg.fileAttributes, msg.symlinkContent))
				return false;
			shared.symlinks.try_emplace(msg.symlinkFilePath.ToString(), msg.symlinkContent.ToString());
		}

		out.directoryTableSize = GetDirectoryTableSize(&overlay);

		#endif
		return true;
	}

	bool Session::CreateDirectory(CreateDirectoryResponse& out, const CreateDirectoryMessage& msg)
	{
		auto& shared = msg.process.m_shared;
		auto& overlay = shared.overlay;

		// To consider. In the future we might have all output files in memory (or backed by guid-based file)
		// And then we can remove these create directory and only add them after process has finished
		if (!m_runningRemote)
			uba::CreateDirectoryW(msg.name.data);

		{
			OverlayEditScope scope(overlay);
			RegisterOverlayCreateOrWrite(scope, msg.nameKey, msg.name, 0, 0, DefaultDirAttributes());

			TString dirPath = msg.name.ToString();
			UBA_ASSERT(!dirPath.empty());
			if (!shared.removedDirectories.erase(dirPath))
				shared.createdDirectories.insert(dirPath);
		}

		out.directoryTableSize = GetDirectoryTableSize(&overlay);

		return true;
	}

	bool Session::RemoveDirectory(RemoveDirectoryResponse& out, const RemoveDirectoryMessage& msg)
	{
		auto& shared = msg.process.m_shared;
		auto& overlay = shared.overlay;

		{
			OverlayEditScope scope(overlay);
			RegisterOverlayDelete(scope, msg.nameKey, msg.name);
			TString dirPath = msg.name.ToString();
			if (!shared.createdDirectories.erase(dirPath))
				shared.removedDirectories.insert(dirPath);
		}

		out.directoryTableSize = GetDirectoryTableSize(&overlay);

		// Read comment in CreateDirectory
		if (!m_runningRemote)
			uba::RemoveDirectoryW(msg.name.data);
		return true;
	}

	bool Session::GetFullFileName(GetFullFileNameResponse& out, const GetFullFileNameMessage& msg)
	{
		// Diagnostic: this RPC should be remote-only (dlopen detour short-circuits in local mode).
		// Echo the input back to keep the build going while we track down the offender.
		m_logger.Info(TC("[GetFullFileName-local] file=%s app=%s edge=%s cwd=%s"), msg.fileName.data, msg.process.m_startInfo.application, msg.process.m_startInfo.GetDescription(), msg.process.m_startInfo.workingDir);
		out.fileName.Append(msg.fileName);
		out.virtualFileName.Append(msg.fileName);
		out.mappedFileTableSize = GetFileMappingSize();
		return true;
	}

	bool Session::GetLongPathName(GetLongPathNameResponse& out, const GetLongPathNameMessage& msg)
	{
		// Same as GetFullFileName: local mode has no remote-binary indirection.
		out.fileName.Append(msg.fileName);
		out.errorCode = 0;
		return true;
	}

	bool Session::GetListDirectoryInfo(ListDirectoryResponse& out, const StringView& dirName, const StringKey& dirKey, DirectoryTableOverlay* overlay)
	{
		DirTableOffset tableOffset;
		u32 tableSize = WriteDirectoryEntries(dirKey, dirName, &tableOffset);
		out.tableOffset = tableOffset;
		u32 overlaySize = 0;
		if (overlay)
		{
			SCOPED_READ_LOCK(overlay->lock, lock);
			if (!IsValidEntry(tableOffset))
			{
				auto findIt = overlay->lookup.find(dirKey);
				if (findIt != overlay->lookup.end())
					tableOffset = {findIt->second.offset | OverlayTableFlag};
			}
			overlaySize = overlay->size;
		}

		out.directoryTableSize = {tableSize, overlaySize};
		return true;
	}

	bool Session::WriteFilesToDisk(ProcessImpl& process, WrittenFile** files, u32 fileCount)
	{
		if (!fileCount)
			return true;

		// This is to not kill I/O when writing lots of pdb/dlls in parallel
		#if PLATFORM_WINDOWS
		BottleneckScope scope(m_writeFilesBottleneck, Stats().waitBottleneck);
		#endif

		if (process.IsCancelled())
			return false;

		Atomic<bool> success = true;
		auto span = std::span(files, fileCount);
		m_workManager.ParallelFor(fileCount - 1, span, [&](const WorkContext&, auto& it)
			{
				KernelStatsScope ks(process.m_kernelStats);
				StorageStatsScope ss(process.m_storageStats);
				SessionStatsScope sessionStatsScope(process.m_sessionStats);
				if (!WriteFileToDisk(process, **it))
					success = false;
			}, TCV("WriteFilesToDisk"));
		return success;
	}

	bool Session::AllocFailed(Process& process, const tchar* allocType, u32 error)
	{
		m_logger.Warning(TC("Allocation failed in %s (%s).. process will sleep and try again"), allocType, LastErrorToText(error).data);
		return true;
	}

	bool Session::GetNextProcess(Process& process, bool& outNewProcess, bool& outShouldExit, NextProcessInfo& outNextProcess, u32 prevExitCode, BinaryReader& prevStats, u32 timeoutMs)
	{
		outShouldExit = false;
		outNewProcess = false;

		if (!m_getNextProcessFunction)
			return true;

		outNewProcess = m_getNextProcessFunction(process, outNextProcess, prevExitCode, prevStats, timeoutMs, outShouldExit);
		if (!outNewProcess)
			return true;

		m_trace.ProcessEnvironmentUpdated(process.GetId(), outNextProcess.description, prevExitCode, prevStats.GetPositionData(), prevStats.GetLeft(), outNextProcess.breadcrumbs);
		return true;
	}

	bool Session::CustomMessage(Process& process, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 recvSize = reader.ReadU32();
		u32* sendSize = (u32*)writer.AllocWrite(4);
		void* sendData = writer.GetData() + writer.GetPosition();
		u32 written = 0;
		if (m_customServiceFunction)
			written = m_customServiceFunction(process, reader.GetPositionData(), recvSize, sendData, u32(writer.GetCapacityLeft()));
		*sendSize = written;
		writer.AllocWrite(written);
		return true;
	}

	bool Session::SHGetKnownFolderPath(Process& process, BinaryReader& reader, BinaryWriter& writer)
	{
		UBA_ASSERT(false); // Should only be called on UbaSessionClient
		return false;
	}

	bool Session::HostRun(BinaryReader& reader, BinaryWriter& writer)
	{
#if PLATFORM_WINDOWS
		TString temp;
		const tchar* subdir = nullptr;
		int csidl = (int)reader.ReadU32();
		DWORD flags = reader.ReadU32();
		if (reader.ReadBool())
		{
			temp = reader.ReadString();
			subdir = temp.data();
		}

		static HMODULE shellDll = LoadLibrary(TC("Shell32.dll"));
		using SHGetFolderPathAndSubDirWFunc = HRESULT(HWND hwnd, int csidl, HANDLE hToken, DWORD dwFlags, LPCWSTR pszSubDir, LPWSTR pszPath);
		static SHGetFolderPathAndSubDirWFunc* SHGetFolderPathAndSubDirW = (SHGetFolderPathAndSubDirWFunc*)(void*)GetProcAddress(shellDll, "SHGetFolderPathAndSubDirW");

		tchar path[MAX_PATH];
		HRESULT res = SHGetFolderPathAndSubDirW(0, csidl, NULL, flags, subdir, path);
		writer.WriteU32(res);
		writer.WriteString(path);
#else
		Vector<TString> args;
		while (reader.GetLeft())
			args.push_back(reader.ReadString());
		bool success = false;

		StringBuffer<> command;
		for (auto& arg : args)
		{
			if (command.count)
				command.Append(' ');
			command.Append(arg);
		}

		char result[4096];
		if (FILE* fp = popen(command.data, "r"))
		{
			char* dest = result;
			errno = 0;
			while (true)
			{
				if (!fgets(dest, sizeof(result) - (dest - result), fp))
				{
					success = errno == 0;
					if (!success)
						snprintf(result, sizeof(result), "fgets failed with command: %s", command.data);
					break;
				}
				dest += strlen(dest);
			}
			pclose(fp);
		}
		else
			snprintf(result, sizeof(result), "popen failed with command: %s", command.data);
		writer.WriteBool(success);
		writer.WriteString(result);
#endif
		return true;
	}

	bool Session::GetSymbols(const tchar* application, bool isLinux, bool isArm, BinaryReader& reader, BinaryWriter& writer)
	{
		StringBuffer<256> detoursLibPath;
		StringBuffer<256> alternativeLibPath;

		detoursLibPath.Append(m_detoursLibrary[IsLinux][IsArmBinary].c_str()).Resize(detoursLibPath.Last(PathSeparator) - detoursLibPath.data);
		GetAlternativeUbaPath(m_logger, alternativeLibPath, detoursLibPath, IsArmBinary);

		StringView searchPaths[3] = { detoursLibPath, alternativeLibPath, {} };

		u64 size = reader.ReadU32();
		BinaryReader reader2(reader.GetPositionData(), 0, size);

		StringBuffer<16*1024> sb;
		ParseCallstackInfo(sb, reader2, application, searchPaths);
		writer.WriteString(sb);
		return true;
	}

	bool Session::CheckRemapping(ProcessImpl& process, BinaryReader& reader, BinaryWriter& writer)
	{
		StringBuffer<> fileName;
		reader.ReadString(fileName);
		StringKey fileNameKey = reader.ReadStringKey();
		UBA_ASSERT(fileNameKey != StringKeyZero);

		#if UBA_DEBUG_TRACK_MAPPING
		//m_debugLogger->Info(TC("Mapping check (%s)"), fileName.data);
		#endif

		auto wg = MakeGuard([&]() { writer.WriteU32(GetFileMappingSize()); });

		MemoryMap out;
		u64 alignment = GetMemoryMapAlignment(fileName);
		bool canBeFreed = true;
		if (process.IsCancelled())
			return false;
		if (!GetOrCreateMemoryMapFromFile(out, fileNameKey, fileName.data, false, alignment, TC("Remap"), &process, canBeFreed))
			return m_logger.Warning(TC("Failed to remap %s"), fileName.data);
		return true;
	}

	bool Session::RunSpecialProgram(ProcessImpl& process, BinaryReader& reader, BinaryWriter& writer)
	{
		UBA_ASSERT(false);
		return true;
	}
	
	bool Session::FlushWrittenFiles(ProcessImpl& process)
	{
		return true;
	}

	bool Session::UpdateEnvironment(ProcessImpl& process, const StringView& reason, bool resetStats)
	{
		if (!resetStats)
			return true;
		UBA_ASSERT(!m_runningRemote); // local do not write session stats
		StackBinaryWriter<16 * 1024> writer;
		process.m_processStats.Write(writer);
		process.m_storageStats.Write(writer);
		process.m_kernelStats.Write(writer);
		m_trace.ProcessEnvironmentUpdated(process.GetId(), reason, 0, writer.GetData(), writer.GetPosition(), ToView(process.GetStartInfo().breadcrumbs));
		process.m_processStats = {};
		process.m_storageStats = {};
		process.m_kernelStats = {};
		return true;
	}

	bool Session::LogLine(ProcessImpl& process, const tchar* line, LogEntryType logType)
	{
		return true;
	}

	void Session::EmergencyShutdown()
	{
		FatalError(EmergencyShutdownExitCode, TC("Emergency shutdown. Likely out-of-memory"));
	}

	void Session::PrintSessionStats(Logger& logger)
	{
		logger.Info(TC("  DirectoryTable      %7u %9s"), u32(m_directoryTable.m_lookup.size()), BytesToText(GetDirectoryTableSize(nullptr).main).str);
		logger.Info(TC("  MappingTable        %7u %9s"), u32(m_fileMappingTableLookup.size()), BytesToText(GetFileMappingSize()).str);
		logger.Info(TC("  MappingBuffer       %7u %9s"), 1, BytesToText(m_fileMappingBuffer.GetUsed()).str);
		logger.Info(TC("  ProcessReuseCount   %7u"), m_reuseCount.load());
		m_stats.Print(logger);
		logger.Info(TC(""));
	}

	bool Session::RegisterVirtualFileInternal(const StringKey& fileNameKey, const StringView& filePath, const tchar* sourceFile, u64 sourceOffset, u64 sourceSize)
	{
		TimerScope ts(Stats().createMmapFromFile);

		VirtualSourceFile virtualFile;
		StringKey sourceFileKey = CaseInsensitiveFs ? ToStringKeyLower(ToView(sourceFile)) : ToStringKey(ToView(sourceFile));
		SCOPED_FUTEX(m_virtualSourceFilesLock, virtualSourceFilesLock);
		auto insres = m_virtualSourceFiles.try_emplace(sourceFileKey); 
		if (insres.second)
		{
			FileHandle fileHandle = uba::CreateFileW(sourceFile, GENERIC_READ, FILE_SHARE_READ, OPEN_EXISTING, DefaultAttributes());
			if (fileHandle == InvalidFileHandle)
				return m_logger.Error(TC("[RegisterVirtualFileInternal] CreateFileW for %s failed (%s)"), sourceFile, LastErrorToText().data);
			auto fg = MakeGuard([&]() { uba::CloseFile(sourceFile, fileHandle); });
			u64 fileSize = 0;
			if (!GetFileSizeEx(fileSize, fileHandle))
				return m_logger.Error(TC("[RegisterVirtualFileInternal] GetFileSizeEx for %s failed (%s)"), sourceFile, LastErrorToText().data);
			virtualFile.mappingHandle = FileMapping_CreateFromFile(m_logger, fileHandle, PAGE_READONLY, fileSize, sourceFile);
			virtualFile.size = fileSize;
			insres.first->second = virtualFile;
		}
		else
			virtualFile = insres.first->second;
		if (!virtualFile.mappingHandle.IsValid())
			return m_logger.Error(TC("[RegisterVirtualFileInternal] CreateFileMapping for %s failed (%s)"), sourceFile, LastErrorToText().data);
		virtualSourceFilesLock.Leave();

		if (sourceSize + sourceOffset > virtualFile.size)
			return m_logger.Error(TC("Virtual file offset(%llu)+size(%llu) outside source file size(%llu)"), sourceOffset, sourceSize, virtualFile.size, filePath.data);

		SharedMemoryHandle memoryHandle = m_sharedMemory.RegisterExternalMapping(virtualFile.mappingHandle);

		return RegisterVirtualFileInternal(fileNameKey, filePath, memoryHandle, sourceSize, sourceOffset, false);
	}

	bool Session::CreateVirtualFileInternal(const StringKey& fileNameKey, const StringView& filePath, const void* memory, u64 memorySize, bool transient)
	{
		TimerScope ts(Stats().createMmapFromFile);

		SCOPED_FUTEX(m_virtualSourceFilesLock, virtualSourceFilesLock);
		auto insres = m_virtualSourceFiles.try_emplace(fileNameKey); 
		if (!insres.second)
			return false;
		VirtualSourceFile& virtualFile = insres.first->second;

		SharedMemoryHandle mapping = m_sharedMemory.CreateHandle(m_logger, filePath.data);
		if (!mapping.IsValid())
			return false;
		m_sharedMemory.ExtendMemory(mapping, memorySize, filePath.data, false);
		u8* mem2 = m_sharedMemory.MapView(mapping, filePath.data, SharedMemoryMapType_ReadWrite);
		if (!mem2)
			return false;
		FileMapping_CopyMem(mem2, memory, memorySize);
		m_sharedMemory.UnmapView(mapping, mem2);

		virtualFile.memoryHandle = mapping;
		virtualFile.size = memorySize;
		return RegisterVirtualFileInternal(fileNameKey, filePath, virtualFile.memoryHandle, memorySize, 0, transient);
	}

	bool Session::RegisterVirtualFileInternal(const StringKey& fileNameKey, const StringView& filePath, SharedMemoryHandle memoryHandle, u64 mappingSize, u64 mappingOffset, bool transient)
	{
		SCOPED_FUTEX(m_fileMappingTableLookupLock, lookupLock);
		auto insres = m_fileMappingTableLookup.try_emplace(fileNameKey);
		FileMappingEntry& entry = insres.first->second;
		lookupLock.Leave();

		SCOPED_FUTEX(entry.lock, entryLock);
		
		if (entry.handled)
			return m_logger.Error(TC("Virtual file %s has already been registered"), filePath.data);

		entry.memoryHandle = memoryHandle;
		entry.memoryOffset = mappingOffset;
		entry.contentSize = mappingSize;
		entry.handled = true;
		entry.lastWriteTime = 0; // TODO: Take lastwritetime of source file?
		entry.createIndependentMapping = true; // We want independent mapping in casdb so they can be deleted
		entry.isInvisible = transient;
		entry.dropCasAfterUse = true;
		entry.canBeFreed = false;

		StringBuffer<> mappingName;
		GetMappingString(mappingName, memoryHandle, mappingOffset, entry.canBeFreed);

		entry.success = true;

		#if UBA_DEBUG_TRACK_MAPPING
		entry.name = filePath.data;
		m_debugLogger->Info(TC("Mapping created 0x%llx (%s) from virtual file"), u64(entry.memoryHandle.ToU64()), entry.name.c_str());
		#endif

		if (!transient)
		{
			SCOPED_WRITE_LOCK(m_fileMappingTableMemLock, lock);
			BinaryWriter writer(m_fileMappingTableMem, m_fileMappingTableSize, FileMappingTableMaxSize);
			writer.WriteStringKey(fileNameKey);
			writer.WriteString(mappingName);
			writer.Write7BitEncoded(mappingSize);
			m_fileMappingTableSize = (u32)writer.GetPosition();
		}
		return true;
	}

	bool Session::GetOutputFileSizeInternal(u64& outSize, const StringKey& fileNameKey, StringView filePath)
	{
		SCOPED_FUTEX(m_fileMappingTableLookupLock, lookupLock);
		auto findIt = m_fileMappingTableLookup.find(fileNameKey);
		if (findIt == m_fileMappingTableLookup.end())
			return false;
		FileMappingEntry& entry = findIt->second;
		lookupLock.Leave();
		SCOPED_FUTEX(entry.lock, entryCs);
		outSize = entry.contentSize;
		return true;
	}

	bool Session::GetOutputFileDataInternal(void* outData, const StringKey& fileNameKey, StringView filePath, bool deleteInternalMapping)
	{
		return GetFileMemory([&](const void* fileMem, u64 fileSize) { FileMapping_CopyMem(outData, fileMem, fileSize); return true; }, fileNameKey, filePath, deleteInternalMapping);
	}

	bool Session::WriteOutputFileInternal(const StringKey& fileNameKey, StringView filePath, bool deleteInternalMapping)
	{
		//m_logger.Info(TC("TRYING TO WRITE OUTPUT %s (%s)"), filePath, KeyToString(fileNameKey).data);
		FileAccessor destinationFile(m_logger, filePath.data);
		if (!GetFileMemory([&](const void* fileMem, u64 fileSize) { return WriteMemoryToDisk(destinationFile, fileMem, fileSize, nullptr); }, fileNameKey, filePath, deleteInternalMapping))
			return false;
		if (!destinationFile.Close())
			return false;
		return true;
	}

	bool Session::CreateProcessJobObject()
	{
		#if PLATFORM_WINDOWS
		m_processJobObject = CreateJobObject(nullptr, nullptr);
		if (!m_processJobObject)
			return m_logger.Error(TC("Failed to create process job object"));
		JOBOBJECT_EXTENDED_LIMIT_INFORMATION info = { };
		info.BasicLimitInformation.LimitFlags = JOB_OBJECT_LIMIT_BREAKAWAY_OK | JOB_OBJECT_LIMIT_KILL_ON_JOB_CLOSE;
		SetInformationJobObject(m_processJobObject, JobObjectExtendedLimitInformation, &info, sizeof(info));
		#endif
		return true;
	}

	void Session::GetSessionInfo(StringBufferBase& out)
	{
		out.Append(TCV("Cas:"));
		out.Append(BytesToText(m_storage.GetStorageUsed()).str).Append('/');
		if (u64 capacity = m_storage.GetStorageCapacity())
			out.Append(BytesToText(capacity).str);
		else
			out.Append(TCV("NoLimit"));

		StringBuffer<128> zone;
		if (m_storage.GetZone(zone))
			out.Append(TCV(" Zone:")).Append(zone);

		if (!m_extraInfo.empty())
			out.Append(m_extraInfo);

		#if UBA_DEBUG
		out.Append(TCV(" - DEBUG"));
		#endif
	}

	bool Session::HasVfs(RootsHandle handle) const
	{
		return (handle & 1ull) == 1ull;
	}

	RootsHandle Session::WithVfs(RootsHandle handle, bool vfs) const
	{
		return vfs ? (handle | 1ull) : (handle & ~1ull);
	}

	const Session::RootsEntry* Session::GetRootsEntry(RootsHandle rootsHandle)
	{
		SCOPED_FUTEX_READ(m_rootsLookupLock, rootsLock);
		auto findIt = m_rootsLookup.find(WithVfs(rootsHandle, false));
		if (findIt == m_rootsLookup.end())
			return m_logger.Error(TC("Can't find entry from roots handle %llu"), rootsHandle) ? nullptr : nullptr;

		RootsEntry& entry = findIt->second;
		rootsLock.Leave();

		UBA_ASSERT(entry.handled);
		return &entry;
	}

	void Session::PopulateRootsEntry(RootsEntry& entry, const void* rootsData, uba::u64 rootsDataSize)
	{
		entry.memory.resize(rootsDataSize);
		memcpy(entry.memory.data(), rootsData, rootsDataSize);

		BinaryReader reader((const u8*)rootsData, 0, rootsDataSize);
		while (reader.GetLeft())
		{
			u8 id = reader.ReadByte();(void)id; // Root id.. ignore for conversion from vfs to local
			StringBuffer<> temp;
			reader.ReadString(temp);
			if (!temp.count) // If vfs is not set it means that vfs should not be used (the roots entry memory might be used for cacheclient though)
				break;
			entry.roots.RegisterRoot(m_logger, temp.data);
			entry.vfs.emplace_back(temp.data, temp.count);
			#if PLATFORM_WINDOWS
			Replace((tchar*)entry.vfs.data(), '/', '\\');
			#endif
			reader.ReadString(temp.Clear());
			entry.locals.emplace_back(temp.data, temp.count);
		}
	}

	u32 Session::PopTempDirectory(TString& out)
	{
		u32 id;
		bool isNew = false;
		SCOPED_FUTEX(m_availableTempDirectoriesLock, lock);
		if (m_availableTempDirectories.empty())
		{
			id = m_availableTempDirectoriesMax++;
			isNew = true;
		}
		else
		{
			id = m_availableTempDirectories.back();
			m_availableTempDirectories.pop_back();
		}
		lock.Leave();

		StringBuffer<> path(m_tempPath);
		path.AppendHex(id).EnsureEndsWithSlash();
		if (isNew)
			if (!uba::CreateDirectoryW(path.data))
				m_logger.Error(TC("Failed to create temp directory %s"), path.data);
		out = path.ToString();
		return id;
	}

	void Session::PushTempDirectory(u32 id)
	{
		if (id == ~0u)
			return;
		ClearTempDirectory(id);
		SCOPED_FUTEX(m_availableTempDirectoriesLock, lock);
		m_availableTempDirectories.push_back(id);
	}

	void Session::ClearTempDirectory(u32 id)
	{
		UBA_ASSERT(id != ~0u);
		StringBuffer<> temp(m_tempPath);
		temp.AppendHex(id).EnsureEndsWithSlash();
		DeleteAllFiles(m_logger, temp.data, false);
	}

	bool Session::CreateSnapshot(u32 currSize)
	{
		const u8* prevSnapshot = nullptr;
		u64 prevSnapshotSize = 0;

		const u8* mainMem = m_directoryTableMem;
		u32 mainSize = currSize;

		const u8* overlayMem = nullptr;
		u32 overlaySize = 0;

		m_dirTableSizeForSnapshot = currSize;

		if (m_dirTableSnapshot)
		{
			prevSnapshot = m_dirTableSnapshot->mappedMemory;
			prevSnapshotSize = DirTableSnapshotMaxSize;
		}

		auto snapshot = new DirTableSnapshot;

		if (!CreateDirectoryTableSnapshot(*snapshot, DirTableSnapshotMaxSize, prevSnapshot, prevSnapshotSize, mainMem, mainSize, overlayMem, overlaySize, TC("DirTableSnapshot"), m_logger))
		{
			delete snapshot;
			return false;
		}

		snapshot->refCount = 1;
		auto oldSnapshot = m_dirTableSnapshot;

		SCOPED_WRITE_LOCK(m_dirTableSnapshotLock, snapshotLock);
		m_dirTableSnapshot = snapshot;
		snapshotLock.Leave();

		ReleaseDirTableSnapshot(oldSnapshot);
		return true;
	}

	DirTableSnapshot* Session::ReferenceDirTableSnapshot()
	{
		SCOPED_READ_LOCK(m_dirTableSnapshotLock, lock);
		if (!m_dirTableSnapshot)
			return nullptr;
		++m_dirTableSnapshot->refCount;
		return m_dirTableSnapshot;
	}

	void Session::ReleaseDirTableSnapshot(DirTableSnapshot* snapshot)
	{
		if (!snapshot || --snapshot->refCount)
			return;
		FileMapping_Unmap(m_logger, snapshot->mappedMemory, DirTableSnapshotMaxSize, TC("DirTableSnapshot"));
		delete snapshot;
	}

	bool Session::ExtractSymbolsFromObjectFile(ProcessImpl& process, const tchar* fileName, const tchar* backedName, u64 fileSize, SharedMemoryHandle memoryHandle, u32 attributes)
	{
		// fileName is the logical (host-format) path that the host overlay knows; backedName is the
		// actual on-disk path (may differ on remote agents where the host path doesn't exist locally).
		// Use backedName for any FileAccessor I/O, but use fileName for keys / overlay registration.
		const tchar* readName = (backedName && *backedName) ? backedName : fileName;

		u8* mem = nullptr;

		auto memClose = MakeGuard([&]() { if (memoryHandle.IsValid()) m_sharedMemory.UnmapView(memoryHandle, mem); });

		FileAccessor objFile(m_logger, readName);

		if (memoryHandle.IsValid())
		{
			if (!fileSize)
				return true;
			mem = m_sharedMemory.MapView(memoryHandle, fileName);
		}
		else
		{
			#if PLATFORM_WINDOWS
			return m_logger.Error(TC("Can't extract symbols from obj file that is written directly to disk (%s writing %s)"), process.m_startInfo.application, fileName);
			#else
			if (!objFile.OpenMemoryRead())
				return false;
			mem = objFile.GetData();
			fileSize = objFile.GetSize();
			if (!fileSize)
				return true;
			#endif
		}

		ObjectFile* objectFile = ObjectFile::Parse(m_logger, ObjectFileParseMode_All, mem, fileSize, fileName);
		if (!objectFile)
			return false;
		auto ofg = MakeGuard([&]() { delete objectFile; });

		const tchar* lastDot = TStrrchr(fileName, '.');
		UBA_ASSERT(lastDot);
		StringBuffer<> exportsFile;
		exportsFile.Append(fileName, lastDot - fileName).Append(TCV(".exi"));

		// Backed (on-disk) path mirrors the .o → .exi extension swap. On local where backedName == fileName
		// this is identical to exportsFile; on remote it lands in the agent's actual disk location and is what
		// SendFile reads from when shipping the .exi back to the host.
		StringBuffer<> exportsBackedFile;
		if (backedName && *backedName)
		{
			if (const tchar* lastDotBacked = TStrrchr(backedName, '.'))
				exportsBackedFile.Append(backedName, lastDotBacked - backedName).Append(TCV(".exi"));
			else
				exportsBackedFile.Append(backedName).Append(TCV(".exi"));
		}
		else
		{
			exportsBackedFile.Append(exportsFile);
		}

		bool verbose = false;
		#if UBA_DEBUG
		verbose = true;
		#endif
		u8 exiVersion = process.m_extractExportsVersion ? process.m_extractExportsVersion : SymbolFileVersion;
		MemoryBlock memoryBlock(TC("ExtractSymbols"), 64*1024*1024); // Seen files that are over 32mb when verbose = true
		if (!objectFile->WriteImportsAndExports(m_logger, memoryBlock, verbose, exiVersion))
			return false;

		SharedMemoryHandle symHandle;
		StringKey symFileKey = CaseInsensitiveFs ? ToStringKeyLower(exportsFile) : ToStringKey(exportsFile);
		u64 lastWriteTime = 0;

		if (m_allowMemoryMaps)
		{
			symHandle = m_sharedMemory.CreateHandle(m_logger, TC("SymHandle"));
			auto mg = MakeGuard([&]() { m_sharedMemory.CloseHandle(m_logger, symHandle, TC("SymHandle")); });
			m_sharedMemory.ExtendMemory(symHandle, memoryBlock.writtenSize, TC("SymHandle"), false);
			u8* mem2 = m_sharedMemory.MapView(symHandle, TC("SymHandle"), SharedMemoryMapType_ReadWrite);
			if (!mem2)
				return false;

			FileMapping_CopyMem(mem2, memoryBlock.memory, memoryBlock.writtenSize);
			m_sharedMemory.UnmapView(symHandle, mem2);

			lastWriteTime = GetSystemTimeAsFileTime();
			{
				OverlayEditScope scope(process.m_shared.overlay);
				if (!RegisterOverlayCreateOrWrite(scope, symFileKey, exportsFile, memoryBlock.writtenSize, lastWriteTime, DefaultAttributes()))
					return false;
			}
			mg.Cancel();
		}
		else
		{
			FileAccessor destFile(m_logger, exportsBackedFile.data);
			if (!destFile.CreateWrite(false, DefaultAttributes(), memoryBlock.writtenSize))
				return false;
			if (!destFile.Write(memoryBlock.memory, memoryBlock.writtenSize))
				return false;
			if (!destFile.Close(&lastWriteTime))
				return false;
			OverlayEditScope scope(process.m_shared.overlay);
			if (!RegisterOverlayCreateOrWrite(scope, symFileKey, exportsFile, memoryBlock.writtenSize, lastWriteTime, DefaultAttributes()))
				return false;
		}

		auto insres = process.m_shared.writtenFiles.try_emplace(symFileKey);
		WrittenFile& writtenFile = insres.first->second;

		UBA_ASSERT(!writtenFile.ownerId || writtenFile.ownerId == process.m_id);
		writtenFile.key = symFileKey;
		writtenFile.ownerId = process.m_id;
		writtenFile.attributes = attributes;
		writtenFile.memoryHandle = symHandle;
		writtenFile.memoryWritten = memoryBlock.writtenSize;
		writtenFile.lastWriteTime = lastWriteTime;
		writtenFile.name = exportsFile.data;
		writtenFile.backedName = exportsBackedFile.data;

		return true;
	}

	bool Session::ExtractSymbolsForProcessTree(ProcessImpl& rootProcess)
	{
		auto& rules = *rootProcess.m_startInfo.rules;

		SCOPED_FUTEX(rootProcess.m_shared.writtenFilesLock, lock);
		for (auto& kv : rootProcess.m_shared.writtenFiles)
		{
			const WrittenFile& wf = kv.second;
			if (wf.name.empty())
				continue;
			if (!rules.ShouldExtractSymbols(StringView(wf.name)))
				continue;
			if (!ExtractSymbolsFromObjectFile(rootProcess, wf.name.c_str(), wf.backedName.c_str(), wf.memoryWritten, wf.memoryHandle, wf.attributes ? wf.attributes : DefaultAttributes()))
				return false;
		}
		return true;
	}

	bool Session::DevirtualizeDepsFile(RootsHandle rootsHandle, MemoryBlock& destData, const void* sourceData, u64 sourceSize, bool escapeSpaces, const tchar* hint)
	{
		auto rootsEntryPtr = GetRootsEntry(rootsHandle);
		if (!rootsEntryPtr)
			return false;
		const RootsEntry& rootsEntry = *rootsEntryPtr;

		UBA_ASSERT(!rootsEntry.locals.empty());

		Vector<std::string> localsAnsi;
		localsAnsi.reserve(rootsEntry.locals.size());

		for (auto& str : rootsEntry.locals)
		{
			char ansi[512];
			u32 ansiPos = 0;
			for (auto c : str)
			{
				UBA_ASSERT(c < 256);
				if (escapeSpaces)
				{
					if (c == ' ')
						ansi[ansiPos++] = '\\';
				}
				else
				{
					if (c == '\\')
						ansi[ansiPos++] = '\\';
				}
				ansi[ansiPos++] = (char)c;
			}
			ansi[ansiPos] = 0;
			localsAnsi.emplace_back(ansi, ansi + ansiPos);
		}

		auto handleString = [&](const char* str, u64 strLen, u32 rootPos)
			{
				if (rootPos == ~0u)
				{
					memcpy(destData.Allocate(strLen, 1, TC("")), str, strLen);
					return;
				}
				auto& path = localsAnsi[(*str - RootPaths::RootStartByte)/PathsPerRoot];
				memcpy(destData.Allocate(path.size(), 1, TC("")), path.c_str(), path.size());
			};

		return rootsEntry.roots.NormalizeString<char>(m_logger, (const char*)sourceData, sourceSize, handleString, true, hint);
	}

#if PLATFORM_LINUX
	bool Session::TryPatchBinary(StringBufferBase& out, StringView in)
	{
		SCOPED_FUTEX(m_patchedBinariesLock, lookupLock);
		auto insres = m_patchedBinaries.try_emplace(in.ToString());
		PatchedBinary& binary = insres.first->second;
		lookupLock.Leave();

		SCOPED_FUTEX(binary.lock, lock);

		if (binary.handled)
		{
			lock.Leave();
			if (binary.key != CasKey())
				out.Clear().Append(m_sessionDir).Append(TCV("bin/1111/")).Append(CasKeyString(binary.key));
			return true;
		}
		binary.handled = true;

		StringBuffer<> temp;
		temp.Append(m_sessionDir).Append(TCV("bin/1111/"));
		if (!m_storage.CreateDirectory(temp.data))
			return false;

		FileAccessor sourceFile(m_logger, in.CheckTerminated());
		if (!sourceFile.OpenMemoryRead())
			return false;

		if (!BinaryRequiresPatching(m_logger, in, sourceFile.GetData(), sourceFile.GetSize()))
			return true;

		StringBuffer<> stubPath;
		if (!GetDirectoryOfCurrentModule(m_logger, stubPath))
			return m_logger.Error(TC("Failed to resolve stub blob directory"));
		stubPath.EnsureEndsWithSlash().Append(TCV("UbaStaticStub.bin"));


		CasKey key = CalculateCasKey(sourceFile.GetData(), sourceFile.GetSize(), false, &m_workManager, in.data);
		if (key == CasKey())
			return false;

		temp.Append(CasKeyString(key));

		SCOPED_FUTEX(m_handledPatchedCasKeysLock, lock2);
		Futex& casLock = m_handledPatchedCasKeys.try_emplace(key).first->second;
		lock2.Leave();

		SCOPED_FUTEX(casLock, lock3);

		if (!FileExists(m_logger, temp.data))
			if (!PatchStaticBinary(m_logger, in, sourceFile.GetData(), sourceFile.GetSize(), temp, m_stubBinary[IsArmBinary]))
				return m_logger.Error(TC("Failed to produce static-detoured copy of %.*s"), in.count, in.data);

		binary.key = key;

		out.Clear().Append(temp);
		return true;
	}
#endif

	void Session::TraceWrittenFile(u32 processId, const StringView& file, u64 size)
	{
		if (!m_traceWrittenFiles)
			return;
		StringBuffer<> str(TC("WrittenFile: "));
		str.Append(file);
		if (!size)
		{
			size = InvalidValue;
			FileBasicInformation info;
			if (!GetFileBasicInformation(info, m_logger, file.data, false))
				str.Append(TCV(" (GetFileBasicInformation failed)")).Append(BytesToText(size).str).Append(')');
			else
				size = info.size;
		}
		if (size != InvalidValue)
			str.Append(TCV( " (size: ")).Append(BytesToText(size).str).Append(')');
		m_trace.ProcessAddBreadcrumbs(processId, str, false);
	}

	void Session::RunDependencyCrawler(ProcessImpl& process)
	{
		auto& startInfo = process.GetStartInfo();

		auto crawlerType = startInfo.rules->GetDependencyCrawlerType();
		if (crawlerType == DependencyCrawlerType_None)
			return;

		bool isLinkCrawler = crawlerType == DependencyCrawlerType_ClangLinker || crawlerType == DependencyCrawlerType_MsvcLinker;

		auto CreateFileFunc = [this, ph = ProcessHandle(&process), rules = startInfo.rules, isLinkCrawler](TrackWorkScope& tracker, const StringView& fileName, const DependencyCrawler::AccessFileFunc& func, bool& stopCrawling) mutable
			{
				return OpenFileForDependencyCrawler(tracker, fileName, func, stopCrawling, ph, isLinkCrawler, *rules);
			};

		auto DevirtualizePathFunc = [this, rootsHandle = startInfo.rootsHandle](StringBufferBase& inOut) { return DevirtualizePath(inOut, rootsHandle, false); };

		if (const tchar* at = TStrchr(startInfo.arguments, '@'))
			m_dependencyCrawler.AddRsp(at + 1, startInfo.workingDir, CreateFileFunc, DevirtualizePathFunc, startInfo.application, crawlerType, startInfo.rules->index);
		else if (!isLinkCrawler)
			m_dependencyCrawler.AddCommandLine(startInfo.arguments, startInfo.workingDir, CreateFileFunc, DevirtualizePathFunc, startInfo.application, crawlerType, startInfo.rules->index);
	}

	bool Session::OpenFileForDependencyCrawler(TrackWorkScope& tracker, StringView fileName, const DependencyCrawler::AccessFileFunc& func, bool& stopCrawling, ProcessHandle& ph, bool isLinkCrawler, const ApplicationRules& rules)
	{
		auto& process = *static_cast<ProcessImpl*>(ph.m_process);
		if (process.m_gotExitMessage || process.IsCancelled() || process.HasExited())
		{
			stopCrawling = true;
			return false;
		}

		if (fileName.IsEmpty())
			return false;

		StringKey fileNameKey = ToStringKey(fileName);

		if (isLinkCrawler)
		{
			SCOPED_FUTEX_READ(process.m_usedFileMappingsLock, lock)
				if (process.m_usedFileMappings.find(fileNameKey) != process.m_usedFileMappings.end())
					return false;
		}

		CreateFileResponse out;
		{
			tracker.AddHint(fileName);

			if (!CreateFileForRead(out, tracker, fileName, fileNameKey, process, rules))
			{
				stopCrawling = process.m_gotExitMessage || process.IsCancelled() || process.HasExited();
				return false;
			}
		}

		if (!func)
			return true;

		if (IsMemoryHandle(out.fileName.data))
		{
			MappedView view = m_fileMappingBuffer.MapView(out.fileName, out.size, fileName.data);
			if (view.memory)
			{
				bool res = func(view.memory, out.size);
				m_fileMappingBuffer.UnmapView(view, fileName.data);
				return res;
			}
			return m_logger.Warning(TC("Failed to open %s"), out.fileName.data);
		}

		if (out.fileName.Equals(TCV("$d")))
		{
			// This can happen on apple targets.. crawler finds some includes that are not proper includes
			return m_logger.Warning(TC("Trying to open directory %s as file"), fileName.data);
		}

		if (out.fileName.Equals(TCV("#")))
			out.fileName.Clear().Append(fileName);

		FileAccessor fa(m_logger, out.fileName.data);
		if (!fa.OpenMemoryRead(0, false))
			return m_logger.Warning(TC("Failed to open %s (%s)"), out.fileName.data, LastErrorToText().data);

		return func(fa.GetData(), fa.GetSize());
	}

#if UBA_USE_NATIVE_MAC_SEMAPHORES
	struct Session::SemaphoreServiceInfo { mach_port_t port = MACH_PORT_NULL; };

	u32 Session::GetSessionId()
	{
		return getpid() + (m_sessionId << 20);
	}

	bool Session::StartSemaphoreService()
	{
		static Atomic<u32> id;
		m_sessionId = id++;
		mach_port_t port = MACH_PORT_NULL;
		kern_return_t kr;
		kr = mach_port_allocate(mach_task_self(), MACH_PORT_RIGHT_RECEIVE, &port);
		if (kr != KERN_SUCCESS)
			return m_logger.Error(("mach_port_allocate failed (%u)"), kr);

		// Give ourselves a send right so the child can contact us if needed.
		// (Optional if you only expect to receive.)
		kr = mach_port_insert_right(mach_task_self(), port, port, MACH_MSG_TYPE_MAKE_SEND);
		if (kr != KERN_SUCCESS)
			return m_logger.Error(("mach_port_insert_right failed (%u)"), kr);

		#pragma clang diagnostic push
		#pragma clang diagnostic ignored "-Wdeprecated-declarations"
		StringBuffer<64> service;
		service.Append("UbaSemaphoreService").AppendValue(GetSessionId());
		//name_t service = "com.epicgames.uba";
		kr = bootstrap_register(bootstrap_port, service.data, port);
		if (kr != KERN_SUCCESS)
			return m_logger.Error(("bootstrap_register failed trying to create %s (%u)"), service.data, kr);
		#pragma clang diagnostic pop

		m_semaphoreServiceInfo = new SemaphoreServiceInfo { port };
		m_semaphoreServiceThread.Start([this]() { ThreadSemaphoreService(); return 0; }, TC("UbaSemSrv"));

		return true;
	}
	
	void Session::StopSemaphoreService()
	{
		#pragma clang diagnostic push
		#pragma clang diagnostic ignored "-Wdeprecated-declarations"
		mach_port_destroy(mach_task_self(), m_semaphoreServiceInfo->port);
		#pragma clang diagnostic pop

		delete m_semaphoreServiceInfo;
		m_semaphoreServiceInfo = nullptr;
		m_semaphoreServiceThread.Wait();
	}

	void Session::ThreadSemaphoreService()
	{
		typedef struct {
			mach_msg_header_t header;
			u32 pid;
			uint8_t           extra[512];
		} request_msg_t;

		typedef struct {
			mach_msg_header_t        header;
			mach_msg_body_t          body;
			mach_msg_port_descriptor_t sem_desc[3]; // carries the semaphore right
		} reply_msg_t;

		kern_return_t kr;
		mach_port_t port = m_semaphoreServiceInfo->port;

		while (m_semaphoreServiceInfo)
		{
			request_msg_t req;
			memset(&req, 0, sizeof(req));
			kr = mach_msg(&req.header, MACH_RCV_MSG, 0, sizeof(req), port, MACH_MSG_TIMEOUT_NONE, MACH_PORT_NULL);
			if (kr != KERN_SUCCESS)
				break;
			UBA_ASSERTF(kr == KERN_SUCCESS, "ERR: %u", kr);
			
			reply_msg_t rep;
			memset(&rep, 0, sizeof(rep));
			rep.header.msgh_bits = MACH_MSGH_BITS(MACH_MSG_TYPE_MOVE_SEND_ONCE, 0) | MACH_MSGH_BITS_COMPLEX;
			rep.header.msgh_size = sizeof(rep);
			rep.header.msgh_remote_port = req.header.msgh_remote_port; // destination is child's reply port
			rep.header.msgh_local_port  = MACH_PORT_NULL;
			rep.body.msgh_descriptor_count = 3;

			//printf("GOT REQUEST FOR PID %u\n", req.pid);

			SCOPED_FUTEX_READ(m_processesLock, plock);
			auto findIt = m_processes.find(req.pid);
			if (findIt == m_processes.end())
				continue;
			ProcessImpl& proc = *(ProcessImpl*)findIt->second.m_process;

			auto setSem = [&](int index, SharedEvent& event)
			{
				StringBuffer<32> str;
				event.ToString(str);
				u64 handle;
				str.Parse(handle);
				rep.sem_desc[index].name        = handle;
				rep.sem_desc[index].disposition = MACH_MSG_TYPE_COPY_SEND;
				rep.sem_desc[index].type        = MACH_MSG_PORT_DESCRIPTOR;
			};
			setSem(0, proc.m_cancelEvent);
			setSem(1, proc.m_writeEvent);
			setSem(2, proc.m_readEvent);

			//printf("SENDING HANDLES FOR PID %u\n", req.pid);

			kr = mach_msg(&rep.header, MACH_SEND_MSG, rep.header.msgh_size, 0, MACH_PORT_NULL, MACH_MSG_TIMEOUT_NONE, MACH_PORT_NULL);
			if (kr != KERN_SUCCESS)
				break;
			UBA_ASSERTF(kr == KERN_SUCCESS, "ERR: %u", kr);
		}
		//printf("EXITED SEMA SERVICE\n");
	}
#endif

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void GenerateNameForProcess(StringBufferBase& out, const tchar* arguments, u32 counterSuffix)
	{
		const tchar* start = arguments;
		const tchar* it = arguments;
		StringBuffer<> temp;
		while (true)
		{
			if (*it != ' ' && *it != 0)
			{
				++it;
				continue;
			}
			temp.Clear();
			temp.Append(start, Min(256ull, u64(it - start)));
			if (!temp.Contains(TC(".rsp")) && !temp.Contains(TC(".bat")))
			{
				if (*it == 0)
					break;
				++it;
				start = it;
				continue;
			}
			out.AppendFileName(temp.data);
			if (out.data[out.count -1] == '"')
				out.Resize(out.count -1);
			break;
		}

		if (out.IsEmpty())
			out.Append(TCV("NoGoodName"));

		if (!counterSuffix)
			return;
			
		if (out.Contains('\"'))	// TODO: Should handle this properly: /home/honk/git/chromium/src/.ubaclientAgent0/sessions/260420_000243/log/liballoc_error_handler_impl.a.rsp/"_4322.log
			out.Clear();
		out.Appendf(TC("_%03u"), counterSuffix);
	}

	bool GetZone(StringBufferBase& outZone)
	{
		outZone.count = GetEnvironmentVariableW(TC("UBA_ZONE"), outZone.data, outZone.capacity);
		if (outZone.count)
			return true;

		// TODO: Remove.
		#if PLATFORM_MAC
		if (!GetComputerNameW(outZone))
			return false;

		if (outZone.StartsWith(TC("dc4-mac")) || outZone.StartsWith(TC("rdu-mac")))
		{
			outZone.Resize(7);
			return true;
		}
		outZone.count = 0;
		#endif

		return false;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
