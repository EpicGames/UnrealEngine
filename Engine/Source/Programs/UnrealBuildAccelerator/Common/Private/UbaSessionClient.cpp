// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaSessionClient.h"
#include "UbaApplicationRules.h"
#include "UbaConfig.h"
#include "UbaEnvironment.h"
#include "UbaNetworkClient.h"
#include "UbaNetworkMessage.h"
#include "UbaProcess.h"
#include "UbaProcessStartInfoHolder.h"
#include "UbaProtocol.h"
#include "UbaStorage.h"

#if PLATFORM_WINDOWS
#include <winerror.h>
#include <tlhelp32.h>
#endif

namespace uba
{
	const tchar* ToString(MessageType type)
	{
		switch (type)
		{
			#define UBA_PROCESS_MESSAGE(x) case MessageType_##x: return TC(#x);
			UBA_PROCESS_MESSAGES
			#undef UBA_PROCESS_MESSAGE
		default:
			return TC("UnknownProcessMessage"); // Should never happen
		}
	}

	SessionClientCreateInfo::SessionClientCreateInfo(Storage& s, NetworkClient& c, LogWriter& writer) : SessionCreateInfo(s, writer), client(c)
	{
		// By default we keep all files in memory
		keepOutputFileMemoryMapsThreshold = InvalidValue;
	}

	void SessionClientCreateInfo::Apply(const Config& config)
	{
		SessionCreateInfo::Apply(config);

		if (const ConfigTable* table = config.GetTable(TC("Session")))
		{
			table->GetValueAsU32(maxProcessCount, TC("MaxProcessCount"));
			table->GetValueAsBool(useDependencyCrawler, TC("UseDependencyCrawler"));
			table->GetValueAsBool(useKnownLocalBinaries, TC("UseKnownLocalBinaries"));
			table->GetValueAsU32(pingTimeoutSecondsPrintCallstacks, TC("PingTimeoutSecondsPrintCallstacks"));
			table->GetValueAsU32(maxIdleSeconds, TC("MaxIdleSeconds"));
			table->GetValueAsBool(killRandom, TC("KillRandom"));
			table->GetValueAsBool(useStorage, TC("UseStorage"));
			u32 temp;
			if (table->GetValueAsU32(temp, TC("MemWaitLoadPercent")))
				memWaitLoadPercent = u8(temp);
			if (table->GetValueAsU32(temp, TC("MemKillLoadPercent")))
				memKillLoadPercent = u8(temp);
		}
	}


	struct SessionClient::ModuleInfo
	{
		ModuleInfo(const tchar* n, const CasKey& c, u32 a) : name(n), casKey(c), attributes(a) {}
		TString name;
		CasKey casKey;
		u32 attributes;
	};

	struct SessionClient::GetNextEntry
	{
		u32 processId;
		bool& outNewProcess;
		bool& outShouldExit;
		NextProcessInfo& outNextProcess;
		GetNextEntry* next;
		Event done;
	};

	SessionClient::SessionClient(const SessionClientCreateInfo& info)
	: Session(info, TC("UbaSessionClient"), true, info.client)
	,	m_client(info.client)
	,	m_name(info.name.data)
	,	m_terminationTime(~0ull)
	,	m_waitToSendEvent(EventResetType_Auto)
	{
		m_maxProcessCount = info.maxProcessCount;
		m_dedicated = info.dedicated;
		m_useStorage = info.useStorage;
		m_downloadDetoursLib = info.downloadDetoursLib;
		m_defaultPriorityClass = info.defaultPriorityClass;
		m_maxIdleSeconds = info.maxIdleSeconds;
		m_osVersion = info.osVersion;
		m_disableCustomAllocator = info.disableCustomAllocator;
		m_useBinariesAsVersion = info.useBinariesAsVersion;
		m_memWaitLoadPercent = info.memWaitLoadPercent;
		m_memKillLoadPercent = info.memKillLoadPercent;
		m_processFinished = info.processFinished;

		#if PLATFORM_WINDOWS || PLATFORM_MAC // Linux will crash with print all callstacks
		m_pingTimeoutSecondsPrintCallstacks = info.pingTimeoutSecondsPrintCallstacks;
		#endif

		m_useDependencyCrawler = info.useDependencyCrawler;
		m_useKnownLocalBinaries = info.useKnownLocalBinaries;

		m_processIdCounter = 1'000'000; // We set this value to a very high value.. because it will be used by child processes and we don't want id from server and child process id to collide

		if (m_name.IsEmpty())
			GetComputerNameW(m_name);

		m_processWorkingDir.Append(m_rootDir).Append(TCV("empty"));// + TC("empty\\");
		m_storage.CreateDirectory(m_processWorkingDir.data);
		m_processWorkingDir.EnsureEndsWithSlash();

		if (info.killRandom)
		{
			Guid g;
			CreateGuid(g);
			m_killRandomIndex = 10 + g.data1 % 30;
		}

		m_nameToHashTableMem.Init(NameToHashMemSize);

		m_trace.SetThreadUpdateCallback([this]() { TraceSessionUpdate(); }, info.traceIntervalMs);

		Create(info);



		if (m_useDependencyCrawler)
			m_dependencyCrawler.Init([this](const StringView& fileName, u32& outAttr) // FileExists
				{
					return Exists(fileName, outAttr);
				},
			[this](const StringView& path, const DependencyCrawler::FileFunc& fileFunc) // TraverseFilesFunc
				{
					DirTableOffset tableOffset;
					if (!EntryExists(path, tableOffset))
						return;
					if (!IsDirectoryEntry(tableOffset))
						return;
					m_directoryTable.TraverseOriginalFilesNoPopulate(tableOffset, fileFunc);
				});
	}

	SessionClient::~SessionClient()
	{
		Stop();
	}

	bool SessionClient::Start()
	{
		m_client.RegisterOnDisconnected([this]()
			{
				m_loop = false;
			});

		m_client.RegisterOnConnected([this]()
			{
				Connect();
			});
		return true;
	}

	void SessionClient::Stop(bool wait)
	{
		m_loop = false;
		m_waitToSendEvent.Set();
		if (wait)
			m_loopThread.Wait();
	}

	bool SessionClient::Wait(u32 milliseconds, Event* wakeupEvent)
	{
		return m_loopThread.Wait(milliseconds, wakeupEvent);
	}

	void SessionClient::SetIsTerminating(const tchar* reason, u64 delayMs)
	{
		m_terminationTime = GetTime() + MsToTime(delayMs);
		m_terminationReason = reason;
		SendNotification(ToView(reason));
	}

	void SessionClient::SetMaxProcessCount(u32 count)
	{
		m_maxProcessCount = count;
	}

	void SessionClient::SetMaxIdleTime(u32 seconds)
	{
		m_maxIdleSeconds = seconds;
	}

	void SessionClient::SetAllowSpawn(bool allow)
	{
		m_allowSpawn = allow;
	}

	u64 SessionClient::GetBestPing()
	{
		return m_bestPing;
	}

	u32 SessionClient::GetMaxProcessCount()
	{
		return m_maxProcessCount;
	}

	u32 SessionClient::GetDesiredConnectionCount()
	{
		return m_desiredConnectionCount;
	}

	bool SessionClient::Exists(const StringView& path, u32& outAttributes)
	{
		DirTableOffset tableOffset;
		if (!EntryExists(path, tableOffset))
			return false;
		outAttributes = m_directoryTable.GetAttributes(tableOffset);
		return outAttributes != 0;
	}

	bool SessionClient::RetrieveCasFile(CasKey& outNewKey, u64& outSize, const CasKey& casKey, StringKey keyHint, const tchar* hint, bool storeUncompressed, bool allowProxy)
	{
		//StringBuffer<> workName;
		//u32 len = TStrlen(hint);
		//workName.Append(TCV("RC:")).Append(len > 30 ? hint + (len - 30) : hint);
		//TrackWorkScope tws(m_client, workName.data);

		TimerScope s(m_stats.storageRetrieve);
		CasKey tempKey = casKey;

		if (storeUncompressed)
			tempKey = AsCompressed(casKey, false);
		//else
		//	storeUncompressed = !IsCompressed(casKey);

		Storage::RetrieveResult result;
		bool res = m_storage.RetrieveCasFile(result, tempKey, keyHint, hint, nullptr, false, 1, allowProxy);
		outNewKey = result.casKey;
		outSize = result.size;
		return res;
	}

	bool SessionClient::GetCasKeyForFile(CasKey& out, u32 processId, const StringView& fileName, const StringKey& fileNameKey)
	{
		TimerScope waitTimer(Stats().waitGetFileMsg);
		SCOPED_FUTEX(m_nameToHashLookupLock, lock);
		auto insres = m_nameToHashLookup.try_emplace(fileNameKey);
		HashRec& rec = insres.first->second;
		lock.Leave();
		SCOPED_FUTEX(rec.lock, lock2);
		if (rec.key == CasKeyZero)//!rec.serverTime)
		{
			waitTimer.Cancel();

			// These will never succeed
			if (fileName.StartsWith(m_sessionBinDir.data) || fileName.StartsWith(TC("c:\\noenvironment")) || fileName.StartsWith(m_processWorkingDir.data))
			{
				out = CasKeyZero;
				return true;
			}

			StackBinaryWriter<1024> writer;
			NetworkMessage msg(m_client, ServiceId, SessionMessageType_GetFileFromServer, writer);
			writer.WriteU32(processId);
			writer.WriteString(fileName);
			writer.WriteStringKey(fileNameKey);

			StackBinaryReader<128> reader;
			if (!msg.Send(reader, Stats().getFileMsg))
				return false;

			rec.key = reader.ReadCasKey();
			if (rec.key != CasKeyZero)
				rec.serverTime = reader.ReadU64();
		}
		out = rec.key;
		return true;
	}

	bool SessionClient::EnsureBinaryFile(StringBufferBase& out, StringBufferBase& outVirtual, u32 processId, StringView fileName, const StringKey& fileNameKey, StringView applicationDir, StringView workingDir, const u8* loaderPaths, u32 loaderPathsSize)
	{
		CasKey casKey;
		u32 fileAttributes = DefaultAttributes(); // TODO: This is wrong.. need to retrieve from server if this is executable or not

		bool isAbsolute = IsAbsolutePath(fileName.data);
		if (isAbsolute)
		{
			UBA_ASSERT(fileNameKey != StringKeyZero);
			if (!GetCasKeyForFile(casKey, processId, fileName, fileNameKey))
				return false;
			outVirtual.Append(fileName);
		}
		else
		{
			UBA_ASSERT(fileNameKey == StringKeyZero);
			StackBinaryWriter<1024> writer;
			NetworkMessage msg(m_client, ServiceId, SessionMessageType_EnsureBinaryFile, writer);
			writer.WriteBool(IsLinux);
			writer.WriteBool(IsRunningArm());
			writer.WriteString(fileName);
			writer.WriteStringKey(fileNameKey);
			writer.WriteString(applicationDir);
			writer.WriteString(workingDir);
			if (loaderPathsSize)
				writer.WriteBytes(loaderPaths, loaderPathsSize);

			StackBinaryReader<1024> reader;
			if (!msg.Send(reader, Stats().getBinaryMsg))
				return false;

			casKey = reader.ReadCasKey();
			reader.ReadString(outVirtual);
		}

		if (casKey == CasKeyZero)
		{
			out.Append(fileName);
			return true;
		}
		bool storeUncompressed = true;
		CasKey newKey;
		u64 fileSize;
		if (!RetrieveCasFile(newKey, fileSize, casKey, {}, outVirtual.data, storeUncompressed))
			UBA_ASSERTF(false, TC("Casfile not found for %s using %s"), outVirtual.data, CasKeyString(casKey).str);
		StringBuffer<> destFile;
		if (isAbsolute || fileName.Contains(TC(".."))) // This is not beautiful, but we need to keep some dlls in the sub folder (for cl.exe etc)
			destFile.AppendFileName(fileName.data);
		else
		{
			const tchar* p = fileName.data;
			while (p[0] == '.' && p[1] == PathSeparator) // Strip leading "./" — CheckPath asserts on resulting /./ when appended after <sessionBinDir>/<casKey>/.
				p += 2;
			destFile.Append(p);
		}

		StringBuffer<> applicationDirLower;
		applicationDirLower.Append(applicationDir).MakeLower();
		KeyToString keyStr(ToStringKey(applicationDirLower));

		return WriteBinFile(out, destFile, newKey, keyStr, fileAttributes);
	}

	bool SessionClient::PrepareProcess(ProcessImpl& process, bool isChild, StringBufferBase& outRealApplication, const tchar*& outRealWorkingDir)
	{
		ProcessStartInfoHolder& startInfo = process.m_startInfo;
		outRealWorkingDir = m_processWorkingDir.data;

		StringView application = ToView(startInfo.application); 
		
		if (application.StartsWith(TC("ubacopy")))
			return true;

		#if PLATFORM_LINUX
		//if (application.EndsWith(TStrlen(startInfo.application), "/sh")) // For some reason the local shell application on the helpers hang (need to investigate aws shell application)
		//	return true;
		#elif PLATFORM_WINDOWS
		if (application.Equals(TCV("cmd.exe")) || application.EndsWith(TCV("\\cmd.exe"))) // We want to use local cmd.exe.. wine does not support kernel calls called by windows 11's cmd.exe for example
		{
			outRealApplication.Clear().Append(m_systemPath).Append(TCV("\\System32\\cmd.exe"));
			return true;
		}
		#endif

		outRealApplication.Clear();

		bool isAbsolute = IsAbsolutePath(startInfo.application);

		// If the exec target is in this tree's writtenFiles, host hasn't seen the bytes yet — flush overlay back (CAS + dirtable) before HandleGetApplication runs.
		// Must flush via the ROOT of the tree; flushing on a fresh child while parents still hold mappings deadlocks SendFiles.
		TString writtenFileBackedName; // captured under lock for the not-a-binary fallback below
		if (isAbsolute)
		{
			StringKey appKey = CaseInsensitiveFs ? ToStringKeyLower(application) : ToStringKey(application);
			bool needFlush = false;
			{
				SCOPED_FUTEX(process.m_shared.writtenFilesLock, writtenLock);
				auto findIt = process.m_shared.writtenFiles.find(appKey);
				if (findIt != process.m_shared.writtenFiles.end())
				{
					needFlush = true;
					writtenFileBackedName = findIt->second.backedName;
				}
			}
			if (needFlush)
			{
				ProcessImpl* root = &process;
				while (root->m_parentProcess)
					root = root->m_parentProcess;
				m_logger.Detail(TC("PrepareProcess: flushing overlay (via root pid %u) before exec of writtenFile binary %s"), root->GetId(), application.data);
				if (!FlushWrittenFiles(*root))
					return m_logger.Error(TC("PrepareProcess: FlushWrittenFiles failed before exec of %s"), application.data);

				StackBinaryWriter<128> writer;
				NetworkMessage msg(m_client, ServiceId, SessionMessageType_FlushProcess, writer);
				writer.WriteU32(root->m_id);
				StackBinaryReader<128> reader;
				if (!msg.Send(reader, m_stats.customMsg))
					return false;
			}
		}

		SCOPED_FUTEX(m_handledApplicationEnvironmentsLock, environmentslock);
		auto insres = m_handledApplicationEnvironments.try_emplace(application.data);
		environmentslock.Leave();

		ApplicationEnvironment& appEnv = insres.first->second;
		SCOPED_FUTEX(appEnv.lock, lock);

		if (!appEnv.realApplication.empty())
		{
			outRealApplication.Append(appEnv.realApplication);
#if PLATFORM_LINUX
			if (appEnv.isShebang)
			{
				// argv[1] must be the script (the original startInfo.application captured in `application`), not the cached interpreter virtualApplication.
				TString scriptPath(application.data, application.count);
				startInfo.applicationStr = appEnv.virtualApplication;
				startInfo.application = startInfo.applicationStr.c_str();
				startInfo.argumentsStr = scriptPath + " " + startInfo.argumentsStr;
				startInfo.arguments = startInfo.argumentsStr.c_str();
				return true;
			}
#endif
			if (!isAbsolute)
			{
				startInfo.applicationStr = appEnv.virtualApplication;
				startInfo.application = startInfo.applicationStr.c_str();
			}
			return true;
		}

		List<ModuleInfo> modules;
		bool isShebang = false;
		StringBuffer<512> virtualApplication;
		if (!ReadModules(modules, isShebang, virtualApplication, process.GetId(), application, startInfo.workingDirStr))
		{
			// Server can't parse this writtenFile as ELF/shebang/static-stub. Mirror local PrepareProcess: hand the staged path to the kernel and let it ENOEXEC.
			if (!writtenFileBackedName.empty())
			{
				m_logger.Detail(TC("PrepareProcess: server couldn't parse writtenFile binary %s — falling through to staged path %s; kernel will decide"), application.data, writtenFileBackedName.c_str());
				outRealApplication.Append(writtenFileBackedName.c_str());
				appEnv.realApplication = writtenFileBackedName;
				appEnv.virtualApplication = application.ToString();
				return true;
			}
			return m_logger.Warning(TC("Failed to get modules from server for %s"), application);
		}
		
#if PLATFORM_LINUX
		if (isShebang)
		{
			// modules[0] is the resolved interpreter (e.g. /bin/bash) after CopyImports recursion; script path is the ORIGINAL startInfo.application captured
			// into `application` before any rewrite. argv[1] must be the script — argv[0] is the interpreter — so prepend BEFORE overwriting applicationStr.
			// Otherwise we get `bash bash --args` ("cannot execute binary file", exit 126).
			TString scriptPath(application.data, application.count);
			appEnv.isShebang = true;
			appEnv.virtualApplication = virtualApplication.ToString();
			startInfo.applicationStr = appEnv.virtualApplication;
			startInfo.application = startInfo.applicationStr.c_str();
			startInfo.argumentsStr = scriptPath + " " + startInfo.argumentsStr;
			startInfo.arguments = startInfo.argumentsStr.c_str();
			application = modules.begin()->name;
		}

		if (m_useKnownLocalBinaries)
		{
			// Some applications we allow using local binaries
			// Common coreutils + tools that AOSP sandbox scripts (sbox.sh) shell out
			// to. Anything in this list runs from /usr/bin or /bin on the helper
			// instead of being staged from the host — needed when host glibc is
			// newer than helper's (Ubuntu 24.04 host vs AL2025 EC2 helper:
			// `version GLIBC_2.38 not found` when staged binary tries to run).
			constexpr StringView knownBinaries[] = {
				TCV("git"), TCV("cp"), TCV("rm"), TCV("ln"), TCV("sh"), TCV("bash"),
				TCV("env"), TCV("cmp"), TCV("rsync"),
				TCV("sed"), TCV("awk"), TCV("grep"), TCV("find"), TCV("sort"),
				TCV("uniq"), TCV("head"), TCV("tail"), TCV("cut"), TCV("tr"),
				TCV("cat"), TCV("mv"), TCV("mkdir"), TCV("rmdir"), TCV("touch"),
				TCV("chmod"), TCV("chown"), TCV("readlink"), TCV("dirname"),
				TCV("basename"), TCV("xargs"), TCV("date"), TCV("printf"),
				TCV("echo"), TCV("test"), TCV("true"), TCV("false"), TCV("which"),
				TCV("ls"), TCV("wc"), TCV("tee"), TCV("diff"), TCV("tar"),
				TCV("gzip"), TCV("gunzip"), TCV("zip"), TCV("unzip"), TCV("md5sum"),
				TCV("sha1sum"), TCV("sha256sum"), TCV("expr"),
			};
			const StringView knownFolders[] = { TCV("/usr/bin/"), TCV("/bin/") };
			for (auto knownFolder : knownFolders)
			{
				if (!virtualApplication.StartsWith(knownFolder))
					continue;
				StringView appName(ToView(virtualApplication).Skip(knownFolder.count));
				for (auto knownBinary : knownBinaries)
				{
					if (!appName.Equals(knownBinary))
						continue;
					if (FileExists(m_logger, virtualApplication.data))
					{
						appEnv.virtualApplication = virtualApplication.ToString();
						appEnv.realApplication = appEnv.virtualApplication;
						outRealApplication.Append(virtualApplication);
						startInfo.applicationStr = appEnv.virtualApplication;
						startInfo.application = startInfo.applicationStr.c_str();
						//m_logger.Info(TC("USING LOCAL BINARY %s"), outRealApplication.data);
						return true;
					}
				}
			}
		}
#endif


		StringBuffer<MaxPath> applicationDir;
		applicationDir.AppendDir(modules.begin()->name.c_str());
		KeyToString keyStr(ToStringKeyLower(applicationDir));

		Atomic<bool> success = true;

		m_client.ParallelFor(u32(modules.size()), modules, [&](const WorkContext&, auto& it)
			{
				auto& m = *it;
				TrackWorkScope tws(m_client, AsView(TC("FetchModule")), ColorWork);
				tws.AddHint(m.name);

				CasKey newCasKey;
				bool storeUncompressed = true;
				u64 fileSize;
				const tchar* moduleName = m.name.c_str();
				if (!RetrieveCasFile(newCasKey, fileSize, m.casKey, {}, moduleName, storeUncompressed))
				{
					m_logger.Error(TC("Casfile not found for %s (%s)"), moduleName, CasKeyString(m.casKey).str);
					success = false;
					return;
				}
				if (const tchar* lastSeparator = TStrrchr(moduleName, PathSeparator))
					moduleName = lastSeparator + 1;
				StringBuffer<MaxPath> temp;
				if (!WriteBinFile(temp, ToView(moduleName), newCasKey, keyStr, m.attributes))
					success = false;
			}, AsView(TC("FetchModule")), WorkPriority_High);

		if (!success)
			return false;
		
		outRealApplication.Append(m_sessionBinDir).Append(keyStr).Append(PathSeparator).AppendFileName(application.data);
		appEnv.realApplication = outRealApplication.data;

		if (!isAbsolute)
		{
			appEnv.virtualApplication = virtualApplication.ToString();
			startInfo.applicationStr = appEnv.virtualApplication;
			startInfo.application = startInfo.applicationStr.c_str();
		}

		return true;
	}

	bool SessionClient::ReadModules(List<ModuleInfo>& outModules, bool& outIsShebang, StringBufferBase& outVirtualApplication, u32 processId, StringView application, StringView workingDir)
	{
		TrackWorkScope tws(m_client, AsView(TC("ReadModules")), ColorWork);

		#if PLATFORM_MAC // Stack overflow on mac
		u8* mem = new u8[SendMaxSize];
		auto g = MakeGuard([mem]() { delete[] mem; });
		#else
		u8 mem[SendMaxSize];
		#endif
		BinaryReader reader(mem, 0, SendMaxSize);

		{
			StackBinaryWriter<1024> writer;
			NetworkMessage msg(m_client, ServiceId, SessionMessageType_GetApplication, writer);
			writer.WriteBool(IsLinux);
			writer.WriteString(application);
			writer.WriteString(workingDir);
			if (!msg.Send(reader, m_stats.getApplicationMsg))
				return false;
		}

		#if !PLATFORM_WINDOWS
		outIsShebang = reader.ReadBool();
		#endif

		u32 serverSystemPathLen = reader.ReadU32();
		u32 moduleCount = reader.ReadU32();
		if (moduleCount == 0)
			return m_logger.Error(TC("Application %s not found"), application.data);

		while (moduleCount--)
		{
			StringBuffer<MaxPath> moduleFile;
			reader.ReadString(moduleFile);
			u32 fileAttributes = reader.ReadU32();
			bool isInSystemFolder = reader.ReadBool();

			CasKey casKey = reader.ReadCasKey();
			if (casKey == CasKeyZero)
				return m_logger.Error(TC("Bad CasKey for %s (%s)"), moduleFile.data, CasKeyString(casKey).str);

			#if PLATFORM_MAC
			u32 minOsVersion = reader.ReadU32();
			if (m_osVersion && m_osVersion < minOsVersion && !IsRunningDarling())
				return m_logger.Error(TC("%s has min os version %u but current os is %u"), moduleFile.data, minOsVersion, m_osVersion);
			#endif

			// These are files in system folder that is not known to system
			if (isInSystemFolder)
			{
				StringBuffer<> localSystemModule;
				localSystemModule.Append(m_systemPath).Append(moduleFile.data + serverSystemPathLen);

				// For non-windows we always use the local system file if it exists.
				#if !PLATFORM_WINDOWS
				if (FileExists(m_logger, localSystemModule.data))
					continue;
				#endif
				if (outModules.empty())
					outVirtualApplication.Append(localSystemModule);
			}
			else if (outModules.empty())
				outVirtualApplication.Append(moduleFile);
			
			outModules.emplace_back(moduleFile.data, casKey, fileAttributes);
		}

		return true;
	}

	Vector<tchar>& SessionClient::GetProcessEnvironmentVariables()
	{
		UBA_ASSERT(!m_environmentVariables.empty());
		return m_environmentVariables;
	}

	bool SessionClient::WriteBinFile(StringBufferBase& out, const StringView& binaryName, const CasKey& casKey, const KeyToString& applicationDir, u32 fileAttributes)
	{
		UBA_ASSERT(fileAttributes);

		out.Append(m_sessionBinDir);
		out.Append(applicationDir).Append(PathSeparator);
		
		StringBuffer<> lower;
		lower.Append(applicationDir).Append(PathSeparator).Append(binaryName);
		lower.MakeLower();
		SCOPED_FUTEX(m_binFileLock, lock);

		auto insres = m_writtenBinFiles.try_emplace(lower.data, casKey);
		if (!insres.second)
		{
			out.Append(binaryName);
			if (insres.first->second != casKey)
				return m_logger.Error(TC("Writing same binary file %s multiple times but with different data! (Current: %s Previous: %s)"), out.data, CasKeyString(casKey).str, CasKeyString(insres.first->second).str);
			return true;
		}

		m_storage.CreateDirectory(out.data);
		out.Append(binaryName);

		//if (GetFileAttributesW(out.data) != INVALID_FILE_ATTRIBUTES)
		//	return true;

		if (TStrchr(binaryName.data, PathSeparator))
		{
			StringBuffer<> binaryDir;
			binaryDir.AppendDir(out);
			if (!m_storage.CreateDirectory(binaryDir.data))
				return false;
		}

		// Special hack to prevent two identical dlls from pointing to the same file (dlls are deduplicated if they are the same file in the backend)
		bool allowHardlink = !IsWindows || !binaryName.GetFileName().StartsWith(TC("c2"));

		constexpr bool writeCompressed = false;
		constexpr bool isTemp = true;
		return m_storage.CopyOrLink(casKey, out.data, fileAttributes, writeCompressed, 0, {}, isTemp, allowHardlink);
	}

	bool SessionClient::ProcessThreadStart(ProcessImpl& process)
	{
		if (!Session::ProcessThreadStart(process))
			return false;

		if (RootsHandle rootsHandle = process.GetStartInfo().rootsHandle)
			if (!SendRootsHandle(rootsHandle))
				return false;

		if (!process.m_parentProcess && m_useDependencyCrawler)
			RunDependencyCrawler(process);

		return true;
	}

	bool SessionClient::CreateFileForRead(CreateFileResponse& out, TrackWorkScope& tws, StringView fileName, StringKey fileNameKey, ProcessImpl& process, const ApplicationRules& rules)
	{
		auto g = MakeGuard([&]()
			{
				out.directoryTableSize = GetDirectoryTableSize(&process.m_shared.overlay);
				out.mappedFileTableSize = GetFileMappingSize();
			});

		{
			auto& shared = process.m_shared;
			SCOPED_FUTEX_READ(shared.writtenFilesLock, lock);
			auto findIt = shared.writtenFiles.find(fileNameKey);
			if (findIt != shared.writtenFiles.end())
			{
				UBA_ASSERT(findIt->second.memoryWritten != InvalidValue);
				// backedName MUST be non-empty: callers (e.g. RunDependencyCrawler) feed it to IsMemoryHandle which asserts *name. Surface a producer bug loudly here, then fall through to the CAS path so the build doesn't tip over.
				UBA_ASSERTF(!findIt->second.backedName.empty(), TC("writtenFiles entry for %s (key %s) has empty backedName — producer site forgot to set it"), fileName.data, KeyToString(fileNameKey).data);
				if (!findIt->second.backedName.empty())
				{
					out.fileName.Append(findIt->second.backedName);
					return true;
				}
			}
			lock.Leave();
			
			// It could be that files have moved that are not written, then we just find the source
			// and use the cas from there
			OverlayEditScope scope(shared.overlay);
			auto findIt2 = shared.movedFiles.find(fileName.data);
			if (findIt2 != shared.movedFiles.end())
			{
				UBA_ASSERT(!findIt2->second.backedName.empty());
				out.fileName.Append(findIt2->second.backedName);
				return true;
			}
		}

		CasKey casKey;
		if (!GetCasKeyForFile(casKey, process.GetId(), fileName, fileNameKey))
			return false;

		// Not finding a file is a valid path. Some applications try with a path and if fails try another path
		if (casKey == CasKeyZero)
		{
			//m_logger.Warning(TC("No casfile found for %s"), fileName.data);
			out.fileName.Append(fileName);
			return true;
		}

		// Code for doing retry if failing to decompress casfile. We've seen cases of corrupt cas files on clients
		bool shouldRetry = true;
		FileMappingEntry* retryEntry = nullptr;
		auto retryEntryGuard = MakeGuard([&]() { if (retryEntry) retryEntry->lock.Leave(); });

		while (true)
		{
			StringBuffer<> newName;
			bool isDir = casKey == CasKeyIsDirectory;
			u64 fileSize = InvalidValue;
			CasKey newCasKey;

			#ifdef __clang_analyzer__
			newCasKey = {};
			#endif

			u32 memoryMapAlignment = 0;
			if (m_allowMemoryMaps)
			{
				memoryMapAlignment = GetMemoryMapAlignment(fileName);
				if (!memoryMapAlignment && !m_useStorage)
					memoryMapAlignment = 64 * 1024;
			}

			if (isDir)
			{
				newName.Append(TCV("$d"));
			}
			else if (casKey != CasKeyZero)
			{
				if (m_useStorage || memoryMapAlignment == 0)
				{
					bool storeUncompressed = memoryMapAlignment == 0;
					bool allowProxy = rules.AllowStorageProxy(fileName);
					if (!RetrieveCasFile(newCasKey, fileSize, casKey, fileNameKey, fileName.data, storeUncompressed, allowProxy))
						return m_logger.Error(TC("Error retrieving cas entry %s (%s) (1)"), CasKeyString(casKey).str, fileName.data);

					if (!m_storage.GetCasFileName(newName, newCasKey))
						return false;
				}
				else
				{
					StorageStats& stats = m_storage.Stats();
					TimerScope ts(stats.ensureCas);

					SCOPED_FUTEX(m_fileMappingTableLookupLock, lookupLock);
					auto insres = m_fileMappingTableLookup.try_emplace(fileNameKey);
					FileMappingEntry& entry = insres.first->second;
					lookupLock.Leave();

					SCOPED_FUTEX(entry.lock, entryCs);
					ts.Leave();

					bool canBeFreed = true; // For now this code path is really only used for one thing (stage compression on console target)

					if (entry.handled)
					{
						if (!entry.success)
							return false;
						UBA_ASSERT(canBeFreed == entry.canBeFreed);
					}
					else
					{
						TimerScope s(m_stats.storageRetrieve);
						casKey = AsCompressed(casKey, false);
						entry.handled = true;
						Storage::RetrieveResult result;
						bool allowProxy = rules.AllowStorageProxy(fileName);
						if (!m_storage.RetrieveCasFile(result, casKey, fileNameKey, fileName.data, &m_fileMappingBuffer, canBeFreed, memoryMapAlignment, allowProxy))
							return m_logger.Error(TC("Error retrieving cas entry %s (%s) (2)"), CasKeyString(casKey).str, fileName.data);
						entry.success = true;
						entry.contentSize = result.size;
						entry.memoryHandle = result.view.handle;
						entry.memoryOffset = result.view.offset;
						entry.canBeFreed = canBeFreed;
					}

					if (canBeFreed)
					{
						ProcessImpl::UsedFileMapping usedFileMapping { true };
						SCOPED_FUTEX(process.m_usedFileMappingsLock, lock);
						if (process.m_usedFileMappings.insert({fileNameKey, usedFileMapping}).second)
							++entry.refCount;
					}

					fileSize = entry.contentSize;
					if (entry.memoryHandle.IsValid())
						GetMappingString(newName, entry.memoryHandle, entry.memoryOffset, entry.canBeFreed);
					else
						newName.Append(entry.isDir ? TC("$d") : TC("$f"));
				}
			}

			UBA_ASSERTF(!newName.IsEmpty(), TC("No casfile available for %s using %s"), fileName.data, CasKeyString(casKey).str);

			if (!IsMemoryHandle(newName.data))
			{
				TrackHintScope ths(tws, AsView(TC("CreateMemoryMap")));
				if (!isDir && memoryMapAlignment)
				{
					if (retryEntry)
						retryEntryGuard.Execute();

					MemoryMap map;
					bool canBeFreed = false;
					if (!GetOrCreateMemoryMapFromFile(map, fileNameKey, newName.data, IsCompressed(newCasKey), memoryMapAlignment, fileName.data, nullptr, canBeFreed))
					{
						if (!shouldRetry)
							return false;
						shouldRetry = false;

						// We need to take a lock around the file map entry since there might be another thread also wanting to map this
						{
							SCOPED_FUTEX(m_fileMappingTableLookupLock, lookupLock);
							retryEntry = &m_fileMappingTableLookup.try_emplace(fileNameKey).first->second;
							lookupLock.Leave();
							retryEntry->lock.Enter();
							retryEntry->handled = false;
						}

						if (!m_storage.ReportBadCasFile(newCasKey))
							return false;

						continue;
					}
					fileSize = map.size;
					newName.Clear().Append(map.name);
				}
				else if (!rules.IsRarelyRead(fileName))
				{
					AddFileMapping(fileNameKey, fileName.data, newName.data, fileSize);
				}
			}

			out.fileName.Append(newName);
			out.size = fileSize;
			return true;
		}
	}

	bool SessionClient::ShouldSend(ProcessImpl& process, StringView path)
	{
		auto& rules = *process.m_startInfo.rules;
		if (path.StartsWith(m_tempPath) && !rules.KeepTempOutputFile(StringView(path)))
			return false;
		if (rules.IsThrowAway(path, true))
			return false;
		return true;
	}

	bool SessionClient::SendFileSystemOperations(ProcessImpl& process)
	{
		auto& shared = process.m_shared;
		auto& overlay = shared.overlay;

		StackBinaryWriter<MaxPath * 2 + 64> writer; // file + symlink-content + headers
		StackBinaryReader<64> reader;

		SCOPED_READ_LOCK(overlay.lock, lock);

		for (auto& deletedFile : shared.deletedFiles)
		{
			if (!ShouldSend(process, deletedFile))
				continue;
			NetworkMessage networkMsg(m_client, ServiceId, SessionMessageType_DeleteFile, writer.Reset());
			writer.WriteU32(process.m_id);
			writer.WriteString(deletedFile);
			if (!networkMsg.Send(reader.Reset(), Stats().deleteFileMsg))
				return false;
			if (!reader.ReadBool())
				return m_logger.Warning(TC("Failed to delete file %s on server side"), deletedFile.c_str());
		}
		shared.deletedFiles.clear();

		for (auto i=shared.removedDirectories.rbegin(), e=shared.removedDirectories.rend(); i!=e; ++i)
		{
			if (!ShouldSend(process, *i))
				continue;
			//UBA_ASSERTF(false, TC("Removing directories from remote not implemented. (%s)"), removedDir.c_str());
		}
		shared.removedDirectories.clear();

		for (auto& createdDir : shared.createdDirectories)
		{
			if (!ShouldSend(process, createdDir))
				continue;
			NetworkMessage msg(m_client, ServiceId, SessionMessageType_CreateDirectory, writer.Reset());
			writer.WriteU32(process.m_id);
			writer.WriteString(createdDir);
			if (!msg.Send(reader.Reset(), Stats().createDirMsg))
				return false;
			if (!reader.ReadBool())
				return m_logger.Warning(TC("Failed to create directory %s on server side"), createdDir.c_str());
		}
		shared.createdDirectories.clear();

		#if !PLATFORM_WINDOWS
		for (auto& symlink : shared.symlinks)
		{
			NetworkMessage networkMsg(m_client, ServiceId, SessionMessageType_Symlink, writer.Reset());
			writer.WriteU32(process.m_id);
			writer.WriteString(symlink.first); // File
			writer.WriteString(symlink.second); // Content
			if (!networkMsg.Send(reader.Reset(), Stats().copyFileMsg))
				return false;
			if (!reader.ReadBool())
				return m_logger.Warning(TC("Failed to create symlink %s->%s on server side"), symlink.first.c_str(), symlink.second.c_str());
		}
		shared.symlinks.clear();
		#endif

		for (auto& move : shared.movedFiles)
		{
			NetworkMessage networkMsg(m_client, ServiceId, SessionMessageType_MoveFile, writer.Reset());
			writer.WriteU32(process.m_id);
			writer.WriteString(move.second.name); // From
			writer.WriteString(move.first); // To
			if (!networkMsg.Send(reader.Reset(), Stats().copyFileMsg))
				return false;
			if (!reader.ReadBool())
				return m_logger.Warning(TC("Failed to move %s->%s on server side"), move.first.c_str(), move.second.name.c_str());
		}
		shared.movedFiles.clear();

		return true;
	}

	bool SessionClient::SendFiles(ProcessImpl& process)
	{
		Atomic<bool> success = true;
		auto& rules = *process.m_startInfo.rules;
		auto& shared = process.m_shared;
		auto& writtenFiles = shared.writtenFiles;
		TimerScope timer(process.m_processStats.sendFiles);

		m_client.ParallelFor(u32(writtenFiles.size()), writtenFiles, [&](const WorkContext&, auto& it)
			{
				StorageStatsScope storageStatsScope(process.m_storageStats);
				WrittenFile& file = it->second;

				if (!ShouldSend(process, file.name))
				{
					m_sharedMemory.CloseHandle(m_logger, file.memoryHandle, file.backedName.c_str());
					file.memoryHandle = {};
					#if UBA_DEBUG
					m_logger.Info(TC("Skipping sending back temp file %s"), file.name.c_str());
					#endif
					return;
				}

				#ifdef _DEBUG
				if (!pair.second.memoryHandle.IsValid())
					m_logger.Warning(TC("%s is not using file mapping"), file.name.c_str());
				#endif

				bool keepMappingInMemory = IsWindows && IsMaybeReadAfterWritten(process, file.name) && file.memoryWritten < m_keepOutputFileMemoryMapsThreshold;
				bool compressed = rules.SendFileCompressedFromClient(file.name);
				if (!SendFile(file, process.GetId(), keepMappingInMemory, compressed))
					success = false;
			}, TCV("SendFile"));
		return success;
	}

	bool SessionClient::SendFile(WrittenFile& file, u32 processId, bool keepMappingInMemory, bool compressed)
	{
		// Invariant: backedName must not live in session-temp (ClearTempDirectory wipes it at tree exit, breaking CAS). Migrate at producer (CreateFile + Shared_rename), never here.
		UBA_ASSERTF(file.memoryHandle.IsValid() || file.backedName.empty() || !StringView(file.backedName).StartsWith(m_tempPath),
			TC("SendFile: writtenFile backedName lives in session-temp dir (will be wiped) — name=%s backedName=%s tempPath=%s"),
			file.name.c_str(), file.backedName.c_str(), m_tempPath.data);
		CasKey casKey;
		{
			TimerScope ts(m_stats.storageSend);
			if (!m_storage.StoreCasFileClient(casKey, file.key, file.backedName.c_str(), file.memoryHandle, 0, file.memoryWritten, file.name.c_str(), keepMappingInMemory, compressed))
				return false;
		}
		if (casKey == CasKeyZero)
			return m_logger.Warning(TC("Failed to store cas on server for local file %s (size %llu destination %s)"), file.backedName.c_str(), file.memoryWritten, file.name.c_str());
	
		m_sharedMemory.CloseHandle(m_logger, file.memoryHandle, file.backedName.c_str());
		file.memoryHandle = {};

		StackBinaryReader<128> reader;
		{
			StackBinaryWriter<1024> writer;
			NetworkMessage msg(m_client, ServiceId, SessionMessageType_SendFileToServer, writer);
			writer.WriteU32(processId);
			writer.WriteString(file.name);
			writer.WriteStringKey(file.key);
			writer.WriteU32(file.attributes);
			writer.WriteCasKey(casKey);
			if (!msg.Send(reader, Stats().sendFileMsg))
				return m_logger.Warning(TC("Failed to send file %s to server"), file.backedName.c_str());
		}

		#if UBA_DEBUG_LOGGER
		m_debugLogger->Info(TC("SENDFILE %s\n"), file.name.c_str());
		#endif

		if (!reader.ReadBool())
			return m_logger.Warning(TC("Server failed to copy cas %s to %s (local source %s)"), CasKeyString(casKey).str, file.name.c_str(), file.backedName.c_str());
		return true;
	}

	bool SessionClient::DeleteFile(DeleteFileResponse& out, const DeleteFileMessage& msg)
	{
		// TODO: Deleting output files should also delete them on disk (for now they will leak until process shutdown)
		RemoveWrittenFile(msg.process, msg.fileNameKey);

		auto& shared = msg.process.m_shared;
		auto& overlay = shared.overlay;

		if (msg.closeId != 0)
		{
			UBA_ASSERTF(false, TC("This has not been tested properly"));
			SCOPED_FUTEX(shared.activeFilesLock, lock);
			shared.activeFiles.erase(msg.closeId);
		}

		{
			OverlayEditScope scope(overlay);
			RegisterOverlayDelete(scope, msg.fileNameKey, msg.fileName);

			// Moves from this file needs to turn into a write or something like that
			UBA_FOR_ASSERT(for (auto& move : shared.movedFiles) UBA_ASSERT(move.second.name != msg.fileName.data);)

			#if !PLATFORM_WINDOWS
			shared.symlinks.erase(msg.fileName.data);
			#endif

			// Remove moves to this file and make the from file a normal delete
			auto findIt = shared.movedFiles.find(msg.fileName.data);
			if (findIt != shared.movedFiles.end())
			{
				shared.deletedFiles.insert(findIt->second.name);
				shared.movedFiles.erase(findIt);
			}
			else
				shared.deletedFiles.insert(msg.fileName.data);
		}

		out.result = true;
		out.errorCode = ERROR_SUCCESS;
		out.directoryTableSize = GetDirectoryTableSize(&overlay);
		return true;
	}

	bool SessionClient::MoveFile(MoveFileResponse& out, const MoveFileMessage& msg)
	{
		StringView fromName = msg.fromName;
		StringView toName = msg.toName;
		auto& process = msg.process;
		auto& shared = process.m_shared;
		auto& overlay = shared.overlay;
		auto& writtenFiles = shared.writtenFiles;

		SCOPED_FUTEX(shared.writtenFilesLock, writtenFilesLock);
		auto findIt = writtenFiles.find(msg.fromKey);
		if (findIt != writtenFiles.end())
		{
			UBA_ASSERT(msg.toKey != StringKeyZero);
			auto temp = std::move(findIt->second);
			writtenFiles.erase(findIt);
			auto insres = writtenFiles.try_emplace(msg.toKey);
			WrittenFile& toFile = insres.first->second;
			toFile = std::move(temp); // Copy all state first and then fixup
			toFile.key = msg.toKey;
			toFile.name = toName.ToString();
			toFile.ownerId = process.m_id;

			// In remote mode rename is overlay-only (no disk rename), so backedName must stay pointing at the staged-output file.
			// Exception: when the virtual name leaves session-temp but backing still lives there, kernel-rename the backing to m_sessionOutputDir before tree exit wipes it.
			if (!toFile.memoryHandle.IsValid() && !toFile.backedName.empty() && StringView(toFile.backedName).StartsWith(m_tempPath) && !toName.StartsWith(m_tempPath))
			{
				StringBuffer<> newBacked;
				newBacked.Append(m_sessionOutputDir).Append(KeyToString(msg.toKey));
				if (MoveFileExW(toFile.backedName.c_str(), newBacked.data, 0))
					toFile.backedName = newBacked.data;
				else
					m_logger.Warning(TC("MoveFile: failed to migrate backing %s -> %s for %s (errno=%d) — SendFile may fail"), toFile.backedName.c_str(), newBacked.data, toName.data, errno);
			}

			writtenFilesLock.Leave();

			// Atomic-write pattern (lld / gcc_solink / Python tempfile): erase any pending delete/move for dest, else producer exit double-sends DeleteFile(dest) + writtenFile.
			SCOPED_WRITE_LOCK(overlay.lock, oLock);
			shared.deletedFiles.erase(toName.data);
			shared.movedFiles.erase(toName.data);
		}
#if UBA_SUPPORT_SYMLINKS
		else if (IsSymlink(msg.fileAttributes))
		{
			// `ln -f -s` (mkstemp+unlink+symlink+rename) renames a symlink that's only in overlay.symlinks (Session::Symlink path), never writtenFiles.
			SCOPED_WRITE_LOCK(overlay.lock, oLock);
			auto sIt = shared.symlinks.find(fromName.ToString());
			TString content = (sIt != shared.symlinks.end()) ? sIt->second : msg.symlinkContent.ToString();
			if (sIt != shared.symlinks.end())
				shared.symlinks.erase(sIt);
			shared.symlinks.try_emplace(toName.ToString(), std::move(content));
		}
#endif
		else if (!fromName.StartsWith(m_tempPath) || !toName.StartsWith(m_tempPath))
		{
			writtenFilesLock.Leave();

			SCOPED_WRITE_LOCK(overlay.lock, oLock);

			// Extend a previous move if one targeted fromName.
			auto findIt2 = shared.movedFiles.find(fromName.data);
			if (findIt2 != shared.movedFiles.end())
			{
				shared.movedFiles.try_emplace(toName.data, findIt2->second);
				shared.deletedFiles.emplace(fromName.data);
				shared.movedFiles.erase(findIt2);
			}
			else
			{
				oLock.Leave();
				CreateFileResponse out2;
				TrackWorkScope tws;
				CreateFileForRead(out2, tws, fromName, CaseInsensitiveFs ? ToStringKeyLower(fromName) : ToStringKey(fromName), process, *process.m_startInfo.rules);
				oLock.Enter();

				shared.movedFiles.try_emplace(toName.ToString(), fromName.ToString(), out2.fileName.ToString());
			}
			shared.deletedFiles.erase(toName.data);
		}

		writtenFilesLock.Leave();

		// Carry symlink target through rename: dest stays S_IFLNK with same content.
		{
			OverlayEditScope scope(overlay);
			RegisterOverlayCreateOrWrite(scope, msg.toKey, toName, msg.fileSize, msg.lastWriteTime, msg.fileAttributes, msg.symlinkContent);
			RegisterOverlayDelete(scope, msg.fromKey, fromName);
		}
		out.directoryTableSize = GetDirectoryTableSize(&overlay);
		out.result = true;
		out.errorCode = ERROR_SUCCESS;
		return true;
	}

	bool SessionClient::Chmod(ChmodResponse& out, const ChmodMessage& msg)
	{
		{
			auto& shared = msg.process.m_shared;
			SCOPED_FUTEX(shared.writtenFilesLock, lock);
			auto& writtenFiles = shared.writtenFiles;
			auto findIt = writtenFiles.find(msg.fileNameKey);
			if (findIt != writtenFiles.end())
			{
				bool executable = false;
				#if !PLATFORM_WINDOWS
				if (msg.fileMode & S_IXUSR)
					executable = true;
				#endif
				auto& f = findIt->second;
				f.attributes = DefaultAttributes(executable);
				out.errorCode = 0;

				{
					OverlayEditScope scope(shared.overlay);
					RegisterOverlayCreateOrWrite(scope, msg.fileNameKey, msg.fileName, f.memoryWritten, f.lastWriteTime, f.attributes);
				}
				//out.directoryTableSize = GetDirectoryTableSize(&shared.overlay);

				#if !PLATFORM_WINDOWS
				// Apply the chmod to the staged file on disk so a subsequent execve of this writtenFile
				// (PrepareProcess writtenFiles short-circuit feeds the kernel <sessionDir>/output/<key>)
				// observes the requested mode bits. CAS-fetched files land at 0644 by default; without
				// this, +x chmods update DIRTABLE only and the kernel rejects execve with EACCES.
				if (!f.backedName.empty())
				{
					if (::chmod(f.backedName.c_str(), (mode_t)msg.fileMode) != 0)
						m_logger.Warning(TC("chmod(%s, 0%o) on backed file %s failed: %s"), msg.fileName.data, msg.fileMode, f.backedName.c_str(), strerror(errno));
				}
				#endif

				return true;
			}
		}

		m_logger.Warning(TC("NotImplemented - chmod %u on non-written file %s (should be sent message to server)"), msg.fileMode, msg.fileName.data);
		out.errorCode = 0;
		//UBA_ASSERTF(false, TC("NotImplemented - chmod on non-written file %s (should be sent message to server"), msg.fileName.data);
		return true;
	}

	bool SessionClient::GetFullFileName(GetFullFileNameResponse& out, const GetFullFileNameMessage& msg)
	{
		StringView workingDir(msg.process.m_startInfo.workingDirStr);

		StringKeyHasher hasher;
		hasher.UpdateNoCheck(msg.process.m_startInfo.applicationStr);
		hasher.UpdateNoCheck(msg.fileName);
		hasher.UpdateNoCheck(workingDir);
		StringKey nameKey = ToStringKey(hasher);

		SCOPED_FUTEX(m_nameToNameLookupLock, lock);
		auto insres = m_nameToNameLookup.try_emplace(nameKey);
		NameRec& rec = insres.first->second;
		lock.Leave();
		SCOPED_FUTEX(rec.lock, lock2);

		if (rec.handled)
		{
			out.fileName.Append(rec.name.c_str());
			out.virtualFileName.Append(rec.virtualName.c_str());
			return true;
		}
		rec.handled = true;

		// JVM dlopens a lib it wrote earlier in this tree (Room/sqlite-jdbc /tmp pattern); server hasn't seen the bytes yet, so check writtenFiles directly and hand back backedName.
		if (IsAbsolutePath(msg.fileName.data))
		{
			StringKey appKey = msg.fileNameKey;
			if (appKey == StringKeyZero)
				appKey = CaseInsensitiveFs ? ToStringKeyLower(msg.fileName) : ToStringKey(msg.fileName);
			SCOPED_FUTEX(msg.process.m_shared.writtenFilesLock, writtenLock);
			auto findIt = msg.process.m_shared.writtenFiles.find(appKey);
			if (findIt != msg.process.m_shared.writtenFiles.end() && !findIt->second.backedName.empty())
			{
				out.fileName.Append(findIt->second.backedName.c_str());
				out.virtualFileName.Append(msg.fileName);
				writtenLock.Leave();
				rec.name = out.fileName.data;
				rec.virtualName = out.virtualFileName.data;
				out.mappedFileTableSize = AddFileMapping(appKey, msg.fileName.data, out.fileName.data);
				return true;
			}
		}

		StringBuffer<> appDir;
		appDir.AppendDir(msg.process.m_startInfo.application);
		if (!EnsureBinaryFile(out.fileName, out.virtualFileName, msg.process.m_id, msg.fileName, msg.fileNameKey, appDir, workingDir, msg.loaderPaths, msg.loaderPathsSize))
			return false;

		StringKey fileNameKey = msg.fileNameKey;
		if (fileNameKey == StringKeyZero)
			fileNameKey =  CaseInsensitiveFs ? ToStringKeyLower(out.virtualFileName) : ToStringKey(out.virtualFileName);

		rec.name = out.fileName.data;
		rec.virtualName = out.virtualFileName.data;
		out.mappedFileTableSize = AddFileMapping(fileNameKey, msg.fileName.data, out.fileName.data);
		return true;
	}

	bool SessionClient::GetLongPathName(GetLongPathNameResponse& out, const GetLongPathNameMessage& msg)
	{
		StackBinaryWriter<1024> writer;
		NetworkMessage networkMsg(m_client, ServiceId, SessionMessageType_GetLongPathName, writer);
		writer.WriteString(msg.fileName);
		StackBinaryReader<1024> reader;
		if (!networkMsg.Send(reader, Stats().getLongNameMsg))
			return false;
		out.errorCode = reader.ReadU32();
		reader.ReadString(out.fileName);
		return true;
	}

	bool SessionClient::GetListDirectoryInfo(ListDirectoryResponse& out, const StringView& dirName, const StringKey& dirKey, DirectoryTableOverlay* overlay)
	{
		TrackWorkScope tws(m_client, AsView(TC("GetListDir")), ColorWork);
		StackBinaryWriter<1024> writer;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_ListDirectory, writer);
		writer.WriteU32(m_sessionId);
		writer.WriteString(dirName);
		writer.WriteStringKey(dirKey);
		
		StackBinaryReader<SendMaxSize> reader;

		if (!msg.Send(reader, Stats().listDirMsg))
			return false;

		DirTableOffset tableOffset { reader.ReadU32() };

		u32 oldTableSize = GetDirectoryTableSize(nullptr).main;
		if (!UpdateDirectoryTableFromServer(reader))
			return false;
		DirTableSize newTableSize = GetDirectoryTableSize(overlay);

		// Ask for a refresh of hashes straight away since they will likely be asked for by the process doing this query
		if (oldTableSize != newTableSize.main)
			m_waitToSendEvent.Set();

		out.tableOffset = tableOffset;
		out.directoryTableSize = newTableSize;
		return true;
	}

	bool SessionClient::WriteFilesToDisk(ProcessImpl& process, WrittenFile** files, u32 fileCount)
	{
		// Do nothing, we will send the data to the host when process is finished
		return true;
	}


	struct SessionClient::ActiveUpdateDirectoryEntry
	{
		Event done;
		u32 readPos = 0;
		ActiveUpdateDirectoryEntry* prev = nullptr;
		ActiveUpdateDirectoryEntry* next = nullptr;
		bool success = true;

		static bool Wait(SessionClient& client, ActiveUpdateDirectoryEntry*& first, ScopedFutex& lock, u32 readPos, const tchar* hint)
		{
			ActiveUpdateDirectoryEntry item;
			item.next = first;
			if (item.next)
				item.next->prev = &item;
			item.readPos = readPos;
			first = &item;
			item.done.Create(EventResetType_Manual);

			lock.Leave();
			bool res = item.done.IsSet(5*60*1000);
			lock.Enter();

			if (item.prev)
				item.prev->next = item.next;
			else
				first = item.next;
			if (item.next)
				item.next->prev = item.prev;

			if (res)
				return item.success;

			UpdateError(first);
			
			if (client.m_directoryTableError)
				return false;
			client.m_directoryTableError = true;

			u32 activeCount = 0;
			for (auto i = first; i; i = i->next)
				++activeCount;
			return client.m_logger.Warning(TC("Timed out after 5 minutes waiting for update directory message to reach read position %u  (%u active in %s wait)"), readPos, activeCount, hint);
		}

		static void UpdateReadPosMatching(ActiveUpdateDirectoryEntry*& first, u32 readPos)
		{
			for (auto i = first; i; i = i->next)
			{
				if (i->readPos != readPos)
					continue;
				i->done.Set();
				break;
			}
		}

		static void UpdateReadPosLessOrEqual(ActiveUpdateDirectoryEntry*& first, u32 readPos)
		{
			for (auto i = first; i; i = i->next)
				if (i->readPos <= readPos)
					i->done.Set();
		}

		static void UpdateError(ActiveUpdateDirectoryEntry*& first)
		{
			for (auto i = first; i; i = i->next)
			{
				i->success = false;
				i->done.Set();
			}
		}
	};

	bool SessionClient::UpdateDirectoryTableFromServer(StackBinaryReader<SendMaxSize>& reader)
	{
		auto& dirTable = m_directoryTable;

		auto updateMemorySizeAndSignal = [&]
			{
				SCOPED_WRITE_LOCK(dirTable.m_memoryLock, lock);
				dirTable.m_memorySize = m_directoryTableMemPos;
				lock.Leave();
				ActiveUpdateDirectoryEntry::UpdateReadPosLessOrEqual(m_firstEmptyWait, m_directoryTableMemPos);
				return true;
			};

		u32 lastWriteEnd = ~0u;

		while (true)
		{
			u32 readPos = reader.ReadU32();

			u8* pos = m_directoryTableMem + readPos;
			u32 toRead = u32(reader.GetLeft());

			SCOPED_FUTEX(m_directoryTableLock, lock);

			if (m_directoryTableError)
				return false;

			EnsureDirectoryTableMemory(readPos + toRead);

			if (toRead == 0)
			{
				// We wrote to lastWriteEnd and now we got an empty message where readPos is the same..
				// This means that it was a good cut-off and we can increase m_memorySize
				// If m_directoryTableMemPos is different it means that we have another thread going on that will update things a little bit later.
				if (lastWriteEnd == readPos && lastWriteEnd == m_directoryTableMemPos)
					return updateMemorySizeAndSignal();

				// We might share this position with others
				if (dirTable.m_memorySize < readPos)
					if (!ActiveUpdateDirectoryEntry::Wait(*this, m_firstEmptyWait, lock, readPos, TC("empty")))
						return false;
				return true;
			}

			reader.ReadBytes(pos, toRead);

			// Wait until all data before readPos has been read
			if (readPos != m_directoryTableMemPos)
				if (!ActiveUpdateDirectoryEntry::Wait(*this, m_firstReadWait, lock, readPos, TC("read")))
					return false;

			m_directoryTableMemPos += toRead;
			
			// Find potential waiter waiting for this exact size and wake it up
			ActiveUpdateDirectoryEntry::UpdateReadPosMatching(m_firstReadWait, m_directoryTableMemPos);


			// If there is space left in the message it means that we caught up with the directory table server side..
			// And we will stop asking for more data.
			// Note, we can only set m_memorySize when getting messages that reads less than capacity since we don't know if we reached a good position in the directory table
			if (reader.GetPosition() < m_client.GetMessageMaxSize() - m_client.GetMessageReceiveHeaderSize())
				return updateMemorySizeAndSignal();

			lastWriteEnd = m_directoryTableMemPos;

			StackBinaryWriter<1024> writer;
			NetworkMessage msg(m_client, ServiceId, SessionMessageType_GetDirectoriesFromServer, writer);
			writer.WriteU32(m_sessionId);

			if (msg.Send(reader.Reset(), Stats().getDirsMsg))
				continue;

			// Let's signal waiters to exit faster since we will not get out of this situation (most likely a disconnect)
			m_directoryTableError = true;
			ActiveUpdateDirectoryEntry::UpdateError(m_firstReadWait);
			ActiveUpdateDirectoryEntry::UpdateError(m_firstEmptyWait);
			return false;
		}
	}

	bool SessionClient::UpdateNameToHashTableFromServer(StackBinaryReader<SendMaxSize>& reader)
	{
		u32 serverTableSize = 0;
		bool isFirst = true;
		u32 readStartPos = u32(m_nameToHashTableMem.writtenSize);
		u32 localTableSize = readStartPos;
		u64 serverTime = 0;
		while (true)
		{
			if (isFirst)
			{
				serverTableSize = reader.ReadU32();
				isFirst = false;
			}
			else
			{
				StackBinaryWriter<1024> writer;
				NetworkMessage msg(m_client, ServiceId, SessionMessageType_GetNameToHashFromServer, writer);
				writer.WriteU32(serverTableSize);
				writer.WriteU32(localTableSize);
				if (!msg.Send(reader.Reset(), Stats().getHashesMsg))
					return false;
			}
			serverTime = reader.ReadU64();

			u8* pos = m_nameToHashTableMem.memory + localTableSize;

			u32 left = u32(reader.GetLeft());
			u32 toRead = serverTableSize - localTableSize;

			bool needMore = left < toRead;
			if (needMore)
				toRead = left;

			m_nameToHashTableMem.AllocateNoLock(toRead, 1, TC("NameToHashTable"));
			reader.ReadBytes(pos, toRead);
			localTableSize += toRead;

			if (!needMore)
				break;
		}

		u32 addCount = 0;
		BinaryReader r(m_nameToHashTableMem.memory, readStartPos, NameToHashMemSize);
		SCOPED_FUTEX(m_nameToHashLookupLock, lock);
		while (r.GetPosition() < localTableSize)
		{
			StringKey name = r.ReadStringKey();
			CasKey hash = r.ReadCasKey();

			HashRec& rec = m_nameToHashLookup[name];
			SCOPED_FUTEX(rec.lock, lock2);
			if (serverTime < rec.serverTime)
				continue;
			rec.key = hash;
			rec.serverTime = serverTime;
			++addCount;
		}

		return true;
	}

	void SessionClient::Connect()
	{
		StackBinaryWriter<1024> writer;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_Connect, writer);
		writer.WriteString(m_name.data);
		writer.WriteU32(SessionNetworkVersion);
		writer.WriteBool(IsRunningArm());
		writer.WriteBool(IsLinux);

		CasKey keys[2];
		if (m_useBinariesAsVersion)
		{
			StringBuffer<> dir;
			GetDirectoryOfCurrentModule(m_logger, dir);
			u64 dirCount = dir.count;
			dir.Append(PathSeparator).Append(UBA_AGENT_EXECUTABLE);
			m_storage.CalculateCasKey(keys[0], {}, dir.data);
			dir.Resize(dirCount).Append(PathSeparator).Append(UBA_DETOURS_LIBRARY);
			m_storage.CalculateCasKey(keys[1], {}, dir.data);
		}

		writer.WriteCasKey(keys[0]);
		writer.WriteCasKey(keys[1]);

		StringBuffer<256> machineId;
		GetMachineId(machineId);
		writer.WriteString(machineId);

		writer.WriteU32(m_maxProcessCount);
		writer.WriteBool(m_dedicated);

		StringBuffer<> info;
		GetSystemInfo(m_logger, info);
		info.Append(' ');
		GetSessionInfo(info);
		writer.WriteString(info);

		u64 memAvail;
		u64 memTotal;
		GetMemoryInfo(m_logger, memAvail, memTotal);
		float cpuLoad = m_trace.UpdateCpuLoad();

		m_cpuUsage = cpuLoad;
		m_memAvail = memAvail;
		m_memTotal = memTotal;

		writer.WriteU64(memAvail);
		writer.WriteU64(memTotal);
		writer.WriteU32(*(u32*)&cpuLoad);


		StackBinaryReader<SendMaxSize> reader;

		if (!msg.Send(reader, m_stats.connectMsg))
			return;

		if (!reader.ReadBool())
		{
			StringBuffer<MaxPath> str;
			reader.ReadString(str);
			m_logger.Error(str.data);

			CasKey exeKey = reader.ReadCasKey();
			CasKey dllKey = reader.ReadCasKey();
			m_client.InvokeVersionMismatch(exeKey, dllKey);
			return;
		}

		bool isServerLinux = reader.ReadBool();(void)isServerLinux;

		u32 isArm = IsRunningArm();

		auto downloadBinary = [&](StringBufferBase& outFile, CasKey binaryKey, StringView fileName, u32 i)
			{
				TimerScope s(m_stats.storageRetrieve);
				Storage::RetrieveResult result;
				if (!m_storage.RetrieveCasFile(result, AsCompressed(binaryKey, false), {}, fileName.data))
					return false;
				s.Leave();

				StringKey key;
				key.a = i;
				KeyToString dir(key);
				if (!WriteBinFile(outFile, fileName, binaryKey, dir, DefaultAttributes()))
					return false;
				return true;
			};

		CasKey detoursBinaryKey[2];
		detoursBinaryKey[0] = reader.ReadCasKey();
		if (isArm)
			detoursBinaryKey[1] = reader.ReadCasKey();

		StringBuffer<> detoursFile[2];

		if (m_downloadDetoursLib)
		{
			for (u32 i=0; i!=isArm+1; ++i)
				if (!downloadBinary(detoursFile[i], detoursBinaryKey[i], AsView(UBA_DETOURS_LIBRARY), i))
					return;
		}
		else
		{
			GetDirectoryOfCurrentModule(m_logger, detoursFile[isArm]);
			detoursFile[isArm].EnsureEndsWithSlash().Append(UBA_DETOURS_LIBRARY);
		}

		for (u32 i=0; i!=isArm+1; ++i)
		{
			#if PLATFORM_WINDOWS
			char dll[1024];
			detoursFile[i].Parse(dll, sizeof_array(dll));
			m_detoursLibrary[IsLinux][i] = dll;
			#else
			m_detoursLibrary[IsLinux][i] = detoursFile[i].data;
			#endif
		}


		#if PLATFORM_LINUX
		if (isServerLinux)
		{
			CasKey stubBinaryKey[2];
			stubBinaryKey[0] = reader.ReadCasKey();
			if (isArm)
				stubBinaryKey[1] = reader.ReadCasKey();
			// Allow these to fail for now. Stub also only exists for x64
			for (u32 i=0; i!=isArm+1; ++i)
				if (downloadBinary(detoursFile[i].Clear(), stubBinaryKey[i], AsView("UbaStaticStub.bin"), i))
					m_stubBinary[i] = detoursFile[i].data;
		}
		#endif

		bool resetCas = reader.ReadBool();
		if (resetCas)
			m_storage.Reset();

		m_sessionId = reader.ReadU32();
		m_uiLanguage = reader.ReadU32();
		m_storeIntermediateFilesCompressed = reader.ReadBool();
		m_detailedTrace = reader.ReadBool();
		m_shouldSendLogToServer = reader.ReadBool();
		m_shouldSendTraceToServer = reader.ReadBool();
		m_readIntermediateFilesCompressed = reader.ReadBool();

		auto& serverName = detoursFile[0].Clear(); // reuse
		reader.ReadString(serverName);
		m_logger.Info(TC("Connected to server %s"), serverName.data);

		if (m_shouldSendLogToServer)
			m_logToFile = true;

		if (m_shouldSendTraceToServer)
		{
			//if (m_detailedTrace)
				m_client.SetWorkTracker(&m_trace);

			m_trace.StartWriteAndThread(nullptr, 256*1024*1024, true);
		}
		else
		{
			m_trace.StartThread(); // We still use trace thread for ping messages
		}

		BuildEnvironmentVariables(reader);

		m_loopThread.Start([this]() { ThreadCreateProcessLoop(); return 0; }, TC("UbaCreateProc"));
	}

	void SessionClient::BuildEnvironmentVariables(BinaryReader& reader)
	{
		TString tempStr;
		while (true)
		{
			tempStr = reader.ReadString();
			if (tempStr.empty())
				break;
			m_environmentVariables.insert(m_environmentVariables.end(), tempStr.begin(), tempStr.end());
			m_environmentVariables.push_back(0);
		}
		//AddEnvironmentVariableNoLock(TCV("WINEDEBUG"), TCV("+process"));
		StringBuffer<> v;
		for (auto& var : m_localEnvironmentVariables)
		{
			if (u32 res = GetEnvironmentVariableW(var, v.data, v.capacity))
			{
				v.count = res;
				AddEnvironmentVariableNoLock(ToView(var), v);
			}
		}
		m_environmentVariables.push_back(0);
	}

	struct SessionClient::InternalProcessStartInfo : ProcessStartInfoHolder
	{
		u32 processId = 0;
		Vector<u8> overlayFilesData;
	};


	bool SessionClient::SendProcessAvailable(Vector<InternalProcessStartInfo>& outStartInfos, Vector<u32>& outCancelled, float availableWeight)
	{
		StackBinaryReader<SendMaxSize> reader;

		{
			TrackWorkScope tws(m_client, AsView(TC("RequestProcesses")), ColorWork);
			StackBinaryWriter<32> writer;
			NetworkMessage msg(m_client, ServiceId, SessionMessageType_ProcessAvailable, writer);
			writer.WriteU32(m_sessionId);
			writer.WriteU32(*(u32*)&availableWeight);

			if (!msg.Send(reader, m_stats.procAvailableMsg))
			{
				if (m_loop)
					m_logger.Error(TC("Failed to send ProcessAvailable message"));
				return false;
			}
		}

		u64 processesLostRaceCount = reader.Read7BitEncoded();
		while (processesLostRaceCount--)
			outCancelled.push_back(reader.ReadU32());

		while (true)
		{
			u32 processId = reader.ReadU32();
			if (processId == 0)
				break;
			if (processId == SessionProcessAvailableResponse_Disconnect)
			{
				m_logger.Info(TC("Got disconnect request from host"));
				return false;
			}
			if (processId == SessionProcessAvailableResponse_RemoteExecutionDisabled)
			{
				m_remoteExecutionEnabled = false;
				break;
			}
			InternalProcessStartInfo& info = outStartInfos.emplace_back();
			info.processId = processId;
			info.Read(reader);

			info.overlayFilesData.resize(reader.Read7BitEncoded());
			reader.ReadBytes(info.overlayFilesData.data(), info.overlayFilesData.size());
		}

		u32 neededDirectoryTableSize = reader.ReadU32();
		u32 neededHashTableSize = reader.ReadU32();

		if (u32 knownInputsCount = reader.ReadU32())
		{
			while (knownInputsCount--)
			{
				StringKey knownFile = reader.ReadStringKey();
				CasKey knownInputKey = reader.ReadCasKey();
				u32 mappingAlignment = reader.ReadU32();
				bool allowProxy = reader.ReadBool();

				if (!m_useStorage) // If storage is not used these will just be downloaded and forgotten since there is no tracking
					continue;

				bool storeUncompressed = !m_allowMemoryMaps || mappingAlignment == 0;
				if (storeUncompressed)
					knownInputKey = AsCompressed(knownInputKey, false);

				m_client.AddWork([knownInputKey, knownFile, allowProxy, this](const WorkContext&)
					{
						TrackWorkScope tws(m_client, StringBuffer<>().Appendf(TC("KnownInput")), ColorWork);
						Storage::RetrieveResult result;
						bool res = m_storage.RetrieveCasFile(result, knownInputKey, knownFile, TC("KnownInput"), nullptr, false, 1, allowProxy);
						(void)res;
					}, 1, TC("KnownInput"));
			}
		}

		if (!outStartInfos.empty())
			if (neededDirectoryTableSize > GetDirectoryTableSize(nullptr).main)
				if (!SendUpdateDirectoryTable(reader.Reset()))
					return false;

		// Always nice to update name-to-hash table since it can reduce number of messages while building.
		u32 hashTableMemSize;
		{
			SCOPED_READ_LOCK(m_nameToHashMemLock, l);
			hashTableMemSize = u32(m_nameToHashTableMem.writtenSize);
		}
		if (neededHashTableSize > hashTableMemSize)
			if (!SendUpdateNameToHashTable(reader.Reset()))
				return false;

		return true;
	}

	void SessionClient::SendReturnProcess(u32 processId, const tchar* reason, ...)
	{
		StringBuffer<> temp;
		va_list arg;
		va_start(arg, reason);
		temp.Append(reason, arg);
		va_end(arg);

		StackBinaryWriter<1024> writer;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_ProcessReturned, writer);
		writer.WriteU32(processId);
		writer.WriteString(temp);
		StackBinaryReader<32> reader;
		if (!msg.Send(reader, m_stats.procReturnedMsg))
			return;
	}

	bool SessionClient::SendProcessInputs(ProcessImpl& process)
	{
		auto& inputs = process.GetTrackedInputs();
		const u8* data = inputs.data();
		u64 left = inputs.size();
		u64 sendChunkSize = SendMaxSize * 5;
		while (left)
		{
			u64 toSend = Min(left, sendChunkSize);

			StackBinaryWriter<SendMaxSize> writer;
			NetworkMessage msg(m_client, ServiceId, SessionMessageType_ProcessInputs, writer);
			writer.Write7BitEncoded(process.m_id);
			writer.Write7BitEncoded(toSend);

			if (!writer.WriteBytesCompressed(data, toSend, false))
			{
				m_logger.Info(TC("Failed to compress %llu bytes of inputs to fit a network message, lower sendChunkSize"));
				sendChunkSize /= 2;
				continue;
			}

			StackBinaryReader<32> reader;
			if (!msg.Send(reader))
				return false;

			data += toSend;
			left -= toSend;
		}
		return true;
	}

	bool SessionClient::SendProcessFinished(ProcessImpl& process, u32 exitCode)
	{
		StackBinaryWriter<SendMaxSize> writer;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_ProcessFinished, writer);
		writer.WriteU32(process.m_id);
		writer.WriteU32(exitCode);
		u32* lineCount = (u32*)writer.AllocWrite(sizeof(u32));
		*lineCount = WriteLogLines(writer, process);

		// This is normally set after callback so we need to calculate it here
		if (process.m_exitTime)
			m_processStats.exitTime = GetTime() - process.m_exitTime;

		// Must be written last
		process.m_processStats.Write(writer);
		process.m_sessionStats.Write(writer);
		process.m_storageStats.Write(writer);
		process.m_kernelStats.Write(writer);

		if (!msg.Send(m_stats.procFinishedMsg) && m_loop)
			return m_logger.Error(TC("Failed to send ProcessFinished message!"));
		return true;
	}

	bool SessionClient::SendUpdateDirectoryTable(StackBinaryReader<SendMaxSize>& reader)
	{
		TrackWorkScope tws(m_client, AsView(TC("UpdateDir")), ColorWork);
		UBA_ASSERT(reader.GetPosition() == 0);
		StackBinaryWriter<32> writer;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_GetDirectoriesFromServer, writer);
		writer.WriteU32(m_sessionId);
		if (!msg.Send(reader, Stats().getDirsMsg))
			return false;
		return UpdateDirectoryTableFromServer(reader);
	}

	bool SessionClient::SendUpdateNameToHashTable(StackBinaryReader<SendMaxSize>& reader)
	{
		TrackWorkScope tws(m_client, AsView(TC("UpdateHashTable")), ColorWork);
		StackBinaryWriter<32> writer;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_GetNameToHashFromServer, writer);
		writer.WriteU32(~u32(0));

		SCOPED_WRITE_LOCK(m_nameToHashMemLock, lock);
		writer.WriteU32(u32(m_nameToHashTableMem.writtenSize));

		if (!msg.Send(reader, Stats().getHashesMsg))
			return false;
		return UpdateNameToHashTableFromServer(reader);
	}

	void SessionClient::SendPing(u64 memAvail, u64 memTotal)
	{
		u64 time = GetTime();
		if (TimeToMs(time - m_lastPingSendTime) < 2000) // Ping every ~2 seconds... this is so server can disconnect a client quickly if no ping is coming
			return;

		float cpuLoad = m_trace.UpdateCpuLoad();
		m_cpuUsage = cpuLoad;

		StackBinaryWriter<128> writer;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_Ping, writer);
		writer.WriteU32(m_sessionId);
		writer.WriteU64(m_lastPing);
		writer.WriteU64(memAvail);
		writer.WriteU64(memTotal);
		writer.WriteU32(*(u32*)&cpuLoad);
		StackBinaryReader<32> reader;

		struct Response
		{
			Event done;
			u64 time;
		} response{EventResetType_Manual, 0};

		auto doneFunc = [](bool error, void* userData)
			{
				auto& response = *(Response*)userData;
				response.time = GetTime();
				response.done.Set();
			};

		time = GetTime();
		
		if (!msg.SendAsync(reader, doneFunc, &response))
		{
			m_loop = false;
			return;
		}

		bool reportPing = false;

		u32 timeoutSeconds = m_pingTimeoutSecondsPrintCallstacks ? m_pingTimeoutSecondsPrintCallstacks : 20;
		if (!response.done.IsSet(timeoutSeconds*1000))
		{
			reportPing = true;
			LoggerWithWriter logger(g_consoleLogWriter);
			logger.Info(TC("Took more than %u seconds to send/receive ping%s"), timeoutSeconds, (m_pingTimeoutSecondsPrintCallstacks ? TC(". Printing callstacks") : TC("")));
			if (m_pingTimeoutSecondsPrintCallstacks)
				PrintAllCallstacks(logger);
			m_client.ValidateNetwork(logger, true);

			while (!response.done.IsSet(2000))
				m_client.ValidateNetwork(logger, false);

		}
		
		response.done.IsSet(); // Wait forever (until message trigger error or finishes)

		bool asyncRes = msg.ProcessAsyncResults(reader);

		if (reportPing)
		{
			LoggerWithWriter logger(g_consoleLogWriter);
			logger.Info(TC("Ping finished after %s (%u)"), TimeToText(GetTime() - time).str, msg.GetError());
		}

		if (!asyncRes || msg.GetError())
		{
			m_loop = false;
			return;
		}

		u64 lastPing = response.time - time;
		m_lastPing = lastPing;
		m_lastPingSendTime = response.time;

		if (lastPing < m_bestPing || m_bestPing == 0)
			m_bestPing = lastPing;

		m_storage.Ping();

		if (reader.ReadBool()) // abort
		{
			LoggerWithWriter(g_consoleLogWriter).Info(TC("Got abort from server"));
			abort();
		}
		if (reader.ReadBool()) // crashdump
		{
			TraverseAllCallstacks([&](const CallstackInfo& cs)
				{
					BinaryReader stackReader(cs.data.data(), 0, cs.data.size());
					StackBinaryWriter<SendMaxSize> stackWriter;
					GetSymbols(UBA_AGENT_EXECUTABLE, IsLinux, IsArmBinary, stackReader, stackWriter);
					BinaryReader resultReader(stackWriter.GetData(), 0, stackWriter.GetPosition());
					TString infoString = resultReader.ReadString();
					m_logger.Info(TC("%s%s"), cs.desc.c_str(), infoString.c_str());
				},
				[&](const StringView& error)
				{
					m_logger.Info(error.data);
				});
		}
	}

	void SessionClient::SendNotification(StringView text)
	{
		StackBinaryWriter<1024> writer;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_Notification, writer);
		writer.WriteU32(m_sessionId);
		writer.WriteString(text);
		msg.Send();
	}

	void SessionClient::SendLogEntry(StringView text, LogEntryType type, bool ack)
	{
		StackBinaryWriter<1024> writer;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_LogEntry, writer);
		writer.WriteByte(type);
		writer.WriteString(text);

		StackBinaryReader<16> reader;
		if (ack)
			msg.Send(reader);
		else
			msg.Send();
	}

	bool SessionClient::SendRootsHandle(RootsHandle rootsHandle)
	{
		SCOPED_FUTEX(m_rootsLookupLock, rootsLock);
		auto insres = m_rootsLookup.try_emplace(WithVfs(rootsHandle, false));
		RootsEntry& entry = insres.first->second;
		rootsLock.Leave();

		SCOPED_FUTEX(entry.lock, entryLock);
		if (entry.handled)
			return true;
		entry.handled = true;

		StackBinaryWriter<128> writer;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_GetRoots, writer);
		writer.WriteU64(rootsHandle);
		StackBinaryReader<8*1024> reader;
		if (!msg.Send(reader, m_stats.getApplicationMsg))
			return false;
		PopulateRootsEntry(entry, reader.GetPositionData(), reader.GetLeft());
		return true;
	}

	void SessionClient::SendSummary(const Function<void(Logger&)>& extraInfo)
	{
		StackBinaryWriter<SendMaxSize> writer;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_Summary, writer);

		writer.WriteU32(m_sessionId);

		Trace::WriteSummaryText(writer, [&](Logger& logger)
			{
				PrintSummary(logger);
				m_storage.PrintSummary(logger);
				m_client.PrintSummary(logger);
				KernelStats::GetGlobal().Print(logger, true);
				PrintContentionSummary(logger);
				if (extraInfo)
					extraInfo(logger);
			});

		msg.Send();
	}

	void SessionClient::SendLogFilesToServer(ProcessImpl& pi)
	{
		List<WrittenFile> logFiles;
		GetLogFiles(logFiles, pi);
		if (logFiles.empty())
			return;
		m_client.ParallelFor(u32(logFiles.size()), logFiles, [&](const WorkContext&, auto& it)
			{
				SendFile(*it, 0, false, true);
			}, AsView(TC("LogFiles")));
	}

	void SessionClient::GetLogFiles(List<WrittenFile>& outLogFiles, ProcessImpl& pi)
	{
		auto logFile = pi.m_startInfo.logFile;
		if (!logFile || !*logFile)
			return;
		pi.m_startInfo.logFile = nullptr;
		WrittenFile f;
		f.backedName = logFile;
		f.attributes = DefaultAttributes();
		StringBuffer<> dest;
		if (const tchar* lastSlash = TStrrchr(logFile, PathSeparator))
			logFile = lastSlash + 1;
		dest.Append(TCV("<log>")).Append(logFile);
		f.name = dest.data;
		f.key = ToStringKeyLower(dest);
		outLogFiles.push_back(std::move(f));
		for (auto& child : pi.m_childProcesses)
			GetLogFiles(outLogFiles, *(ProcessImpl*)child.m_process);
	}

	void SessionClient::GetLogFileName(StringBufferBase& out, const tchar* logFile, const tchar* arguments, u32 processId)
	{
		out.Append(m_sessionLogDir.data);
		if (logFile && *logFile)
		{
			if (const tchar* lastSeparator = TStrrchr(logFile, PathSeparator))
				logFile = lastSeparator + 1;
			out.Append(logFile);
		}
		else
		{
			GenerateNameForProcess(out, arguments, processId);
			out.Append(TCV(".log"));
		}
	}

	void SessionClient::ThreadCreateProcessLoop()
	{
		m_sendPing = true;

		// Ask for dir and hash table straight away to minimize latency
		m_client.AddWork([this](const WorkContext&)
			{
				StackBinaryReader<SendMaxSize> reader;
				SendUpdateDirectoryTable(reader);
			}, 1, TC("InitGetDirTable"), ColorWork);
		m_client.AddWork([this](const WorkContext&)
			{
				StackBinaryReader<SendMaxSize> reader;
				SendUpdateNameToHashTable(reader);
			}, 1, TC("InitGetHashTable"), ColorWork);


		struct ProcessRec
		{
			ProcessRec(ProcessImpl* impl) : handle(impl) {}
			ProcessHandle handle;
			Futex lock;
			Atomic<bool> isKilled = false;
			Atomic<bool> isDone = false;
			float weight = 1.0f;
		};
		List<ProcessRec> activeProcesses;

		u64 lastWaitTime = 0;
		u64 waitForMemoryPressureStartTime = 0;

		constexpr u64 waitTimeToSpawnAfterKillMs = 5 * 1000;

		u64 memAvail = m_memAvail;
		u64 memTotal = m_memTotal;

		u64 memRequiredToSpawn = u64(double(memTotal) * double(100 - m_memWaitLoadPercent) / 100.0);
		u64 memRequiredFree = u64(double(memTotal) * double(100 - m_memKillLoadPercent) / 100.0);

		float activeWeight = 0;
		ReaderWriterLock activeWeightLock;

		u64 idleStartTime = GetTime();
		u32 processRequestCount = 0;

		auto RemoveInactiveProcesses = [&]()
		{
			for (auto it=activeProcesses.begin();it!=activeProcesses.end();)
			{
				ProcessRec& r = *it;
				if (!r.isDone)
				{
					++it;
					continue;
				}
				r.lock.Enter();
				r.lock.Leave();
				it = activeProcesses.erase(it);
			}

			if (m_remoteExecutionEnabled && m_terminationReason)
			{
				m_remoteExecutionEnabled = false;
				m_logger.Info(TC("%s. Will stop scheduling processes and send failing processes back for retry"), m_terminationReason.load());
			}

			if (!activeProcesses.empty() || !m_allowSpawn)
			{
				idleStartTime = GetTime();
				processRequestCount = 0;
			}
			else if (m_remoteExecutionEnabled)
			{
				u32 idleTime = u32(TimeToS(GetTime() - idleStartTime));
				if (idleTime > m_maxIdleSeconds)
				{
					m_logger.Info(TC("Session has been idle longer than max idle time (%u seconds). Disconnecting (Did %u process requests during idle)"), m_maxIdleSeconds.load(), processRequestCount);
					SendNotification(AsView(TC("Idle time timeout")));

					m_waitToSendEvent.Set();
					m_remoteExecutionEnabled = false;
				}
			}
		};

		Vector<InternalProcessStartInfo> startInfos;
		Vector<u32> cancelledProcesses;

		auto TryKill = [&](ProcessRec& rec, const tchar* reason)
			{
				if (rec.isKilled || rec.isDone)
					return false;
				SCOPED_FUTEX(rec.lock, lock);
				if (rec.isDone)
					return false;
				if (!rec.handle.Cancel())
					return false;
				rec.isKilled = true;
				SendReturnProcess(rec.handle.GetId(), reason);
				++m_stats.killCount;

				// We must always wait for cancelled processes to really finish since we might get new processes from host
				// which depend on this process and then outputs might collide.
				lock.Leave();
				if (!rec.handle.WaitForExit(60*1000))
					m_logger.Warning(TC("Timed out waiting for killed process to exit"));
				return true;
			};

		while (m_loop)
		{
			float maxWeight = float(m_maxProcessCount);
			u32 waitTimeoutMs = 3000;

			FlushDeadProcesses(true);
			UpdateDirTableSnapshot(false);

			GetMemoryInfo(m_logger, memAvail, memTotal);
			m_memAvail = memAvail;
			m_memTotal = memTotal;

			if (memAvail < memRequiredFree)
			{
				for (auto it = activeProcesses.rbegin(); it != activeProcesses.rend(); ++it)
				{
					ProcessRec& rec = *it;
					if (!TryKill(rec, TC("Running out of memory")))
						continue;
					m_logger.Warning(TC("Killed process due to memory pressure (Available: %s Total: %s)"), BytesToText(memAvail).str, BytesToText(memTotal).str);
					break;
				}
				lastWaitTime = GetTime();
			}

			if (!SendGetNextMessage())
			{
				m_loop = false;
				break;
			}

			bool canSpawn = TimeToMs(GetTime() - lastWaitTime) > waitTimeToSpawnAfterKillMs && m_allowSpawn;
			if (!canSpawn)
				waitTimeoutMs = 500;

			bool firstCall = true;

			while (m_remoteExecutionEnabled && canSpawn && m_loop)
			{
				float availableWeight;
				{
					SCOPED_READ_LOCK(activeWeightLock, lock);
					if (activeWeight >= maxWeight)
						break;
					availableWeight = maxWeight - activeWeight;
				}

				if (!firstCall)
				{
					GetMemoryInfo(m_logger, memAvail, memTotal);
					m_memAvail = memAvail;
					m_memTotal = memTotal;
				}
				if (memAvail < memRequiredToSpawn)
				{
					if (waitForMemoryPressureStartTime == 0)
					{
						m_logger.Info(TC("Delaying spawn due to memory pressure (Available: %s Total: %s)"), BytesToText(memAvail).str, BytesToText(memTotal).str);
						waitForMemoryPressureStartTime = GetTime();
					}
					break;
				}

				if (waitForMemoryPressureStartTime)
				{
					u64 waitTime = GetTime() - waitForMemoryPressureStartTime;
					m_logger.Info(TC("Waited %s for memory pressure to go down (Available: %s Total: %s)"), TimeToText(waitTime).str, BytesToText(memAvail).str, BytesToText(memTotal).str);
					m_stats.waitMemPressure += waitTime;
					waitForMemoryPressureStartTime = 0;
					lastWaitTime = GetTime();
					waitTimeoutMs = 200;
					availableWeight = Min(availableWeight, 1.0f);
				}

				startInfos.clear();
				cancelledProcesses.clear();

				if (!SendProcessAvailable(startInfos, cancelledProcesses, availableWeight))
				{
					m_loop = false;
					break;
				}
				++processRequestCount;

				if (!m_remoteExecutionEnabled)
				{
					m_logger.Info(TC("Got remote execution disabled response from host (will finish %llu active processes)"), startInfos.size() + activeProcesses.size());


						// We must always wait for cancelled processes to really finish since we might get new processes from host
						// which depend on this process and then outputs might collide.


						// We must always wait for cancelled processes to really finish since we might get new processes from host
						// which depend on this process and then outputs might collide.
				}

				for (u32 cancelled : cancelledProcesses)
				{
					for (auto& rec : activeProcesses)
					{
						if (rec.handle.GetId() != cancelled)
							continue;
						TryKill(rec, TC("Killed by server"));
					}
				}

				if (startInfos.empty())
				{
					canSpawn = false;
					waitTimeoutMs = 200;
				}

				for (InternalProcessStartInfo& startInfo : startInfos)
				{
					if (startInfo.applicationStr == TC("speedtest"))
					{
						RunSpeedTest(startInfo.argumentsStr);
						SendReturnProcess(startInfo.processId, TC("speedtest"));
						continue;
					}


					startInfo.uiLanguage = int(m_uiLanguage);
					startInfo.priorityClass = m_defaultPriorityClass;
					startInfo.useCustomAllocator = !m_disableCustomAllocator;
					startInfo.rules = GetRules(startInfo);

					StringBuffer<> logFile;
					if (m_logToFile)
					{
						GetLogFileName(logFile, startInfo.logFile, startInfo.arguments, startInfo.processId);
						startInfo.logFile = logFile.data;
					}

					Vector<tchar>& env = GetProcessEnvironmentVariables();

					bool detourEnabled = !StringView(startInfo.applicationStr).GetFileName().Contains(TCV("ubatest.exe"));

					auto process = new ProcessImpl(*this, startInfo.processId, nullptr, detourEnabled);

					activeProcesses.emplace_back(process);
					ProcessRec* rec = &activeProcesses.back();

					rec->weight = startInfo.weight;

					{
						SCOPED_WRITE_LOCK(activeWeightLock, lock);
						activeWeight += rec->weight;
					}

					struct ExitedRec
					{
						ExitedRec(SessionClient& s, ReaderWriterLock& l, float& w, ProcessRec* r) : session(s), activeWeightLock(l), activeWeight(w), rec(r) {}
						SessionClient& session;
						ReaderWriterLock& activeWeightLock;
						float& activeWeight;
						ProcessRec* rec;
					};

					ExitedRec* exitedRec = new ExitedRec(*this, activeWeightLock, activeWeight, rec);
					startInfo.userData = exitedRec;
					startInfo.exitedFunc = [](void* userData, const ProcessHandle& h, ProcessExitedResponse& exitedResponse)
					{
						auto er = (ExitedRec*)userData;
						SessionClient& session = er->session;
						ReaderWriterLock& activeWeightLock = er->activeWeightLock;
						float& activeWeight = er->activeWeight;
						ProcessRec* rec = er->rec;
						delete er;

						auto& startInfo = h.GetStartInfo();
						if (session.m_shouldSendLogToServer)
							session.SendLogFilesToServer(*(ProcessImpl*)h.m_process);

						float weight = rec->weight;
						auto decreaseWeight = MakeGuard([&]()
							{
								SCOPED_WRITE_LOCK(activeWeightLock, weightLock);
								activeWeight -= weight;
								session.m_waitToSendEvent.Set();
							});

						SCOPED_FUTEX(rec->lock, lock);
						auto doneGuard = MakeGuard([&]() { rec->isDone = true; session.m_waitToSendEvent.Set(); });

						if (rec->isKilled)
							return;

						auto& process = *(ProcessImpl*)h.m_process;

						if (session.m_killRandomIndex != ~0u && session.m_killRandomCounter++ == session.m_killRandomIndex)
						{
							session.m_loop = false;
							session.m_logger.Info(TC("Killed random process (%s)"), process.m_startInfo.GetDescription());
							return;
						}

						u32 exitCode = process.m_exitCode;

						if (exitCode != 0)
						{
							if (exitCode == 44324 || exitCode == 44325) // Failed straight away inside process->Start call
							{
								if (session.m_loop)
									session.SendReturnProcess(rec->handle.GetId(), TC("Failed to find executable %s"), startInfo.application);
								return;
							}

							if (GetTime() >= session.m_terminationTime)
							{
								if (session.m_loop)
									session.SendReturnProcess(rec->handle.GetId(), session.m_terminationReason);
								return;
							}

							if (process.HasFailedMessage()) // If there are failure caused by failed messages we send back for retry
							{
								if (session.m_loop)
									session.SendReturnProcess(rec->handle.GetId(), TC("Failed message (%s)"), ToString(process.m_firstMessageError));
								return;
							}
						}

						if (exitCode == 0 || startInfo.writeOutputFilesOnFail)
						{
							if (!process.m_parentProcess && process.m_extractExportsVersion)
								session.ExtractSymbolsForProcessTree(process);

							// Should we decrease weight before or after sending files?
							//decreaseWeight.Execute();
							bool success = session.SendFileSystemOperations(process);
							success = success && session.SendFiles(process);
							if (!success)
							{
								const tchar* desc = TC("Failed to send output files to host");
								session.m_logger.Error(desc);
								if (session.m_loop)
									session.SendReturnProcess(rec->handle.GetId(), desc);
								return;
							}
						}

						decreaseWeight.Execute();

						if (process.IsCancelled())
						{
							if (session.m_loop)
								session.SendReturnProcess(rec->handle.GetId(), TC("Cancelled"));
							return;
						}

						if (startInfo.trackInputs)
							session.SendProcessInputs(process);

						session.SendProcessFinished(process, exitCode);

						// TODO: These should be removed and instead added in TraceReader (so it will update over time)
						session.m_stats.stats.Add(process.m_sessionStats);
						session.m_storage.AddStats(process.m_storageStats);

						if (session.m_processFinished)
							session.m_processFinished(&process);
					};

					if (!startInfo.overlayFilesData.empty())
					{
						UBA_ASSERT(startInfo.overlayFilesStr.empty());
						auto& overlay = process->m_shared.overlay;
						CreateOverlay(overlay);
						ResetOverlay(overlay, startInfo.overlayFilesData.data(), startInfo.overlayFilesData.size());
					}

					if (!process->Start(startInfo, true, env, true, exitedRec))
						m_remoteExecutionEnabled = false;
				}

				RemoveInactiveProcesses();

				firstCall = false;
			}

			//SendUpdateNameToHashTable(); // It is always nice to populate this at certain cadence since it might speed up running processes queries

			m_waitToSendEvent.IsSet(waitTimeoutMs);

			RemoveInactiveProcesses();

			if (activeProcesses.empty() && !m_remoteExecutionEnabled)
			{
				// There can be processes that are done (isDone is true) but are still in m_processes list (since they are removed from that after). give them some time
				u64 counter = 300;
				while (true)
				{
					if (!counter--)
					{
						m_logger.Warning(TC("Took a long time for processes to be removed after being finished"));
						break;
					}

					SCOPED_FUTEX_READ(m_processesLock, processesLock);
					if (m_processes.empty())
						break;
					processesLock.Leave();
					Sleep(10);
				}
				break;
			}
		}

		CancelGetNextMessageQueue();
		CancelAllProcessesAndWait(); // If we got the exit from server there is no point sending anything more back.. cancel everything

		u32 retry = 0;
		while (true)
		{
			if (retry++ == 100)
			{
				m_logger.Error(TC("This should never happen!"));
				break;
			}
			RemoveInactiveProcesses();
			if (activeProcesses.empty())
				break;
			m_waitToSendEvent.IsSet(100);
		};


		m_client.FlushWork();


		m_trace.StopThread();
		m_trace.SetThreadUpdateCallback({});

		if (m_shouldSendTraceToServer)
		{
			// Can't disable this, it can cause race conditions
			//m_client.SetWorkTracker(nullptr);

			m_trace.WriteSessionSummary([&](Logger& logger)
				{
					PrintSummary(logger);
					m_storage.PrintSummary(logger);
					m_client.PrintSummary(logger);
					KernelStats::GetGlobal().Print(logger, true);
					PrintContentionSummary(logger);
				});

			StringBuffer<> ubaFile(m_sessionLogDir);
			ubaFile.Append(TCV("Trace.uba"));
			if (m_trace.StopWrite(ubaFile.data))
			{
				WrittenFile f;
				f.backedName = ubaFile.data;
				f.attributes = DefaultAttributes();
				StringBuffer<> dest(TC("<uba>"));
				f.name = dest.data;
				f.key = ToStringKeyLower(dest);

				SendFile(f, 0, false, true);
			}
		}
	}

	u32 SessionClient::WriteLogLines(BinaryWriter& writer, ProcessImpl& process)
	{
		u32 logLineCount = 0;
		for (auto& child : process.m_childProcesses)
			logLineCount += WriteLogLines(writer, *(ProcessImpl*)child.m_process);
		for (auto& line : process.m_logLines)
		{
			if (line.text.size()*sizeof(tchar) + 1000 >= writer.GetCapacityLeft())
				break;
			writer.WriteString(line.text);
			writer.WriteByte(line.type);
			++logLineCount;
		}
		return logLineCount;
	}

	bool SessionClient::ParseDirectoryTable()
	{
		SCOPED_WRITE_LOCK(m_directoryTable.m_lookupLock, lock);
		SCOPED_READ_LOCK(m_directoryTable.m_memoryLock, lock2);
		u32 newMemPosition = m_directoryTable.m_memorySize;
		lock2.Leave();
		if (newMemPosition == m_dirtableParsedPosition)
			return false;
		m_directoryTable.ParseDirectoryTableNoLock(m_directoryTable.m_memory, m_dirtableParsedPosition, newMemPosition, false);
		m_dirtableParsedPosition = newMemPosition;
		return true;
	}

	bool SessionClient::EntryExists(StringView path, DirTableOffset& outTableOffset)
	{
		StringKey key;
		if (path.data[path.count - 1] == PathSeparator)
			key = ToStringKey(path.data, path.count - 1);
		else
			key = ToStringKey(path);

		StringBuffer<> dirName;
		DirTableOffset tableOffset;
		DirectoryTable::Exists exists = m_directoryTable.EntryExists(key, path, true, &tableOffset);
		if (exists == DirectoryTable::Exists_Maybe)
		{
			if (ParseDirectoryTable())
			{
				exists = m_directoryTable.EntryExists(key, path, true, &tableOffset);
			}

			if (exists == DirectoryTable::Exists_Maybe)
			{
				if (const tchar* lastSeparator = TStrrchr(path.data, PathSeparator))
					dirName.Append(path.data, lastSeparator - path.data);
				
				StringKey dirKey = ToStringKey(dirName);

				{	// Optimization to reduce number of messages
					SCOPED_FUTEX(m_dirVisitedLock, l);
					DirVisitedEntry& entry = m_dirVisited[dirKey];
					l.Leave();
				
					SCOPED_FUTEX(entry.lock, l2);
					if (!entry.handled)
					{
						ListDirectoryResponse out;
						if (!GetListDirectoryInfo(out, dirName, dirKey, nullptr))
							return false;
						ParseDirectoryTable();
						entry.handled = true;
					}
				}

				exists = m_directoryTable.EntryExists(key, path, true, &tableOffset);
			}
		}

		UBA_ASSERTF(exists != DirectoryTable::Exists_Maybe, TC("This should not happen. Asking for directory %s"), dirName.data);

		if (exists != DirectoryTable::Exists_Yes)
			return false;

		outTableOffset = tableOffset;

		return true;
	}

	void SessionClient::RunSpeedTest(StringView args)
	{
		struct Logger2 : Logger
		{
			SessionClient& sc;
			Logger2(SessionClient& c) : sc(c) {}
			virtual void BeginScope() override {}
			virtual void EndScope() override {}
			virtual void Log(LogEntryType type, const tchar* str, u32 strLen)
			{
				//sc.m_logger.Log(type, str, strLen);
				sc.SendLogEntry(StringView(str, strLen));
			}
		} logger(*this);


		u64 connectionCount = 1;
		ParseArguments(args.data, args.count, [&](const tchar* arg, u32 argLen)
			{
				StringView v(arg, argLen);
				if (v.StartsWith(TCV("-c=")))
					Parse(connectionCount, arg+3, argLen-3);
			});

		logger.Info(TC("Starting speed test.. Waiting for %llu connections (use -c=<connectioncount>)"), connectionCount);

		m_desiredConnectionCount = u32(connectionCount);
		while (m_client.IsConnected() && m_client.GetConnectionCount() != connectionCount)
			Sleep(1);

		struct Connection
		{
			Atomic<u64> send = 0;
			Atomic<u64> recv = 0;

			u64 oldSend = 0;
			u64 oldRecv = 0;

			u64 sendDelta = 0;
			u64 recvDelta = 0;
		};

		UnorderedMap<void*, Connection> connections;

		m_client.TraverseConnections([&](NetworkBackend& backend, void* conn)
			{
				Connection& c = connections[conn];
				backend.SetDataSentCallback(conn, &c, [](void* context, u32 bytes) { ((Connection*)context)->send += bytes; });
				backend.SetDataReceivedCallback(conn, &c, [](void* context, u32 bytes) { ((Connection*)context)->recv += bytes; });
			});


		u64 dataSize = 1024ull * 1024 * 1024;
		u8* dest = new u8[dataSize];
		auto destGuard = MakeGuard([&](){ delete[] dest; });

		u64 messageCount = dataSize / (SendMaxSize - 10);

		struct Entry
		{
			Entry(NetworkClient& c, BinaryWriter& w, u8* d) : msg(c, ServiceId, SessionMessageType_SpeedTest, w), ev(EventResetType_Manual), reader(d) {}
			NetworkMessage msg;
			EventSlim ev;
			BinaryReader reader;
		};

		Event logLoop(EventResetType_Manual);
		Thread logThread([&]()
			{
				u64 startTime = GetTime();
				u64 lastTime = GetTime();
				while (!logLoop.IsSet(1000))
				{
					u64 time = GetTime();
					if (TimeToMs(time - startTime) > 10 * 1000)
						break;

					u64 deltaTime = time - lastTime;
					lastTime = time;
					u32 index = 0;
					u64 totalRecv = 0;

					for (auto& kv : connections)
					{
						auto& c = kv.second;
						u64 send = c.send;
						u64 recv = c.recv;
						c.sendDelta = send - c.oldSend;
						c.recvDelta = recv - c.oldRecv;
						c.oldSend = send;
						c.oldRecv = recv;
					}

					StringBuffer<2048> str;
					for (auto& kv : connections)
					{
						auto& c = kv.second;

						float bps = float(c.recvDelta * 8) / TimeToS(deltaTime);

						str.Appendf(TC("\r\n [%3i] %s %sit/sec"), index++, BytesToText(c.recvDelta).str, BytesToText(u64(bps)).str);
						totalRecv += c.recvDelta;
					}

					if (connections.size() > 1)
					{
						float bps = float(totalRecv * 8) / TimeToS(deltaTime);
						str.Appendf(TC("\r\n [SUM] %s %sit/sec"), BytesToText(totalRecv).str, BytesToText(u64(bps)).str);
					}
					logger.Log(LogEntryType_Info, str.data, str.count);
				}
				return 0;
			}, TC("UbaSpdTst"));

		struct EntryData { u8 data[sizeof(Entry)]; };

		Vector<EntryData> entries;
		entries.resize(messageCount);
		u64 entryCounter = 0;
		while (!logThread.Wait(0))
		{
			u64 entryIndex = entryCounter % messageCount;
			StackBinaryWriter<16> writer;
			auto& entry = *new (entries[entryIndex].data) Entry(m_client, writer, dest + (SendMaxSize - 10) * entryIndex);
			writer.WriteByte(0);
			if (!entry.msg.SendAsync(entry.reader, [](bool err, void* userData) { ((EventSlim*)userData)->Set(); }, & entry.ev))
				break;

			++entryCounter;
			if (entryCounter >= messageCount)
			{
				u64 prevEntryIndex = entryCounter % messageCount;
				u8* data = entries[prevEntryIndex].data;
				auto& prevEntry = *(Entry*)data;
				prevEntry.ev.IsSet();
				// prevEntry.msg.ProcessAsyncResults(prevEntry.reader); // Not really needed, just decrypts
				prevEntry.~Entry();
				*(u64*)data = ~0ull;
			}
		}

		logLoop.Set();
		logThread.Wait();

		for (u64 i=0, e=Min(entryCounter, messageCount); i!=e; ++i)
		{
			u8* data = entries[i].data;
			if (*(u64*)data == ~0ull)
				continue;
			auto& entry = *(Entry*)data;
			entry.ev.IsSet();
			entry.~Entry();
		}

		logger.Info(TC("Speed test done"));

		m_client.TraverseConnections([&](NetworkBackend& backend, void* conn)
			{
				backend.SetDataSentCallback(conn, nullptr, nullptr);
				backend.SetDataReceivedCallback(conn, nullptr, nullptr);
			});
	}

	bool SessionClient::AllocFailed(Process& process, const tchar* allocType, u32 error)
	{
		//StackBinaryWriter<32> writer;
		//NetworkMessage msg(m_client, ServiceId, SessionMessageType_VirtualAllocFailed, writer);
		//if (!msg.Send())
		//	m_logger.Error(TC("Failed to send VirtualAllocFailed message!"));
		return Session::AllocFailed(process, allocType, error);
	}

	void SessionClient::PrintSessionStats(Logger& logger)
	{
		Session::PrintSessionStats(logger);
	}

	bool SessionClient::SendGetNextMessage()
	{
		SCOPED_FUTEX(m_getNextEntryLock, lock);
		if (!m_firstGetNextEntry)
			return true;

		if (!m_remoteExecutionEnabled)
		{
			lock.Leave();
			CancelGetNextMessageQueue();
			return true;
		}

		StackBinaryReader<SendMaxSize> reader;
		StackBinaryWriter<16 * 1024> writer;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_GetNextProcess, writer);
		writer.WriteBool(false);
		for (auto i=m_firstGetNextEntry; i; i=i->next)
			writer.WriteU32(i->processId);

		if (!msg.Send(reader, m_stats.customMsg))
			return false;

		GetNextEntry* prev = nullptr;
		GetNextEntry* i = m_firstGetNextEntry;
		while (i)
		{
			GetNextEntry* next = i->next;
			if (HandleGetNextResponse(reader, i->processId, i->outNewProcess, i->outShouldExit, i->outNextProcess))
			{
				if (prev)
					prev->next = next;
				else
					m_firstGetNextEntry = next;
				i->done.Set();
			}
			else
				prev = i;
			i = next;
		}

		return true;
	}

	void SessionClient::CancelGetNextMessageQueue()
	{
		SCOPED_FUTEX(m_getNextEntryLock, lock);
		while (GetNextEntry* i = m_firstGetNextEntry)
		{
			m_firstGetNextEntry = i->next;
			i->outNewProcess = false;
			i->outShouldExit = true;
			i->done.Set();
		}
	}

	bool SessionClient::HandleGetNextResponse(BinaryReader& reader, u32 processId, bool& outNewProcess, bool& outShouldExit, NextProcessInfo& outNextProcess)
	{
		outNewProcess = reader.ReadBool();
		outShouldExit = reader.ReadBool();
		if (!outNewProcess)
			return false;

		outNextProcess.arguments = reader.ReadString();
		outNextProcess.workingDir = reader.ReadString();
		outNextProcess.description = reader.ReadString();
		outNextProcess.logFile = reader.ReadString();
		outNextProcess.overlayFiles.resize(reader.Read7BitEncoded());
		reader.ReadBytes(outNextProcess.overlayFiles.data(), outNextProcess.overlayFiles.size());

		if (m_logToFile)
		{
			StringBuffer<512> logFile;
			GetLogFileName(logFile, outNextProcess.logFile.c_str(), outNextProcess.arguments.c_str(), processId);
			outNextProcess.logFile = logFile.data;
		}
		return true;
	}

	bool SessionClient::GetNextProcess(Process& process, bool& outNewProcess, bool& outShouldExit, NextProcessInfo& outNextProcess, u32 prevExitCode, BinaryReader& statsReader, u32 timeoutMs)
	{
		outShouldExit = true;
		outNewProcess = false;

		auto& pi = (ProcessImpl&)process;

		if (!m_remoteExecutionEnabled)
			return true;

		if (!FlushWrittenFiles(pi))
			return false;

		if (m_shouldSendLogToServer)
			SendLogFilesToServer(pi);

		// Replace process stats i reader
		ProcessStats processStats;
		processStats.Read(statsReader, TraceVersion);
		processStats.sendFiles = pi.m_processStats.sendFiles;

		StackBinaryReader<SendMaxSize> reader;
		StackBinaryWriter<16 * 1024> writer;

		// Try getting process from this thread
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_GetNextProcess, writer);
		writer.WriteBool(true);
		writer.WriteU32(pi.m_id);
		writer.WriteU32(prevExitCode);
		processStats.Write(writer);
		writer.WriteBytes(statsReader.GetPositionData(), statsReader.GetLeft());

		pi.m_exitCode = prevExitCode;
		if (m_processFinished)
			m_processFinished(&process);

		if (!msg.Send(reader, m_stats.customMsg))
			return false;
		if (HandleGetNextResponse(reader, pi.m_id, outNewProcess, outShouldExit, outNextProcess))
			return SendUpdateDirectoryTable(reader.Reset());


		if (!timeoutMs)
			return false;

		SetProcessAsIdle(pi, statsReader, prevExitCode);

		// No process found, let main loop query free processes

		SCOPED_FUTEX(m_getNextEntryLock, lock);
		GetNextEntry entry { pi.m_id, outNewProcess, outShouldExit, outNextProcess };
		entry.next = m_firstGetNextEntry;
		m_firstGetNextEntry = &entry;
		entry.done.Create(EventResetType_Manual);
		lock.Leave();

		// Wait for main loop to set event (or timeout)
		entry.done.IsSet(timeoutMs);

		// Remove from linked list
		lock.Enter();
		for (GetNextEntry* i=m_firstGetNextEntry, *p=nullptr; i; p=i, i=i->next)
		{
			if (i != &entry)
				continue;
			if (!p)
				m_firstGetNextEntry = entry.next;
			else
				p->next = entry.next;
			break;
		}
		lock.Leave();

		if (outNewProcess)
			return SendUpdateDirectoryTable(reader.Reset());

		if (!m_remoteExecutionEnabled)
			outShouldExit = true;

		return true;

		/*
		u64 usedTimeMs = TimeToMs(GetTime() - startTime);
		if (usedTimeMs >= timeoutMs)
			return true;

		if (timeoutIndex == 0)
			SetProcessAsIdle(pi, statsReader);

		Sleep(500); // TODO: BAAAAD.. never want to sleep.. this should incorporate with update thread request for processes
		*/
	}

	bool SessionClient::CustomMessage(Process& process, BinaryReader& reader, BinaryWriter& writer)
	{
		StackBinaryWriter<SendMaxSize> msgWriter;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_Custom, msgWriter);

		u32 recvSize = reader.ReadU32();
		msgWriter.WriteU32(process.GetId());
		msgWriter.WriteU32(recvSize);
		msgWriter.WriteBytes(reader.GetPositionData(), recvSize);

		BinaryReader msgReader(writer.GetData(), 0);
		if (!msg.Send(msgReader, m_stats.customMsg))
			return false;

		u32 responseSize = msgReader.ReadU32();
		writer.AllocWrite(4ull + responseSize);
		return true;
	}

	bool SessionClient::SHGetKnownFolderPath(Process& process, BinaryReader& reader, BinaryWriter& writer)
	{
#if PLATFORM_WINDOWS
		StackBinaryWriter<SendMaxSize> msgWriter;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_SHGetKnownFolderPath, msgWriter);
		msgWriter.WriteBytes(reader.GetPositionData(), reader.GetLeft());
		BinaryReader msgReader(writer.GetData(), 0);
		if (!msg.Send(msgReader, m_stats.customMsg))
		{
			writer.WriteU32(u32(E_FAIL));
			return false;
		}
		writer.AllocWrite(msgReader.GetPosition());
#endif
		return true;
	}

	bool SessionClient::HostRun(BinaryReader& reader, BinaryWriter& writer)
	{
		const void* data = reader.GetPositionData();
		u64 size = reader.GetLeft();

		CasKey key = ToCasKey(CasKeyHasher().Update(data, size), false);

		SCOPED_FUTEX(m_hostRunCacheLock, l);
		auto insres = m_hostRunCache.try_emplace(key);
		auto& buffer = insres.first->second;
		if (!insres.second)
		{
			writer.WriteBytes(buffer.data(), buffer.size());
			return true;
		}

		StackBinaryWriter<SendMaxSize> msgWriter;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_HostRun, msgWriter);
		msgWriter.WriteBytes(data, size);
		BinaryReader msgReader(writer.GetData(), 0);
		if (!msg.Send(msgReader, m_stats.customMsg))
			return false;
		writer.AllocWrite(msgReader.GetLeft());

		buffer.resize(msgReader.GetLeft());
		memcpy(buffer.data(), msgReader.GetPositionData(), buffer.size());
		return true;
	}

	bool SessionClient::GetSymbols(const tchar* application, bool isLinux, bool isArm, BinaryReader& reader, BinaryWriter& writer)
	{
		StackBinaryWriter<SendMaxSize> msgWriter;
		NetworkMessage msg(m_client, ServiceId, SessionMessageType_GetSymbols, msgWriter);
		msgWriter.WriteString(application);
		msgWriter.WriteBool(isLinux);
		msgWriter.WriteBool(isArm);
		u32 size = reader.ReadU32();
		msgWriter.WriteU32(size);
		msgWriter.WriteBytes(reader.GetPositionData(), size);

		BinaryReader responseReader(writer.GetData(), 0, writer.GetCapacityLeft());
		if (!msg.Send(responseReader, m_stats.customMsg))
			return false;
		writer.AllocWrite(responseReader.GetLeft());

		if constexpr (DownloadDebugSymbols)
		{
			CasKey detoursSymbolsKey = responseReader.ReadCasKey();
			if (detoursSymbolsKey == CasKeyZero)
				return true;
			StringBuffer<128> symbolsFile(UBA_DETOURS_LIBRARY);
			#if PLATFORM_WINDOWS
			symbolsFile.Resize(symbolsFile.count - 3).Append("pdb");
			#else
			symbolsFile.Resize(symbolsFile.count - 2).Append("debug");
			#endif
			Storage::RetrieveResult result;
			StringBuffer<> throwaway;
			if (m_storage.RetrieveCasFile(result, AsCompressed(detoursSymbolsKey, false), {}, symbolsFile.data))
				WriteBinFile(throwaway, symbolsFile, detoursSymbolsKey, KeyToString(StringKeyZero), DefaultAttributes());
		}
		return true;
	}


	bool SessionClient::FlushWrittenFiles(ProcessImpl& process)
	{
		bool success = SendFileSystemOperations(process);
		SCOPED_FUTEX(process.m_shared.writtenFilesLock, lock);
		success = success && SendFiles(process);
		process.m_shared.writtenFiles.clear();
		return success;
	}

	bool SessionClient::UpdateEnvironment(ProcessImpl& process, const StringView& reason, bool resetStats)
	{
		StackBinaryReader<SendMaxSize> reader;

		if (resetStats)
		{
			StackBinaryWriter<16 * 1024> writer;
			NetworkMessage msg(m_client, ServiceId, SessionMessageType_UpdateEnvironment, writer);
			writer.WriteU32(process.m_id);
			writer.WriteString(reason);
			process.m_processStats.Write(writer);
			process.m_sessionStats.Write(writer);
			process.m_storageStats.Write(writer);
			process.m_kernelStats.Write(writer);

			process.m_processStats = {};
			process.m_sessionStats = {};
			process.m_storageStats = {};
			process.m_kernelStats = {};

			if (!msg.Send(reader, m_stats.customMsg))
				return false;
			reader.Reset();
		}
		return SendUpdateDirectoryTable(reader);
	}

	bool SessionClient::LogLine(ProcessImpl& process, const tchar* line, LogEntryType logType)
	{
#if 0 // PLATFORM_WINDOWS

		auto rules = process.m_startInfo.rules;
		if (!rules)
			return true;

		const tchar* errorPos = nullptr;
		if (rules->index == SpecialRulesIndex_ClExe)
		{
			if (!Contains(line, TC("C1083"), false, &errorPos))
				return true;
		}
		else
		{
			if (!Contains(line, TC("' file not found")))
				return true;
			if (!Contains(line, TC("fatal error: '"), false, &errorPos))
				return true;
		}

		const tchar* fileBegin = TStrchr(errorPos, '\'');
		if (!fileBegin)
			return true;
		++fileBegin;
		const tchar* fileEnd = TStrchr(fileBegin, '\'');
		if (!fileEnd)
			return true;

		MemoryBlock memoryBlock;
		DirectoryTable dirTable(&memoryBlock);
		{
			SCOPED_WRITE_LOCK(m_directoryTable.m_memoryLock, lock2);
			m_directoryTable.m_memorySize = m_directoryTableMemPos;
			dirTable.Init(m_directoryTable.m_memory, 0, m_directoryTable.m_memorySize);
		}

		StringBuffer<> errorPath;
		errorPath.Append(fileBegin, fileEnd - fileBegin).Replace('/', PathSeparator);

		{
			StackBinaryWriter<1024> writer;
			NetworkMessage msg(m_client, ServiceId, SessionMessageType_DebugFileNotFoundError, writer);
			writer.WriteString(errorPath);
			writer.WriteString(process.m_startInfo.workingDir);
			msg.Send();
		}

		StringView searchString = errorPath;
		if (searchString.data[0] == '.' && searchString.data[1] == '.')
		{
			searchString.data += 3;
			searchString.count -= 3;
		}

		u32 foundCount = 0;
		dirTable.TraverseAllFilesNoLock([&](const DirectoryTable::EntryInformation& info, const StringBufferBase& path, DirTableOffset dirOffset)
			{
				if (!path.EndsWith(searchString))
					return;
				if (path[path.count - searchString.count - 1] != PathSeparator)
					return;

				auto ToString = [](bool b) { return b ? TC("true") : TC("false"); };

				++foundCount;
				StringBuffer<> logStr;
				logStr.Appendf(TC("File %s found in directory table at offset %u of %u while searching for matches for %s (File size %llu attr %u)"), path.data, dirOffset, dirTable.m_memorySize, searchString.data, info.size, info.attributes);
				process.LogLine(false, logStr.data, logType);

				StringKey fileNameKey = ToStringKey(path);
				SCOPED_READ_LOCK(m_fileMappingTableLookupLock, mlock);
				auto findIt = m_fileMappingTableLookup.find(fileNameKey);
				if (findIt != m_fileMappingTableLookup.end())
				{
					auto& entry = findIt->second;
					SCOPED_READ_LOCK(entry.lock, entryCs);
					logStr.Clear().Appendf(TC("File %s found in mapping table table."), path.data);
					if (entry.handled)
					{
						StringBuffer<128> mappingName;
						if (entry.mapping.IsValid())
							GetMappingString(mappingName, entry.mapping, entry.mappingOffset);
						else
							mappingName.Append(TCV("Not valid"));
						logStr.Appendf(TC(" Success: %s Size: %u IsDir: %s Mapping name: %s Mapping offset: %u"), ToString(entry.success), entry.size, ToString(entry.isDir), mappingName.data, entry.mappingOffset);
					}
					else
					{
						logStr.Appendf(TC(" Entry not handled"));
					}
				}
				else
					logStr.Clear().Appendf(TC("File %s not found in mapping table table."), path.data);
				process.LogLine(false, logStr.data, logType);

				CasKey key;
				if (GetCasKeyForFile(key, process.m_id, path, fileNameKey))
				{
					logStr.Clear().Appendf(TC("File %s caskey is %s."), path.data, CasKeyString(key).str);

					StringBuffer<512> casKeyFile;
					if (m_storage.GetCasFileName(casKeyFile, key))
					{
						logStr.Appendf(TC(" CasKeyFile: %s"), casKeyFile.data);
						u64 size = 0;
						u32 attributes = 0;
						bool exists = FileExists(m_logger, casKeyFile.data, &size, &attributes);
						logStr.Appendf(TC(" Exists: %s"), ToString(exists));
						if (exists)
						{
							logStr.Appendf(TC(" Size: %llu Attr: %u"), size, attributes);

							FileHandle fileHandle = uba::CreateFileW(casKeyFile.data, GENERIC_READ, FILE_SHARE_READ, OPEN_EXISTING, DefaultAttributes());
							if (fileHandle == InvalidFileHandle)
							{
								logStr.Appendf(TC(" Failed to open file %s (%s)"), casKeyFile.data, LastErrorToText().data);
							}
							else
							{
								logStr.Appendf(TC(" CreateFile for read successful"));
								uba::CloseFile(casKeyFile.data, fileHandle);
							}
						}
					}
					else
						logStr.Appendf(TC(" Failed to get cas filename for cas key"));
				}
				else
					logStr.Clear().Appendf(TC("File %s caskey not found"), path.data);
				process.LogLine(false, logStr.data, logType);
			});

		if (!foundCount)
		{
			StringBuffer<> logStr;
			logStr.Appendf(TC("No matching entry found in directory table while searching for matches for %s. DirTable size: %u"), searchString.data, GetDirectoryTableSize());
			process.LogLine(false, logStr.data, logType);
			if (errorPath.StartsWith(TC("..\\Intermediate")))
			{
				auto workDir = process.m_startInfo.workingDir;
				StringBuffer<> fullPath;
				FixPath(errorPath.data, workDir, TStrlen(workDir), fullPath);
			}
		}
#endif
		return true;
	}

	void SessionClient::TraceSessionUpdate()
	{
		if (m_loop && m_sendPing)
			SendPing(m_memAvail, m_memTotal);

		// TODO: There should be some sort of log entry if we have a process that has done no progress for 30 minutes

		if (!m_trace.IsWriting())
			return;

		u64 send;
		u64 recv;
		if (auto backend = m_client.GetFirstConnectionBackend())
		{
			backend->GetTotalSendAndRecv(send, recv);
		}
		else
		{
			recv = m_client.GetTotalRecvBytes();
			send = m_client.GetTotalSentBytes();
		}

		// send and recv are swapped on purpose because that is how visualizer is visualizing 
		m_trace.SessionUpdate(0, 0, send, recv, m_lastPing, m_memAvail, m_memTotal, m_cpuUsage);

		#if UBA_TRACK_SHARED_MEMORY_ALLOCATIONS
		m_sharedMemory.TraceStats(m_trace, 30);
		#endif

		//m_fileMappingBuffer.TraceStatus(m_trace, 20, TC("Session"), 1);
		//m_storage.TraceUpdate(m_trace, 300);

		m_trace.PublishIfQuiet();
	}
}
