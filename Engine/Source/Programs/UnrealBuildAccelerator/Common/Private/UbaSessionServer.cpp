// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaSessionServer.h"
#include "UbaApplicationRules.h"
#include "UbaConfig.h"
#include "UbaEnvironment.h"
#include "UbaNetworkServer.h"
#include "UbaProcess.h"
#include "UbaProcessStartInfoHolder.h"
#include "UbaScheduler.h"
#include "UbaStaticPatcher.h"
#include "UbaStorage.h"

// Temp hack to test running linux application from windows. (I have manually copied gcc to d:\\temp)
#define UBA_TEST_LINUX_HELPING_WINDOWS 0

namespace uba
{
	struct SessionServer::ReceivedFile
	{
		u32 processId;
		CasKey casKey;
		StringKey destinationKey;
		TString destination;
		u32 attributes;
		bool casRefcounted;
	};

	struct SessionServer::RacingProcessExitInfo
	{
		SessionServer* server;
		EventSlim allowedToExit;
		ProcessHandle remoteProcess;
		ProcessStartInfo::ExitedCallback* exitedFunc;
	};

	class SessionServer::RemoteProcess final : public Process
	{
	public:
		RemoteProcess(SessionServer* server, const ProcessStartInfo& si, u32 processId, float weight_)
		:	m_server(server)
		,	m_startInfo(si)
		,	m_processId(processId)
		,	m_done(EventResetType_Manual)
		{
			m_startInfo.weight = weight_;
		}

		~RemoteProcess()
		{
			if (m_knownInputsDone.IsCreated())
				if (!m_knownInputsDone.IsSet(50*1000))
					LoggerWithWriter(g_consoleLogWriter).Error(TC("RemoteProcess dtor took more than 50 seconds to exit. Something is very wrong"));
			delete[] m_knownInputs;
		}

		virtual const ProcessStartInfo& GetStartInfo() const override { return m_startInfo; }
		virtual u32 GetId() override { return m_processId; }
		virtual u32 GetExitCode() override { UBA_ASSERT(m_done.IsSet(0)); return m_exitCode; }
		virtual bool HasExited() override { return m_done.IsSet(0); }
		virtual bool WaitForExit(u32 millisecondsTimeout) override { return m_done.IsSet(millisecondsTimeout); }
		virtual u64 GetTotalProcessorTime() const override { return m_processorTime; }
		virtual u64 GetTotalWallTime() const override { return m_wallTime; }
		virtual u64 GetPeakMemory() const override { return m_peakMemory;  }
		virtual const Vector<ProcessLogLine>& GetLogLines() const override { return m_logLines; }
		virtual const Vector<u8>& GetTrackedInputs() const override { return m_trackedInputs; }
		virtual const Vector<u8>& GetTrackedOutputs() const override { return m_trackedOutputs; };
		virtual bool Cancel() override
		{
			if (m_cancelled.exchange(true))
				return true;
			m_exitCode = ProcessCancelExitCode;
			if (auto s = m_server)
				s->OnCancelled(this);
			else
				m_done.Set();

			ProcessHandle h;
			h.m_process = this;
			CallProcessExit(h);
			h.m_process = nullptr;
			return true;
		}

		virtual const tchar* GetExecutingHost() const override { return m_executingHost.c_str(); }
		virtual bool IsRemote() const override { return true; }
		virtual ProcessExecutionType GetExecutionType() const override { return ProcessExecutionType_Remote; }
		virtual bool IsChild() override { return false; }

		virtual void TraverseOutputFiles(const Function<void(StringView file)>& func) override
		{
			SCOPED_READ_LOCK(m_writtenFilesLock, lock);
			for (auto& kv : m_writtenFiles)
				func(kv.first);
		}

		void CallProcessExit(ProcessHandle& h)
		{
			SCOPED_FUTEX(m_exitedLock, lock);
			if (!m_startInfo.exitedFunc)
				return;
			auto exitedFunc = m_startInfo.exitedFunc;
			auto userData = m_startInfo.userData;
			m_startInfo.exitedFunc = nullptr;
			m_startInfo.userData = nullptr;
			ProcessExitedResponse response = ProcessExitedResponse_None;
			exitedFunc(userData, h, response);
		}

		using Process::AddRef;
		using Process::Release;

		SessionServer* m_server;
		ProcessStartInfoHolder m_startInfo;
		Futex m_exitedLock;
		u32 m_processId;
		u32 m_exitCode = ~0u;
		u64 m_processorTime = 0;
		u64 m_wallTime = 0;
		u64 m_peakMemory = 0;
		Event m_done;
		Vector<ProcessLogLine> m_logLines;
		Futex m_receivedFilesLock;
		Vector<ReceivedFile> m_receivedFiles;
		Vector<u8> m_trackedInputs;
		Vector<u8> m_trackedOutputs;
		Atomic<bool> m_cancelled = false;
		bool m_canCrossArchitecture = false;
		bool m_canCrossPlatform = false;
		bool m_isIdle = false;
		u32 m_clientId = ~0u;
		u32 m_sessionId = 0;
		u64 m_startTime = 0;
		TString m_executingHost;

		ProcessHandle m_racingProcess;

		struct KnownInput { StringKey file; CasKey key; u32 mappingAlignment = 0; bool allowProxy = true; };
		KnownInput* m_knownInputs = nullptr;
		u32 m_knownInputsCount = 0;
		Event m_knownInputsDone;

		ReaderWriterLock m_writtenFilesLock;
		UnorderedMap<TString, CasKey> m_writtenFiles;
	};

	void SessionServerCreateInfo::Apply(const Config& config)
	{
		SessionCreateInfo::Apply(config);

		if (const ConfigTable* table = config.GetTable(TC("Session")))
		{
			table->GetValueAsBool(remoteLogEnabled, TC("RemoteLogEnabled"));
			table->GetValueAsBool(remoteTraceEnabled, TC("RemoteTraceEnabled"));
			table->GetValueAsBool(nameToHashTableEnabled, TC("NameToHashTableEnabled"));
			table->GetValueAsBool(traceIOEnabled, TC("TraceIOEnabled"));
			table->GetValueAsBool(customCasKeysEnabled, TC("CustomCasKeysEnabled"));
			table->GetValueAsBool(useFillLock, TC("UseFillLock"));
		}
	}

	bool GetCrossArchitectureDir(Logger& logger, StringBufferBase& dir, bool reportError)
	{
		// UBT has the path win-x64/native or win-arm64/native
		bool isUbtPath = dir.EndsWith(TCV("native"));
		if (isUbtPath)
			dir.Resize(dir.count - 7); // Remove native and slash
		const tchar* archPath[2] = { TC("x64"), TC("arm64") };
		if (!dir.EndsWith(archPath[IsArmBinary]))
			return reportError ? logger.Error(TC("Module dir is not under supported folder (%s) to be able to run cross architectures, can't figure out matching x64/arm64 folder"), dir.data) : false;
		dir.Resize(dir.count - TStrlen(archPath[IsArmBinary])).Append(archPath[!IsArmBinary]);
		if (isUbtPath)
			dir.Append(PathSeparator).Append(TCV("native"));
		return true;
	}

#if !PLATFORM_LINUX
	bool GetLinuxDir(Logger& logger, StringBufferBase& dir, bool reportError)
	{
		// UBT has the path win-x64/native or win-arm64/native
		bool isUbtPath = dir.EndsWith(TCV("native"));
		if (isUbtPath)
		{
			const tchar* runtimesDir = IsWindows ? TC("\\runtimes\\") : TC("/runtimes/");
			const tchar* pos = nullptr;
			if (!dir.Contains(runtimesDir, true, &pos))
				return false;
			dir.Resize(pos - dir.data + 10).Append(TCV("linux-x64")).Append(PathSeparator).Append(TCV("native"));
		}
		else
		{
			const tchar* platformStr = IsWindows ? TC("\\Win64\\") : TC("/Mac/");
			const tchar* pos = nullptr;
			if (!dir.Contains(platformStr, true, &pos))
				return false;
			dir.Resize(pos - dir.data + 1).Append(TCV("Linux")).Append(PathSeparator).Append(TCV("UnrealBuildAccelerator"));
		}
		return true;
	}
#endif

	SessionServer::SessionServer(const SessionServerCreateInfo& info, const u8* environment, u32 environmentSize)
	:	Session(info, TC("UbaSessionServer"), false, info.server)
	,	m_server(info.server)
	{
		m_server.RegisterOnClientDisconnected(ServiceId, [this](const Guid& clientUid, u32 clientId) { OnDisconnected(clientUid, clientId); });

		m_server.RegisterService(ServiceId,
			[this](const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer)
			{
				switch (messageInfo.type)
				{
					#define UBA_SESSION_MESSAGE(x) case SessionMessageType_##x: return Handle##x(connectionInfo, workContext, reader, writer);
					UBA_SESSION_MESSAGES
					#undef UBA_SESSION_MESSAGE
				}

				UBA_ASSERT(false);
				return false;
			},
			[](u8 type)
			{
				switch (type)
				{
					#define UBA_SESSION_MESSAGE(x) case SessionMessageType_##x: return AsView(TC("")#x);
					UBA_SESSION_MESSAGES
					#undef UBA_SESSION_MESSAGE
				default:
					return ToView(TC("Unknown"));
				}
			}
		);

		if (environmentSize)
		{
			m_environmentMemory.resize(environmentSize);
			memcpy(m_environmentMemory.data(), environment, environmentSize);
		}

		m_uiLanguage = GetUserDefaultUILanguage();
		m_resetCas = info.resetCas;
		m_remoteExecutionEnabled = info.remoteExecutionEnabled;
		m_nameToHashTableEnabled = info.nameToHashTableEnabled;
		m_remoteLogEnabled = info.remoteLogEnabled;
		m_remoteTraceEnabled = info.remoteTraceEnabled;
		m_traceIOEnabled = info.traceIOEnabled;
		m_customCasKeysEnabled = info.customCasKeysEnabled;
		m_useFillLock = info.useFillLock;

		if (m_resetCas)
			m_storage.Reset();

		m_storage.SetTrace(&m_trace, m_detailedTrace);
		m_server.SetTrace(&m_trace);

		if (m_detailedTrace)
			m_server.SetWorkTracker(&m_trace);

		StringBuffer<512> detoursFile;
		if (!GetDirectoryOfCurrentModule(m_logger, detoursFile))
		{
			UBA_ASSERT(false);
			return;
		}
		u32 dirLength = detoursFile.count;(void)dirLength;

		detoursFile.Append(PathSeparator).Append(UBA_DETOURS_LIBRARY);
		
		char temp[1024];

		#if !PLATFORM_LINUX
		StringBuffer<512> detoursFileLinux;
		detoursFileLinux.AppendDir(detoursFile);
		if (GetLinuxDir(m_logger, detoursFileLinux, false))
		{
			detoursFileLinux.Append(PathSeparator).Append(UBA_DETOURS_LIBRARY_LINUX).Parse(temp, sizeof_array(temp));
			m_detoursLibrary[true][false] = temp;
		}
		#endif

		#if PLATFORM_WINDOWS
		detoursFile.Parse(temp, sizeof_array(temp));
		m_detoursLibrary[IsLinux][IsArmBinary] = temp;
		if (GetCrossArchitectureDir(m_logger, detoursFile.Resize(dirLength), false))
		{
			detoursFile.Append(PathSeparator).Append(UBA_DETOURS_LIBRARY).Parse(temp, sizeof_array(temp));
			m_detoursLibrary[IsLinux][!IsArmBinary] = temp;
		}
		#else
		m_detoursLibrary[IsLinux][IsArmBinary] = detoursFile.data;
		#endif

		#if PLATFORM_LINUX
		m_stubBinary[IsArmBinary] = detoursFile.Resize(dirLength).Append(PathSeparator).Append(TCV("UbaStaticStub.bin")).ToString();
		#endif

		m_trace.SetThreadUpdateCallback([this]() { TraceSessionUpdate(); }, info.traceIntervalMs);

		if (!Create(info))
		{
			UBA_SHIPPING_ASSERT(false);
			return;
		}

		#if UBA_TEST_LINUX_HELPING_WINDOWS
		RegisterCrossArchitectureMapping(TC("d:\\temp\\dummy"), TC("d:\\temp\\gcc"));
		#endif
	}

	SessionServer::~SessionServer()
	{
		if (m_ownsTrace)
			m_trace.StopThread();

		m_trace.SetThreadUpdateCallback({});

		m_server.SetWorkTracker(nullptr);
		m_server.UnregisterOnClientDisconnected(ServiceId);
		m_server.UnregisterService(ServiceId);

		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, lock);
		for (ProcessHandle& p : m_queuedRemoteProcesses)
		{
			((RemoteProcess*)p.m_process)->m_server = nullptr;
			p.Cancel();
		}
		m_queuedRemoteProcesses.clear();
		for (const ProcessHandle& p : m_activeRemoteProcesses)
		{
			((RemoteProcess*)p.m_process)->m_server = nullptr;
			p.Cancel();
		}
		m_activeRemoteProcesses.clear();

		if (m_trace.IsWriting())
		{
			m_sharedMemory.TraceStats(m_trace, 30);

			m_trace.WriteSessionSummary([&](Logger& logger)
				{
					PrintSummary(logger);
					m_storage.PrintSummary(logger);
					m_server.PrintSummary(logger);
					KernelStats::GetGlobal().Print(logger, true);
					PrintContentionSummary(logger);
				});
		}

		for (auto s : m_clientSessions)
		{
			s->~ClientSession();
			aligned_free(s);
		}
		m_clientSessions.clear();
	}

	ProcessHandle SessionServer::RunProcessRacing(const ProcessToRaceCallback& callback)
	{
		u64 newestStartTime = 0;
		RemoteProcess* newest = nullptr;

		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, lock);
		for (const ProcessHandle& h : m_activeRemoteProcesses)
		{
			RemoteProcess& rp = *(RemoteProcess*)h.m_process;
			if (rp.m_startTime <= newestStartTime)
				continue;
			if (!rp.m_startInfo.rules->SupportsRacing())
				continue;
			if (rp.m_racingProcess.IsValid()) // && !rc.m_racingProcess.IsCancelled()
				continue;
			newest = &rp;
			newestStartTime = rp.m_startTime;
		}

		if (!newest)
			return {};

		ProcessStartInfo psi = newest->m_startInfo;
		ProcessHandle remoteHandle(newest);
		if (!callback(remoteHandle, psi))
			return {};

		StringBuffer<> logFile;
		if (*psi.logFile)
		{
			logFile.Append(psi.logFile);
			if (const tchar* dot = logFile.Last('.'))
			{
				u64 offset = dot - logFile.data;
				logFile.Resize(offset).Append(TCV("_racing")).Append(psi.logFile + offset);
			}
			else
				logFile.Append(TCV("_racing"));
			psi.logFile = logFile.data;
		}

		auto rpi = new RacingProcessExitInfo;
		rpi->server = this;
		rpi->remoteProcess = std::move(remoteHandle);
		rpi->exitedFunc = psi.exitedFunc;
		rpi->allowedToExit.Create(EventResetType_Manual);

		psi.exitedFunc = [](void* userData, const ProcessHandle& h, ProcessExitedResponse& r)
			{
				auto& rpi = *(RacingProcessExitInfo*)userData;
				auto& racingProcess = *(ProcessImpl*)h.m_process;

				auto exitGuard = MakeGuard([&]()
					{
						rpi.exitedFunc(racingProcess.m_startInfo.userData, h, r);
						delete &rpi;
					});

				// We spawn the process async, and if we get here before m_messageThread is set that means
				if (!racingProcess.m_messageThread)
					return;

				rpi.allowedToExit.IsSet();
				auto& remoteProcess = *(RemoteProcess*)rpi.remoteProcess.m_process;

				if (racingProcess.IsCancelled())
				{
					remoteProcess.m_racingProcess = {};
					return;
				}

				// We got here before remote process.. need to make sure process exit callback is called first
				rpi.server->OnRaceLost(remoteProcess);
				remoteProcess.m_exitCode = ProcessCancelExitCode;
				remoteProcess.m_done.Set();
				remoteProcess.CallProcessExit(rpi.remoteProcess);
			};

		// This function can call directly into psi.exitedFunc if it fails in certain ways (

		ProcessHandle racingProcess = InternalRunProcess(psi, true, nullptr, m_allowLocalDetour, rpi);
		if (!racingProcess.IsValid())
			return {};

		m_trace.ProcessRace(racingProcess.GetId(), newest->GetId());

		newest->m_racingProcess = racingProcess;

		rpi->allowedToExit.Set();

		return racingProcess;
	}

	ProcessHandle SessionServer::RunProcessRemote(const ProcessStartInfo& startInfo, float weight, const void* knownInputs, u32 knownInputsCount, bool canCrossArchitecture, bool canCrossPlatform)
	{
		//TrackWorkScope tws(m_trace, AsView(TC("RunProcessRemote")), ColorWork);

		UBA_ASSERT(!startInfo.startSuspended);

		FlushDeadProcesses(false);
		ValidateStartInfo(startInfo);
		u32 processId = CreateProcessId();
		RemoteProcess* remoteProcess = new RemoteProcess(this, startInfo, processId, weight);

		ProcessHandle h(remoteProcess); // Keep ref count up even if process is removed by callbacks etc.

		auto rules = GetRules(remoteProcess->m_startInfo);

		#if UBA_TEST_LINUX_HELPING_WINDOWS
		remoteProcess->m_startInfo.applicationStr = TC("d:\\temp\\dummy");
		remoteProcess->m_startInfo.application = remoteProcess->m_startInfo.applicationStr.c_str();
		m_canCrossPlatform = true;
		#endif

		remoteProcess->m_startInfo.rules = rules;
		remoteProcess->m_canCrossArchitecture = canCrossArchitecture;
		remoteProcess->m_canCrossPlatform = canCrossPlatform;

		if (knownInputsCount)
		{
			remoteProcess->m_knownInputsDone.Create(EventResetType_Manual);


			auto kiBegin = (const tchar*)knownInputs;
			auto kiEnd = kiBegin;
			for (u32 i=0;i!=knownInputsCount; ++i)
				kiEnd += TStrlen(kiEnd) + 1;

			u32 knownInputsBytes = u32((kiEnd - kiBegin)*sizeof(tchar));
			void* knownInputsCopy = malloc(knownInputsBytes);
			if (knownInputsCopy) // static analysis check
				memcpy(knownInputsCopy, knownInputs, knownInputsBytes);

			#if defined(__clang_analyzer__)
			free(knownInputsCopy); // analyzer doesn't seem to understand it is handed over
			#endif

			m_server.AddWork([this, h, knownInputsCopy, knownInputsCount, rules](const WorkContext& context)
				{
					auto remoteProcess = (RemoteProcess*)h.m_process;
					auto keys = remoteProcess->m_knownInputs = new RemoteProcess::KnownInput[knownInputsCount];

					struct Container
					{
						struct iterator
						{
							iterator() : ptr(nullptr), index(0) {}
							iterator(const tchar* p, u32 i) : ptr(p), index(i) {}
							iterator operator++(int) { auto prev = ptr; ptr += TStrlen(ptr) + 1; return iterator(prev, index++); }
							const tchar* operator*() { return ptr; }
							bool operator==(const iterator& o) const { return index == o.index; }
							const tchar* ptr;
							u32 index;
						};
						Container(const tchar* b, u32 c) : ptr(b), count(c) {}
						iterator begin() const { return iterator(ptr, 0); }
						iterator end() const { return iterator(ptr, count); }
						u32 size() { return count; }
						const tchar* ptr;
						u32 count;
					};

					Container container((const tchar*)knownInputsCopy, knownInputsCount);

					Atomic<u32> keysIndex = 0;
					const TString& workingDir = remoteProcess->m_startInfo.workingDirStr;

					m_server.ParallelFor(knownInputsCount, container, [&](const WorkContext&, auto& it)
						{
							//TrackWorkScope tws2(m_trace, AsView(TC("KnownInputs")), ColorWork);
							StringBuffer<> fileName;
							FixPath(*it, workingDir.c_str(), u32(workingDir.size()), fileName);
							StringKey fileNameKey = CaseInsensitiveFs ? ToStringKeyLower(fileName) : ToStringKey(fileName);

							//tws2.AddHint(fileName);

							// Make sure cas entry exists and caskey is calculated (cas content creation is deferred in case client already has it)
							CasKey casKey;
							if (!StoreCasFile(casKey, fileNameKey, fileName.data) || casKey == CasKeyZero)
								return;

							u32 keyIndex = keysIndex.fetch_add(1);
							UBA_ASSERT(keyIndex < knownInputsCount);
							auto& ki = keys[keyIndex];
							ki.file = fileNameKey;
							ki.key = casKey;
							ki.mappingAlignment = GetMemoryMapAlignment(fileName, true);
							ki.allowProxy = rules->AllowStorageProxy(fileName);


							// Update name to hash table
							SCOPED_FUTEX(m_nameToHashLookupLock, lock);
							CasKey& lookupCasKey = m_nameToHashLookup[fileNameKey];
							if (lookupCasKey != casKey)
							{
								lookupCasKey = casKey;
								BinaryWriter w(m_nameToHashTableMem.memory, m_nameToHashTableMem.writtenSize, NameToHashMemSize);
								m_nameToHashTableMem.AllocateNoLock(sizeof(StringKey) + sizeof(CasKey), 1, TC("NameToHashTable"));
								w.WriteStringKey(fileNameKey);
								w.WriteCasKey(lookupCasKey);
							}
						}, AsView(TC("KnownInputsLoop")), WorkPriority_High);

					remoteProcess->m_knownInputsCount = keysIndex;
					remoteProcess->m_knownInputsDone.Set();
					free(knownInputsCopy);

				}, 1, TC("KnownInputs"));
		}

		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, lock);
		m_queuedRemoteProcesses.push_back(remoteProcess);

		SCOPED_READ_LOCK(m_remoteProcessReturnedEventLock, lock2);
		if (m_remoteProcessReturnedEvent)
		{
			if (!m_remoteExecutionEnabled)
			{
				m_logger.Info(TC("Process queued for remote but remote execution was disabled, returning process to queue"));
				m_remoteProcessReturnedEvent(*remoteProcess);
			}
			else if (!m_connectionCount)
			{
				m_logger.Info(TC("Process queued for remote but there are no active connections, returning process to queue"));
				m_remoteProcessReturnedEvent(*remoteProcess);
			}
		}
		return h;
	}

	void SessionServer::DisableRemoteExecution()
	{
		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, lock);
		if (m_remoteExecutionEnabled)
			m_logger.Info(TC("Disable remote execution (remote sessions will finish current processes)"));
		m_remoteExecutionEnabled = false;
		m_trace.RemoteExecutionDisabled();
	}

	bool SessionServer::IsRemoteExecutionDisabled()
	{
		return !m_remoteExecutionEnabled;
	}

	void SessionServer::ReenableRemoteExecution()
	{
		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, lock);
		if (m_remoteExecutionEnabled)
			return;
		m_logger.Info(TC("Reenabled remote execution"));
		m_remoteExecutionEnabled = true;
		//m_trace.RemoteExecutionDisabled();
	}

	bool SessionServer::DisableRemoteExecutionOnAgent(const tchar* agentName)
	{
		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, queueLock);
		for (auto s : m_clientSessions)
		{
			if (s->name != agentName)
				continue;
			s->remoteExecutionEnabled = false;
			return true;
		}
		return false;
	}

	void SessionServer::SetCustomCasKeyFromTrackedInputs(const tchar* fileName_, const tchar* workingDir_, const u8* trackedInputs, u32 trackedInputsBytes)
	{
		if (!m_customCasKeysEnabled)
		{
			m_logger.Error(TC("Custom cas keys not enabled. Enable with config option CustomCasKeysEnabled"));
			return;
		}
		StringBuffer<> workingDir;
		FixFileName(workingDir, workingDir_, nullptr);
		if (workingDir[workingDir.count - 1] != '\\')
			workingDir.Append(TCV("\\"));
		StringBuffer<> fileName;
		FixFileName(fileName, fileName_, workingDir.data);
		StringKey fileNameKey = ToStringKey(fileName);
		
		SCOPED_FUTEX(m_customCasKeysLock, lock);
		auto insres = m_customCasKeys.try_emplace(fileNameKey);
		CustomCasKey& customKey = insres.first->second;
		customKey.casKey = CasKeyZero;
		customKey.workingDir = workingDir.data;
		customKey.trackedInputs.resize(trackedInputsBytes);
		memcpy(customKey.trackedInputs.data(), trackedInputs, trackedInputsBytes);

		//m_logger.Debug(TC("Registered file using custom cas %s (%s)"), fileName_, GuidToString(fileNameHash).str);
	}

	bool SessionServer::GetCasKeyFromTrackedInputs(CasKey& out, const tchar* fileName, const tchar* workingDir, const u8* data, u32 dataLen)
	{
		u64 workingDirLen = TStrlen(workingDir);

		BinaryReader reader(data);

		CasKeyHasher hasher;

		while (reader.GetPosition() < dataLen)
		{
			tchar str[512] = { 0 };
			reader.ReadString(str, sizeof_array(str));
			tchar* path = str;

			tchar temp[512];
			if (str[1] != ':' && (TStrstr(str, TC(".dll")) || TStrstr(str, TC(".exe"))))
			{
				bool res = SearchPathW(NULL, str, NULL, 512, temp, NULL);
				UBA_ASSERT(res);
				if (!res)
					return false;
				path = temp;
			}
			
			StringBuffer<> inputFileName;
			FixPath(path, workingDir, workingDirLen, inputFileName);

			if (inputFileName.StartsWith(m_tempPath.data))
				continue;
			if (inputFileName.Equals(fileName))
				continue;
			if (inputFileName.StartsWith(m_systemPath.data))
				continue;

			CasKey casKey;
			bool deferCreation = true;
			if (!m_storage.StoreCasFile(casKey, path, CasKeyZero, deferCreation))
				return false;
			UBA_ASSERTF(casKey != CasKeyZero, TC("Failed to store cas for %s when calculating key for tracked inputs on %s"), path, fileName);
			hasher.Update(&casKey, sizeof(CasKey));
		}

		out = ToCasKey(hasher, m_storage.StoreCompressed());
		return true;
	}

	void SessionServer::SetRemoteProcessSlotAvailableEvent(const RemoteProcessAvailableCallback& remoteProcessSlotAvailableEvent)
	{
		SCOPED_WRITE_LOCK(m_remoteProcessSlotAvailableEventLock, lock);
		m_remoteProcessSlotAvailableEvent = remoteProcessSlotAvailableEvent;
	}

	void SessionServer::SetRemoteProcessReturnedEvent(const Function<void(Process&)>& remoteProcessReturnedEvent)
	{
		SCOPED_WRITE_LOCK(m_remoteProcessReturnedEventLock, lock);
		m_remoteProcessReturnedEvent = remoteProcessReturnedEvent;
	}

	void SessionServer::SetNativeProcessCreatedFunc(const Function<void(ProcessImpl&)>& func)
	{
		m_nativeProcessCreatedFunc = func;
	}

	void SessionServer::WaitOnAllTasks()
	{
		while (true)
		{
			SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, lock);
			if (m_activeRemoteProcesses.empty() && m_queuedRemoteProcesses.empty())
				break;
			lock.Leave();
			Sleep(200);
		}

		bool isEmpty = false;
		while (!isEmpty)
		{
			Vector<ProcessHandle> processes;
			{
				SCOPED_FUTEX(m_processesLock, lock);
				isEmpty = m_processes.empty();
				processes.reserve(m_processes.size());
				for (auto& pair : m_processes)
					processes.push_back(pair.second);
			}

			for (auto& process : processes)
				process.WaitForExit(100000);
		}

		FlushDeadProcesses(true);
	}

	void SessionServer::WaitOnAllClients()
	{
		while (true)
		{
			SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, queueLock);
			bool connected = false;
			for (auto s : m_clientSessions)
				connected |= s->connected;
			queueLock.Leave();
			if (!connected)
				break;
			Sleep(10);
		}
	}

	void SessionServer::SetMaxRemoteProcessCount(u32 count)
	{
		m_maxRemoteProcessCount.exchange(count);
	}

	u32 SessionServer::BeginExternalProcess(const tchar* description, const tchar* breadcrumbs)
	{
		u32 processId = CreateProcessId();
		m_trace.ProcessAdded(0, processId, ToView(description), ToView(breadcrumbs));
		return processId;
	}

	void SessionServer::EndExternalProcess(u32 id, u32 exitCode)
	{
		StackBinaryWriter<1024> statsWriter;
		ProcessStats processStats;
		processStats.Write(statsWriter);
		m_trace.ProcessExited(id, exitCode, statsWriter.GetData(), statsWriter.GetPosition(), Vector<ProcessLogLine>());
	}

	void SessionServer::UpdateProgress(u32 processesTotal, u32 processesDone, u32 errorCount)
	{
		m_trace.ProgressUpdate(processesTotal, processesDone, errorCount);
	}

	void SessionServer::UpdateStatus(u32 statusRow, u32 statusColumn, const tchar* statusText, LogEntryType statusType, const tchar* statusLink)
	{
		m_trace.StatusUpdate(statusRow, statusColumn, ToView(statusText), statusType, statusLink ? ToView(statusLink) : StringView());
	}

	void SessionServer::AddProcessBreadcrumbs(u32 processId, const tchar* breadcrumbs, bool deleteOld)
	{
		m_trace.ProcessAddBreadcrumbs(processId, ToView(breadcrumbs), deleteOld);
	}

	NetworkServer& SessionServer::GetServer()
	{
		return m_server;
	}

	void SessionServer::RegisterNetworkTrafficProvider(u64 key, const NetworkTrafficProvider& provider)
	{
		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, lock);
		m_providers.emplace_back(key, provider);
	}

	void SessionServer::UnregisterNetworkTrafficProvider(u64 key)
	{
		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, lock);
		for (auto i=m_providers.begin();i!=m_providers.end();++i)
		{
			if (i->key != key)
				continue;
			m_providers.erase(i);
			break;
		}
	}

	bool SessionServer::RegisterCrossArchitectureMapping(const tchar* from, const tchar* to)
	{
		if (!IsAbsolutePath(from))
			return m_logger.Error(TC("Must register absolute paths for cross mapping (%s)"), from);
		if (!IsAbsolutePath(to))
			return m_logger.Error(TC("Must register absolute paths for cross mapping (%s)"), to);
		m_crossArchitectureMappings.emplace_back(CrossArchitectureMapping{from, to});
		return true;
	}

	void SessionServer::SetOuterScheduler(Scheduler* scheduler)
	{
		UBA_ASSERT(!m_outerScheduler || !scheduler);
		m_outerScheduler = scheduler;
	}

	Scheduler* SessionServer::GetOuterScheduler()
	{
		return m_outerScheduler;
	}

	bool SessionServer::GetRemoteTraceEnabled()
	{
		return m_remoteTraceEnabled;
	}

	ProcessHandle SessionServer::GetNewestLocalProcess()
	{
		u64 newestTime = 0;
		ProcessImpl* newestProcess = nullptr;
		SCOPED_FUTEX(m_processesLock, lock);
		for (auto& kv : m_processes)
		{
			ProcessHandle& h = kv.second;
			if (h.IsRemote())
				continue;
			auto& p = *(ProcessImpl*)h.m_process;
			if (p.m_startTime <= newestTime)
				continue;
			if (p.m_parentProcess) // Only get root processes
				continue;
			if (p.IsCancelled()) // Already cancelled
				continue;
			if (p.WaitForExit(0)) // Process already exited, let it finish its work
				continue;

			newestTime = p.m_startTime;
			newestProcess = &p;
		}
		if (!newestProcess)
			return {};
		ProcessHandle ph(newestProcess);
		return ph;
	}

	bool SessionServer::WasEverConnected()
	{
		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, lock);
		return !m_clientSessions.empty();
	}

	void SessionServer::OnDisconnected(const Guid& clientUid, u32 clientId)
	{
		u32 returnCount = 0;
		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, queueLock);
		for (auto it=m_activeRemoteProcesses.begin(); it!=m_activeRemoteProcesses.end();)
		{
			RemoteProcess* remoteProcess = (RemoteProcess*)it->m_process;
			if (remoteProcess->m_clientId != clientId)
			{
				++it;
				continue;
			}
			m_queuedRemoteProcesses.push_front(*it);
			it = m_activeRemoteProcesses.erase(it);
			remoteProcess->m_executingHost.clear();

			m_trace.ProcessReturned(remoteProcess->m_processId, AsView(TC("Disconnected")));

			ProcessHandle h = ProcessRemoved(remoteProcess->m_processId);
			if (!h.m_process)
				m_logger.Warning(TC("Trying to remove process on client %u that does not exist in active list.. investigate me"), clientId);

			++returnCount;

			remoteProcess->m_clientId = ~0u;
			remoteProcess->m_sessionId = 0;

			SCOPED_READ_LOCK(m_remoteProcessReturnedEventLock, lock2);
			if (m_remoteProcessReturnedEvent)
				m_remoteProcessReturnedEvent(*remoteProcess);
		}

		m_returnedRemoteProcessCount += returnCount;

		u32 sessionId = 0;
		StringBuffer<> sessionName;
		for (auto sptr : m_clientSessions)
		{
			++sessionId;
			auto& s = *sptr;
			if (s.clientId != clientId)
				continue;

			if (!returnCount && !s.hasNotification && !s.enabled)
				m_trace.SessionNotification(sessionId, TC("Done"));

			m_trace.SessionDisconnect(sessionId);

			sessionName.Append(s.name);
			UBA_ASSERTF(s.usedSlotCount == returnCount || m_logger.isMuted, TC("Used slot count different than return count (%u vs %u)"), s.usedSlotCount, returnCount);
			s.usedSlotCount -= returnCount;

			if (s.enabled)
				m_availableRemoteSlotCount -= s.processSlotCount - returnCount;
			s.enabled = false;
			s.connected = false;
			--m_connectionCount;
		}

		if (returnCount)
		{
			if (sessionName.IsEmpty())
				sessionName.Append(TCV("<can't find session>"));

			m_logger.Info(TC("Client session %s (%s) disconnected. Returned %u process(s) to queue"), sessionName.data, GuidToString(clientUid).str, returnCount);
		}

		if (m_connectionCount)
			return;

		if (!m_queuedRemoteProcesses.empty())
		{
			SCOPED_READ_LOCK(m_remoteProcessReturnedEventLock, lock2);
			if (m_remoteProcessReturnedEvent)
			{
				m_logger.Info(TC("No client sessions connected and there are %llu processes left in the remote queue. Will return all queued remote processes"), m_queuedRemoteProcesses.size());
				List<ProcessHandle> temp(m_queuedRemoteProcesses);
				for (ProcessHandle& remoteProcess : temp)
					m_remoteProcessReturnedEvent(*remoteProcess.m_process);
			}
			else
			{
				m_logger.Info(TC("No client sessions connected and there are %llu processes left in the remote queue. processes will be picked up when remote connection is established"), m_queuedRemoteProcesses.size());
			}
		}

		if (m_activeRemoteProcesses.empty())
			return;

		// This path has been seen in the wild (twice over million of runs)....
		// It doesn't seem to have caused hangs so maybe there is a timing with disconnecting and getting results back.
		// Let's wait 1 second and see again and log out.
		m_logger.Warning(TC("No client sessions connected but there are %llu active remote processes. This should not happen"), m_activeRemoteProcesses.size());
		queueLock.Leave();
		Sleep(1000);
		queueLock.Enter();
		m_logger.Warning(TC("%llu active remote processes over %u connections"), m_activeRemoteProcesses.size(), m_connectionCount);
	}

	bool SessionServer::HandleConnect(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		StringBuffer<128> name;
		reader.ReadString(name);
		u32 clientVersion = reader.ReadU32();
		bool isClientArm = false;
		bool isClientLinux = false;
		if (clientVersion >= 36)
			isClientArm = reader.ReadBool();
		if (clientVersion >= 49)
			isClientLinux = reader.ReadBool();

		m_logger.Detail(TC("Client session %s connected (Id: %u, Uid: %s%s)"), name.data, connectionInfo.GetId(), GuidToString(connectionInfo.GetUid()).str, (isClientArm ? TC(", IsArm: true") : TC("")));

		CasKey clientAgentKey = reader.ReadCasKey();
		CasKey clientDetoursKey = reader.ReadCasKey();(void)clientDetoursKey;

		CasKey detoursBinaryKey[2];
		CasKey& agentBinaryKey = m_agentBinaryKey[isClientArm];

		#if PLATFORM_LINUX
		CasKey stubBinaryKey[2];
		#endif

		bool binAsVersion = clientAgentKey != CasKeyZero;
		{
			SCOPED_FUTEX(m_binKeysLock, lock);

			detoursBinaryKey[0] = m_detoursBinaryKey[isClientLinux][0];
			detoursBinaryKey[1] = m_detoursBinaryKey[isClientLinux][1];

			StringBuffer<> temp;
			bool deferCreation = true;

			// Handle x64 for both architectures and arm64 only for arm64 clients
			for (u32 i=0; i!=(isClientArm ? 2u : 1u); ++i)
			{
				if (detoursBinaryKey[i] != CasKeyZero)
					continue;
				temp.Clear().Append(m_detoursLibrary[isClientLinux][i].c_str());
				if (!m_storage.StoreCasFile(detoursBinaryKey[i], temp.data, CasKeyZero, deferCreation) || detoursBinaryKey[i] == CasKeyZero)
					return m_logger.Error(TC("Failed to create cas for %s"), temp.data);
				m_detoursBinaryKey[isClientLinux][i] = detoursBinaryKey[i];
			}

			#if PLATFORM_LINUX
			stubBinaryKey[0] = m_stubBinaryKey[0];
			stubBinaryKey[1] = m_stubBinaryKey[1];
			for (u32 i=0; i!=(isClientArm ? 2u : 1u); ++i)
			{
				if (stubBinaryKey[i] != CasKeyZero)
					continue;
				temp.Clear().Append(m_stubBinary[i].c_str());
				if (!m_storage.StoreCasFile(stubBinaryKey[i], temp.data, CasKeyZero, deferCreation) || stubBinaryKey[i] == CasKeyZero)
					continue; //return m_logger.Error(TC("Failed to create cas for %s"), temp.data);
				m_stubBinaryKey[i] = stubBinaryKey[i];
			}
			#endif

			if (binAsVersion && agentBinaryKey == CasKeyZero)
			{
				UBA_ASSERT(IsLinux == isClientLinux);
				StringBuffer<> agentDir;
				if (!GetDirectoryOfCurrentModule(m_logger, agentDir))
					return false;
				if (IsArmBinary != isClientArm)
					if (!GetCrossArchitectureDir(m_logger, agentDir, true))
						return false;
				UBA_ASSERT(IsWindows);
				agentDir.Append(PathSeparator).Append(UBA_AGENT_EXECUTABLE);
				if (!m_storage.StoreCasFile(agentBinaryKey, agentDir.data, CasKeyZero, deferCreation) || agentBinaryKey == CasKeyZero)
				{
					// This is hacky but uba binary is not copied by ubt anymore.
					StringBuffer<> dir2;
					if (!GetAlternativeUbaPath(m_logger, dir2, agentDir, IsWindows && isClientArm))
						return false;
					dir2.Append(UBA_AGENT_EXECUTABLE);
					if (!m_storage.StoreCasFile(agentBinaryKey, dir2.data, CasKeyZero, deferCreation) || agentBinaryKey == CasKeyZero)
						return m_logger.Error(TC("Failed to create cas for %s"), dir2.data);
				}
			}
		}

		StringBuffer<> tempBuffer;
		auto& disconnectResponse = tempBuffer;

		if (binAsVersion && clientAgentKey != agentBinaryKey)
		{
			m_logger.Warning(TC("UbaAgent binaries mismatch. Disconnecting %s"), name.data);
			disconnectResponse.Appendf(TC("UbaAgent binaries mismatch. Disconnecting..."));
		}
		else if (clientVersion != SessionNetworkVersion)
		{
			m_logger.Warning(TC("Version mismatch. Server is on version %u while client is on %u. Disconnecting %s (connection %u)"), SessionNetworkVersion, clientVersion, name.data, connectionInfo.GetId());
			disconnectResponse.Appendf(TC("Version mismatch. Server is on version %u while client is on %u. Disconnecting..."), SessionNetworkVersion, clientVersion);
		}

		writer.WriteBool(disconnectResponse.IsEmpty());

		if (!disconnectResponse.IsEmpty())
		{
			writer.WriteString(disconnectResponse);
			writer.WriteCasKey(agentBinaryKey);
			writer.WriteCasKey(detoursBinaryKey[0]);
			if (isClientArm)
				writer.WriteCasKey(detoursBinaryKey[1]);
			return true;
		}
		writer.WriteBool(IsLinux);

		TString machineId = reader.ReadString();

		u32 processSlotCount = reader.ReadU32();
		bool dedicated = reader.ReadBool();

		TString info = reader.ReadString();

		u64 memAvail = reader.ReadU64();
		u64 memTotal = reader.ReadU64();
		u32 cpuLoadValue = reader.ReadU32();
		float cpuLoad = *(float*)&cpuLoadValue;


		// I have no explanation for this. On linux we get a shutdown crash when running through UBT if session is allocated with normal new
		// For now we will work around it by using aligned_alloc which seems to be working on all platforms
		auto& session = *new (aligned_alloc(alignof(ClientSession), sizeof(ClientSession))) ClientSession();
		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, lock);
		m_clientSessions.push_back(&session);
		u32 sessionId = u32(m_clientSessions.size());
		session.name = name.data;
		session.clientId = connectionInfo.GetId();
		session.processSlotCount = processSlotCount;
		session.dedicated = dedicated;
		session.isArm = isClientArm;
		session.isLinux = isClientLinux;
		session.memAvail = memAvail;
		session.memTotal = memTotal;
		session.cpuLoad = cpuLoad;
		m_availableRemoteSlotCount += processSlotCount;
		++m_connectionCount;

		if (!InitializeNameToHashTable())
			return false;

		writer.WriteCasKey(m_detoursBinaryKey[isClientLinux][0]);
		if (isClientArm)
			writer.WriteCasKey(m_detoursBinaryKey[isClientLinux][1]);

		#if PLATFORM_LINUX
		writer.WriteCasKey(m_stubBinaryKey[0]);
		if (isClientArm)
			writer.WriteCasKey(m_stubBinaryKey[1]);
		#endif

		writer.WriteBool(m_resetCas);
		writer.WriteU32(sessionId);
		writer.WriteU32(m_uiLanguage);
		writer.WriteBool(m_storeIntermediateFilesCompressed);
		writer.WriteBool(m_detailedTrace);
		writer.WriteBool(m_remoteLogEnabled);
		writer.WriteBool(m_remoteTraceEnabled);
		writer.WriteBool(m_readIntermediateFilesCompressed);

		auto& computerName = tempBuffer.Clear();
		GetComputerNameW(computerName);
		writer.WriteString(computerName);

		WriteRemoteEnvironmentVariables(writer);

		m_trace.SessionAdded(sessionId, connectionInfo.GetId(), name, machineId); // Must be inside lock for TraceSessionUpdate() to not include
		m_trace.SessionInfo(sessionId, info);
		m_trace.SessionUpdate(sessionId, 1, 0, 0, 0, memAvail, memTotal, cpuLoad);

		lock.Leave();
		return true;
	}

	bool SessionServer::HandleEnsureBinaryFile(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		bool isClientLinux = reader.ReadBool();
		bool isClientArm = reader.ReadBool();

		StringBuffer<MaxPath> fileName;
		reader.ReadString(fileName);
		StringKey fileNameKey = reader.ReadStringKey();
		TString applicationDir = reader.ReadString();
		TString workingDir = reader.ReadString();

		StringBuffer<MaxPath*4> lookupStr;
		lookupStr.Append(fileName).Append(applicationDir).Append(workingDir).Append('#');
		lookupStr.MakeLower();
		StringKey lookupKey = ToStringKeyNoCheck(lookupStr.data, lookupStr.count);

		SCOPED_FUTEX(m_applicationDataLock, lock);
		auto insres = m_applicationData.try_emplace(lookupKey);
		ApplicationData& data = insres.first->second;
		lock.Leave();

		SCOPED_FUTEX(data.lock, lock2);
		if (!data.bytes.empty())
		{
			writer.WriteBytes(data.bytes.data(), data.bytes.size());
			return true;
		}

		Vector<TString> loaderPaths;
		while (reader.GetLeft())
			loaderPaths.push_back(reader.ReadString());

		CasKey casKey = CasKeyZero;
		StringBuffer<> absoluteFile;

		bool isCross = isClientLinux != IsLinux || isClientArm != IsArmBinary;

		auto FixCrossArchitecture = [this, isCross](StringBuffer<>& absoluteFile)
			{
				if (!isCross)
					return;
				for (auto& mapping : m_crossArchitectureMappings)
					if (absoluteFile.StartsWith(mapping.from.c_str()))
					{
						StringBuffer<> temp;
						temp.Append(absoluteFile.data + mapping.from.size());
						absoluteFile.Clear().Append(mapping.to).Append(temp);
						break;
					}
			};


		if (!loaderPaths.empty())
		{
			for (auto& loaderPath : loaderPaths)
			{
				StringBuffer<> fullPath;

				#if PLATFORM_LINUX
				if (loaderPath[0] != '/') // TODO: Revisit this.. should be done in a less hacky way.
				#endif
					fullPath.Append(applicationDir).EnsureEndsWithSlash();
				fullPath.Append(loaderPath).EnsureEndsWithSlash().Append(fileName);
				if (GetFileAttributesW(fullPath.data) == INVALID_FILE_ATTRIBUTES)
					continue;
				FixPath(fullPath.data, nullptr, 0, absoluteFile);
				FixCrossArchitecture(absoluteFile);
				fileNameKey = ToStringKeyLower(absoluteFile);
				if (!IsKnownSystemFile(fileName.data))
					if (!StoreCasFile(casKey, fileNameKey, absoluteFile.data))
						return false;
				break;
			}

			// dlopen(3) slash-but-not-absolute paths resolve against cwd (bindgen via LIBCLANG_PATH); loaderPaths loop misses since Shared_LoadLibrary hands {"/"}.
			#if PLATFORM_LINUX
			if (casKey == CasKeyZero && !IsAbsolutePath(fileName.data) && TStrchr(fileName.data, PathSeparator))
			{
				StringBuffer<> fullPath;
				fullPath.Append(workingDir).EnsureEndsWithSlash().Append(fileName);
				if (GetFileAttributesW(fullPath.data) != INVALID_FILE_ATTRIBUTES)
				{
					FixPath(fullPath.data, nullptr, 0, absoluteFile);
					FixCrossArchitecture(absoluteFile);
					fileNameKey = ToStringKeyLower(absoluteFile);
					if (!IsKnownSystemFile(fileName.data))
						if (!StoreCasFile(casKey, fileNameKey, absoluteFile.data))
							return false;
				}
			}
			#endif

			#if 0
			if (casKey == CasKeyZero)
			{
				m_logger.Warning(TC("HandleEnsureBinaryFile - Failed to find file %s"), fileName.data);
				for (auto& loaderPath : loaderPaths)
				{
					m_logger.Warning(TC("   LoaderPath %s"), loaderPath.c_str());
				}
			}
			#endif
		}
		else if (m_searchPathCache.SearchPathForFile(m_logger, absoluteFile, fileName, workingDir, applicationDir))
		{

			if (!absoluteFile.StartsWith(m_systemPath.data) || !IsKnownSystemFile(fileName.data))
			{
				FixCrossArchitecture(absoluteFile);
				fileNameKey = ToStringKeyLower(absoluteFile);
				if (!StoreCasFile(casKey, fileNameKey, absoluteFile.data))
					return false;
			}
		}

		u64 startPos = writer.GetPosition();
		writer.WriteCasKey(casKey);
		writer.WriteString(absoluteFile);

		u64 bytesSize = writer.GetPosition() - startPos;
		data.bytes.resize(bytesSize);
		memcpy(data.bytes.data(), writer.GetData() + startPos, bytesSize);

		return true;
	}

	bool SessionServer::HandleGetApplication(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		bool isClientLinux = reader.ReadBool();
		StringBuffer<MaxPath> applicationName;
		reader.ReadString(applicationName);
		StringBuffer<MaxPath> workingDir;
		reader.ReadString(workingDir);

		#if UBA_TEST_LINUX_HELPING_WINDOWS
		if (isClientLinux && applicationName.Equals(TCV("/usr/bin/gcc")))
			applicationName.Clear().Append(TCV("d:\\temp\\gcc"));
		#endif

		if (!IsAbsolutePath(applicationName.data))
		{
			StringBuffer<> temp;
			if (!m_searchPathCache.SearchPathForFile(m_logger, temp, applicationName, workingDir, {}))
				return false;
			applicationName.Clear().Append(temp);
		}

		StringKey applicationKey = ToStringKeyLower(applicationName);

		SCOPED_FUTEX(m_applicationDataLock, lock);
		auto insres = m_applicationData.try_emplace(applicationKey);
		ApplicationData& data = insres.first->second;
		lock.Leave();
				
		SCOPED_FUTEX(data.lock, lock2);
		if (!data.bytes.empty())
		{
			writer.WriteBytes(data.bytes.data(), data.bytes.size());
			return true;
		}

		u64 startPos = writer.GetPosition();
		BinaryAndDeps binaryAndDeps;
		if (!GetBinaryModules(binaryAndDeps, applicationName.data, isClientLinux))
		{
			// Must respond: skipping makes the agent's ReadModules block forever (10m timeout). moduleCount=0 triggers the writtenFile-staged fallback in PrepareProcess.
			#if !PLATFORM_WINDOWS
			writer.WriteBool(false); // isShebang
			#endif
			writer.WriteU32(m_systemPath.count);
			writer.WriteU32(0);
			return true;
		}

		#if PLATFORM_WINDOWS
		if (applicationName.EndsWith(TCV("ubatest.exe"))) // Since we don't detour ubatest.exe we need to make sure to transfer all the files needed to do the unit tests
		{
			StringBuffer<> extra;
			extra.AppendDir(applicationName).EnsureEndsWithSlash().Append(TCV("UbaWine.dll.so"));
			binaryAndDeps.modules.emplace_back(TC("UbaWine.dll.so"), extra.data, DefaultAttributes());
			extra.Clear().AppendDir(applicationName).EnsureEndsWithSlash().Append(TCV("UbaDetours.dll"));
			binaryAndDeps.modules.emplace_back(TC("UbaDetours.dll"), extra.data, DefaultAttributes());
		}
		#endif

		#if PLATFORM_LINUX
		StringBuffer<> patchedModule0Path;
		if (!binaryAndDeps.modules.empty())
			if (!TryPatchBinary(patchedModule0Path, binaryAndDeps.modules[0].path))
				return false;
		bool isFirstModule = true;
		#endif

		#if !PLATFORM_WINDOWS
		writer.WriteBool(binaryAndDeps.isShebang);
		#endif

		writer.WriteU32(m_systemPath.count);
		writer.WriteU32(u32(binaryAndDeps.modules.size()));

		for (BinaryModule& m : binaryAndDeps.modules)
		{
			CasKey casKey;
			StringView path = m.path;
			u32 fileAttributes = m.fileAttributes;

			#if PLATFORM_WINDOWS
			TString tmp;
			if (isClientLinux)
			{
				bool execute = true;
				tmp = m.path;
				Replace(tmp.data(), NonPathSeparator, PathSeparator);
				path = tmp;
				fileAttributes = 0644 | (execute ? 0100 : 0);
			}
			#endif

			// CAS read points at the patched copy; module path sent to the client stays m.path
			// so the client doesn't see the detour.
			#if PLATFORM_LINUX
			if (isFirstModule && patchedModule0Path.count)
				path = patchedModule0Path;
			isFirstModule = false;
			#endif

			if (!StoreCasFile(casKey, StringKeyZero, path.data))
				return false;
			writer.WriteString(m.path);
			writer.WriteU32(fileAttributes);
			writer.WriteBool(m.isSystem);
			writer.WriteCasKey(casKey);
			#if PLATFORM_MAC
			if (!isClientLinux)
				writer.WriteU32(m.minOsVersion);
			#endif
		}

		u64 bytesSize = writer.GetPosition() - startPos;
		data.bytes.resize(bytesSize);
		memcpy(data.bytes.data(), writer.GetData() + startPos, bytesSize);

		return true;
	}

	bool SessionServer::HandleGetFileFromServer(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 processId = reader.ReadU32(); (void)processId;
		StringBuffer<MaxPath> fileName;
		reader.ReadString(fileName);
		StringKey fileNameKey = reader.ReadStringKey();

		workContext.tracker.AddHint(StringView(fileName).GetFileName());

		CasKey casKey;
		if (!StoreCasFile(casKey, fileNameKey, fileName.data))
			return false;
		if (casKey == CasKeyZero)
		{
			// TODO: Should this instead use DirectoryTable? (it is currently not properly populated for lookups)
			u32 attr = GetFileAttributesW(fileName.data);
			if (attr == INVALID_FILE_ATTRIBUTES || !IsDirectory(attr))
			{
				// Not finding a file is a valid path. Some applications try with a path and if fails try another path
				//m_logger.Error(TC("Failed to create cas for %s (not found)"), fileName.data);
				writer.WriteCasKey(casKey);
				return true;
			}

			casKey = CasKeyIsDirectory;
		}

		u64 serverTime;
		if (m_nameToHashInitialized && casKey != CasKeyIsDirectory)
		{
			SCOPED_FUTEX(m_nameToHashLookupLock, lock);
			serverTime = GetTime();
			CasKey& lookupCasKey = m_nameToHashLookup[fileNameKey];
			if (lookupCasKey != casKey)
			{
				lookupCasKey = casKey;
				BinaryWriter w(m_nameToHashTableMem.memory, m_nameToHashTableMem.writtenSize, NameToHashMemSize);
				m_nameToHashTableMem.AllocateNoLock(sizeof(StringKey) + sizeof(CasKey), 1, TC("NameToHashTable"));
				w.WriteStringKey(fileNameKey);
				w.WriteCasKey(casKey);
			}
		}
		else
			serverTime = GetTime();

		writer.WriteCasKey(casKey);
		writer.WriteU64(serverTime);
		return true;
	}

	bool SessionServer::HandleGetLongPathName(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		#if PLATFORM_WINDOWS
		StringBuffer<MaxPath> shortPath;
		reader.ReadString(shortPath);
		StringBuffer<MaxPath> longPath;
		longPath.count = ::GetLongPathNameW(shortPath.data, longPath.data, longPath.capacity);
		writer.WriteU32(GetLastError());
		writer.WriteString(longPath);
		return true;
		#else
		return false;
		#endif
	}

	bool SessionServer::WriteReceivedFile(ReceivedFile& file, RemoteProcess* process, const ConnectionInfo& connectionInfo)
	{
		u32 clientId = connectionInfo.GetId();
		StringView destination = file.destination;
		CasKey casKey = file.casKey;

		if (destination.StartsWith(TC("<log>")))
		{
			StringBuffer<> logPath;
			logPath.Append(m_sessionLogDir).Append(destination.data + 5);
			if (!m_storage.CopyOrLink(casKey, logPath.data, file.attributes, false, clientId))
				m_logger.Error(TC("Failed to copy cas from %s to %s"), CasKeyString(casKey).str, logPath.data);
			else if (!m_storage.DropCasFile(casKey, false, logPath.data))
				m_logger.Error(TC("Failed to drop cas %s"), CasKeyString(casKey).str);
			return true;
		}

		if (destination.StartsWith(TC("<uba>")))
		{
			StringBuffer<> ubaPath;
			ubaPath.Append(m_sessionLogDir);

			ClientSession* session = nullptr;
			for (auto& s : m_clientSessions)
				if (s->clientId == clientId)
					session = s;
			if (session)
				ubaPath.Append(session->name);
			else
				ubaPath.Append(TCV("Connection")).AppendValue(clientId);

			ubaPath.Append(TCV(".uba"));
			m_storage.CopyOrLink(casKey, ubaPath.data, file.attributes, false, clientId);
			m_storage.DropCasFile(casKey, false, ubaPath.data);
			return true;
		}

		if (!process)
			return m_logger.Info(TC("Failed to find process for id %u when receiving SendFileToServer message for file %s"), file.processId, destination.data).ToFalse();

		auto& startInfo = process->GetStartInfo();
		auto& rules = *startInfo.rules;

		bool shouldWriteToDisk = ShouldWriteToDisk(destination);

		{
			SCOPED_WRITE_LOCK(process->m_writtenFilesLock, lock);
			process->m_writtenFiles.try_emplace(destination.ToString(), casKey);
		}

		// We need to report before writing to flush out potential deferred cas creations
		m_storage.ReportFileWrite(file.destinationKey, destination.data);

		if (shouldWriteToDisk)
		{
			bool writeCompressed = false;
			bool shouldWritePlaceholder = m_writePlaceholders && rules.SupportsPlaceholder(destination);

			RootsHandle rootsHandle = startInfo.rootsHandle;

			Storage::FormattingFunc formattingFunc;
			bool escapeSpaces;
			if (HasVfs(rootsHandle) && rules.ShouldDevirtualizeFile(destination, escapeSpaces))
			{
				UBA_ASSERT(!shouldWritePlaceholder);
				formattingFunc = [&](MemoryBlock& destData, const void* sourceData, u64 sourceSize, const tchar* hint)
					{
						return DevirtualizeDepsFile(rootsHandle, destData, sourceData, sourceSize, escapeSpaces, hint);
					};
			}
			else if (m_storeIntermediateFilesCompressed)
			{
				writeCompressed = g_globalRules.FileCanBeCompressed(destination);
			}

			if (shouldWritePlaceholder)
			{
				if (!m_storage.WritePlaceholder(casKey, destination.data, file.attributes, clientId))
					return m_logger.Error(TC("Failed to write placeholder file for cas from %s to %s (%s)"), CasKeyString(casKey).str, destination.data, startInfo.GetDescription());
			}
			else
			{
				if (!m_storage.CopyOrLink(casKey, destination.data, file.attributes, writeCompressed, clientId, formattingFunc))
					return m_logger.Warning(TC("Failed to copy cas from %s to %s (%s)"), CasKeyString(casKey).str, destination.data, startInfo.GetDescription());
			}
			TraceWrittenFile(file.processId, destination);
		}
		else
		{
			if (!rules.IsInvisible(destination))
				if (!m_storage.FakeCopy(casKey, destination.data, 0, 0, file.attributes))
					return m_logger.Error(TC("Failed to fake copy cas from %s to %s (%s)"), CasKeyString(casKey).str, destination.data, startInfo.GetDescription());

			// We need to transfer ref count to this list
			// and release the one inside StorageServer
			if (!m_storage.AddRef(casKey))
				return m_logger.Error(TC("Failed to add ref count to memory caskey %s (%s)"), CasKeyString(casKey).str, destination.data);

			m_storage.SkipWrite(casKey, clientId);

			SCOPED_WRITE_LOCK(m_receivedFilesLock, lock);
			m_receivedFiles.try_emplace(file.destinationKey, casKey);
		}

		if (!rules.IsInvisible(destination))
		{
			bool invalidateStorage = false; // No need, already handled in m_storage.CopyOrLink
			RegisterCreateFileForWrite(StringKeyZero, destination, 0, 0, file.attributes, shouldWriteToDisk, invalidateStorage);

			if (startInfo.trackInputs)
			{
				u64 bytes = GetStringWriteSize(destination.data, destination.count);
				SCOPED_WRITE_LOCK(m_receivedFilesLock, lock); // Abuse of existing lock (could use its own lock but not worth it)
				u64 prevSize = process->m_trackedOutputs.size();
				process->m_trackedOutputs.resize(prevSize + bytes);
				BinaryWriter w2(process->m_trackedOutputs.data(), prevSize, prevSize + bytes);
				w2.WriteString(destination);
			}
		}
		return true;
	}

	bool SessionServer::WriteReceivedFiles(Vector<ReceivedFile>& receivedFiles, RemoteProcess& process, Timer& writeFilesTimer, const ConnectionInfo& connectionInfo)
	{
		if (receivedFiles.empty())
			return true;
		TimerScope ts(writeFilesTimer);
		Atomic<bool> writeSuccess = true;
		m_server.ParallelFor(u32(receivedFiles.size()), receivedFiles, [&](const WorkContext&, auto& it)
			{
				if (!WriteReceivedFile(*it, &process, connectionInfo))
					writeSuccess = false;
				++writeFilesTimer.count;
			}, AsView(TC("WriteReceivedFiles")));
		--writeFilesTimer.count;
		return writeSuccess;
	}

	void SessionServer::SkipReceivedFiles(Vector<ReceivedFile>& receivedFiles, const ConnectionInfo& connectionInfo)
	{
		u32 clientId = connectionInfo.GetId();
		for (auto& file : receivedFiles)
			m_storage.SkipWrite(file.casKey, clientId);
	}

	void SessionServer::ReleaseReceivedFiles(Vector<ReceivedFile>& receivedFiles)
	{
		for (auto& file : receivedFiles)
			if (file.casRefcounted)
				m_storage.ReleaseRef(file.casKey, true);
	}

	bool SessionServer::HandleSendFileToServer(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 clientId = connectionInfo.GetId();

		ReceivedFile file;
		file.processId = reader.ReadU32();
		file.destination = reader.ReadString();
		file.destinationKey = reader.ReadStringKey();
		file.attributes = reader.ReadU32();
		file.casRefcounted = false;
		CasKey casKey = reader.ReadCasKey();

		Storage::RetrieveResult res;
		bool success = m_storage.RetrieveCasFile(res, casKey, file.destinationKey, file.destination.c_str(), nullptr, false, 1, true, clientId);
		auto writeResponse = MakeGuard([&]() { writer.WriteBool(success); });

		if (!success)
		{
			auto logType = connectionInfo.ShouldDisconnect() ? LogEntryType_Info : LogEntryType_Warning;
			m_logger.Logf(logType, TC("Failed to retrieve cas for %s from client %u (Needed to write %s)"), CasKeyString(casKey).str, clientId, file.destination.c_str());
			return true;
		}
		file.casKey = res.casKey;

		if (!file.processId)
		{
			success = WriteReceivedFile(file, nullptr, connectionInfo);
			return true;
		}

		ProcessHandle h = GetProcess(file.processId);
		if (!h.IsValid())
		{
			m_logger.Info(TC("Failed to find process for id %u when receiving SendFileToServer message for %s"), file.processId, file.destination.c_str());
			success = false;
			return false;
		}

		if (!m_storage.AddRef(res.casKey))
		{
			m_logger.Info(TC("Failed to add ref count to caskey %s (%s)"), CasKeyString(casKey).str, file.destination.c_str());
			success = false;
			return true;
		}

		file.casRefcounted = true;

		auto& process = *(RemoteProcess*)h.m_process;

		SCOPED_FUTEX(process.m_receivedFilesLock, lock);
		process.m_receivedFiles.push_back(file);
		return true;
	}

	bool SessionServer::HandleDeleteFile(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 processId = reader.ReadU32();(void)processId;
		StringBuffer<MaxPath> fileName;
		reader.ReadString(fileName);
		bool res = uba::DeleteFileW(fileName.data);
		if (res)
		{
			StringKey fileNameKey = CaseInsensitiveFs ? ToStringKeyLower(fileName) : ToStringKey(fileName);
			RegisterDeleteFile(fileNameKey, fileName);
		}
		else if (GetLastError() == ERROR_FILE_NOT_FOUND || GetLastError() == ERROR_PATH_NOT_FOUND)
			res = true;
		writer.WriteBool(res);
		return true;
	}

	bool SessionServer::HandleCopyFile(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		// TODO: This should be removed and replaced with overlay
		StringKey fromNameKey = reader.ReadStringKey(); (void)fromNameKey;
		StringBuffer<MaxPath> fromName;
		reader.ReadString(fromName);
		StringKey toNameKey = reader.ReadStringKey();
		StringBuffer<MaxPath> toName;
		reader.ReadString(toName);
		bool result = uba::CopyFileW(fromName.data, toName.data, false);
		u32 errorCode = GetLastError();
		if (result)
			RegisterCreateFileForWrite(toNameKey, toName, 0, 0, 0, true, true);
		writer.WriteU32(errorCode);
		return true;
	}

	bool SessionServer::HandleSymlink(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		#if PLATFORM_WINDOWS
		return false;
		#else
		u32 processId = reader.ReadU32();(void)processId;
		StringBuffer<MaxPath> symlinkFile;
		StringBuffer<MaxPath> symlinkContent;
		reader.ReadString(symlinkFile);
		reader.ReadString(symlinkContent);
		bool result = symlink(symlinkContent.data, symlinkFile.data) == 0;
		if (result)
		{
			StringKey symlinkKey = CaseInsensitiveFs ? ToStringKeyLower(symlinkFile) : ToStringKey(symlinkFile);
			RegisterCreateFileForWrite(symlinkKey, symlinkFile, 0, 0, 0, true, true);
		}
		writer.WriteBool(result);
		return true;
		#endif
	}

	bool SessionServer::HandleCreateDirectory(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 processId = reader.ReadU32();(void)processId;
		StringBuffer<> createdDir;
		reader.ReadString(createdDir);
		bool res = uba::CreateDirectoryW(createdDir.data);
		if (res)
		{
			StringKey createdDirKey = CaseInsensitiveFs ? ToStringKeyLower(createdDir) : ToStringKey(createdDir);
			StringKey dirKey;
			const tchar* lastSlash;
			StringBuffer<> dirName;
			GetDirKey(dirKey, dirName, lastSlash, createdDir);
			RegisterCreateFileForWrite(createdDirKey, createdDir, 0, 0, DefaultDirAttributes(), true, true);
			WriteDirectoryEntries(dirKey, dirName);
		}
		else if (GetLastError() == ERROR_ALREADY_EXISTS)
			res = true;
		writer.WriteBool(res);
		return true;
	}

	bool SessionServer::HandleRemoveDirectory(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		UBA_ASSERT(false);
		#if 0
		RemoveDirectoryMessage msg;
		reader.ReadString(msg.name);
		RemoveDirectoryResponse response;
		if (!Session::RemoveDirectory(response, msg, nullptr))
			return false;
		writer.WriteBool(response.result);
		writer.WriteU32(response.errorCode);
		#endif
		return true;
	}

	bool SessionServer::HandleMoveFile(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 processId = reader.ReadU32();(void)processId;
		StringBuffer<MaxPath> fromPath;
		StringBuffer<MaxPath> toPath;
		reader.ReadString(fromPath);
		reader.ReadString(toPath);
		bool res = uba::MoveFileExW(fromPath.data, toPath.data, 0);
		if (res)
		{
			StringKey fromPathKey = CaseInsensitiveFs ? ToStringKeyLower(fromPath) : ToStringKey(fromPath);
			StringKey toPathKey = CaseInsensitiveFs ? ToStringKeyLower(toPath) : ToStringKey(toPath);
			RegisterDeleteFile(fromPathKey, fromPath);
			RegisterCreateFileForWrite(toPathKey, toPath, 0, 0, 0, true, true);
		}
		else
			m_logger.Error(TC("Failed to move file from %s to %s"), fromPath.data, toPath.data);
		writer.WriteBool(res);
		return true;
	}

	bool SessionServer::HandleListDirectory(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 sessionId = reader.ReadU32();
		u32 sessionIndex = sessionId - 1;
		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, lock);
		if (sessionIndex >= m_clientSessions.size())
			return m_logger.Error(TC("Got ListDirectory message from connection using bad sessionid (%u/%llu)"), sessionIndex, m_clientSessions.size());
		ClientSession& session = *m_clientSessions[sessionIndex];
		lock.Leave();

		StringBuffer<> dirName;
		reader.ReadString(dirName);
		StringKey dirKey = reader.ReadStringKey();
		ListDirectoryResponse out;
		GetListDirectoryInfo(out, dirName, dirKey, nullptr);
		writer.WriteU32(out.tableOffset.internal);
		WriteDirectoryTable(session, reader, writer);
		return true;
	}

	bool SessionServer::HandleGetDirectoriesFromServer(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 sessionId = reader.ReadU32();
		u32 sessionIndex = sessionId - 1;
		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, lock);
		if (sessionIndex >= m_clientSessions.size())
			return m_logger.Error(TC("Got GetDirectories message from connection using bad sessionid (%u/%llu)"), sessionIndex, m_clientSessions.size());
		ClientSession& session = *m_clientSessions[sessionIndex];
		lock.Leave();
		WriteDirectoryTable(session, reader, writer);
		return true;
	}

	bool SessionServer::HandleGetNameToHashFromServer(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 requestedSize = reader.ReadU32();

		SCOPED_FUTEX_READ(m_nameToHashLookupLock, lock);
		if (requestedSize == ~0u)
		{
			requestedSize = u32(m_nameToHashTableMem.writtenSize);
			writer.WriteU32(requestedSize);
		}
		writer.WriteU64(GetTime());
		lock.Leave();

		WriteNameToHashTable(reader, writer, requestedSize);
		return true;
	}

	bool SessionServer::HandleProcessAvailable(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 sessionId = reader.ReadU32();
		u32 sessionIndex = sessionId - 1;

		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, sessionsLock);
		if (sessionIndex >= m_clientSessions.size())
			return m_logger.Error(TC("Got ProcessAvailable message from connection using bad sessionid (%u/%llu)"), sessionIndex, m_clientSessions.size());
		ClientSession& session = *m_clientSessions[sessionIndex];
		auto processesLostRace = std::move(session.processesLostRace);
		sessionsLock.Leave();

		bool isClientCrossPlatform = IsLinux != session.isLinux;
		bool isClientCrossArchitecture = IsArmBinary != session.isArm;

		u32 weight32 = reader.ReadU32();
		float availableWeight = *(float*)&weight32;

		writer.Write7BitEncoded(processesLostRace.size());
		for (auto& lostProcess : processesLostRace)
			writer.WriteU32(lostProcess);

		if ((!m_remoteExecutionEnabled || !session.remoteExecutionEnabled) && m_queuedRemoteProcesses.empty())
			availableWeight = 0.0f;

		Vector<RemoteProcess*> processesWithKnownInputsToSend;

		float weightLeft = availableWeight;
		u32 addCount = 0;

		if (m_useFillLock)
			m_fillUpOneAtTheTimeLock.Enter();
		auto exitFileLock = MakeGuard([&]() { if (m_useFillLock) m_fillUpOneAtTheTimeLock.Leave(); });

		while (weightLeft > 0)
		{
			RemoteProcess* process = DequeueProcess(session, sessionId, connectionInfo.GetId());
			if (process == nullptr)
				break;

			auto returnProcess = [&]()
				{
					m_queuedRemoteProcesses.push_front({process});
					SCOPED_READ_LOCK(m_remoteProcessReturnedEventLock, lock2);
					if (m_remoteProcessReturnedEvent)
						m_remoteProcessReturnedEvent(*process);
				};

			auto& startInfo = process->m_startInfo;

			if (!IsAbsolutePath(startInfo.application) && !Equals(startInfo.application, TC("speedtest")))
			{
				StringBuffer<> temp;
				if (!m_searchPathCache.SearchPathForFile(m_logger, temp, startInfo.applicationStr, startInfo.workingDirStr, {}))
				{
					// TODO: This should be seen as a failed process
					m_logger.Error(TC("BAD PATH TO APPLICATION %s"), startInfo.application);
					returnProcess();
					continue;
				}
				startInfo.applicationStr = temp.ToString();
				startInfo.application = startInfo.applicationStr.data();
			}

			StringBuffer<> applicationOverride;

			if (isClientCrossArchitecture || isClientCrossPlatform)
			{
				if ((!process->m_canCrossArchitecture && isClientCrossArchitecture) || (!process->m_canCrossPlatform && isClientCrossPlatform))
				{
					// There is a risk cross architecture client dequeues another client's process that is not cross architecture
					// and in that case we just have to return it and try again later.. (break out)
					returnProcess();
					break;
				}

				for (auto& mapping : m_crossArchitectureMappings)
					if (StartsWith(startInfo.application, mapping.from.c_str()))
					{
						applicationOverride.Append(mapping.to).Append(startInfo.application + mapping.from.size());
						break;
					}

				if (applicationOverride.count)
				{
					//m_logger.Info(TC("Couldn't find cross architecture mapping for %s"), startInfo.application);
					//returnProcess();
					//break;
					if (!FileExists(m_logger, applicationOverride.data))
					{
						m_logger.Info(TC("Couldn't find cross architecture executable %s"), applicationOverride.data);
						returnProcess();
						break;
					}
				}

				#if UBA_TEST_LINUX_HELPING_WINDOWS // PLATFORM_WINDOWS
				applicationOverride.Clear().Append(TCV("/usr/bin/gcc"));
				startInfo.workingDirStr = TC("/usr/bin");
				startInfo.workingDir = startInfo.workingDirStr.c_str();
				#endif
			}

			ProcessAdded(*process, sessionId);
			writer.WriteU32(process->m_processId);
			startInfo.Write(writer, applicationOverride);

			if (!startInfo.overlayFilesStr.empty())
			{
				Vector<u8> data;
				data.resize(16*1024);
				DirectoryTableOverlay overlay;
				overlay.committed = u32(data.size());
				overlay.memory = data.data();
				if (!PopulateOverlayFiles(overlay, startInfo.overlayFilesStr))
					m_logger.Error(TC("Failed to populate overlay for process %u (%s) — overlay files: %s"), process->m_processId, startInfo.application, startInfo.overlayFilesStr.c_str());
				writer.Write7BitEncoded(overlay.size);
				writer.WriteBytes(data.data(), overlay.size);
			}
			else
				writer.Write7BitEncoded(0);

			if (process->m_knownInputsDone.IsCreated())
				processesWithKnownInputsToSend.push_back(process);

			++addCount;

			if (writer.GetCapacityLeft() < 5000) // Arbitrary number to cover all parameters above
				break;

			weightLeft -= startInfo.weight;
		}
		exitFileLock.Execute();

		u32 neededDirectoryTableSize = GetDirectoryTableSize(nullptr).main;
		u32 neededHashTableSize;
		{
			SCOPED_FUTEX_READ(m_nameToHashLookupLock, l);
			neededHashTableSize = u32(m_nameToHashTableMem.writtenSize);
		}

		sessionsLock.Enter();
		//if (addCount)
		//	m_logger.Debug(TC("Gave %u processes to %s using up %.1f weight out of %.1f available"), addCount, session.name.c_str(), availableWeight - weightLeft, availableWeight);

		bool remoteExecutionEnabled = (m_remoteExecutionEnabled && session.remoteExecutionEnabled) || !m_queuedRemoteProcesses.empty();
		if (!remoteExecutionEnabled)
		{
			if (session.enabled)
				m_availableRemoteSlotCount -= session.processSlotCount - session.usedSlotCount;
			session.enabled = false;
			m_logger.Detail(TC("Disable remote execution on %s because remote execution has been disabled and queue is empty (will finish %u processes)"), session.name.c_str(), session.usedSlotCount);
		}

		// If this client session has 0 active processes and m_maxRemoteProcessCount < total available compute - client session, then we can disconnect this client
		if (remoteExecutionEnabled && !addCount && m_maxRemoteProcessCount != ~0u)
		{
			if (!session.dedicated && !session.usedSlotCount)
			{
				if (m_maxRemoteProcessCount < m_availableRemoteSlotCount - session.processSlotCount)
				{
					if (session.enabled)
						m_availableRemoteSlotCount -= session.processSlotCount - session.usedSlotCount;
					session.enabled = false;
					remoteExecutionEnabled = false;
					m_logger.Info(TC("Disable remote execution on %s because host session has enough help (%u left and %u remote slots)"), session.name.c_str(), m_maxRemoteProcessCount.load(), m_availableRemoteSlotCount);
				}
			}
		}
		sessionsLock.Leave();

		writer.WriteU32(remoteExecutionEnabled ? SessionProcessAvailableResponse_None : SessionProcessAvailableResponse_RemoteExecutionDisabled);

				
		// Write in the needed dir and hash table offset to be up-to-date (to potentially avoid additional messages from client
		writer.WriteU32(neededDirectoryTableSize);
		writer.WriteU32(neededHashTableSize);


		// Collect known inputs
		Vector<RemoteProcess::KnownInput*> knownInputsToSend;
		for (auto process : processesWithKnownInputsToSend)
			if (process->m_knownInputsDone.IsSet(50*1000))
				for (auto kiIt = process->m_knownInputs, kiEnd = kiIt + process->m_knownInputsCount; kiIt!=kiEnd; ++kiIt)
					if (session.sentKeys.insert(kiIt->key).second)
						knownInputsToSend.push_back(kiIt);

		// Send caskeys of known inputs so client can start retrieving them straight away
		u32 kiCapacity = u32(writer.GetCapacityLeft() - sizeof(u32)) / sizeof(RemoteProcess::KnownInput);
		u32 toSendCount = Min(kiCapacity, u32(knownInputsToSend.size()));
		writer.WriteU32(toSendCount);
		for (auto kv : knownInputsToSend)
		{
			if (!toSendCount--)
				break;
			writer.WriteStringKey(kv->file);
			writer.WriteCasKey(kv->key);
			writer.WriteU32(kv->mappingAlignment);
			writer.WriteBool(kv->allowProxy);
		}
		return true;
	}

	bool SessionServer::HandleProcessInputs(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 processId = u32(reader.Read7BitEncoded());
		ProcessHandle h(GetProcess(processId));
		if (!h.IsValid())
		{
			m_logger.Info(TC("Failed to find process for id %u when receiving custom message"), processId);
			return false;
		}
		auto& process = *(RemoteProcess*)h.m_process;
		auto& inputs = process.m_trackedInputs;
		u64 inputsSize = reader.Read7BitEncoded();
		u64 prevSize = inputs.size();
		inputs.resize(prevSize + inputsSize);
		reader.ReadBytesCompressed(inputs.data() + prevSize, inputsSize);
		return true;
	}

	bool SessionServer::HandleProcessFinished(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 processId = reader.ReadU32();

		ProcessHandle h = ProcessRemoved(processId);
		if (!h.m_process)
		{
			m_logger.Info(TC("Client finished process with id %u that is not found on server"), processId);
			return false;
		}
		auto& process = *(RemoteProcess*)h.m_process;

		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, cs2);

		ProcessHandle racingProcess = process.m_racingProcess;
		bool hasRacingProcess = racingProcess.IsValid();
		bool useRacingProcess = false;
		if (hasRacingProcess)
			useRacingProcess = !racingProcess.Cancel(); // If cancel fails it means that racing process has started writing files and we have to let it finish

		if (!m_activeRemoteProcesses.erase(&process))
		{
			cs2.Leave();
			m_logger.Info(TC("Got finished process but process was not in active remote processes. Was there a disconnect happening directly after but executed before?"));
			return false;
		}
		u32 sessionIndex = process.m_sessionId - 1;
		if (sessionIndex >= m_clientSessions.size())
			return m_logger.Error(TC("Got ProcessFinished message from connection using bad sessionid (%u/%llu)"), sessionIndex, m_clientSessions.size());
		auto& session = *m_clientSessions[sessionIndex];
		++m_finishedRemoteProcessCount;
		--session.usedSlotCount;
		if (session.enabled)
			++m_availableRemoteSlotCount;
		process.m_clientId = ~0u;
		auto receivedFiles = std::move(process.m_receivedFiles);
		cs2.Leave();

		Timer writeFilesTimer;

		u32 exitCode = reader.ReadU32();
		u32 logLineCount = reader.ReadU32();
		Vector<ProcessLogLine> logLines;
		logLines.reserve(logLineCount);
		while (logLineCount-- != 0)
		{
			TString text = reader.ReadString();
			LogEntryType type = LogEntryType(reader.ReadByte());
			logLines.push_back({ std::move(text), type });
		}

		process.m_logLines = std::move(logLines);

		bool filesWritten = false;
		if (!useRacingProcess)
		{
			auto& startInfo = process.m_startInfo;

			if (auto func = startInfo.logLineFunc)
				for (auto& line : process.m_logLines)
					func(startInfo.logLineUserData, line.text.c_str(), u32(line.text.size()), line.type);
			if (startInfo.writeOutputFilesOnFail || startInfo.rules->IsExitCodeSuccess(exitCode))
			{
				filesWritten = true;
				if (!WriteReceivedFiles(receivedFiles, process, writeFilesTimer, connectionInfo)) // Reduce refcount
					exitCode = UBA_EXIT_CODE(20);
			}
		}
		else
		{
			exitCode = ProcessCancelExitCode;
		}

		if (!filesWritten)
			SkipReceivedFiles(receivedFiles, connectionInfo); // We need to inform storage server that we don't need these anymore (to reduce ref count for active casentries)

		u32 id = process.m_processId;
		Vector<ProcessLogLine> emptyLines;
		auto& tracedLogLines = (exitCode != 0 || m_detailedTrace) ? process.m_logLines : emptyLines;
		m_trace.ProcessExited(id, exitCode, reader.GetPositionData(), reader.GetLeft(), tracedLogLines);

		ProcessStats processStats;
		processStats.Read(reader, ~0u);

		processStats.writeFiles = writeFilesTimer; // The writeFiles stats from client session is not really useful since it is a noop
		process.m_processorTime = processStats.cpuTime;
		process.m_wallTime = processStats.wallTime;
		process.m_peakMemory = processStats.peakJobMemory > 0 ? processStats.peakJobMemory : processStats.peakMemory;
		process.m_server = nullptr;

		// If not using the racing process we want it to call exit function with cancel first before we call the remote process exit function
		if (hasRacingProcess)
			if (!racingProcess.WaitForExit(60*60*1000))
				m_logger.Error(TC("Timed out waiting for cancelled racing process!"));

		process.m_exitCode = exitCode;
		process.m_done.Set();
		process.CallProcessExit(h);

		ReleaseReceivedFiles(receivedFiles);

		return true;
	}

	bool SessionServer::HandleProcessReturned(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 processId = reader.ReadU32();
		StringBuffer<> reason;
		reader.ReadString(reason);

		ProcessHandle h = ProcessRemoved(processId);
		RemoteProcess* process = (RemoteProcess*)h.m_process;
		if (!process)
		{
			m_logger.Warning(TC("Client %s returned process %u that is not found on server (%s)"), GuidToString(connectionInfo.GetUid()).str, processId, reason.data);
			return true;
		}

		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, cs2);
		if (!m_activeRemoteProcesses.erase(process))
		{
			cs2.Leave();
			m_logger.Warning(TC("Got returned process %u from client %s but process was not in active remote processes. Was there a disconnect happening directly after but executed before?"), processId, GuidToString(connectionInfo.GetUid()).str);
			return true;
		}
		u32 sessionIndex = process->m_sessionId - 1;
		if (sessionIndex >= m_clientSessions.size())
			return m_logger.Error(TC("Got ProcessReturned message from connection using bad sessionid (%u/%llu)"), sessionIndex, m_clientSessions.size());
		auto& session = *m_clientSessions[sessionIndex];
		--session.usedSlotCount;
		if (session.enabled)
			++m_availableRemoteSlotCount;

		if (process->m_startInfo.applicationStr == TC("speedtest"))
		{
			m_trace.ProcessReturned(process->m_processId, TCV("speedtest"));
			process->m_exitCode = 0;
			process->m_server = nullptr;
			process->m_done.Set();
			process->CallProcessExit(h);
			return true;
		}

		m_logger.Detail(TC("Client %s returned process %u to queue (%s)"), session.name.c_str(), processId, reason.data);
		++m_returnedRemoteProcessCount;

		process->m_executingHost.clear();
		process->m_clientId = ~0u;
		process->m_sessionId = 0;

		m_trace.ProcessReturned(process->m_processId, reason);
		m_queuedRemoteProcesses.push_front(h);

		SCOPED_READ_LOCK(m_remoteProcessReturnedEventLock, lock2);
		if (m_remoteProcessReturnedEvent)
			m_remoteProcessReturnedEvent(*process);
		return true;
	}

	bool SessionServer::HandleGetRoots(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		RootsHandle rootsHandle = reader.ReadU64();
		auto rootsEntry = GetRootsEntry(rootsHandle);
		if (!rootsEntry)
			return false;
		writer.WriteBytes(rootsEntry->memory.data(), rootsEntry->memory.size());
		return true;
	}

	bool SessionServer::HandleVirtualAllocFailed(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		m_logger.Error(TC("VIRTUAL ALLOC FAILING ON REMOTE MACHINE %s !"), GuidToString(connectionInfo.GetUid()).str);
		return true;
	}

	bool SessionServer::HandleGetTraceInformation(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		// This code is deprecated and moved to UbaNetworkServer. Trace is now a system message
		u32 remotePos = reader.ReadU32();
		u32 localPos = m_trace.GetPublishedSize();

		writer.WriteU32(localPos);
		u32 toWrite = Min(localPos - remotePos, u32(writer.GetCapacityLeft()));
		writer.WriteBytes(m_trace.m_memoryBegin + remotePos, toWrite);
		return true;
	}

	bool SessionServer::HandlePing(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		LOG_STALL_SCOPE(m_logger, 5, TC("HandlePing took more than %s"));
		
		u32 sessionId = reader.ReadU32();
		u64 lastPing = reader.ReadU64();
		u64 memAvail = reader.ReadU64();
		u64 memTotal = reader.ReadU64();
		u32 cpuLoadValue = reader.ReadU32();

		u64 pingTime = GetTime();
		u32 sessionIndex = sessionId - 1;
		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, lock);
		if (sessionIndex >= m_clientSessions.size())
			return m_logger.Warning(TC("Got PingMessage from connection %u using bad sessionid (%u/%llu)"), connectionInfo.GetId(), sessionIndex, m_clientSessions.size());
		auto& session = *m_clientSessions[sessionIndex];
		session.pingTime = pingTime;
		session.lastPing = lastPing;
		session.memAvail = memAvail;
		session.memTotal = memTotal;
		session.cpuLoad = *(float*)&cpuLoadValue;
		writer.WriteBool(session.abort);
		writer.WriteBool(session.crashdump);
		session.crashdump = false;

		return true;
	}

	bool SessionServer::HandleNotification(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 sessionId = reader.ReadU32();

		u32 sessionIndex = sessionId - 1;
		{
			SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, lock);
			if (sessionIndex < m_clientSessions.size())
				m_clientSessions[sessionIndex]->hasNotification = true;
		}

		StringBuffer<1024> str;
		reader.ReadString(str);
		m_trace.SessionNotification(sessionId, str.data);
		return true;
	}

	bool SessionServer::HandleGetNextProcess(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		bool isFirstGet = reader.ReadBool();

		// Trace stream does not handle empty
		StackBinaryWriter<1024> statsWriter;
		if (!isFirstGet)
		{
			ProcessStats().Write(statsWriter);
			SessionStats().Write(statsWriter);
			StorageStats().Write(statsWriter);
			KernelStats().Write(statsWriter);
		}

		do
		{
			u32 processId = reader.ReadU32();
			u32 prevExitCode = 0; // Idle exit code is always success
			if (isFirstGet)
				prevExitCode = reader.ReadU32();

			ProcessHandle h(GetProcess(processId));
			if (!h.IsValid())
			{
				m_logger.Info(TC("Failed to find process for id %u when receiving GetNextProcess message"), processId);
				return false;
			}

			auto& remoteProcess = *(RemoteProcess*)h.m_process;
			auto receivedFiles = std::move(remoteProcess.m_receivedFiles);
			remoteProcess.m_writtenFiles.clear();

			Timer writeFilesTimer; // TODO: This should be added somewhere
		
			auto& startInfo = remoteProcess.m_startInfo; // This is kind of wrong with processor reuse but we know the rules is the same
			if (startInfo.writeOutputFilesOnFail || startInfo.rules->IsExitCodeSuccess(prevExitCode))
			{
				if (!WriteReceivedFiles(receivedFiles, remoteProcess, writeFilesTimer, connectionInfo))
					prevExitCode = UBA_EXIT_CODE(20);
			}
			else
				SkipReceivedFiles(receivedFiles, connectionInfo);
			//processStats.writeFiles = writeFilesTimer

			BinaryReader prevStats = isFirstGet ? reader : BinaryReader(statsWriter.GetData(), 0, statsWriter.GetPosition());


			SCOPED_FUTEX(remoteProcess.m_exitedLock, exitedLock);
			NextProcessInfo nextProcess;
			bool newProcess;
			bool shouldExit;
			u32 timeoutMs = 0;
			remoteProcess.m_exitCode = prevExitCode;
			remoteProcess.m_done.Set();
			bool success = GetNextProcess(remoteProcess, newProcess, shouldExit, nextProcess, prevExitCode, prevStats, timeoutMs);
			remoteProcess.m_exitCode = ~0u;
			remoteProcess.m_done.Reset();
			exitedLock.Leave();

			ReleaseReceivedFiles(receivedFiles);

			if (!success)
				return false;

			writer.WriteBool(newProcess);
			writer.WriteBool(shouldExit);
			if (newProcess)
			{
				writer.WriteString(nextProcess.arguments);
				writer.WriteString(nextProcess.workingDir);
				writer.WriteString(nextProcess.description);
				writer.WriteString(nextProcess.logFile);
				writer.Write7BitEncoded(nextProcess.overlayFiles.size());
				writer.WriteBytes(nextProcess.overlayFiles.data(), nextProcess.overlayFiles.size());
				remoteProcess.m_isIdle = false;
			}
			else if (!remoteProcess.m_isIdle)
			{
				// Stats are the previous call's stats
				m_trace.ProcessEnvironmentUpdated(processId, TCV("IDLE"), prevExitCode, prevStats.GetPositionData(), prevStats.GetLeft(), TCV(""));
				remoteProcess.m_isIdle = true;
			}
		}
		while (!isFirstGet && reader.GetLeft());

		return true;
	}

	bool SessionServer::HandleFlushProcess(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 processId = reader.ReadU32();
		ProcessHandle h(GetProcess(processId));
		if (!h.IsValid())
		{
			m_logger.Info(TC("Failed to find process for id %u when receiving GetNextProcess message"), processId);
			return false;
		}

		Timer writeFilesTimer; // TODO: This should be added somewhere

		auto& remoteProcess = *(RemoteProcess*)h.m_process;
		auto receivedFiles = std::move(remoteProcess.m_receivedFiles);
		if (!WriteReceivedFiles(receivedFiles, remoteProcess, writeFilesTimer, connectionInfo))
			return false;

		return true;
	}

	bool SessionServer::HandleCustom(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 processId = reader.ReadU32();
		ProcessHandle h(GetProcess(processId));
		if (!h.IsValid())
		{
			m_logger.Info(TC("Failed to find process for id %u when receiving custom message"), processId);
			return false;
		}
		auto& remoteProcess = *(RemoteProcess*)h.m_process;
		SCOPED_FUTEX(remoteProcess.m_exitedLock, exitedLock);
		CustomMessage(remoteProcess, reader, writer);
		return true;
	}

	bool SessionServer::HandleUpdateEnvironment(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 processId = reader.ReadU32();
		ProcessHandle h(GetProcess(processId));
		if (!h.IsValid())
		{
			m_logger.Info(TC("Failed to find process for id %u when receiving update environment message"), processId);
			return false;
		}
		StringBuffer<> reason;
		reader.ReadString(reason);
		m_trace.ProcessEnvironmentUpdated(processId, reason, 0, reader.GetPositionData(), reader.GetLeft(), ToView(h.GetStartInfo().breadcrumbs));
		return true;
	}

	bool SessionServer::HandleSummary(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		u32 sessionId = reader.ReadU32();
		m_trace.SessionSummary(sessionId, reader.GetPositionData(), reader.GetLeft());
		return true;
	}

	bool SessionServer::HandleCommand(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		StringBuffer<128> command;
		reader.ReadString(command);

		auto WriteString = [&](const tchar* str, LogEntryType type = LogEntryType_Info) { writer.WriteByte(type); writer.WriteString(str); };

		if (command.Equals(TCV("status")))
		{
			u32 totalUsed = 0;
			u32 totalSlots = 0;
			SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, queueLock);
			u64 time = GetTime();
			for (auto& s : m_clientSessions)
			{
				if (!s->enabled)
					continue;
				WriteString(StringBuffer<>().Appendf(TC("Session %u (%s)"), s->clientId, s->name.c_str()).data);
				WriteString(StringBuffer<>().Appendf(TC("   Process slots used %u/%u"), s->usedSlotCount, s->processSlotCount).data);
				if (s->pingTime)
					WriteString(StringBuffer<>().Appendf(TC("   Last ping %s ago"), TimeToText(time - s->pingTime).str).data);
				totalUsed += s->usedSlotCount;
				totalSlots += s->processSlotCount;
			}
			WriteString(StringBuffer<>().Appendf(TC("Total remote slots used %u/%u"), totalUsed, totalSlots).data);
		}
		if (command.Equals(TCV("crashdump")))
		{
			WriteString(TC("Requesting crashdumps from all remotes on next ping"));
			SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, queueLock);
			for (auto& s : m_clientSessions)
				s->crashdump = true;
		}
		else if (command.StartsWith(TC("abort")))
		{
			bool abortWithProxy = command.Equals(TCV("abortproxy"));
			bool abortUseProxy = command.Equals(TCV("abortnonproxy"));
			if (!abortWithProxy && !abortUseProxy)
			{
				abortWithProxy = true;
				abortUseProxy = true;
			}
			SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, queueLock);
			u32 abortCount = 0;
			for (auto& s : m_clientSessions)
			{
				if (!s->enabled || s->abort)
					continue;
				bool hasProxy = m_storage.HasProxy(s->clientId);
				if (abortWithProxy && hasProxy)
					s->abort = true;
				else if (abortUseProxy && !hasProxy)
					s->abort = true;
				if (s->abort)
					++abortCount;
			}
			WriteString(StringBuffer<>().Appendf(TC("Aborting: %u remote sessions"), abortCount).data);
		}
		else if (command.Equals(TCV("disableremote")))
		{
			DisableRemoteExecution();
			WriteString(StringBuffer<>().Appendf(TC("Remote execution is disabled")).data);
		}
		else
		{
			WriteString(StringBuffer<>().Appendf(TC("Unknown command: %s"), command.data).data, LogEntryType_Error);
		}
		writer.WriteByte(255);
		return true;
	}

	bool SessionServer::HandleSHGetKnownFolderPath(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
#if PLATFORM_WINDOWS
		GUID kfid;
		reader.ReadBytes(&kfid, sizeof(GUID));
		u32 flags = reader.ReadU32();
		PWSTR str;

		static HMODULE moduleHandle = LoadLibrary(L"Shell32.dll");
		using SHGetKnownFolderPathFunc = HRESULT(const GUID& rfid, DWORD dwFlags, HANDLE hToken, PWSTR* ppszPath);
		static SHGetKnownFolderPathFunc* SHGetKnownFolderPath = (SHGetKnownFolderPathFunc*)(void*)GetProcAddress(moduleHandle, "SHGetKnownFolderPath");;
		HRESULT res = SHGetKnownFolderPath(kfid, flags, NULL, &str);
		writer.WriteU32(res);
		if (res == S_OK)
		{
			writer.WriteString(str);
			CoTaskMemFree(str);
		}
#endif
		return true;
	}

	bool SessionServer::StoreCasFile(CasKey& out, const StringKey& fileNameKey, const tchar* fileName)
	{
		CasKey casKeyOverride = CasKeyZero;

		bool deferCreation = true;
		if (m_customCasKeysEnabled)
		{
			SCOPED_FUTEX(m_customCasKeysLock, lock);
			auto findIt = m_customCasKeys.find(fileNameKey);
			if (findIt != m_customCasKeys.end())
			{
				CustomCasKey& customKey = findIt->second;
				if (customKey.casKey == CasKeyZero)
				{
					if (!GetCasKeyFromTrackedInputs(customKey.casKey, fileName, customKey.workingDir.c_str(), customKey.trackedInputs.data(), u32(customKey.trackedInputs.size())))
						return false;
					UBA_ASSERTF(customKey.casKey != CasKeyZero, TC("This should never happen!!"));
					//m_logger.Debug(TC("Calculated custom key: %s (%s)"), GuidToString(customKey.casKey).str, fileName);
				}
				casKeyOverride = customKey.casKey;
			}
		}

		if (!m_storage.StoreCasFile(out, fileName, casKeyOverride, deferCreation)) // We can defer the creation of the cas file since client might already have it
			return false;
		return true;//out != CasKeyZero;
	}

	bool SessionServer::WriteDirectoryTable(ClientSession& session, BinaryReader& reader, BinaryWriter& writer)
	{
		auto& dirTable = m_directoryTable;

		SCOPED_FUTEX(session.dirTablePosLock, lock2);

		//m_logger.Info(TC("WritePos: %llu"), session.dirTablePos);
		writer.WriteU32(session.dirTablePos); // We can figure out on the other side if everything was written based on if the message is full or not.

		u32 toSend = GetDirectoryTableSize(nullptr).main - session.dirTablePos;
		if (toSend == 0)
			return true;

		u32 capacityLeft = u32(writer.GetCapacityLeft());
		if (capacityLeft < toSend)
			toSend = capacityLeft;

		writer.WriteBytes(dirTable.m_memory + session.dirTablePos, toSend);

		session.dirTablePos += toSend;
		return true;
	}

	bool SessionServer::WriteNameToHashTable(BinaryReader& reader, BinaryWriter& writer, u32 requestedSize)
	{
		u32 remoteTableSize = reader.ReadU32();
				
		u32 toSend = requestedSize - remoteTableSize;
		if (toSend == 0)
			return true;

		u32 capacityLeft = u32(writer.GetCapacityLeft());
		if (capacityLeft < toSend)
			toSend = capacityLeft;

		writer.WriteBytes(m_nameToHashTableMem.memory + remoteTableSize, toSend);
		return true;
	}

	SessionServer::RemoteProcess* SessionServer::DequeueProcess(ClientSession& session, u32 sessionId, u32 clientId)
	{
		//TrackWorkScope tws(m_trace, AsView(TC("DequeueProcess")), ColorWork);
		SCOPED_READ_LOCK(m_remoteProcessSlotAvailableEventLock, lock);
		bool hasCalledCallback = !m_remoteProcessSlotAvailableEvent;

		while (true)
		{
			SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, queueLock);

			if (!session.connected) // This should not be possible
			{
				m_logger.Warning(TC("Dequeing process to session that is not connected. This should never happen. Report to Epic (%u)"), clientId);
				return nullptr;
			}

			while (!m_queuedRemoteProcesses.empty())
			{
				auto processHandle = m_queuedRemoteProcesses.front();
				auto process = (RemoteProcess*)processHandle.m_process;
				m_queuedRemoteProcesses.pop_front();
				if (process->m_cancelled)
					continue;
				
				if (session.enabled)
					--m_availableRemoteSlotCount;
				++session.usedSlotCount;

				process->m_clientId = clientId;
				process->m_sessionId = sessionId;
				process->m_executingHost = session.name;
				process->m_startTime = GetTime();
				UBA_ASSERT(!process->m_cancelled);
				m_activeRemoteProcesses.insert(process);
				return process;
			}
			queueLock.Leave();

			if (hasCalledCallback)
				return nullptr;

			m_remoteProcessSlotAvailableEvent(IsArmBinary != session.isArm, IsLinux != session.isLinux);
			hasCalledCallback = true;
		}
	}

	void SessionServer::OnCancelled(RemoteProcess* process)
	{
		ProcessHandle h(process);

		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, queueLock);
		process->m_server = nullptr;

		if (process->m_clientId == ~0u)
		{
			for (auto it=m_queuedRemoteProcesses.begin(); it!=m_queuedRemoteProcesses.end(); ++it)
			{
				if (it->m_process != process)
					continue;
				m_queuedRemoteProcesses.erase(it);
				break;
			}
		}
		else
		{
			u32 sessionIndex = process->m_sessionId - 1;
			UBA_ASSERT(sessionIndex < m_clientSessions.size());
			ClientSession& session = *m_clientSessions[sessionIndex];
			--session.usedSlotCount;
			
			m_activeRemoteProcesses.erase(process);

			{
				SCOPED_FUTEX(m_processesLock, lock);
				m_processes.erase(process->m_processId);
			}

			queueLock.Leave();

			StackBinaryWriter<1024> writer;
			ProcessStats().Write(writer);
			SessionStats().Write(writer);
			StorageStats().Write(writer);
			KernelStats().Write(writer);
			m_trace.ProcessExited(process->m_processId, process->m_exitCode, writer.GetData(), writer.GetPosition(), Vector<ProcessLogLine>());
		}

		auto receivedFiles = std::move(process->m_receivedFiles);

		process->m_done.Set();

		ReleaseReceivedFiles(receivedFiles);
	}

	void SessionServer::OnRaceLost(RemoteProcess& process)
	{
		// We want to tell client to remove this process
		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, queueLock);
		if (process.m_sessionId == 0) // Was disconnected
			return;
		u32 sessionIndex = process.m_sessionId - 1;
		UBA_ASSERT(sessionIndex < m_clientSessions.size());
		ClientSession& session = *m_clientSessions[sessionIndex];
		session.processesLostRace.push_back(process.GetId());
	}

	ProcessHandle SessionServer::ProcessRemoved(u32 processId)
	{
		SCOPED_FUTEX(m_processesLock, lock);
		auto findIt = m_processes.find(processId);
		if (findIt == m_processes.end())
			return {};
		ProcessHandle h(findIt->second);
		m_processes.erase(findIt);
		return h;
	}

	ProcessHandle SessionServer::GetProcess(u32 processId)
	{
		SCOPED_FUTEX_READ(m_processesLock, lock);
		auto findIt = m_processes.find(processId);
		if (findIt == m_processes.end())
			return {};
		return ProcessHandle(findIt->second);
	}

	TString SessionServer::GetProcessDescription(u32 processId)
	{
		StringBuffer<512> str;
		SCOPED_FUTEX_READ(m_processesLock, lock);
		auto findIt = m_processes.find(processId);
		if (findIt == m_processes.end())
			return str.Appendf(TC("<Process with id %u not found>"), processId).data;
		return str.Appendf(TC("%s"), findIt->second.GetStartInfo().GetDescription()).data;
	}

	bool SessionServer::ProcessNativeCreated(ProcessImpl& process)
	{
		if (m_nativeProcessCreatedFunc)
			m_nativeProcessCreatedFunc(process);
		return true;
	}

	bool SessionServer::CreateFileForRead(CreateFileResponse& out, TrackWorkScope& tws, StringView fileName, StringKey fileNameKey, ProcessImpl& process, const ApplicationRules& rules)
	{
		if (!m_shouldWriteToDisk)
		{
			SCOPED_READ_LOCK(m_receivedFilesLock, lock);
			auto findIt = m_receivedFiles.find(fileNameKey);
			if (findIt != m_receivedFiles.end())
			{
				u64 memoryMapAlignment = GetMemoryMapAlignment(fileName);
				if (!memoryMapAlignment)
					memoryMapAlignment = 4096;
				MemoryMap map;
				if (!GetOrCreateMemoryMapFromStorage(map, fileNameKey, fileName.data, findIt->second, memoryMapAlignment))
					return false;
				out.directoryTableSize = GetDirectoryTableSize(&process.m_shared.overlay);
				out.mappedFileTableSize = GetFileMappingSize();
				out.fileName.Append(map.name);
				out.size = map.size;
				return true;
			}
		}
		return Session::CreateFileForRead(out, tws, fileName, fileNameKey, process, rules);
	}

	bool SessionServer::GetOutputFileSizeInternal(u64& outSize, const StringKey& fileNameKey, StringView filePath)
	{
		if (Session::GetOutputFileSizeInternal(outSize, fileNameKey, filePath))
			return true;
		SCOPED_READ_LOCK(m_receivedFilesLock, lock);
		auto findIt = m_receivedFiles.find(fileNameKey);
		if (findIt == m_receivedFiles.end())
			return false;
		CasKey casKey = findIt->second;
		lock.Leave();
		return m_storage.GetFileSize(outSize, casKey, filePath.data);
	}

	bool SessionServer::GetOutputFileDataInternal(void* outData, const StringKey& fileNameKey, StringView filePath, bool deleteInternalMapping)
	{
		if (Session::GetOutputFileDataInternal(outData, fileNameKey, filePath, deleteInternalMapping))
			return true;
		SCOPED_READ_LOCK(m_receivedFilesLock, lock);
		auto findIt = m_receivedFiles.find(fileNameKey);
		if (findIt == m_receivedFiles.end())
			return false;
		CasKey casKey = findIt->second;
		lock.Leave();
		if (!m_storage.GetFileData(outData, casKey, filePath.data))
			return false;
		if (!deleteInternalMapping)
			return true;

		SCOPED_WRITE_LOCK(m_receivedFilesLock, lock2);
		auto eraseIt = m_receivedFiles.find(fileNameKey); // Recheck, can't reuse iterator
		if (eraseIt == m_receivedFiles.end())
			return true; // somebody else already erased it
		m_receivedFiles.erase(eraseIt);
		lock2.Leave();

		m_storage.DropCasFile(casKey, true, filePath.data, true);
		return true;
	}

	bool SessionServer::WriteOutputFileInternal(const StringKey& fileNameKey, StringView filePath, bool deleteInternalMapping)
	{
		if (Session::WriteOutputFileInternal(fileNameKey, filePath, deleteInternalMapping))
			return true;

		SCOPED_READ_LOCK(m_receivedFilesLock, lock);
		auto findIt = m_receivedFiles.find(fileNameKey);
		if (findIt == m_receivedFiles.end())
			return false;
		CasKey casKey = findIt->second;
		lock.Leave();

		bool res = m_storage.CopyOrLink(casKey, filePath.data, DefaultAttributes());

		if (!deleteInternalMapping)
			return res;

		SCOPED_WRITE_LOCK(m_receivedFilesLock, lock2);
		auto eraseIt = m_receivedFiles.find(fileNameKey); // Recheck, can't reuse iterator
		if (eraseIt == m_receivedFiles.end())
			return res; // somebody else already erased it
		m_receivedFiles.erase(eraseIt);
		lock2.Leave();

		m_storage.DropCasFile(casKey, true, filePath.data, true);
		return res;
	}

	void SessionServer::FileEntryAdded(StringKey fileNameKey, u64 lastWritten, u64 size)
	{
		SCOPED_FUTEX(m_nameToHashLookupLock, lock);

		if (!m_nameToHashInitialized)
			return;
		
		Storage::CachedFileInfo cachedInfo;
		if (!m_storage.VerifyAndGetCachedFileInfo(cachedInfo, fileNameKey, lastWritten, size))
			if (m_nameToHashLookup.find(fileNameKey) == m_nameToHashLookup.end())
				return;
		CasKey& lookupCasKey = m_nameToHashLookup[fileNameKey];
		if (lookupCasKey == cachedInfo.casKey)
			return;

		#if 0
		//m_debugLogger->Info(TC("NAMETOHASHADD    %s %s\n"), KeyToString(fileNameKey).data, CasKeyString(lookupCasKey).str);
		#endif

		lookupCasKey = cachedInfo.casKey;
		BinaryWriter w(m_nameToHashTableMem.memory, m_nameToHashTableMem.writtenSize, NameToHashMemSize);
		m_nameToHashTableMem.AllocateNoLock(sizeof(StringKey) + sizeof(CasKey), 1, TC("NameToHashTable"), true);
		w.WriteStringKey(fileNameKey);
		w.WriteCasKey(lookupCasKey);
	}

	bool SessionServer::RunSpecialProgram(ProcessImpl& process, BinaryReader& reader, BinaryWriter& writer)
	{
		TString application = reader.ReadString();
		TString cmdLine = reader.ReadLongString();
		TString workingDir = reader.ReadString();
		UBA_ASSERT(StringView(application).Contains(TCV("UbaCli.exe")));

		StringBuffer<> jsonFile;
		ParseArguments(cmdLine.data(), cmdLine.size(), [&](const tchar* arg, u32 argLen)
			{
				StringView sv(arg, argLen);
				if (sv.Contains(TCV(".json")))
					jsonFile.Append(workingDir).EnsureEndsWithSlash().Append(sv);
			});

		if (jsonFile.IsEmpty())
			return false;

		ProcessImpl* rootProcess = &process;
		while (rootProcess->m_parentProcess)
			rootProcess = rootProcess->m_parentProcess;
		auto& startInfo = rootProcess->GetStartInfo();
		UBA_ASSERTF(m_outerScheduler, TC("No outer scheduler set"));
		return m_outerScheduler->EnqueueFromSpecialJson(jsonFile.data, workingDir.c_str(), TC("UbaDistributor"), startInfo.rootsHandle, startInfo.userData);
	}

	void SessionServer::PrintSessionStats(Logger& logger)
	{
		Session::PrintSessionStats(logger);

		if (m_nameToHashLookup.size())
			logger.Info(TC("  NameToHashLookup    %7u %9s"), u32(m_nameToHashLookup.size()), BytesToText(m_nameToHashTableMem.writtenSize).str);
		logger.Info(TC("  Remote processes finished    %8u"), m_finishedRemoteProcessCount);
		logger.Info(TC("  Remote processes returned    %8u"), m_returnedRemoteProcessCount);
		logger.Info(TC(""));
	}

	void SessionServer::TraceSessionUpdate()
	{
		u32 sessionIndex = 1;

		u64 serverSend = m_server.GetTotalSentBytes();
		u64 serverRecv = m_server.GetTotalRecvBytes();
		u32 serverConnectionCount = m_server.GetConnectionCount();

		SCOPED_CRITICAL_SECTION(m_remoteProcessAndSessionLock, lock);
		for (auto sptr : m_clientSessions)
		{
			auto& s = *sptr;
			NetworkServer::ClientStats stats;
			m_server.GetClientStats(stats, s.clientId);
			if (stats.connectionCount && (stats.send || stats.recv))
				m_trace.SessionUpdate(sessionIndex, stats.connectionCount, stats.send, stats.recv, s.lastPing, s.memAvail, s.memTotal, s.cpuLoad);
			++sessionIndex;
		}
		for (auto& rec : m_providers)
		{
			u64 send;
			u64 recv;
			u32 connectionCount;
			rec.provider(send, recv, connectionCount);
			serverSend += send;
			serverRecv += recv;
			serverConnectionCount += connectionCount;
		}
		lock.Leave();

		float cpuLoad = m_trace.UpdateCpuLoad();
		u64 memAvail;
		u64 memTotal;
		GetMemoryInfo(m_logger, memAvail, memTotal);

		if (m_traceIOEnabled)
		{
			for (auto& volume : m_volumeCache.volumes)
			{
				if (volume.drives.empty())
					continue;
				u8 busyPercent;
				u32 readCount;
				u64 readBytes;
				u32 writeCount;
				u64 writeBytes;
				if (!volume.UpdateStats(busyPercent, readCount, readBytes, writeCount, writeBytes))
					continue;
				if (!busyPercent && !readCount && !readBytes && !writeCount && !writeBytes)
					continue;
				m_trace.DriveUpdate(volume.drives[0], busyPercent, readCount, readBytes, writeCount, writeBytes);
			}
		}

		m_trace.SessionUpdate(0, serverConnectionCount, serverSend, serverRecv, 0, memAvail, memTotal, cpuLoad);

		#if UBA_TRACK_SHARED_MEMORY_ALLOCATIONS
		m_sharedMemory.TraceStats(m_trace, 30);
		#endif

		#if 0
		StringBuffer<> str;
		str.Appendf(TC("DirTable: %s, MapTable: %s"), BytesToText(m_directoryTable.m_memorySize).str, BytesToText(m_fileMappingTableSize).str);
		m_trace.StatusUpdate(40, 7, str, LogEntryType_Info);
		#endif

		#if 0
		m_fileMappingBuffer.TraceStatus(m_trace, 20, TC("Session"), 1);

		m_trace.StatusUpdate(40, 1, TCV("Independents"), LogEntryType_Info);
		StringBuffer<> str;
		u64 created = m_independentMappingCreated;
		u64 count = m_independentMappingActive;
		str.Appendf(TC("Active: %llu, Created: %llu"), count, created);
		m_trace.StatusUpdate(40, 7, str, LogEntryType_Info);
		#endif

		m_storage.TraceUpdate(m_trace, 300);

		// Backstop publish: under steady write pressure no single writer ends up the
		// last-out, so the size header lags. Drive it forward each tick.
		m_trace.PublishIfQuiet();
	}

	void SessionServer::WriteRemoteEnvironmentVariables(BinaryWriter& writer)
	{
		if (!m_remoteEnvironmentVariables.empty())
		{
			writer.WriteBytes(m_remoteEnvironmentVariables.data(), m_remoteEnvironmentVariables.size());
			return;
		}

		u64 startPos = writer.GetPosition();

		auto strs = (const tchar*)GetProcessEnvironmentVariables().data();

		for (auto it = strs; *it; it += TStrlen(it) + 1)
		{
			StringBuffer<> varName;
			varName.Append(it, TStrchr(it, '=') - it);
			if (!varName.IsEmpty() && !varName.Equals(TCV("CL")) && !varName.Equals(TCV("_CL_")))
				if (m_localEnvironmentVariables.find(varName.data) == m_localEnvironmentVariables.end())
					writer.WriteString(it);
		}

		writer.WriteString(TC(""));
		
		u64 size = writer.GetPosition() - startPos;
		m_remoteEnvironmentVariables.resize(size);
		memcpy(m_remoteEnvironmentVariables.data(), writer.GetData() + startPos, size);
	}

	bool SessionServer::InitializeNameToHashTable()
	{
		if (!m_nameToHashTableEnabled || m_nameToHashInitialized)
			return true;

		SCOPED_FUTEX(m_nameToHashLookupLock, lock);
		m_nameToHashTableMem.Init(NameToHashMemSize);
		m_nameToHashInitialized = true;
		lock.Leave();

		auto& dirTable = m_directoryTable;

		{
			Vector<DirectoryTable::Directory*> dirs;
			SCOPED_READ_LOCK(dirTable.m_lookupLock, dirsLock);
			dirs.reserve(dirTable.m_lookup.size());
			for (auto& kv : dirTable.m_lookup)
				dirs.push_back(&kv.second);
			dirsLock.Leave();

			for (auto dirPtr : dirs)
			{
				DirectoryTable::Directory& dir = *dirPtr;
				SCOPED_READ_LOCK(dir.lock, dirLock);
				for (auto& fileKv : dir.files)
				{
					StringKey fileNameKey = fileKv.first;
					BinaryReader reader(dirTable.m_memory, fileKv.second.internal);
					reader.SkipString();
					u32 attr = reader.ReadU32();
					if (IsDirectory(attr))
						continue;
					reader.ReadVolumeSerial();
					reader.ReadFileIndex();

					#if UBA_SUPPORT_SYMLINKS
					if (IsSymlink(attr))
						reader.SkipString();
					#endif

					u64 lastWritten = reader.ReadFileTime();
					u64 size = reader.ReadFileSize();
					FileEntryAdded(fileNameKey, lastWritten, size);
				}
			}
		}
		SCOPED_FUTEX(m_nameToHashLookupLock, lock2);
		u64 entryCount = m_nameToHashLookup.size();
		lock2.Leave();

		m_logger.Debug(TC("Prepopulated NameToHash table with %u entries"), entryCount);

		return true;
	}

	bool SessionServer::HandleDebugFileNotFoundError(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
#if 0//PLATFORM_WINDOWS
		StringBuffer<> errorPath;
		reader.ReadString(errorPath);
		StringBuffer<> workDir;
		reader.ReadString(workDir);

		StringView searchString = errorPath;
		if (searchString.data[0] == '.' && searchString.data[1] == '.')
		{
			searchString.data += 3;
			searchString.count -= 3;
		}

		auto LogLine = [&](const StringView& text) { m_logger.Log(LogEntryType_Warning, text.data, text.count); };

		// Make a copy of dir table since we can't populate it
		MemoryBlock block(64*1024*1024);
		DirectoryTable dirTable(block);
		u8* dirMem;
		u32 dirMemSize;
		{
			SCOPED_READ_LOCK(m_directoryTable.m_memoryLock, lock);
			dirMem = m_directoryTableMem;
			dirMemSize = m_directoryTable.m_memorySize;
		}

		dirTable.Init(dirMem, 0, dirMemSize);

		u32 foundCount = 0;
		dirTable.TraverseAllFilesNoLock([&](const DirectoryTable::EntryInformation& info, const StringBufferBase& path, u32 dirOffset)
			{
				if (!path.EndsWith(searchString))
					return;
				if (path[path.count - searchString.count - 1] != PathSeparator)
					return;

				auto ToString = [](bool b) { return b ? TC("true") : TC("false"); };

				++foundCount;
				StringBuffer<> logStr;
				logStr.Appendf(TC("File %s found in directory table at offset %u of %u while searching for matches for %s (File size %llu attr %u)"), path.data, dirOffset, dirTable.m_memorySize, searchString.data, info.size, info.attributes);
				LogLine(logStr);

				StringKey fileNameKey = ToStringKey(path);
				{
					SCOPED_FUTEX_READ(m_fileMappingTableLookupLock, mlock);
					auto findIt = m_fileMappingTableLookup.find(fileNameKey);
					if (findIt != m_fileMappingTableLookup.end())
					{
						auto& entry = findIt->second;
						SCOPED_FUTEX_READ(entry.lock, entryCs);
						logStr.Clear().Appendf(TC("File %s found in mapping table table."), path.data);
						if (entry.handled)
						{
							StringBuffer<128> mappingName;
							if (entry.mapping.IsValid())
								Storage::GetMappingString(mappingName, entry.mapping, entry.mappingOffset);
							else
								mappingName.Append(TCV("Not valid"));
							logStr.Appendf(TC(" Success: %s Size: %u IsDir: %s Mapping name: %s Mapping offset: %u"), ToString(entry.success), entry.contentSize, ToString(entry.isDir), mappingName.data, entry.mappingOffset);
						}
						else
						{
							logStr.Appendf(TC(" Entry not handled"));
						}
					}
					else
						logStr.Clear().Appendf(TC("File %s not found in mapping table table."), path.data);
					LogLine(logStr);
				}
				{
					SCOPED_FUTEX_READ(m_nameToHashLookupLock, hlock);
					auto findIt = m_nameToHashLookup.find(fileNameKey);
					if (findIt != m_nameToHashLookup.end())
						logStr.Clear().Appendf(TC("File %s found in name-to-hash lookup. CasKey is %s"), path.data, CasKeyString(findIt->second).str);
					else
						logStr.Clear().Appendf(TC("File %s not found in name-to-hash lookup"), path.data);
					LogLine(logStr);
				}
			});

		if (!foundCount)
		{
			StringBuffer<> logStr;
			logStr.Appendf(TC("No matching entry found in directory table while searching for matches for %s. DirTable size: %u"), searchString.data, GetDirectoryTableSize());
			LogLine(logStr);
			/*
			if (errorPath.StartsWith(TC("..\\Intermediate")))
			{
				StringBuffer<> fullPath;
				FixPath(errorPath.data, workDir.data, workDir.count, fullPath);
				GetFileAttributes
			}
			*/
		}
#endif
		return true;
	}

	bool SessionServer::HandleHostRun(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		return HostRun(reader, writer);
	}

	bool SessionServer::HandleGetSymbols(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		TString application = reader.ReadString();
		bool isClientLinux = reader.ReadBool();
		bool isClientArm = reader.ReadBool();

		if (IsLinux != isClientLinux)
		{
			writer.WriteString(TC("Can't resolve callstack on cross platforms"));
			return true;
		}

		if (IsArmBinary != isClientArm)
		{
			writer.WriteString(TC("Can't resolve callstack on cross architectures"));
			return true;
		}

		GetSymbols(application.c_str(), isClientLinux, isClientArm, reader, writer);

		if constexpr (DownloadDebugSymbols)
		{
			CasKey detoursSymbolsKey;
			StringBuffer<> dir;
			if (GetDirectoryOfCurrentModule(m_logger, dir))
			{
				bool deferCreation = true;
				auto ChangeToSymbolExtension = [](StringBufferBase& str) -> StringBufferBase& { IsWindows ? str.Resize(str.count - 3).Append("pdb") : str.Resize(str.count - 2).Append("debug"); return str; };
				if (!m_storage.StoreCasFile(detoursSymbolsKey, ChangeToSymbolExtension(dir).data, CasKeyZero, deferCreation) || detoursSymbolsKey == CasKeyZero)
				{
					StringBuffer<> dir2;
					if (GetAlternativeUbaPath(m_logger, dir2, dir, IsWindows && isClientArm))
						m_storage.StoreCasFile(detoursSymbolsKey, ChangeToSymbolExtension(dir2.Append(UBA_DETOURS_LIBRARY)).data, CasKeyZero, deferCreation);
				}
			}
			writer.WriteCasKey(detoursSymbolsKey);
		}

		return true;
	}

	bool SessionServer::HandleSpeedTest(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		writer.AllocWrite(SendMaxSize-10);
		return true;
	}

	bool SessionServer::HandleLogEntry(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer)
	{
		auto logType = (LogEntryType)reader.ReadByte();
		TString str = reader.ReadString();
		m_logger.Log(logType, str.c_str(), u32(str.size()));
		return true;
	}
}
