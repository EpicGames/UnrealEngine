// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaSharedMemoryAllocator.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaFile.h"
#include "UbaFileMapping.h"
#include "UbaHash.h"
#include "UbaTrace.h"

#if PLATFORM_WINDOWS
#include <windows.h>
#else
#define ZeroMemory(mem, size) memset(mem, 0, size)
#define DiscardVirtualMemory(...)
#define PAGE_WRITECOPY PROT_NONE
#endif

#define UBA_DEBUG_HINTS (UBA_DEBUG || UBA_TRACK_SHARED_MEMORY_ALLOCATIONS)

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	u8 g_emptyMemory;

	struct SharedMemoryAllocator::Slice
	{
		u64 offset;
		u64 size;
	};

	struct SharedMemoryAllocator::InternalHandle
	{
		using Slices = Vector<Slice>;
		Slices slices;
		Atomic<u32> refCount = 1;

		#if UBA_DEBUG_HINTS
		TString hint;
		const tchar* GetHint() { return hint.c_str(); }
		#else
		const tchar* GetHint() { return TC("EnableDebugHints"); }
		#endif

		#if UBA_TRACK_SHARED_MEMORY_ALLOCATIONS
		Atomic<u64> committed = 0;
		InternalHandle* next;
		InternalHandle* prev;
		#endif
	};


	#define UBA_REF_COUNT(f) UBA_ASSERTF(h.refCount, TC( #f " accessing released shared memory handle (%s)"), h.GetHint());

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool SharedMemoryAllocator::Init(u64 capacity, u64 commitStepSize, const tchar* tempFile, bool tempFileSparse)
	{
		m_commitStepSize = Min(commitStepSize, capacity);
		m_capacity = AlignUp(capacity, PageSize);

		auto tempFileGuard = MakeGuard([&](){ CloseFile(tempFile, m_tempFile); });

		#if PLATFORM_WINDOWS
		if (tempFile && *tempFile)
		{
			m_tempFile = CreateFileW(tempFile, GENERIC_READ|GENERIC_WRITE, 0, CREATE_ALWAYS, FILE_ATTRIBUTE_TEMPORARY|FILE_FLAG_DELETE_ON_CLOSE);
			if (m_tempFile == InvalidFileHandle)
				return m_logger.Error(TC("Failed to create file %s (%s)"), tempFile, LastErrorToText().data);
			if (tempFileSparse)
				if (!MakeSparse(m_logger, tempFile, m_tempFile, m_capacity))
					return false;
			m_fileMapping = m_fileMappingBackend.CreateFromFile(m_logger, m_tempFile, PAGE_READWRITE, m_capacity, TC("SharedMemoryAllocator"));
		}
		else
		#endif
		{
			m_fileMapping = m_fileMappingBackend.Create(m_logger, PAGE_READWRITE|SEC_RESERVE, m_capacity, nullptr, TC("SharedMemoryAllocator"));
		}
		if (!m_fileMapping.IsValid())
			return m_logger.Error(TC("Failed to create file mapping for SharedMemoryAllocator"));

		m_memory = m_fileMappingBackend.Map(m_logger, m_fileMapping, FILE_MAP_READ|FILE_MAP_WRITE, 0, m_capacity, false);
		UBA_ASSERTF(m_memory, TC("MapViewOfFile failed trying to map %llu bytes (%s)"), m_capacity, LastErrorToText().data);

		if (!m_memory)
		{
			m_fileMappingBackend.Close(m_logger, m_fileMapping, TC("SharedMemoryAllocator"));
			return false;
		}

		tempFileGuard.Cancel();
		return true;
	}

	void SharedMemoryAllocator::Deinit(bool keepFileMapping)
	{
		if (m_memory)
			m_fileMappingBackend.Unmap(m_logger, m_memory, m_capacity, TC("SharedMemoryAllocator"), false);
		m_memory = nullptr;

		if (m_fileMapping.IsValid() && !keepFileMapping)
			m_fileMappingBackend.Close(m_logger, m_fileMapping, TC("SharedMemoryAllocator"));
		m_fileMapping = {};

		if (m_tempFile != InvalidFileHandle && !keepFileMapping)
			CloseFile(TC(""), m_tempFile);
		m_tempFile = InvalidFileHandle;
	}

	void SharedMemoryAllocator::Reserve(u64 size)
	{
		FreeNoLock(AllocateNoLock(size));
	}

	SharedMemoryAllocator::operator SharedMemoryAllocatorHandle() const
	{
		return m_fileMapping;
	}

	u64 SharedMemoryAllocator::GetCapacity()
	{
		return m_capacity;
	}

	u64 SharedMemoryAllocator::GetUsed()
	{
		return m_used;
	}

	SharedMemoryHandle SharedMemoryAllocator::CreateHandle(Logger& logger, const tchar* hint, bool priority)
	{
		++m_handleCount;
		auto h = new InternalHandle();

		#if UBA_DEBUG_HINTS
		h->hint = hint;
		#endif

		#if UBA_TRACK_SHARED_MEMORY_ALLOCATIONS
		{
			SCOPED_FUTEX(m_handleLock, l);
			if (m_firstHandle)
				m_firstHandle->prev = h;
			h->next = m_firstHandle;
			m_firstHandle = h;
		}
		#endif

		if (priority)
		{
			u32 index = m_priorityHandleCounter++;
			if (index < sizeof_array(m_priorityHandles))
			{
				m_priorityHandles[index] = h;
				return { index + 1 };
			}
		}

		return { u64(h) };
	}

	SharedMemoryHandle SharedMemoryAllocator::DuplicateHandle(SharedMemoryHandle handle, const tchar* hint)
	{
		if (!handle.internalHandle)
			return {};
		if (IsExternalMapping(handle))
			return handle;
		auto& h = GetHandle(handle);
		UBA_REF_COUNT(DuplicateHandle);
		++h.refCount;
		return handle;
	}

	void SharedMemoryAllocator::ClearHandle(SharedMemoryHandle handle)
	{
		if (!handle.internalHandle)
			return;
		auto& h = GetHandle(handle);
		UBA_REF_COUNT(ClearHandle);
		ClearHandle(h);
	}

	void SharedMemoryAllocator::CloseHandle(Logger& logger, SharedMemoryHandle handle, const tchar* hint)
	{
		if (!handle.internalHandle)
			return;
		if (IsExternalMapping(handle))
			return;
		ReleaseHandle(handle);
	}

	void SharedMemoryAllocator::ExtendMemory(BinaryWriter& out, SharedMemoryHandle handle, u64 size, const tchar* hint, bool zeroMemory)
	{
		UBA_ASSERT(handle.internalHandle);
		auto& h = GetHandle(handle);
		UBA_REF_COUNT(ExtendMemory);

		auto& slices = h.slices;
		
		u32 sliceCount = u32(slices.size());
		ExtendHandle(h, size, hint, zeroMemory);
		u32 newCount = u32(slices.size());

		out.WriteU32(newCount - sliceCount);
		for (u32 i = sliceCount, e = newCount; i != e; ++i)
		{
			out.Write7BitEncoded(slices[i].offset);
			out.Write7BitEncoded(slices[i].size);
		}
	}

	void SharedMemoryAllocator::ExtendMemory(SharedMemoryHandle handle, u64 size, const tchar* hint, bool zeroMemory)
	{
		UBA_ASSERTF(m_fileMapping.IsValid(), TC("Allocator handle not initialized"));
		UBA_ASSERT(handle.internalHandle);
		auto& h = GetHandle(handle);
		UBA_REF_COUNT(ExtendMemory);
		ExtendHandle(h, size, hint, zeroMemory);
	}

	void SharedMemoryAllocator::ExtendMemory(SharedMemoryView& view, SharedMemoryHandle handle, u64 size, const tchar* hint, bool zeroMemory)
	{
		UBA_ASSERT(handle.internalHandle);
		auto& h = GetHandle(handle);
		UBA_REF_COUNT(ExtendMemory);

		auto& slices = h.slices;
		u32 sliceCount = u32(slices.size());
		ExtendHandle(h, size, hint, zeroMemory);

		for (u32 i=sliceCount, e= u32(slices.size()); i!=e; ++i)
			view.ExtendMemory(*this, slices[i].offset, slices[i].size, SharedMemoryMapType_ReadWrite, hint);
	}

	void SharedMemoryAllocator::TrimMemory(SharedMemoryHandle handle, u64 usedSize)
	{
		UBA_ASSERT(handle.internalHandle);
		auto& h = GetHandle(handle);
		UBA_REF_COUNT(TrimMemory);
		UBA_ASSERT(h.refCount == 1);

		usedSize = AlignUp(usedSize, PageSize);

		#if UBA_TRACK_SHARED_MEMORY_ALLOCATIONS
		h.committed = usedSize;
		#endif

		auto& slices = h.slices;
		for (u32 i = 0, e = u32(slices.size()); i != e; ++i)
		{
			u64 sliceSize = slices[i].size;
			if (usedSize <= sliceSize)
			{
				SCOPED_FUTEX(m_lock, lock);
				if (u64 left = sliceSize - usedSize)
				{
					u64 offset = slices[i].offset + usedSize;
					FreeNoLock({ offset, left });
					slices[i].size = usedSize;
				}
				for (u32 j = ++i; j != e; ++j)
					FreeNoLock(slices[j]);
				slices.resize(i);
				return;
			}

			usedSize -= sliceSize;
		}
	}

	u8* SharedMemoryAllocator::ReserveMemory(SharedMemoryHandle handle, u64 size)
	{
		UBA_ASSERT(AlignUp(size, PageSize) == size);
		UBA_ASSERT(handle.internalHandle);
		auto& h = GetHandle(handle);
		UBA_REF_COUNT(ReserveMemory);

		UBA_ASSERT(h.slices.empty());

		auto& slice = h.slices.emplace_back();
		SCOPED_FUTEX(m_lock, lock);
		slice.offset = m_reserved;
		m_reserved += size;
		slice.size = size;
		lock.Leave();

		return m_memory + slice.offset;
	}

	void SharedMemoryAllocator::CommitMemory(SharedMemoryHandle handle, u64 offset, u64 size)
	{
		UBA_ASSERT(AlignUp(offset, PageSize) == offset);
		UBA_ASSERT(AlignUp(size, PageSize) == size);
		UBA_ASSERT(handle.internalHandle);
		auto& h = GetHandle(handle);
		UBA_REF_COUNT(CommitMemory);

		UBA_ASSERT(h.slices.size() == 1);
		auto& slice = h.slices.back();
		if (m_tempFile == InvalidFileHandle)
			m_fileMappingBackend.Commit(m_logger, m_memory + slice.offset + offset, size, false);

		SCOPED_FUTEX(m_lock, lock);
		m_committed += size;
		m_used += size;

		#if UBA_TRACK_SHARED_MEMORY_ALLOCATIONS
		h.committed += size;
		#endif
	}

	void SharedMemoryAllocator::DecommitMemory(SharedMemoryHandle handle, u64 offset, u64 size)
	{
		if (!handle.internalHandle)
			return;
		if (!size)
			return;
		UBA_ASSERT(AlignUp(offset, PageSize) == offset);
		UBA_ASSERT(AlignUp(size, PageSize) == size);
		UBA_ASSERT(handle.internalHandle);
		auto& h = GetHandle(handle);
		UBA_REF_COUNT(DecommitMemory);
		UBA_ASSERT(h.slices.size() == 1);
		auto& slice = h.slices.back();

		#if UBA_TRACK_SHARED_MEMORY_ALLOCATIONS
		h.committed -= size;
		#endif

		// Unfortunately windows has no way of decommitting memory from anonymous file mappings.
		SCOPED_FUTEX(m_lock, lock);
		FreeNoLock({ slice.offset + offset, size });
	}

	void SharedMemoryAllocator::UnreserveMemory(SharedMemoryHandle handle)
	{
		if (!handle.internalHandle)
			return;
		UBA_ASSERT(handle.internalHandle);
		auto& h = GetHandle(handle);
		UBA_REF_COUNT(UnreserveMemory);
		h.slices.clear();
	}

	u64 SharedMemoryAllocator::GetCommitted(SharedMemoryHandle handle)
	{
		auto& h = GetHandle(handle);
		UBA_REF_COUNT(GetCommitted);
		u64 committed = 0;
		for (auto& slice : h.slices)
			committed += slice.size;
		return committed;
	}

	u64 SharedMemoryAllocator::GetRefCount(SharedMemoryHandle handle)
	{
		auto& h = GetHandle(handle);
		UBA_REF_COUNT(GetRefCount);
		return h.refCount;
	}

	FileMappingBackend& SharedMemoryAllocator::GetFileMappingBackend()
	{
		return m_fileMappingBackend;
	}

	void SharedMemoryAllocator::WriteViewSlices(BinaryWriter& out, SharedMemoryHandle handle, u64 offset, u64 size)
	{
		auto& h = GetHandle(handle);
		UBA_REF_COUNT(WriteViewSlices);
		auto& totCount = *(u32*)out.AllocWrite(4);
		u32 count = 0;
		for (auto& slice : h.slices)
		{
			u64 sliceOffset = slice.offset;
			u64 sliceSize = slice.size;
			if (offset >= sliceSize)
			{
				offset -= sliceSize;
				continue;
			}
			else if (offset)
			{
				sliceOffset += offset;
				sliceSize -= offset;
				offset = 0;
			}

			++count;

			out.Write7BitEncoded(sliceOffset);

			if (size <= sliceSize)
			{
				out.Write7BitEncoded(AlignUp(size, PageSize));
				break;
			}

			out.Write7BitEncoded(sliceSize);
			size -= sliceSize;
		}

		totCount = count;
	}

	bool SharedMemoryAllocator::MapView(SharedMemoryView& view, SharedMemoryHandle handle, const tchar* hint, SharedMemoryMapType mapType, u64 offset, u64 size)
	{
		if (!handle.internalHandle)
			return false;

		if (handle.internalHandle & 0x8000'0000'0000'0000)
		{
			// TODO: Right now we leak this mapping. Not good..
			// .. but only used in very special situations
			FileMappingHandle file = FileMappingHandle::FromU64(handle.internalHandle & ~0x8000'0000'0000'0000);
			u8* mem = m_fileMappingBackend.Map(m_logger, file, FILE_MAP_READ, offset, size, false);
			UBA_ASSERT(mem);
			view.ReferenceExternalMemory(mem, size);
			return true;
		}

		auto& h = GetHandle(handle);
		UBA_REF_COUNT(MapView);

		if (size == 0)
			return true;

		u64 totalSize = size;
		if (totalSize == ~0ull)
		{
			totalSize = 0;
			for (auto& slice : h.slices)
				totalSize += slice.size;
		}

		bool initialized = false;

		for (auto& slice : h.slices)
		{
			u64 sliceOffset = slice.offset;
			u64 sliceSize = slice.size;
			if (offset >= sliceSize)
			{
				offset -= sliceSize;
				continue;
			}
			else if (offset)
			{
				sliceOffset += offset;
				sliceSize -= offset;
				offset = 0;
			}

			if (totalSize <= sliceSize)
			{
				if (!initialized)
					view.ReferenceExternalMemory(m_memory + sliceOffset, totalSize);
				else
					view.ExtendMemory(*this, sliceOffset, totalSize, mapType, hint);
				break;
			}

			if (!initialized)
				view.Init(*this, totalSize);
			initialized = true;

			view.ExtendMemory(*this, sliceOffset, sliceSize, mapType, hint);
			totalSize -= sliceSize;
			if (!totalSize)
				break;
		}
		return true;
	}

	u8* SharedMemoryAllocator::MapView(SharedMemoryHandle handle, const tchar* hint, SharedMemoryMapType mapType)
	{
		if (!handle.internalHandle)
			return nullptr;

		auto& h = GetHandle(handle);
		UBA_REF_COUNT(MapView);

		if (h.slices.empty())
			return &g_emptyMemory;

		++h.refCount;

		if (h.slices.size() == 1)
			return m_memory + h.slices.front().offset;

		u64 committed = 0;
		for (auto& slice : h.slices)
			committed += slice.size;

		u32 protect = PAGE_READWRITE;
		if (mapType == SharedMemoryMapType_ReadOnly)
			protect = PAGE_READONLY;
		else if (mapType == SharedMemoryMapType_CopyOnWrite)
			protect = PAGE_WRITECOPY;

		auto& kernelStats = KernelStats::GetCurrent();

		TimerScope ts(kernelStats.virtualAlloc2);
		u8* memory = (u8*)m_fileMappingBackend.ReservePlaceholder(nullptr, committed);
		ts.Leave();

		u64 pos = 0;
		for (Slice slice : h.slices)
		{
			kernelStats.mapViewOfFile3.bytes += slice.size;
			TimerScope ts2(kernelStats.mapViewOfFile3);

			void* viewA = m_fileMappingBackend.MapPlaceholder(m_fileMapping, (MemoryMapType)mapType, memory, pos, committed, slice.offset, slice.size);
			UBA_ASSERTF(viewA == memory+pos, TC("MapViewOfFile3 failed in MapView 0x%p %llu for % s (%s)"), memory, pos, h.GetHint(), LastErrorToText().data); (void)viewA;
			pos += slice.size;
		}

		return memory;
	}

	void SharedMemoryAllocator::UnmapView(SharedMemoryHandle handle, u8* memory)
	{
		if (!memory || memory == &g_emptyMemory)
			return;

		UBA_ASSERT(handle.internalHandle);
		auto& h = GetHandle(handle);
		UBA_REF_COUNT(UnmapView);

		if (h.slices.size() == 1)
		{
			// UBA_ASSERT(memory >= m_memory && memory < m_memory + m_reserved); // TSAN does not like the m_reserved 
			ReleaseHandle(handle);
			return;
		}

		Vector<u64> subMappings;
		subMappings.reserve(h.slices.size() - 1);
		u64 committed = 0;
		for (Slice slice : h.slices)
		{
			if (committed != 0)
				subMappings.push_back(committed);
			committed += slice.size;
		}
		m_fileMappingBackend.UnmapPlaceholder(memory, committed, committed, subMappings.data(), subMappings.size());

		ReleaseHandle(handle);
	}

	SharedMemoryAllocator::SharedMemoryAllocator(Logger& logger, FileMappingBackend& fileMappingBackend)
	:	m_logger(logger)
	,	m_fileMappingBackend(fileMappingBackend)
	{
	}

	SharedMemoryAllocator::~SharedMemoryAllocator()
	{
		Deinit(false);
	}

	SharedMemoryHandle SharedMemoryAllocator::RegisterExternalMapping(FileMappingHandle file)
	{
		SharedMemoryHandle h;
		h.internalHandle = 0x8000'0000'0000'0000 + file.ToU64();
		return h;
	}

	bool SharedMemoryAllocator::IsExternalMapping(SharedMemoryHandle handle)
	{
		return (handle.internalHandle & 0x8000'0000'0000'0000) != 0;
	}

	void SharedMemoryAllocator::UnmapExternalMapping(SharedMemoryHandle handle, u8* memory, u64 size)
	{
		UBA_ASSERT((handle.internalHandle & 0x8000'0000'0000'0000) != 0);
		bool res = m_fileMappingBackend.Unmap(m_logger, memory, size, TC("ExternalMapping"), false);
		UBA_ASSERT(res); (void)res;
	}

	SharedMemoryAllocator::InternalHandle& SharedMemoryAllocator::GetHandle(SharedMemoryHandle h)
	{
		UBA_ASSERT(!(h.internalHandle & 0x8000'0000'0000'0000));
		if (h.internalHandle <= sizeof_array(m_priorityHandles))
			return *m_priorityHandles[h.internalHandle - 1];
		return *(InternalHandle*)h.internalHandle;
	}

	void SharedMemoryAllocator::ReleaseHandle(SharedMemoryHandle handle)
	{
		InternalHandle& h = GetHandle(handle);
		UBA_REF_COUNT(ReleaseHandle);
		if (--h.refCount)
			return;

		#if UBA_TRACK_SHARED_MEMORY_ALLOCATIONS
		{
			SCOPED_FUTEX(m_handleLock, l);
			if (m_firstHandle == &h)
				m_firstHandle = h.next;
			if (h.prev)
				h.prev->next = h.next;
			if (h.next)
				h.next->prev = h.prev;
		}
		#endif

		ClearHandle(h);

		#if !UBA_DEBUG_HINTS
		delete& h;
		#endif
		if (handle.internalHandle <= sizeof_array(m_priorityHandles))
			m_priorityHandles[handle.internalHandle - 1] = 0;
		--m_handleCount;
	}

	void SharedMemoryAllocator::ExtendHandle(InternalHandle& handle, u64 size, const tchar* hint, bool zeroMemory)
	{
		size = AlignUp(size, PageSize);
		u64 left = size;

		#if UBA_TRACK_SHARED_MEMORY_ALLOCATIONS
		handle.committed += size;
		#endif

		auto& slices = handle.slices;

		u32 sliceCount = u32(slices.size());

		SCOPED_FUTEX(m_lock, lock);
		while (left)
		{
			Slice slice = AllocateNoLock(left);
			slices.push_back(slice);
			left -= slice.size;
		}
		lock.Leave();

		if (zeroMemory)
			for (u32 i = sliceCount, e = u32(slices.size()); i != e; ++i)
				ZeroMemory(m_memory + slices[i].offset, slices[i].size);
	}

	void SharedMemoryAllocator::ClearHandle(InternalHandle& handle)
	{
		if (handle.slices.empty())
			return;

		#if UBA_TRACK_SHARED_MEMORY_ALLOCATIONS
		handle.committed = 0;
		#endif

		SCOPED_FUTEX(m_lock, lock);
		for (auto& slice : handle.slices)
			FreeNoLock(slice);
		lock.Leave();
		handle.slices.clear();
	}

	SharedMemoryAllocator::Slice SharedMemoryAllocator::AllocateNoLock(u64 desiredSize)
	{
		// Find best fit.
		// Will return first iterator not less than desiredSize
		auto it = m_freeMemoryBySize.lower_bound({ desiredSize, 0 });
		if (it == m_freeMemoryBySize.end())
		{
			bool isEmpty = m_freeMemory.empty();

			if (!isEmpty)
			{
				--it;
				u64 size = it->size;
				if (size >= desiredSize / 4) // Quarter the size is ok
				{
					u64 offset = it->offset;
					m_freeMemoryBySize.erase(it);
					m_freeMemory.erase(offset);
					m_used += size;
					return { offset, size };
				}
			}

			UBA_ASSERT(m_memory);
			// No block is big enough, create new one
			u64 toCommit = Min(AlignUp(desiredSize, m_commitStepSize), m_capacity);
			UBA_SHIPPING_ASSERTF(m_reserved + toCommit <= m_capacity, TC("Not enough space in capacity for commit (Capacity: %llu, Reserved: %llu Requested: %llu"), m_capacity, m_reserved, toCommit);
			if (m_tempFile == InvalidFileHandle)
			{
				bool res = m_fileMappingBackend.Commit(m_logger, m_memory + m_reserved, toCommit, false);
				UBA_SHIPPING_ASSERTF(res, TC("VirtualAlloc failed commit %llu bytes at address 0x%llx + %llu (%s)"), toCommit, u64(m_memory), m_reserved, LastErrorToText().data);
			}
			u64 offset = m_reserved;
			m_reserved += toCommit;
			m_committed += toCommit;

			// Coalece with last block if it ends at offset
			if (!isEmpty)
			{
				auto lastIt = m_freeMemory.rbegin();
				if (lastIt->first + lastIt->second == offset)
				{
					offset = lastIt->first;
					toCommit += lastIt->second;
					m_freeMemoryBySize.erase({ lastIt->second, lastIt->first });
					m_freeMemory.erase(lastIt->first);
				}
			}

			u64 size = toCommit;
			if (toCommit > desiredSize)
			{
				u64 newOffset = offset + desiredSize;
				u64 newSize = toCommit - desiredSize;
				m_freeMemory.emplace(newOffset, newSize);
				m_freeMemoryBySize.insert({ newSize, newOffset });
				size = desiredSize;
			}
			m_used += size;
			return { offset, size };
		}
		else
		{
			u64 offset = it->offset;
			u64 size = it->size;
			if (size == desiredSize) // Perfect fit
			{
				m_freeMemoryBySize.erase(it);
				m_freeMemory.erase(offset);
				m_used += desiredSize;
				return { offset, desiredSize };
			}
			else // Slot is larger, take first part of it
			{
				m_freeMemory.erase(offset);
				m_freeMemoryBySize.erase(it);

				u64 newOffset = offset + desiredSize;
				u64 newSize = size - desiredSize;

				m_freeMemory.emplace(newOffset, newSize);
				m_freeMemoryBySize.insert({ newSize, newOffset });

				m_used += desiredSize;
				return { offset, desiredSize };
			}
		}
	}

	void SharedMemoryAllocator::FreeNoLock(Slice slice)
	{
		// Returns it to first slice not earlier than offset
		auto it = m_freeMemory.lower_bound(slice.offset);

		UBA_ASSERT(slice.size);

		m_used -= slice.size;

		// Need to measure if this is a good idea
		// DiscardVirtualMemory(m_memory + slice.offset, slice.size);

		if (it == m_freeMemory.end()) // The slice to add is after last
		{
			if (m_freeMemory.empty()) // No slices found at all
			{
				bool res = m_freeMemory.try_emplace(slice.offset, slice.size).second;
				UBA_ASSERT(res);(void)res;
				m_freeMemoryBySize.insert({ slice.size, slice.offset });
				return;
			}
			else // There are slots before, let's check if we can coalece
			{
				--it;
				u64 end = it->first + it->second;
				if (end == slice.offset) // We can combine the previous slot
				{
					m_freeMemoryBySize.erase({ it->second, it->first });
					it->second += slice.size;
					m_freeMemoryBySize.insert({ it->second, it->first });
					return;
				}
				else // This is a new slot
				{
					bool res = m_freeMemory.try_emplace(slice.offset, slice.size).second;
					UBA_ASSERT(res);(void)res;
					m_freeMemoryBySize.insert({ slice.size, slice.offset });
					return;
				}
			}
		}
		else
		{
			bool addedToPrev = false;

			if (it != m_freeMemory.begin()) // There are slots both before and after
			{
				auto prevIt = it; --prevIt;
				u64 end = prevIt->first + prevIt->second;
				if (end == slice.offset) // This slice can be coaleced with previous slice (reuse the already added one
				{
					m_freeMemoryBySize.erase({ prevIt->second, prevIt->first });
					prevIt->second += slice.size;
					it = prevIt;
					addedToPrev = true;
					slice.offset = prevIt->first;
					slice.size = prevIt->second;
				}
			}

			if (!addedToPrev)
			{
				auto res = m_freeMemory.try_emplace(slice.offset, slice.size);
				UBA_ASSERT(res.second);
				it = res.first;
			}

			auto next = it; ++next;
			if (it->first + it->second == next->first) // We can coalece with the one after
			{
				u64 nextSize = next->second;
				m_freeMemoryBySize.erase({ nextSize, next->first });
				it->second += nextSize;
				m_freeMemory.erase(next);
				slice.size = it->second;
			}

			m_freeMemoryBySize.insert({ slice.size, slice.offset });
			return;
		}
	}

	void SharedMemoryAllocator::TraceStats(Trace& trace, u32 startRow)
	{
		trace.StatusUpdate(startRow, 1, TCV("SharedMemory"), LogEntryType_Info);

		StringBuffer<> str;
		str.Appendf(TC("%s/%s (%llu)"), BytesToText(m_used).str, BytesToText(m_committed).str, m_handleCount.load());
		trace.StatusUpdate(startRow, 7, str, LogEntryType_Info);

		#if UBA_TRACK_SHARED_MEMORY_ALLOCATIONS
		SCOPED_FUTEX(m_handleLock, lock);
		Map<SizeAndOffset, const tchar*> handles;
		for (InternalHandle* it = m_firstHandle; it; it = it->next)
			if (u64 committed = it->committed.load())
				handles.emplace(SizeAndOffset{ committed, u64(it) }, it->GetHint());
		Vector<u32> items;
		CasKeyHasher hasher;
		for (auto ri = handles.rbegin(), re = handles.rend(); ri !=re; ++ri)
		{
			const tchar* hint = ri->second;
			if (const tchar* lastSlash = TStrrchr(hint, PathSeparator))
				hint = lastSlash + 1;
			str.Clear().Appendf(TC("%s  %s"), BytesToText(ri->first.size).str, hint);
			u32 stringId = trace.StringAdd(str);
			items.push_back(stringId);
			hasher.Update(&stringId, sizeof(u32));
		}
		lock.Leave();

		CasKey toSend = ToCasKey(hasher, false);
		if (m_lastSent == toSend)
			return;
		m_lastSent = toSend;

		trace.TableUpdate(trace.StringAdd(TCV("Allocs")), 25, 50, items.data(), u32(items.size()));
		#endif
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void GetMappingString(StringBufferBase& out, SharedMemoryHandle handle, u64 offset, bool canBeFreed)
	{
		out.Append(canBeFreed ? FreeableMemoryHandleChar : MemoryHandleChar);
		out.AppendBase62(handle.internalHandle);
		if (offset)
			out.Append('-').AppendBase62(offset);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
