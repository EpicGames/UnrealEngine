// Copyright Epic Games, Inc. All Rights Reserved.

#if PLATFORM_LINUX

#include "UbaStaticPatcher.h"
#include "UbaFileAccessor.h"
#include "UbaLogger.h"
#include "UbaPlatform.h"
#include <stdint.h>
#include <string.h>
#if !PLATFORM_WINDOWS
#include <sys/stat.h>
#endif

namespace uba
{
	// ELF64 little-endian layout constants — enough to patch x86_64 binaries without <elf.h>.
	namespace ElfX64
	{
		constexpr u8  ElfMagic[4]    = { 0x7F, 'E', 'L', 'F' };
		constexpr u8  ElfClass64     = 2;
		constexpr u8  ElfData2Lsb    = 1;
		constexpr u16 EmX86_64       = 62;

		constexpr u32 PtLoad         = 1;
		constexpr u32 PtInterp       = 3;

		constexpr u32 PfX            = 1;
		constexpr u32 PfW            = 2;
		constexpr u32 PfR            = 4;

		constexpr u32 ShtSymtab      = 2;
		constexpr u32 ShtStrtab      = 3;

		// ELF64 header field offsets
		constexpr u64 OffEiClass     = 4;
		constexpr u64 OffEiData      = 5;
		constexpr u64 OffEMachine    = 18;
		constexpr u64 OffEEntry      = 24;
		constexpr u64 OffEPhoff      = 32;
		constexpr u64 OffEShoff      = 40;
		constexpr u64 OffEPhentsize  = 54;
		constexpr u64 OffEPhnum      = 56;
		constexpr u64 OffEShentsize  = 58;
		constexpr u64 OffEShnum      = 60;

		constexpr u64 PhdrEntSize    = 56;   // sizeof(Elf64_Phdr)
		constexpr u64 ShdrEntSize    = 64;   // sizeof(Elf64_Shdr)
		constexpr u64 SymEntSize     = 24;   // sizeof(Elf64_Sym)

		// Elf64_Shdr offsets (sh_name, sh_type, sh_flags, sh_addr, sh_offset,
		// sh_size, sh_link, sh_info, sh_addralign, sh_entsize)
		constexpr u64 ShOffType      = 4;
		constexpr u64 ShOffOffset    = 24;
		constexpr u64 ShOffSize      = 32;
		constexpr u64 ShOffLink      = 40;

		// Elf64_Sym offsets (st_name, st_info, st_other, st_shndx, st_value, st_size)
		constexpr u64 SymOffName     = 0;
		constexpr u64 SymOffInfo     = 4;
		constexpr u64 SymOffShndx    = 6;
		constexpr u64 SymOffValue    = 8;

		// st_info packs (bind << 4) | (type & 0xF).
		constexpr u8  SttFunc        = 2;   // STT_FUNC
		constexpr u8  StbGlobal      = 1;   // STB_GLOBAL
		constexpr u8  StbWeak        = 2;   // STB_WEAK

		constexpr u64 Page           = 0x1000;

		// Layout of an Elf64_Phdr (little-endian, packed):
		//   u32 p_type
		//   u32 p_flags
		//   u64 p_offset
		//   u64 p_vaddr
		//   u64 p_paddr
		//   u64 p_filesz
		//   u64 p_memsz
		//   u64 p_align
	}

	// Sentinel rewritten by the patcher to the target's original e_entry. Must match UbaStaticStub.S.
	constexpr u64 StubOriginalEntrySentinel = 0xDEADBEEFCAFEBABEULL;

	// 16-byte [Sentinel][Guard] pair baked with VA of internal/runtime/syscall.Syscall6. Guard disambiguates from incidental compile-time immediates.
	constexpr u64 StubSyscall6Sentinel = 0xDEADBEEFC0FFEE00ULL;
	constexpr u64 StubSyscall6Guard    = 0xFEEDFACE12345678ULL;

	// runtime.exit.abi0 — Go's low-level exit_group stub. Hooking it lets us send MessageType_Exit before the kernel tears the process down.
	constexpr u64 StubExitSentinel     = 0xDEADBEEFEEEEE500ULL;
	constexpr u64 StubExitGuard        = 0xCAFEBABE87654321ULL;

	// syscall.rawVforkSyscall.abi0. We redirect to plain SYS_fork so child's chdir can't poison the parent's vwd via shared memory.
	constexpr u64 StubVforkSentinel    = 0xDEADBEEF7AC0FFEEULL;
	constexpr u64 StubVforkGuard       = 0xBADCAFE0BADCAFE1ULL;

	// Go 1.21+ moved the syscall package; pre-1.21 binaries (e.g. AOSP's clang-r530567 wrapper) use the old path.
	constexpr const char Syscall6SymbolName[]    = "internal/runtime/syscall.Syscall6";
	constexpr const char Syscall6SymbolNameOld[] = "runtime/internal/syscall.Syscall6";
	constexpr const char ExitSymbolName[]     = "runtime.exit.abi0";
	constexpr const char VforkSymbolName[]    = "syscall.rawVforkSyscall.abi0";

	// Guaranteed 0 by stub layout (placed first).
	constexpr u64 StubInitOffset = 0;

	namespace
	{
		template<typename T>
		inline void ReadLE(const u8* data, u64 offset, T& out)
		{
			memcpy(&out, data + offset, sizeof(T));
		}
		template<typename T>
		inline void WriteLE(u8* data, u64 offset, T value)
		{
			memcpy(data + offset, &value, sizeof(T));
		}

		// Copy entire file into a buffer (we append to it later, so can't work against mmap).
		bool ReadAll(Logger& logger, StringView path, Vector<u8>& out)
		{
			FileAccessor fa(logger, path.CheckTerminated());
			if (!fa.OpenMemoryRead())
				return false;
			out.resize(fa.GetSize());
			memcpy(out.data(), fa.GetData(), fa.GetSize());
			return true;
		}

		bool WriteAll(Logger& logger, StringView path, const u8* data, u64 size)
		{
			FileAccessor fa(logger, path.CheckTerminated());
			if (!fa.CreateWrite(false, DefaultAttributes(), size))
				return false;
			if (size && !fa.Write(data, size, 0, true))
				return false;
			return fa.Close();
		}

		struct SymtabInfo
		{
			u64         symtabOff;
			u64         symtabSize;
			const char* strtab;
			u64         strtabSize;
		};

		bool LocateSymtab(const u8* image, u64 imageSize, SymtabInfo& out)
		{
			using namespace ElfX64;

			if (imageSize < 64)
				return false;

			u64 eShoff = 0;
			u16 eShentsize = 0, eShnum = 0;
			ReadLE(image, OffEShoff,     eShoff);
			ReadLE(image, OffEShentsize, eShentsize);
			ReadLE(image, OffEShnum,     eShnum);
			if (eShoff == 0 || eShentsize != ShdrEntSize || eShnum == 0)
				return false;
			if (eShoff + u64(eShnum) * ShdrEntSize > imageSize)
				return false;

			u64 symtabOff = 0, symtabSize = 0;
			u32 symtabLink = 0;
			for (u16 i = 0; i < eShnum; ++i)
			{
				const u8* shdr = image + eShoff + u64(i) * ShdrEntSize;
				u32 shType = 0; u64 shOffset = 0, shSize = 0; u32 shLink = 0;
				ReadLE(shdr, ShOffType,   shType);
				ReadLE(shdr, ShOffOffset, shOffset);
				ReadLE(shdr, ShOffSize,   shSize);
				ReadLE(shdr, ShOffLink,   shLink);
				if (shType == ShtSymtab)
				{
					symtabOff  = shOffset;
					symtabSize = shSize;
					symtabLink = shLink;
					break;
				}
			}
			if (!symtabOff || !symtabSize)
				return false;
			if (symtabLink >= eShnum)
				return false;

			const u8* strShdr = image + eShoff + u64(symtabLink) * ShdrEntSize;
			u32 strType = 0; u64 strOffset = 0, strSize = 0;
			ReadLE(strShdr, ShOffType,   strType);
			ReadLE(strShdr, ShOffOffset, strOffset);
			ReadLE(strShdr, ShOffSize,   strSize);
			if (strType != ShtStrtab || strSize == 0)
				return false;
			if (strOffset + strSize > imageSize || symtabOff + symtabSize > imageSize)
				return false;

			out.symtabOff  = symtabOff;
			out.symtabSize = symtabSize;
			out.strtab     = (const char*)(image + strOffset);
			out.strtabSize = strSize;
			return true;
		}

		struct SymbolHit
		{
			u64 value;        // st_value (virtual address)
			u8  info;         // st_info (bind << 4 | type)
		};

		bool FindSymbol(const u8* image, u64 imageSize, const char* name, SymbolHit& out)
		{
			using namespace ElfX64;

			SymtabInfo st;
			if (!LocateSymtab(image, imageSize, st))
				return false;

			u64 nameLen = 0; while (name[nameLen]) ++nameLen;

			for (u64 off = 0; off + SymEntSize <= st.symtabSize; off += SymEntSize)
			{
				const u8* sym = image + st.symtabOff + off;
				u32 stName = 0; u64 stValue = 0;
				ReadLE(sym, SymOffName,  stName);
				ReadLE(sym, SymOffValue, stValue);
				if (stName == 0 || stName >= st.strtabSize)
					continue;
				if (stName + nameLen + 1 > st.strtabSize)
					continue;
				const char* nm = st.strtab + stName;
				if (nm[nameLen] != 0)
					continue;
				bool match = true;
				for (u64 k = 0; k < nameLen; ++k) { if (nm[k] != name[k]) { match = false; break; } }
				if (!match)
					continue;
				out.value = stValue;
				out.info  = sym[SymOffInfo];
				return true;
			}
			return false;
		}

		u64 FindSymbolValue(const u8* image, u64 imageSize, const char* name)
		{
			SymbolHit hit{};
			return FindSymbol(image, imageSize, name, hit) ? hit.value : 0;
		}

		// Returns ~0ULL if no PT_LOAD covers vaddr.
		u64 VaddrToFileOffset(const u8* image, u64 imageSize, u64 vaddr)
		{
			using namespace ElfX64;
			if (imageSize < 64)
				return ~0ULL;
			u64 ePhoff = 0;
			u16 ePhentsize = 0, ePhnum = 0;
			ReadLE(image, OffEPhoff,     ePhoff);
			ReadLE(image, OffEPhentsize, ePhentsize);
			ReadLE(image, OffEPhnum,     ePhnum);
			if (ePhentsize != PhdrEntSize)
				return ~0ULL;

			for (u16 i = 0; i < ePhnum; ++i)
			{
				u64 phdrOff = ePhoff + u64(i) * PhdrEntSize;
				if (phdrOff + PhdrEntSize > imageSize)
					return ~0ULL;
				u32 pType = 0;
				u64 pOffset = 0, pVaddr = 0, pFilesz = 0;
				ReadLE(image, phdrOff +  0, pType);
				ReadLE(image, phdrOff +  8, pOffset);
				ReadLE(image, phdrOff + 16, pVaddr);
				ReadLE(image, phdrOff + 32, pFilesz);
				if (pType != PtLoad)
					continue;
				if (vaddr >= pVaddr && vaddr < pVaddr + pFilesz)
					return pOffset + (vaddr - pVaddr);
			}
			return ~0ULL;
		}

		// Glibc emits multiple aliases at the same VA — first match wins. Priority: public __libc_* (strong T) → __name → version-tagged.
		struct GlibcAliasRow
		{
			const char* tableName;          // matches GlibcHandlerEntry::name
			const char* aliases[6];         // null-terminated list
		};

		constexpr GlibcAliasRow kGlibcAliasTable[] = {
			// MVP set (used for IsGlibcStaticBinary detection + patching)
			{ "open",       { "__libc_open",  "__open",                           nullptr } },
			{ "openat",     { "__openat",     "__libc_openat",                    nullptr } },
			{ "read",       { "__libc_read",  "__read",                           nullptr } },
			{ "write",      { "__libc_write", "__write",                          nullptr } },
			{ "close",      { "__libc_close", "__close",                          nullptr } },
			{ "lseek",      { "__libc_lseek", "__lseek",                          nullptr } },
			{ "fxstat",     { "__fxstat",     "__fxstat64",                       nullptr } },
			{ "mmap",       { "__mmap",       "__mmap64",                         nullptr } },
			{ "access",     { "__access",                                         nullptr } },
			// _exit/_Exit/__exit all resolve to the same address in glibc.
			{ "exit",       { "_exit", "_Exit", "__exit",                         nullptr } },

			// Abort-on-reach rows (patching only; not counted by detection). Glibc exposes legacy __xstat/__lxstat/__fxstatat + modern stat/lstat/fstatat variants.
			{ "stat",       { "__xstat",      "__xstat64",    "stat",   "stat64",   nullptr } },
			{ "lstat",      { "__lxstat",     "__lxstat64",   "lstat",  "lstat64",  nullptr } },
			{ "fstatat",    { "__fxstatat",   "__fxstatat64", "fstatat","newfstatat", nullptr } },
			{ "statx",      { "statx",                                              nullptr } },
			{ "unlink",     { "__unlink",     "unlink",                             nullptr } },
			{ "unlinkat",   { "__unlinkat",   "unlinkat",                           nullptr } },
			{ "rename",     { "__rename",     "rename",                             nullptr } },
			{ "renameat",   { "__renameat",   "renameat",                           nullptr } },
			{ "renameat2",  { "renameat2",                                          nullptr } },
			{ "link",       { "__link",       "link",                               nullptr } },
			{ "linkat",     { "__linkat",     "linkat",                             nullptr } },
			{ "symlink",    { "__symlink",    "symlink",                            nullptr } },
			{ "symlinkat",  { "__symlinkat",  "symlinkat",                          nullptr } },
			{ "readlink",   { "__readlink",   "readlink",                           nullptr } },
			{ "readlinkat", { "__readlinkat", "readlinkat",                         nullptr } },
			{ "truncate",   { "__truncate",   "__truncate64", "truncate", "truncate64", nullptr } },
			{ "ftruncate",  { "__ftruncate",  "__ftruncate64","ftruncate","ftruncate64", nullptr } },
			{ "mkdir",      { "__mkdir",      "mkdir",                              nullptr } },
			{ "mkdirat",    { "__mkdirat",    "mkdirat",                            nullptr } },
			{ "rmdir",      { "__rmdir",      "rmdir",                              nullptr } },
			{ "fork",       { "__fork",       "__libc_fork",  "fork",               nullptr } },
			{ "vfork",      { "__vfork",      "vfork",                              nullptr } },
			{ "clone",      { "__clone",      "clone",                              nullptr } },
			{ "clone3",     { "clone3",                                             nullptr } },
			{ "execve",     { "__execve",     "__libc_execve","execve",             nullptr } },
			{ "execveat",   { "__execveat",   "execveat",                           nullptr } },
			{ "dup",        { "__dup",        "dup",                                nullptr } },
			{ "dup2",       { "__dup2",       "dup2",                               nullptr } },
			{ "dup3",       { "__dup3",       "dup3",                               nullptr } },
			{ "pipe",       { "__pipe",       "pipe",                               nullptr } },
			{ "pipe2",      { "__pipe2",      "pipe2",                              nullptr } },
			{ "opendir",    { "__opendir",    "opendir",                            nullptr } },
			{ "fdopendir",  { "__fdopendir",  "fdopendir",                          nullptr } },
			{ "readdir",    { "__readdir",    "__readdir64",  "readdir","readdir64",nullptr } },
			{ "closedir",   { "__closedir",   "closedir",                           nullptr } },
			{ "getdents64", { "__getdents64", "getdents64",                         nullptr } },
			{ "chdir",      { "__chdir",      "chdir",                              nullptr } },
			{ "fchdir",     { "__fchdir",     "fchdir",                             nullptr } },
		};
		constexpr u32 kGlibcAliasRowCount = sizeof(kGlibcAliasTable) / sizeof(kGlibcAliasTable[0]);
		// Only MVP-set rows count toward IsGlibcStaticBinary's "≥5 present" threshold.
		constexpr u32 kGlibcDetectionRowCount = 9;

		bool ResolveGlibcAlias(const u8* image, u64 imageSize, const GlibcAliasRow& row, SymbolHit& outHit, const char*& outName)
		{
			for (u32 i = 0; row.aliases[i]; ++i)
			{
				if (FindSymbol(image, imageSize, row.aliases[i], outHit))
				{
					outName = row.aliases[i];
					return true;
				}
			}
			return false;
		}

		inline bool IsHookableFunc(u8 info)
		{
			using namespace ElfX64;
			u8 type = info & 0xF;
			u8 bind = info >> 4;
			return type == SttFunc && (bind == StbGlobal || bind == StbWeak);
		}

		// Reject only SYSCALL prefix (0x0F 0x05) — our 5-byte JMP would silently consume it. Wrapper body is dead after patch.
		inline bool IsFirstByteSafeForJmpRel32(const u8* p)
		{
			return !(p[0] == 0x0F && p[1] == 0x05);
		}

		// Kept in sync with Build.sh — same magic, same scan.
		constexpr u64 kGlibcHandlerTableMagic = 0x5448424C47414255ULL;

		u64 FindGlibcHandlerTable(const u8* blob, u64 blobSize)
		{
			for (u64 i = 0; i + sizeof(u64) <= blobSize; i += sizeof(u64))
			{
				u64 v = 0;
				ReadLE(blob, i, v);
				if (v == kGlibcHandlerTableMagic)
					return i;
			}
			return ~0ULL;
		}

		// GlibcHandlerEntry on-disk shape (32 bytes): char name[24], u32 offset, u32 kind.
		constexpr u64 kGlibcEntryNameLen = 24;
		constexpr u64 kGlibcEntrySize    = 32;
		constexpr u64 kGlibcEntryOffOff  = 24;
		constexpr u64 kGlibcEntryKindOff = 28;
		constexpr u64 kGlibcTableHeader  = 16;   // magic(8) + count(4) + reserved(4)
		constexpr u32 kGlibcKindHandlerHook = 0;
		constexpr u32 kGlibcKindImportSlot  = 1;
	}

	namespace
	{
		// ELF64 LE x86-64 no-PT_INTERP pre-filter shared by IsGoStaticBinary / IsGlibcStaticBinary.
		bool IsBareElf64StaticImage(const u8* data, u64 size)
		{
			using namespace ElfX64;
			if (size < 64)
				return false;
			if (memcmp(data, ElfMagic, sizeof(ElfMagic)) != 0)
				return false;
			if (data[OffEiClass] != ElfClass64 || data[OffEiData] != ElfData2Lsb)
				return false;

			u16 eMachine = 0;
			ReadLE(data, OffEMachine, eMachine);
			if (eMachine != EmX86_64)
				return false;

			u64 ePhoff = 0;
			u16 ePhentsize = 0, ePhnum = 0;
			ReadLE(data, OffEPhoff, ePhoff);
			ReadLE(data, OffEPhentsize, ePhentsize);
			ReadLE(data, OffEPhnum, ePhnum);
			if (ePhentsize != PhdrEntSize)
				return false;

			for (u16 i = 0; i < ePhnum; ++i)
			{
				u64 phdrOff = ePhoff + u64(i) * PhdrEntSize;
				if (phdrOff + sizeof(u32) > size)
					return false;
				u32 pType = 0;
				ReadLE(data, phdrOff, pType);
				if (pType == PtInterp)
					return false;
			}
			return true;
		}
	}

	bool IsGoStaticBinary(Logger& logger, StringView path, const u8* inputData, u32 inputSize);
	bool IsGlibcStaticBinary(Logger& logger, StringView path, const u8* inputData, u32 inputSize);
	bool PatchGoStaticBinary(Logger& logger, StringView inputPath, const u8* inputData, u32 inputSize, StringView outputPath, StringView stubBlobPath);
	bool PatchGlibcStaticBinary(Logger& logger, StringView inputPath, const u8* inputData, u32 inputSize, StringView outputPath, StringView stubBlobPath);

	bool IsGoStaticBinary(Logger& logger, StringView inputPath, const u8* inputData, u32 inputSize)
	{
		if (!IsBareElf64StaticImage(inputData, inputSize))
			return false;

		// Go-static binaries always export Syscall6. 1.21+ uses internal/runtime/syscall.Syscall6; pre-1.21 uses runtime/internal/syscall.Syscall6.
		if (FindSymbolValue(inputData, inputSize, Syscall6SymbolName) != 0)
			return true;
		return FindSymbolValue(inputData, inputSize, Syscall6SymbolNameOld) != 0;
	}

	namespace
	{
		struct InjectedStub
		{
			Vector<u8> out;             // assembled output ELF
			u64        blobFileOff;     // byte offset of blob inside `out`
			u64        blobVaddr;       // load VA of the blob in the new PT_LOAD
			u64        origEntry;       // pre-patch e_entry (also baked into the sentinel)
			u64        sentinelOffset;
			u64        blobSize;
		};

		// Inject stub blob into a static ELF: pad to page, append PHDR table + blob, new PT_LOAD, rewrite original-entry sentinel,
		// update e_entry/e_phoff/e_phnum. Does NOT write the file.
		bool InjectStubBlob(Logger& logger, StringView inputPath, const u8* inputData, u32 inputSize, StringView stubBlobPath, InjectedStub& out)
		{
			using namespace ElfX64;

			Vector<u8> image;
			image.resize(inputSize);
			memcpy(image.data(), inputData, inputSize);

			Vector<u8> blob;
			if (!ReadAll(logger, stubBlobPath, blob))
				return logger.Error(TC("Failed to read stub blob: %s"), stubBlobPath);

			if (image.size() < 64)
				return logger.Error(TC("Input too small to be ELF: %s"), inputPath);
			if (memcmp(image.data(), ElfMagic, sizeof(ElfMagic)) != 0)
				return logger.Error(TC("Not an ELF file: %s"), inputPath);
			if (image[OffEiClass] != ElfClass64)
				return logger.Error(TC("Only ELF64 supported: %s"), inputPath);
			if (image[OffEiData] != ElfData2Lsb)
				return logger.Error(TC("Only little-endian ELF supported: %s"), inputPath);

			u16 eMachine = 0;
			ReadLE(image.data(), OffEMachine, eMachine);
			if (eMachine != EmX86_64)
				return logger.Error(TC("Only x86_64 supported (got e_machine=%u): %s"), (u32)eMachine, inputPath);

			u64 eEntry = 0, ePhoff = 0;
			u16 ePhentsize = 0, ePhnum = 0;
			ReadLE(image.data(), OffEEntry, eEntry);
			ReadLE(image.data(), OffEPhoff, ePhoff);
			ReadLE(image.data(), OffEPhentsize, ePhentsize);
			ReadLE(image.data(), OffEPhnum, ePhnum);

			if (ePhentsize != PhdrEntSize)
				return logger.Error(TC("Unexpected e_phentsize %u"), (u32)ePhentsize);

			// Refuse dynamic ELFs, track highest VA.
			u64 maxVaddrEnd = 0;
			for (u16 i = 0; i < ePhnum; ++i)
			{
				u64 phdrOff = ePhoff + u64(i) * PhdrEntSize;
				if (phdrOff + PhdrEntSize > image.size())
					return logger.Error(TC("Corrupt PHDR table"));
				u32 pType = 0;
				u64 pVaddr = 0, pMemsz = 0;
				ReadLE(image.data(), phdrOff + 0,  pType);
				ReadLE(image.data(), phdrOff + 16, pVaddr);
				ReadLE(image.data(), phdrOff + 40, pMemsz);
				if (pType == PtInterp)
					return logger.Error(TC("%s is dynamically linked (PT_INTERP present) — use LD_PRELOAD path instead"), inputPath);
				if (pType == PtLoad)
				{
					u64 end = pVaddr + pMemsz;
					if (end > maxVaddrEnd)
						maxVaddrEnd = end;
				}
			}
			if (maxVaddrEnd == 0)
				return logger.Error(TC("No PT_LOAD segments in %s"), inputPath);

			u64 sentinelOffset = 0;
			bool foundSentinel = false;
			for (u64 i = 0; i + sizeof(u64) <= blob.size(); ++i)
			{
				u64 v = 0;
				ReadLE(blob.data(), i, v);
				if (v == StubOriginalEntrySentinel)
				{
					sentinelOffset = i;
					foundSentinel = true;
					break;
				}
			}
			if (!foundSentinel)
				return logger.Error(TC("Sentinel 0xDEADBEEFCAFEBABE not found in %s — rebuild stub"), stubBlobPath);

			u64 newPhdrCount       = u64(ePhnum) + 1;
			u64 newPhdrTableSize   = newPhdrCount * PhdrEntSize;

			u64 newFileOff         = AlignUp(image.size(), Page);
			u64 newVaddr           = AlignUp(maxVaddrEnd, Page);

			u64 blobFileOff        = newFileOff + newPhdrTableSize;
			u64 blobVaddr          = newVaddr   + newPhdrTableSize;

			u64 segFilesz          = newPhdrTableSize + blob.size();
			// segMemsz > segFilesz makes the kernel zero-fill trailing pages. Stub's .got/.tbss/.relro_padding live just past blob's .text bytes
			// (linker script doesn't catch them); without BSS extension, GOT loads from TLS-aware code SIGSEGV — with it, they return 0.
			u64 segMemsz           = AlignUp(segFilesz + 32 * 1024, Page);

			u64 finalSize = newFileOff + newPhdrTableSize + blob.size();
			out.out.resize(finalSize, 0);
			memcpy(out.out.data(), image.data(), image.size());

			u8* newPhdrs = out.out.data() + newFileOff;
			memcpy(newPhdrs, image.data() + ePhoff, u64(ePhnum) * PhdrEntSize);
			u8* newEntryPtr = newPhdrs + u64(ePhnum) * PhdrEntSize;
			WriteLE<u32>(newEntryPtr,  0, PtLoad);
			// RWX: stub mixes code and writable globals in one blob.
			WriteLE<u32>(newEntryPtr,  4, PfR | PfW | PfX);
			WriteLE<u64>(newEntryPtr,  8, newFileOff);
			WriteLE<u64>(newEntryPtr, 16, newVaddr);
			WriteLE<u64>(newEntryPtr, 24, newVaddr);
			WriteLE<u64>(newEntryPtr, 32, segFilesz);
			WriteLE<u64>(newEntryPtr, 40, segMemsz);
			WriteLE<u64>(newEntryPtr, 48, Page);

			u8* blobOut = out.out.data() + blobFileOff;
			memcpy(blobOut, blob.data(), blob.size());
			WriteLE<u64>(blobOut, sentinelOffset, eEntry);

			// e_entry → uba_detour_init (blob offset 0), e_phoff → new PHDR table, e_phnum += 1.
			u64 newEntry = blobVaddr + StubInitOffset;
			WriteLE<u64>(out.out.data(), OffEEntry,  newEntry);
			WriteLE<u64>(out.out.data(), OffEPhoff,  newFileOff);
			WriteLE<u16>(out.out.data(), OffEPhnum,  u16(newPhdrCount));

			out.blobFileOff    = blobFileOff;
			out.blobVaddr      = blobVaddr;
			out.origEntry      = eEntry;
			out.sentinelOffset = sentinelOffset;
			out.blobSize       = blob.size();

			#if 0
			logger.Info(TC("Patched static binary: %s"), inputPath);
			logger.Info(TC("  original e_entry    = 0x%llx"), (unsigned long long)eEntry);
			logger.Info(TC("  new e_entry         = 0x%llx (stub)"), (unsigned long long)newEntry);
			logger.Info(TC("  new PT_LOAD vaddr   = 0x%llx (size 0x%llx)"), (unsigned long long)newVaddr, (unsigned long long)segMemsz);
			logger.Info(TC("  stub blob bytes     = %llu (sentinel at +%llu)"), (unsigned long long)blob.size(), (unsigned long long)sentinelOffset);
			#endif

			return true;
		}
	}

	bool PatchGoStaticBinary(Logger& logger, StringView inputPath, const u8* inputData, u32 inputSize, StringView outputPath, StringView stubBlobPath)
	{
		using namespace ElfX64;

		InjectedStub stub;
		if (!InjectStubBlob(logger, inputPath, inputData, inputSize, stubBlobPath, stub))
			return false;

		u8* blobOut = stub.out.data() + stub.blobFileOff;
		const u64 blobSize = stub.blobSize;

		// Bake Syscall6 / runtime.exit VAs into the blob. 16-byte [Sentinel][Guard] pair disambiguates from incidental compile-time-immediate matches.
		u64 sys6Off = 0;
		bool sys6SentinelFound = false;
		u64 exitOff = 0;
		bool exitSentinelFound = false;
		u64 vforkOff = 0;
		bool vforkSentinelFound = false;
		for (u64 i = 0; i + 2 * sizeof(u64) <= blobSize; i += sizeof(u64))
		{
			u64 a = 0, b = 0;
			ReadLE(blobOut, i,                  a);
			ReadLE(blobOut, i + sizeof(u64), b);
			if (!sys6SentinelFound && a == StubSyscall6Sentinel && b == StubSyscall6Guard)
			{
				sys6Off = i;
				sys6SentinelFound = true;
			}
			if (!exitSentinelFound && a == StubExitSentinel && b == StubExitGuard)
			{
				exitOff = i;
				exitSentinelFound = true;
			}
			if (!vforkSentinelFound && a == StubVforkSentinel && b == StubVforkGuard)
			{
				vforkOff = i;
				vforkSentinelFound = true;
			}
			if (sys6SentinelFound && exitSentinelFound && vforkSentinelFound)
				break;
		}
		u64 sys6Addr = FindSymbolValue(stub.out.data(), stub.out.size(), Syscall6SymbolName);
		if (sys6Addr == 0)
			sys6Addr = FindSymbolValue(stub.out.data(), stub.out.size(), Syscall6SymbolNameOld);
		if (sys6SentinelFound && sys6Addr != 0)
			WriteLE<u64>(blobOut, sys6Off, sys6Addr);

		u64 exitAddr = FindSymbolValue(stub.out.data(), stub.out.size(), ExitSymbolName);
		if (exitSentinelFound && exitAddr != 0)
			WriteLE<u64>(blobOut, exitOff, exitAddr);

		// Only present in Go binaries that use os/exec.
		u64 vforkAddr = FindSymbolValue(stub.out.data(), stub.out.size(), VforkSymbolName);
		if (vforkSentinelFound && vforkAddr != 0)
			WriteLE<u64>(blobOut, vforkOff, vforkAddr);

		#if 0
		logger.Info(TC("  syscall6 VA         = 0x%llx (%s)"), (unsigned long long)sys6Addr, (sys6Addr != 0 ? TC("baked") : TC("not found — stub won't install Go hook")));
		logger.Info(TC("  runtime.exit VA     = 0x%llx (%s)"), (unsigned long long)exitAddr, (exitAddr != 0 ? TC("baked") : TC("not found — stub won't send Exit RPC")));
		logger.Info(TC("  output -> %s"), outputPath);
		#endif

		if (!WriteAll(logger, outputPath, stub.out.data(), stub.out.size()))
			return logger.Error(TC("Failed to write output: %s"), outputPath);

		if (chmod(outputPath.CheckTerminated(), 0755) == -1)
			return logger.Error(TC("chmod failed!"));

		return true;
	}

	bool IsGlibcStaticBinary(Logger& logger, StringView path, const u8* inputData, u32 inputSize)
	{
		if (!IsBareElf64StaticImage(inputData, inputSize))
			return false;

		// No symtab → can't hook by name (musl-static stripped case).
		SymtabInfo dummy;
		if (!LocateSymtab(inputData, inputSize, dummy))
		{
			logger.Info(TC("%s has no .symtab — glibc-static detector cannot identify hooks (likely musl-static or stripped)"), path);
			return false;
		}

		// Count MVP libc symbols that pass: present, FUNC, global/weak, first 5 bytes safe. Threshold 5 of kGlibcDetectionRowCount tolerates partial
		// stripping. Abort-handler rows past the MVP set are NOT counted — only patched when present.
		u32 hits = 0;
		for (u32 r = 0; r < kGlibcDetectionRowCount; ++r)
		{
			SymbolHit h{};
			const char* matchedName = nullptr;
			if (!ResolveGlibcAlias(inputData, inputSize, kGlibcAliasTable[r], h, matchedName))
				continue;
			if (!IsHookableFunc(h.info))
				continue;
			u64 fileOff = VaddrToFileOffset(inputData, inputSize, h.value);
			if (fileOff == ~0ULL || fileOff + 5 > inputSize)
				continue;
			if (!IsFirstByteSafeForJmpRel32(inputData + fileOff))
				continue;
			++hits;
		}

		constexpr u32 kMinHits = 5;
		if (hits < kMinHits)
			return false;

		return true;
	}

	bool PatchGlibcStaticBinary(Logger& logger, StringView inputPath, const u8* inputData, u32 inputSize, StringView outputPath, StringView stubBlobPath)
	{
		using namespace ElfX64;

		InjectedStub stub;
		if (!InjectStubBlob(logger, inputPath, inputData, inputSize, stubBlobPath, stub))
			return false;

		u8* outData  = stub.out.data();
		u64 outSize  = stub.out.size();
		u8* blobOut  = outData + stub.blobFileOff;

		u64 tableOff = FindGlibcHandlerTable(blobOut, stub.blobSize);
		if (tableOff == ~0ULL)
			return logger.Error(TC("GlibcHandlerTable magic 0x5448424C47414255 not found in stub blob — rebuild DetoursStatic/UbaStaticStub.bin"));

		u32 entryCount = 0;
		ReadLE(blobOut, tableOff + 8, entryCount);
		if (entryCount == 0)
			return logger.Error(TC("GlibcHandlerTable count == 0 — Phase 0 scaffold lost?"));

		const u64 entriesStart = tableOff + kGlibcTableHeader;
		if (entriesStart + u64(entryCount) * kGlibcEntrySize > stub.blobSize)
			return logger.Error(TC("GlibcHandlerTable claims %u entries — overruns blob"), entryCount);

		// ImportSlot resolution failures are non-fatal — stub falls back.
		u32 hookedCount  = 0;
		u32 importStamped = 0;
		u32 importMissing = 0;
		u32 skippedCount = 0;
		for (u32 i = 0; i < entryCount; ++i)
		{
			const u64 e = entriesStart + u64(i) * kGlibcEntrySize;
			char nameBuf[25] = {};
			memcpy(nameBuf, blobOut + e, kGlibcEntryNameLen);
			nameBuf[kGlibcEntryNameLen] = 0;

			u32 entryOff  = 0;
			u32 entryKind = 0;
			ReadLE(blobOut, e + kGlibcEntryOffOff,  entryOff);
			ReadLE(blobOut, e + kGlibcEntryKindOff, entryKind);
			if (entryOff == 0)
				return logger.Error(TC("GlibcHandlerTable entry %u (%hs) has offset == 0 — DetoursStatic/Build.sh post-link stamping did not run; rebuild the stub blob"), i, nameBuf);
			if (entryOff >= stub.blobSize)
				return logger.Error(TC("GlibcHandlerTable entry %u (%hs) offset 0x%x is outside blob (size 0x%llx)"), i, nameBuf, entryOff, (unsigned long long)stub.blobSize);

			if (entryKind == kGlibcKindImportSlot)
			{
				if (entryOff + sizeof(u64) > stub.blobSize)
					return logger.Error(TC("GlibcHandlerTable ImportSlot entry %u (%hs) offset 0x%x + 8 exceeds blob (size 0x%llx)"), i, nameBuf, entryOff, (unsigned long long)stub.blobSize);

				SymbolHit imp{};
				if (FindSymbol(outData, outSize, nameBuf, imp) && imp.value != 0)
				{
					WriteLE<u64>(blobOut, entryOff, imp.value);
					logger.Info(TC("  imported %hs @ 0x%llx -> stub slot at blob+0x%x"),
						nameBuf, (unsigned long long)imp.value, entryOff);
					++importStamped;
				}
				else
				{
					logger.Info(TC("  imported %hs not present in target — stub falls back to local default"), nameBuf);
					++importMissing;
				}
				continue;
			}

			if (entryKind != kGlibcKindHandlerHook)
				return logger.Error(TC("GlibcHandlerTable entry %u (%hs) has unknown kind %u"), i, nameBuf, entryKind);

			const GlibcAliasRow* row = nullptr;
			for (u32 r = 0; r < kGlibcAliasRowCount; ++r)
			{
				bool match = true;
				for (u32 k = 0; k < kGlibcEntryNameLen; ++k)
				{
					char c = nameBuf[k];
					char d = kGlibcAliasTable[r].tableName[k];
					if (c == 0 && d == 0) break;
					if (c != d) { match = false; break; }
				}
				if (match) { row = &kGlibcAliasTable[r]; break; }
			}
			if (!row)
			{
				logger.Info(TC("  glibc handler '%hs' has no alias resolution — extend kGlibcAliasTable in UbaStaticPatcher.cpp"), nameBuf);
				++skippedCount;
				continue;
			}

			SymbolHit h{};
			const char* matchedName = nullptr;
			if (!ResolveGlibcAlias(outData, outSize, *row, h, matchedName))
			{
				++skippedCount;
				continue;
			}
			if (!IsHookableFunc(h.info))
			{
				logger.Info(TC("  '%hs' resolved as %hs but type/bind are not FUNC+GLOBAL/WEAK (info=0x%02x) — skipping"), nameBuf, matchedName, h.info);
				++skippedCount;
				continue;
			}

			u64 wrapperFileOff = VaddrToFileOffset(outData, outSize, h.value);
			if (wrapperFileOff == ~0ULL || wrapperFileOff + 5 > outSize)
			{
				logger.Info(TC("  '%hs' (%hs @ 0x%llx) — could not map to file offset; skipping"), nameBuf, matchedName, (unsigned long long)h.value);
				++skippedCount;
				continue;
			}
			if (!IsFirstByteSafeForJmpRel32(outData + wrapperFileOff))
				return logger.Error(TC("'%hs' (%hs @ 0x%llx): first 2 bytes are SYSCALL (0x0F 0x05) — patch would destroy the syscall opcode; refusing"), nameBuf, matchedName, (unsigned long long)h.value);

			// rel32 destination is computed from the END of the 5-byte instruction.
			u64 handlerVa = stub.blobVaddr + entryOff;
			int64_t rel = int64_t(handlerVa) - int64_t(h.value + 5);
			constexpr int64_t kRelMin = -(int64_t(1) << 31);
			constexpr int64_t kRelMax =  (int64_t(1) << 31) - 1;
			if (rel < kRelMin || rel > kRelMax)
				return logger.Error(TC("'%hs' (%hs @ 0x%llx → handler 0x%llx): rel32 displacement %lld is out of range; the new PT_LOAD landed too far from the original .text"), nameBuf, matchedName, (unsigned long long)h.value, (unsigned long long)handlerVa, (long long)rel);

			outData[wrapperFileOff + 0] = 0xE9;
			int32_t rel32 = int32_t(rel);
			memcpy(outData + wrapperFileOff + 1, &rel32, 4);
			++hookedCount;

			logger.Info(TC("  hooked %-7hs via %-18hs @ 0x%llx -> 0x%llx (rel32=%d)"),
				nameBuf, matchedName,
				(unsigned long long)h.value,
				(unsigned long long)handlerVa,
				(int)rel32);
		}

		logger.Info(TC("  glibc table: %u hooks / %u imports stamped / %u imports missing / %u skipped (count %u)"),
			hookedCount, importStamped, importMissing, skippedCount, entryCount);

		if (hookedCount == 0)
			return logger.Error(TC("PatchGlibcStaticBinary: no MVP libc wrappers were resolvable in %s — would not detour anything"), inputPath);

		if (!WriteAll(logger, outputPath, stub.out.data(), stub.out.size()))
			return logger.Error(TC("Failed to write output: %s"), outputPath);

		if (chmod(outputPath.CheckTerminated(), 0755) == -1)
			return logger.Error(TC("chmod failed!"));

		return true;
	}

	bool BinaryRequiresPatching(Logger& logger, StringView inputPath, const u8* inputData, u32 inputSize)
	{
		return IsGoStaticBinary(logger, inputPath, inputData, inputSize) || IsGlibcStaticBinary(logger, inputPath, inputData, inputSize);
	}

	// PatchStaticBinary refuses to re-patch — the inner stub's e_entry would re-trigger init and re-open UBA_LOGFILE with O_TRUNC, wiping the outer's writes.
	bool LooksAlreadyPatched(const u8* data, u32 size)
	{
		constexpr u64 kMagic = 0x5448424C47414255ULL;
		if (size < sizeof(kMagic)) return false;
		const u32 limit = size - (u32)sizeof(kMagic);
		for (u32 i = 0; i + sizeof(kMagic) <= size; i += sizeof(u64))
		{
			u64 v;
			memcpy(&v, data + i, sizeof(v));
			if (v == kMagic)
				return true;
		}
		// Tail-pad alignment stragglers.
		for (u32 i = (limit & ~(u32)7) + 1; i + sizeof(kMagic) <= size; ++i)
		{
			u64 v;
			memcpy(&v, data + i, sizeof(v));
			if (v == kMagic)
				return true;
		}
		return false;
	}

	bool PatchStaticBinary(Logger& logger, StringView inputPath, const u8* inputData, u32 inputSize, StringView outputPath, StringView stubBlobPath)
	{
		if (LooksAlreadyPatched(inputData, inputSize))
			return logger.Error(TC("PatchBinary: %.*s already contains an injected UBA stub blob (UBAGLBHT magic present); refusing to re-patch — rebuild the original from source"), inputPath.count, inputPath.data);

		if (IsGlibcStaticBinary(logger, inputPath, inputData, inputSize))
			return PatchGlibcStaticBinary(logger, inputPath, inputData, inputSize, outputPath, stubBlobPath);
		if (IsGoStaticBinary(logger, inputPath, inputData, inputSize))
			return PatchGoStaticBinary(logger, inputPath, inputData, inputSize, outputPath, stubBlobPath);
		return logger.Error(TC("PatchBinary: %s is not a recognised static-detour target (neither Go-static nor glibc-static)"), inputPath);
	}
}

#endif
