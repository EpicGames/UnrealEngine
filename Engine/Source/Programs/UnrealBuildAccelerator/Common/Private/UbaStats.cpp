// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaStats.h"
#include "UbaLogger.h"

namespace uba
{
	KernelStats g_kernelStatsInternal;
	KernelStats& g_kernelStats = g_kernelStatsInternal;

	inline void Write(BinaryWriter& writer, u64 v) { writer.Write7BitEncoded(v); }
	inline void Write(BinaryWriter& writer, u32 v) { writer.Write7BitEncoded(v); }

	inline void Read(BinaryReader& reader, u64& v, u32 version) { v = reader.Read7BitEncoded(); }
	inline void Read(BinaryReader& reader, u32& v, u32 version) { v = u32(reader.Read7BitEncoded()); }
	inline void Read(BinaryReader& reader, Timer& timer, u32 version) { timer.time = reader.Read7BitEncoded(); timer.count = (u32)reader.Read7BitEncoded(); }
	inline void Read(BinaryReader& reader, AtomicU64& v, u32 version) { v = reader.Read7BitEncoded(); }
	inline void Read(BinaryReader& reader, TimeAndBytes& timer, u32 version) { timer.time = reader.Read7BitEncoded(); timer.count = (u32)reader.Read7BitEncoded(); if (version >= 30) timer.bytes = reader.Read7BitEncoded(); }

	inline bool IsEmpty(u64 v) { return v == 0; }
	inline bool IsEmpty(u32 v) { return v == 0; }

	const tchar EmptyCharArray[] = TC("                    ");

	void ProcessStats::Print(Logger& logger, u64 frequency)
	{
		if (hostTotalTime)
		{
			logger.Info(TC("  Total              %8u %9s"), GetTotalCount(), TimeToText(GetTotalTime(), false, frequency).str);
			logger.Info(TC("  WaitOnResponse     %8u %9s"), waitOnResponse.count.load(), TimeToText(waitOnResponse.time, false, frequency).str);
			logger.Info(TC("  Host                %17s"), TimeToText(hostTotalTime, false, frequency).str);
			logger.Info(TC(""));

			struct Stat { const char* name; u64 nameLen; const Timer& timer; };
			Stat stats[] =
			{
				#define UBA_PROCESS_STAT(T, ver) { #T, sizeof(#T), T },
				UBA_PROCESS_STATS
				#undef UBA_PROCESS_STAT
			};

			for (Stat& s : stats)
				if (s.timer.count)
					logger.Info(TC("  %c") PERCENT_HS TC("%s%8u %9s"), ToUpper(s.name[0]), s.name + 1, EmptyCharArray + s.nameLen, s.timer.count.load(), TimeToText(s.timer.time, false, frequency).str);

			logger.Info(TC(""));

			logger.Info(TC("  Startup Time                %9s"), TimeToText(startupTime, false, frequency).str);
			logger.Info(TC("  Exit Time                   %9s"), TimeToText(exitTime, false, frequency).str);
			if (detoursMemory)
				logger.Info(TC("  DetoursMem                  %9s"), BytesToText(detoursMemory).str);
		}
		if (iopsRead)
			logger.Info(TC("  IopsRead                    %9llu"), iopsRead.load());
		if (iopsWrite)
			logger.Info(TC("  IopsWrite                   %9llu"), iopsWrite.load());
		if (iopsOther)
			logger.Info(TC("  IopsOther                   %9llu"), iopsOther.load());

		if (peakProcessMemory)
			logger.Info(TC("  PeakProcessMem (Win)        %9s"), BytesToText(peakProcessMemory).str);
		if (peakJobMemory)
			logger.Info(TC("  PeakJobMem (Win)            %9s"), BytesToText(peakJobMemory).str);
		if (peakMemory)
			logger.Info(TC("  PeakMem                     %9s"), BytesToText(peakMemory).str);
		if (cpuTime)
			logger.Info(TC("  CPU Time                    %9s"), TimeToText(cpuTime, false, frequency).str);
		logger.Info(TC("  Wall Time                   %9s"), TimeToText(wallTime, false, frequency).str);
	}

	u64 ProcessStats::GetTotalTime()
	{
		return 
		#define UBA_PROCESS_STAT(T, ver) + T.time
		UBA_PROCESS_STATS
		#undef UBA_PROCESS_STAT
			;
	}

	u32 ProcessStats::GetTotalCount()
	{
		return 
		#define UBA_PROCESS_STAT(T, ver) + T.count
			UBA_PROCESS_STATS
		#undef UBA_PROCESS_STAT
			;
	}

	void ProcessStats::Read(BinaryReader& reader, u32 version)
	{
		uba::Read(reader, waitOnResponse, version);

		if (version < 30)
		{
			#define UBA_PROCESS_STAT(T, ver) if (ver <= version) uba::Read(reader, T, version);
			UBA_PROCESS_STATS
			#undef UBA_PROCESS_STAT
		}
		else
		{
			u64 bits = reader.Read7BitEncoded();
			#define UBA_PROCESS_STAT(var, ver) if (bits & (1ull << Bit_##var)) uba::Read(reader, var, version);
			UBA_PROCESS_STATS
			#undef UBA_PROCESS_STAT
		}

		if (version >= 37)
		{
			startupTime = reader.Read7BitEncoded();
			exitTime = reader.Read7BitEncoded();
			wallTime = reader.Read7BitEncoded();
			cpuTime = reader.Read7BitEncoded();
			if (version >= 49)
			{
				peakProcessMemory = reader.Read7BitEncoded();
				peakJobMemory = reader.Read7BitEncoded();
			}
			detoursMemory = reader.Read7BitEncoded();
			peakMemory = reader.Read7BitEncoded();
			if (version >= 39)
			{
				iopsRead = reader.Read7BitEncoded();
				iopsWrite = reader.Read7BitEncoded();
				iopsOther = reader.Read7BitEncoded();
			}
			hostTotalTime = reader.Read7BitEncoded();
		}
		else
		{
			startupTime = reader.ReadU64();
			exitTime = reader.ReadU64();
			wallTime = reader.ReadU64();
			cpuTime = reader.ReadU64();
			detoursMemory = reader.ReadU32();
			hostTotalTime = reader.ReadU64();
		}
	}

	void ProcessStats::Add(const ProcessStats& other)
	{
		waitOnResponse += other.waitOnResponse;
			
		#define UBA_PROCESS_STAT(T, ver) T += other.T;
		UBA_PROCESS_STATS
		#undef UBA_PROCESS_STAT

		startupTime += other.startupTime;
		exitTime += other.exitTime;
		wallTime += other.wallTime;
		cpuTime += other.cpuTime;
		//detoursMemory = Max(detoursMemory, other.detoursMemory); // Handled on the outside
		//peakMemory = Max(peakMemory, other.peakMemory); // Handled on the outside
		iopsRead += other.iopsRead;
		iopsWrite += other.iopsWrite;
		iopsOther += other.iopsOther;
		hostTotalTime += other.hostTotalTime;
	}

	template<typename T>
	void LogStat(Logger& logger, const char* name, const T&, u64 frequency) {}

	void LogStat(Logger& logger, const char* name, const Timer& timer, u64 frequency)
	{
		if (!timer.count)
			return;
		logger.Info(TC("  %c") PERCENT_HS TC("%s%8u %9s"), ToUpper(name[0]), name+1, EmptyCharArray + strlen(name)+1, timer.count.load(), TimeToText(timer.time, false, frequency).str);
	}

	void LogStat(Logger& logger, const char* name, const ExtendedTimer& timer, u64 frequency)
	{
		LogStat(logger, name, (const Timer&)timer, frequency);
	}

	void LogStat(Logger& logger, const char* name, const TimeAndBytes& timer, u64 frequency)
	{
		if (!timer.count)
			return;
		logger.Info(TC("  %c") PERCENT_HS TC("%s%8u %9s"), ToUpper(name[0]), name+1, EmptyCharArray + strlen(name)+1, timer.count.load(), TimeToText(timer.time, false, frequency).str);
		if (timer.bytes)
			logger.Info(TC("     Bytes                    %9s"), BytesToText(timer.bytes).str);
	}

	void KernelStats::Print(Logger& logger, bool writeHeader, u64 frequency)
	{
		if (writeHeader)
			logger.Info(TC("  ------- Kernel stats summary --------"));

		#define UBA_KERNEL_STAT(type, var, ver) LogStat(logger, #var, var, frequency);
		UBA_KERNEL_STATS
		#undef UBA_KERNEL_STAT

		if (writeHeader)
			logger.Info(TC(""));
	}

	bool KernelStats::IsEmpty()
	{
		#define UBA_KERNEL_STAT(type, var, ver) if (var.count) return false;
		UBA_KERNEL_STATS
		#undef UBA_KERNEL_STAT
		return true;
	}

	void KernelStats::Add(const KernelStats& other)
	{
		#define UBA_KERNEL_STAT(type, var, ver) var += other.var;
		UBA_KERNEL_STATS
		#undef UBA_KERNEL_STAT
	}

	void KernelStats::Read(BinaryReader& reader, u32 version)
	{
		if (version < 30)
		{
			#define UBA_KERNEL_STAT(type, var, ver) if (ver <= version) uba::Read(reader, var, version);
			UBA_KERNEL_STATS
			#undef UBA_KERNEL_STAT
			return;
		}

		u64 bits;
		if (version < 43)
			bits = reader.ReadU16();
		else
			bits = reader.Read7BitEncoded();

		#define UBA_KERNEL_STAT(type, var, ver) if (bits & (1 << Bit_##var)) uba::Read(reader, var, version);
		UBA_KERNEL_STATS
		#undef UBA_KERNEL_STAT
	}

	void StorageStats::Write(BinaryWriter& writer)
	{
		u64 bits = 0;
		#define UBA_STORAGE_STAT(type, var, ver) if (!uba::IsEmpty(var)) bits |= (1 << Bit_##var);
		UBA_STORAGE_STATS
		#undef UBA_STORAGE_STAT

		writer.Write7BitEncoded(bits);

		#define UBA_STORAGE_STAT(type, var, ver) if (!uba::IsEmpty(var)) uba::Write(writer, var);
		UBA_STORAGE_STATS
		#undef UBA_STORAGE_STAT
	}

	void StorageStats::Read(BinaryReader& reader, u32 version)
	{
		if (version < 30)
		{
			#define UBA_STORAGE_STAT(type, var, ver) if (ver <= version) uba::Read(reader, var, version);
			UBA_STORAGE_STATS
			#undef UBA_STORAGE_STAT
			return;
		}

		u64 bits = reader.Read7BitEncoded();
		#define UBA_STORAGE_STAT(type, var, ver) if (bits & (1 << Bit_##var)) uba::Read(reader, var, version);
		UBA_STORAGE_STATS
		#undef UBA_STORAGE_STAT
	}

	void StorageStats::Add(const StorageStats& other)
	{
		#define UBA_STORAGE_STAT(type, var, ver) var += other.var;
		UBA_STORAGE_STATS
		#undef UBA_STORAGE_STAT
	}

	void StorageStats::Print(Logger& logger, u64 frequency)
	{
		if (calculateCasKey.count)
			logger.Info(TC("  CalculateCasKeys     %6u %9s"), calculateCasKey.count.load(), TimeToText(calculateCasKey.time, false, frequency).str);
		if (ensureCas.count)
			logger.Info(TC("  EnsureCas            %6u %9s"), ensureCas.count.load(), TimeToText(ensureCas.time, false, frequency).str);
		if (recvCas.count)
		{
			logger.Info(TC("  ReceiveCas           %6u %9s"), recvCas.count.load(), TimeToText(recvCas.time, false, frequency).str);
			logger.Info(TC("     Bytes Raw/Comp %9s %9s"), BytesToText(recvCasBytesRaw).str, BytesToText(recvCasBytesComp).str);
			if (decompressRecv.count)
				logger.Info(TC("     Decompress        %6u %9s"), decompressRecv.count.load(), TimeToText(decompressRecv.time, false, frequency).str);
		}
		if (sendCas.count)
		{
			logger.Info(TC("  SendCas              %6u %9s"), sendCas.count.load(), TimeToText(sendCas.time, false, frequency).str);
			logger.Info(TC("     Bytes Raw/Comp %9s %9s"), BytesToText(sendCasBytesRaw).str, BytesToText(sendCasBytesComp).str);
			logger.Info(TC("     Compress          %6u %9s"), compressSend.count.load(), TimeToText(compressSend.time, false, frequency).str);
		}
		if (createCas.count)
		{
			logger.Info(TC("  CreateCas            %6u %9s"), createCas.count.load(), TimeToText(createCas.time, false, frequency).str);
			logger.Info(TC("     Bytes Raw/Comp %9s %9s"), BytesToText(createCasBytesRaw).str, BytesToText(createCasBytesComp).str);
			logger.Info(TC("     Compress          %6u %9s"), compressWrite.count.load(), TimeToText(compressWrite.time, false, frequency).str);
		}
		if (copyOrLink.count)
			logger.Info(TC("  CopyOrLink           %6u %9s"), copyOrLink.count.load(), TimeToText(copyOrLink.time, false, frequency).str);
		if (copyOrLinkWait.count)
			logger.Info(TC("  CopyOrLinkWait       %6u %9s"), copyOrLinkWait.count.load(), TimeToText(copyOrLinkWait.time, false, frequency).str);
		if (compressWrite.count)
			logger.Info(TC("  CompressToMem        %6u %9s"), compressWrite.count.load(), TimeToText(compressWrite.time, false, frequency).str);
		if (decompressToMem.count)
			logger.Info(TC("  DecompressToMem      %6u %9s"), decompressToMem.count.load(), TimeToText(decompressToMem.time, false, frequency).str);

		if (memoryCopy.count)
			logger.Info(TC("  MemoryCopy           %6u %9s"), memoryCopy.count.load(), TimeToText(memoryCopy.time, false, frequency).str);
	}

	bool StorageStats::IsEmpty()
	{
		#define UBA_STORAGE_STAT(type, var, ver) if (var != type()) return false;
		UBA_STORAGE_STATS
		#undef UBA_STORAGE_STAT
		return true;
	}

	thread_local StorageStats* t_storageStats;

	StorageStats* StorageStats::GetCurrent()
	{
		return t_storageStats;
	}


	StorageStatsScope::StorageStatsScope(StorageStats& s) : stats(s)
	{
		t_storageStats = &stats;
	}

	StorageStatsScope::~StorageStatsScope()
	{
		t_storageStats = nullptr;
	}


	void SessionStats::Write(BinaryWriter& writer)
	{
		u16 bits = 0;
		#define UBA_SESSION_STAT(type, var, ver) if (var.count) bits |= (1 << Bit_##var);
		UBA_SESSION_STATS
		#undef UBA_SESSION_STAT

		writer.WriteU16(bits);

		#define UBA_SESSION_STAT(type, var, ver) if (var.count) uba::Write(writer, var);
		UBA_SESSION_STATS
		#undef UBA_SESSION_STAT
	}

	void SessionStats::Read(BinaryReader& reader, u32 version)
	{
		if (version < 30)
		{
			#define UBA_SESSION_STAT(type, var, ver) if (ver <= version) uba::Read(reader, var, version);
			UBA_SESSION_STATS
			#undef UBA_SESSION_STAT
			return;
		}

		u16 bits = reader.ReadU16();
		#define UBA_SESSION_STAT(type, var, ver) if (bits & (1 << Bit_##var)) uba::Read(reader, var, version);
		UBA_SESSION_STATS
		#undef UBA_SESSION_STAT
	}

	void SessionStats::Add(const SessionStats& other)
	{
		#define UBA_SESSION_STAT(type, var, ver) var += other.var;
		UBA_SESSION_STATS
		#undef UBA_SESSION_STAT
	}

	void SessionStats::Print(Logger& logger, u64 frequency)
	{
		#define UBA_SESSION_STAT(type, var, ver) LogStat(logger, #var, var, frequency);
		UBA_SESSION_STATS
		#undef UBA_SESSION_STAT
	}

	bool SessionStats::IsEmpty()
	{
		#define UBA_SESSION_STAT(type, var, ver) if (var.count) return false;
		UBA_SESSION_STATS
		#undef UBA_SESSION_STAT
		return true;
	}

	thread_local SessionStats* t_sessionStats;

	SessionStats* SessionStats::GetCurrent()
	{
		return t_sessionStats;
	}


	SessionStatsScope::SessionStatsScope(SessionStats& s) : stats(s)
	{
		t_sessionStats = &stats;
	}

	SessionStatsScope::~SessionStatsScope()
	{
		t_sessionStats = nullptr;
	}

	void SessionSummaryStats::Write(BinaryWriter& writer)
	{
		stats.Write(writer);
		#define UBA_SESSION_SUMMARY_STAT(type, var) uba::Write(writer, var);
		UBA_SESSION_SUMMARY_STATS
		#undef UBA_SESSION_SUMMARY_STAT
	}

	void SessionSummaryStats::Read(BinaryReader& reader, u32 version)
	{
		stats.Read(reader, version);
		#define UBA_SESSION_SUMMARY_STAT(type, var) uba::Read(reader, var, version);
		UBA_SESSION_SUMMARY_STATS
		#undef UBA_SESSION_SUMMARY_STAT
	}

	void SessionSummaryStats::Print(Logger& logger, u64 frequency)
	{
		#define UBA_SESSION_SUMMARY_STAT(T, V) LogStat(logger, #V, V, frequency);
		UBA_SESSION_SUMMARY_STATS
		#undef UBA_SESSION_SUMMARY_STAT
		stats.Print(logger, frequency);
		logger.Info(TC("  MemoryPressureWait          %9s"), TimeToText(waitMemPressure, false, frequency).str);
		logger.Info(TC("  ProcessesKilled             %9llu"), killCount);
		logger.Info(TC(""));
	}

	void CacheFetchStats::Write(BinaryWriter& writer)
	{
		#define UBA_CACHE_STAT(type, var, ver) uba::Write(writer, var);
		UBA_CACHE_FETCH_STATS
		#undef UBA_CACHE_STAT
	}

	void CacheFetchStats::Read(BinaryReader& reader, u32 version)
	{
		#define UBA_CACHE_STAT(type, var, ver) if (ver <= version) uba::Read(reader, var, version);
		UBA_CACHE_FETCH_STATS
		#undef UBA_CACHE_STAT
	}

	void CacheFetchStats::Print(Logger& logger, u64 frequency)
	{
		#define UBA_CACHE_STAT(type, var, ver) LogStat(logger, #var, var, frequency);
		UBA_CACHE_FETCH_STATS
		#undef UBA_CACHE_STAT
		if (fetchBytesComp)
			logger.Info(TC("   Bytes   Raw/Comp %9s %9s"), BytesToText(fetchBytesRaw).str, BytesToText(fetchBytesComp).str);
	}

	bool CacheFetchStats::IsEmpty()
	{
		#define UBA_CACHE_STAT(type, var, ver) if (var != type()) return false;
		UBA_CACHE_FETCH_STATS
		#undef UBA_CACHE_STAT
		return true;
	}

	void CacheSendStats::Write(BinaryWriter& writer)
	{
		#define UBA_CACHE_STAT(type, var, ver) uba::Write(writer, var);
		UBA_CACHE_SEND_STATS
		#undef UBA_CACHE_STAT
	}

	void CacheSendStats::Read(BinaryReader& reader, u32 version)
	{
		#define UBA_CACHE_STAT(type, var, ver) if (ver <= version) uba::Read(reader, var, version);
		UBA_CACHE_SEND_STATS
		#undef UBA_CACHE_STAT
	}

	void CacheSendStats::Print(Logger& logger, u64 frequency)
	{
		#define UBA_CACHE_STAT(type, var, ver) LogStat(logger, #var, var, frequency);
		UBA_CACHE_SEND_STATS
		#undef UBA_CACHE_STAT
	}

	bool CacheSendStats::IsEmpty()
	{
		#define UBA_CACHE_STAT(type, var, ver) if (var != type()) return false;
		UBA_CACHE_SEND_STATS
		#undef UBA_CACHE_STAT
		return true;
	}
}
