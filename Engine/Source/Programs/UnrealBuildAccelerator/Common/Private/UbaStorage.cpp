// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaStorage.h"
#include "UbaApplicationRules.h"
#include "UbaBottleneck.h"
#include "UbaCompressedFileHeader.h"
#include "UbaConfig.h"
#include "UbaFileAccessor.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaDirectoryIterator.h"
#include "UbaStorageUtils.h"
#include "UbaTrace.h"
#include "UbaWorkManager.h"

#define UBA_CHECK_BUFFER_SLOTS 0

namespace uba
{
	static_assert(sizeof(Atomic<u16>) == sizeof(u16));

	constexpr u32 CasTableVersionWithoutIsExecutable = 34;

	constexpr u32 CasTableVersion = IsWindows ? 32 : 35;
	constexpr u32 MaxWorkItemsPerAction = 128; // Cap this to not starve other things

    CasKey EmptyFileKey = []() { CasKeyHasher hasher; return ToCasKey(hasher, false); }();

	StorageCreateInfo::StorageCreateInfo(const tchar* rootDir_, LogWriter& w, WorkManager& wm)
	:	writer(w)
	,	workManager(wm)
	,	rootDir(rootDir_)
	,	casCompressor(DefaultCompressor)
	,	casCompressionLevel(DefaultCompressionLevel)
	{
	}

	void StorageCreateInfo::Apply(const Config& config)
	{
		const ConfigTable* tablePtr = config.GetTable(TC("Storage"));
		if (!tablePtr)
			return;
		const ConfigTable& table = *tablePtr;
		table.GetValueAsString(rootDir, TC("RootDir"));
		table.GetValueAsBool(writeToDisk, TC("WriteToDisk"));
		table.GetValueAsBool(asyncUnmapViewOfFile, TC("AsyncUnmapViewOfFile"));
		table.GetValueAsBytes(casCapacityBytes, TC("CasCapacityBytes"));
		table.GetValueAsBool(storeCompressed, TC("StoreCompressed"));
		table.GetValueAsBool(allowDeferredDeletes, TC("AllowDeferredDeletes"));
		table.GetValueAsBool(allowMemfd, TC("AllowMemfd"));
		table.GetValueAsBool(skipCleanupOnShutdown, TC("SkipCleanupOnShutdown"));
		table.GetValueAsBool(allowUnDropOfCas, TC("AllowUnDropOfCas"));
		table.GetValueAsBool(sideStepWineForSharedMemory, TC("SideStepWineForSharedMemory"));
		table.GetValueAsU32(fileEntryTableWipeThreshold, TC("FileEntryTableWipeThreshold"));
		table.GetValueAsU32(maxParallelCopyOrLink, TC("MaxParallelCopyOrLink"));
		table.GetValueAsU32(deleteCasBatchSize, TC("DeleteCasBatchSize"));

		table.GetValueAsBytes(sharedMemoryCapacity, TC("SharedMemoryCapacity"));
		table.GetValueAsBytes(sharedMemoryCommitSize, TC("SharedMemoryCommitSize"));
		table.GetValueAsBytes(sharedMemoryInitialCommit, TC("SharedMemoryInitialCommit"));
		table.GetValueAsString(sharedMemoryTempFile, TC("SharedMemoryTempFile"));
		table.GetValueAsBool(sharedMemoryTempFileSparse, TC("SharedMemoryTempFileSparse"));

		const tchar* compressor;
		if (table.GetValueAsString(compressor, TC("Compressor")))
			casCompressor = GetCompressor(compressor);

		const tchar* compressionLevel;
		if (table.GetValueAsString(compressionLevel, TC("CompressionLevel")))
			casCompressionLevel = GetCompressionLevel(compressionLevel);
	}

	struct MemoryMappingScope
	{
		u8* MapView(const tchar* func, SharedMemoryHandle memoryHandle, u64 memoryOffset, u64 fileSize, const tchar* fileName_)
		{
			handle = memoryHandle;
			fileName = fileName_;
			u64 alignedOffsetStart = AlignUp(memoryOffset - (PageSize - 1), u64(PageSize));
			mapSize = (memoryOffset - alignedOffsetStart) + fileSize;

			//u64 endOffset = memoryOffset + fileSize;
			//u64 alignedOffsetStart = AlignUp(memoryOffset - (PageSize - 1), PageSize);
			//u64 alignedOffsetEnd = AlignUp(endOffset, PageSize);
			//mapSize = alignedOffsetEnd - alignedOffsetStart;
			
			allocator.MapView(view, handle, fileName, SharedMemoryMapType_ReadOnly, alignedOffsetStart, mapSize);
			mappedMem = view.GetMemory();
			if (mappedMem)
				return mappedMem + (memoryOffset - alignedOffsetStart);
			logger.Error(TC("[%s] Failed to map view of file %s at offset %llu and size %llu (%s)"), func, fileName, alignedOffsetStart, mapSize, LastErrorToText().data);
			return nullptr;
		}

		MemoryMappingScope(Logger& l, SharedMemoryAllocator& a) : logger(l), allocator(a), view(a.GetFileMappingBackend()) {}
		~MemoryMappingScope()
		{
			if (!mappedMem)
				return;
			if (allocator.IsExternalMapping(handle))
				allocator.UnmapExternalMapping(handle, mappedMem, mapSize);
			view.Reset();
		}

		Logger& logger;
		SharedMemoryAllocator& allocator;
		SharedMemoryView view;

		const tchar* fileName;
		SharedMemoryHandle handle;
		u8* mappedMem = nullptr;
		u64 mapSize = 0;
	};

	StorageImpl::CasEntryReadLock::CasEntryReadLock(CasEntry& e) : entry(e)
	{
		active = true;
		entry.internalLock.EnterRead();
	}

	StorageImpl::CasEntryReadLock::~CasEntryReadLock()
	{
		if (active)
			entry.internalLock.LeaveRead();
	}

	StorageImpl::CasEntryWriteLock::CasEntryWriteLock(CasEntry& e) : entry(e)
	{
		active = true;
		entry.internalLock.Enter();
	}

	StorageImpl::CasEntryWriteLock::~CasEntryWriteLock()
	{
		if (active)
			entry.internalLock.Leave();
	}

	void StorageImpl::CasEntryWriteLock::Enter()
	{
		if (active)
			return;
		active = true;
		entry.internalLock.Enter();
	}

	void StorageImpl::CasEntryWriteLock::Leave()
	{
		if (!active)
			return;
		active = false;
		entry.internalLock.Leave();
	}

	void StorageImpl::CasEntryAccessed(CasEntry& casEntry)
	{
		{
			CasEntryReadLock casEntryLock(casEntry); // Note, this lock is taken again outside CasEntryAccessed.. so if this takes a long time it won't help to remove this lock
			if (casEntry.dropped)
				return;
			//if (casEntry.memoryHandle.IsValid())
			//	return;
		}

		SCOPED_FUTEX(m_accessLock, lock);

		CasEntry* prevAccessed = casEntry.prevAccessed;
		if (prevAccessed == nullptr)
		{
			if (m_newestAccessed == &casEntry) // We are already first
				return;
		}
		else
			prevAccessed->nextAccessed = casEntry.nextAccessed;

		if (casEntry.nextAccessed)
			casEntry.nextAccessed->prevAccessed = prevAccessed;
		else if (prevAccessed)
			m_oldestAccessed = prevAccessed;
		else if (!m_oldestAccessed)
			m_oldestAccessed = &casEntry;

		if (m_newestAccessed)
			m_newestAccessed->prevAccessed = &casEntry;
		casEntry.nextAccessed = m_newestAccessed;
		casEntry.prevAccessed = nullptr;
		m_newestAccessed = &casEntry;
	}

	void StorageImpl::CasEntryWritten(CasEntry& casEntry, u64 size)
	{
		SCOPED_FUTEX(m_accessLock, lock);

		m_casTotalBytes += size - casEntry.size;
		m_casMaxBytes = Max(m_casTotalBytes, m_casMaxBytes);

		casEntry.size = size;

		UBA_ASSERT(!m_casCapacityBytes || !casEntry.memoryHandle.IsValid());

		if (!m_casCapacityBytes || m_overflowReported || m_manuallyHandleOverflow)
			return;

		u64 casTotalBytes = m_casTotalBytes - m_casQueuedDeleteBytes;
		if (casTotalBytes <= m_casCapacityBytes)
		{
			if (!casEntry.prevAccessed && !casEntry.nextAccessed && (!m_newestAccessed || m_newestAccessed != &casEntry))
			{
				UBA_ASSERT(!casEntry.dropped);
				AttachEntry(casEntry);
			}
			return;
		}

		// Locking cas entries and then create work to delete them does not work since it can cause deadlocks
		// We need to estimate approximately how many files we need to delete and then kick the work
		u32 deleteCountNeeded = 0;
		for (CasEntry* it = m_oldestAccessed; it; it = it->prevAccessed)
		{
			++deleteCountNeeded;
			casTotalBytes -= it->size;
			m_casQueuedDeleteBytes += it->size;
			if (casTotalBytes <= m_casCapacityBytes)
				break;
		}

		lock.Leave();

		u32 workCount = (deleteCountNeeded + m_deleteCasBatchSize-1) / m_deleteCasBatchSize;

		if (m_allowDeferredDeletes)
		{
			m_workManager.AddWork([this](const WorkContext&) { DeleteOldestCasEntries(); }, workCount, TC("DeleteCasFiles"));
		}
		else
		{
			while (workCount--)
				DeleteOldestCasEntries();
		}
	}

	void StorageImpl::DeleteOldestCasEntries()
	{
		struct DeleteRec { CasEntry& casEntry; u64 size; };
		List<DeleteRec> toDelete;
		u32 left = m_deleteCasBatchSize;

		SCOPED_FUTEX(m_accessLock, lock);
		for (CasEntry* it = m_oldestAccessed; it;)
		{
			CasEntry& ce = *it;
			ce.internalLock.Enter();

			if (ce.verified && !m_allowDeleteVerified) // Can't remove these since they might be in mapping tables and actively being used. It is tempting to delete dropped cas files but we can't
			{
				ce.internalLock.Leave();
				break;
			}

			if (ce.beingWritten || ce.refCount)
			{
				if (ce.beingWritten)
					m_logger.Info(TC("Want to delete file that is begin written. We should never end up here"));
				ce.internalLock.Leave();
				it = ce.prevAccessed;
				continue;
			}

			UBA_ASSERT(ce.key != CasKeyZero);
			toDelete.emplace_back(ce, ce.size);

			if (m_trackedDeletes)
				m_trackedDeletes->insert(ce.key);

			m_casEvictedBytes += ce.size;
			++m_casEvictedCount;
			m_casTotalBytes -= ce.size;
			if (ce.size <= m_casQueuedDeleteBytes)
				m_casQueuedDeleteBytes -= ce.size;
			else
				m_casQueuedDeleteBytes = 0;

			ce.exists = false;
			ce.size = 0;

			it = ce.prevAccessed;
			DetachEntry(ce);

			if (!--left)
				break;
		}

		if (left != 0 && !m_overflowReported)
		{
			m_overflowReported = true;
			m_logger.Info(TC("Exceeding maximum size set for cas (%s). Current session needs more storage to be able to finish (will now overflow). User memory reported on session exit"), BytesToText(m_casCapacityBytes).str);
		}

		lock.Leave();

		for (DeleteRec& rec : toDelete)
		{
			CasEntry& ce = rec.casEntry;
			StringBuffer<> casFile;
			StorageImpl::GetCasFileName(casFile, ce.key);

			if (!DeleteFileW(casFile.data))
			{
				u32 error = GetLastError();
				if (error != ERROR_FILE_NOT_FOUND && error != ERROR_PATH_NOT_FOUND)
				{
					m_logger.Error(TC("Failed to delete %s while handling overflow (%s)"), casFile.data, LastErrorToText(error).data);
					ce.exists = true;
					ce.size = rec.size;
					ce.internalLock.Leave();
					continue;
				}
			}
			ce.verified = true; // Verified to be deleted
			ce.internalLock.Leave();
		}
	}

	void StorageImpl::CasEntryDeleted(CasEntry& casEntry, u64 size)
	{
		SCOPED_FUTEX(m_accessLock, lock);
		m_casTotalBytes -= size;
		casEntry.size = 0;
		DetachEntry(casEntry);
	}

	void StorageImpl::AttachEntry(CasEntry& casEntry)
	{
		if (m_oldestAccessed)
			m_oldestAccessed->nextAccessed = &casEntry;
		casEntry.prevAccessed = m_oldestAccessed;
		casEntry.nextAccessed = nullptr;
		if (!m_newestAccessed)
			m_newestAccessed = &casEntry;
		m_oldestAccessed = &casEntry;
	}

	void StorageImpl::DetachEntry(CasEntry& casEntry)
	{
		CasEntry* prevAccessed = casEntry.prevAccessed;
		if (prevAccessed)
			prevAccessed->nextAccessed = casEntry.nextAccessed;
		else if (m_newestAccessed == &casEntry)
			m_newestAccessed = casEntry.nextAccessed;

		if (casEntry.nextAccessed)
			casEntry.nextAccessed->prevAccessed = prevAccessed;
		else if (m_oldestAccessed == &casEntry)
			m_oldestAccessed = prevAccessed;

		casEntry.prevAccessed = nullptr;
		casEntry.nextAccessed = nullptr;
	}

	void StorageImpl::RegisterExternalFileMappingsProvider(ExternalFileMappingsProvider&& provider)
	{
		m_externalFileMappingsProvider = std::move(provider);
	}

	void StorageImpl::DeleteExternalFileMapping(StringKey fileNameKey, const tchar* fileName)
	{
		SCOPED_FUTEX(m_deferredCasCreationLookupLock, deferredLock);
		auto it = m_deferredCasCreationLookupByName.find(fileNameKey);
		if (it == m_deferredCasCreationLookupByName.end())
			return;
		UBA_ASSERT(it->second.manualDelete);
		CasKey key = it->second.key;
		m_deferredCasCreationLookupByName.erase(it);

		auto it2 = m_deferredCasCreationLookup.find(key);
		if (it2 == m_deferredCasCreationLookup.end())
		{
			UBA_ASSERT(false);
			return;
		}

		DeferredCasCreation& dcc = it2->second;
		++dcc.refCount;
		deferredLock.Leave();

		SCOPED_FUTEX(dcc.lock, dccLock);

		for (auto i=dcc.sources.begin(), e=dcc.sources.end(); i!=e; ++i)
		{
			if (*i != fileNameKey)
				continue;
			dcc.sources.erase(i);
			break;
		}

		deferredLock.Enter();
		if (!--dcc.refCount && dcc.sources.empty())
		{
			dccLock.Leave();
			m_deferredCasCreationLookup.erase(it2);
		}
	}

	bool StorageImpl::WriteCompressedFile(WriteResult& out, const tchar* from, FileHandle readHandle, u8* readMem, u64 fileSize, const tchar* to, const void* header, u64 headerSize, u64 lastWriteTime)
	{
		CompressWriter w(m_logger, m_sharedMemory, m_bufferSlots, m_workManager, Stats(), m_casCompressor, m_casCompressionLevel, m_asyncUnmapViewOfFile);
		return w.CompressToFile(out.storedSize, from, readHandle, readMem, fileSize, to, header, headerSize, lastWriteTime, m_tempPath.data);
	}

	bool StorageImpl::WriteCasFileNoCheck(WriteCasResult& out, const StringKey& fileNameKey, const tchar* fileName, const CasKey& casKey, const tchar* casFile, bool storeCompressed, bool useMapping, DeferredSource* deferredSource, bool allowFail)
	{
		StorageStats& stats = Stats();
		TimerScope ts(stats.createCas);

		if (m_externalFileMappingsProvider)
		{
			ExternalFileMapping externalMapping;
			auto externalMappingGuard = MakeGuard([&]() { m_sharedMemory.CloseHandle(m_logger, externalMapping.handle, TC("ExternalFileMapping")); });

			if (m_externalFileMappingsProvider(externalMapping, fileNameKey, fileName))
			{
				MemoryMappingScope scope(m_logger, m_sharedMemory);
				u8* fileMem = scope.MapView(TC("WriteCompressed"), externalMapping.handle, externalMapping.offset, externalMapping.size, fileName);
				if (!fileMem)
					return false;
				out.sourceStoredSize = externalMapping.storedSize;
				out.isIndependent = externalMapping.createIndependentMapping || useMapping;
				out.dropAfterUse = externalMapping.dropCasAfterUse || useMapping;
				if (out.isIndependent)
				{
					CompressWriter w(m_logger, m_sharedMemory, m_bufferSlots, m_workManager, Stats(), m_casCompressor, m_casCompressionLevel, m_asyncUnmapViewOfFile);
					out.memoryOffset = 0;
					return w.CompressToMapping(out.memoryHandle, out.storedSize, fileMem, externalMapping.size, fileName);//casFile);
				}
				else
				{
					UBA_ASSERTF(!out.dropAfterUse, TC("External file mapping %s has dropAfterUse set but is not an independent mapping"), fileName);
					return WriteCompressedFile(out, fileName, InvalidFileHandle, fileMem, externalMapping.size, casFile, nullptr, 0);
				}
			}
		}

		FileHandle readHandle;
		constexpr int maxOpenFileAttempts = 5;
		int numFailedOpenFileAttempts = 0;

		while (!OpenFileSequentialRead(m_logger, fileName, readHandle) && ++numFailedOpenFileAttempts < maxOpenFileAttempts)
			Sleep(100); // Delay next attempt for 100 milliseconds to open the file in case of a race condition

		if (numFailedOpenFileAttempts == maxOpenFileAttempts)
			return m_logger.Logf(allowFail?LogEntryType_Debug:LogEntryType_Error, TC("[WriteCasFileNoCheck] Failed to open file %s for read (%s)"), fileName, LastErrorToText().data).ToFalse();

		auto fileGuard = MakeGuard([&](){ CloseFile(fileName, readHandle); });

		if (numFailedOpenFileAttempts)
			m_logger.Warning(TC("[WriteCasFileNoCheck] First attempts (%d) to open file %s for read failed; This may indicate a race condition"), numFailedOpenFileAttempts, fileName);

		FileBasicInformation basicInfo;
		if (!GetFileBasicInformationByHandle(basicInfo, m_logger, fileName, readHandle, false))
			return m_logger.Error(TC("[WriteCasFileNoCheck] GetFileBasicInformationByHandle failed for %s (%s)"), fileName, LastErrorToText().data);
	
		if (deferredSource && deferredSource->lastWriteTime != basicInfo.lastWriteTime)
		{
			// If last write time is newer for deferred source, then it must be a bug where last write time is stored wrong when the file is written.
			LogEntryType logType = deferredSource->lastWriteTime < basicInfo.lastWriteTime ? LogEntryType_Warning : LogEntryType_Info;
			m_logger.Logf(logType, TC("[WriteCasFileNoCheck] Last write time has changed since %s was deferred for cas creation. (LastWrite %llu to %llu, Size %llu to %llu)"), fileName, deferredSource->lastWriteTime, basicInfo.lastWriteTime, deferredSource->fileSize, basicInfo.size);
			if (allowFail)
				return false;
		}
		u64 readFileSize = basicInfo.size;

		out.sourceStoredSize = readFileSize;

		bool readIsCompressed = false;
		if (readFileSize >= sizeof(CompressedFileHeader) && g_globalRules.FileCanBeCompressed(ToView(fileName)))
		{
			CompressedFileHeader header(CasKeyZero);
			if (!ReadFile(m_logger, fileName, readHandle, &header, sizeof(header)))
				return m_logger.Error(TC("Failed to read header of compressed file %s (%s)"), fileName, LastErrorToText().data);
			if (header.IsValid())
			{
				if (AsCompressed(casKey, false) != AsCompressed(header.casKey, false))
					return m_logger.Error(TC("Compressed file has different caskey than what was expected (%s)"), fileName);
				readIsCompressed = true;
				readFileSize -= sizeof(header);
			}
			else
			{
				if (!SetFilePointer(m_logger, fileName, readHandle, 0))
					return false;
			}
		}

		if (!readIsCompressed && storeCompressed)
		{
			if (useMapping)
			{
				CompressWriter w(m_logger, m_sharedMemory, m_bufferSlots, m_workManager, Stats(), m_casCompressor, m_casCompressionLevel, m_asyncUnmapViewOfFile);
				out.memoryOffset = 0;
				out.isIndependent = true;
				out.dropAfterUse = true; // This is only used for cache client right now.. so we know this is an output file that can be dropped
				return w.CompressToMappingFromFile(out.memoryHandle, out.storedSize, readHandle, readFileSize, fileName);
			}
			return WriteCompressedFile(out, fileName, readHandle, 0, readFileSize, casFile, nullptr, 0);
		}
		if (readIsCompressed && !storeCompressed)
			return m_logger.Error(TC("[WriteCasFileNoCheck] Writing compressed file to non-compressed storage not supported (%s)"), fileName);

		if (useMapping)
		{
			SharedMemoryHandle destMapping = m_sharedMemory.CreateHandle(m_logger, fileName);
			auto destGuard = MakeGuard([&]() { m_sharedMemory.CloseHandle(m_logger, destMapping, fileName); });
			m_sharedMemory.ExtendMemory(destMapping, readFileSize, fileName, false);
			u8* destData = m_sharedMemory.MapView(destMapping, fileName, SharedMemoryMapType_ReadWrite);
			if (!destData)
				return false;
			auto unmapGuard = MakeGuard([&]() { m_sharedMemory.UnmapView(destMapping, destData); });
			if (!ReadFile(m_logger, fileName, readHandle, destData, readFileSize))
				return false;
			destGuard.Cancel();

			out.memoryHandle = destMapping;
			out.storedSize = readFileSize;
			out.memoryOffset = 0;
			out.isIndependent = true;
			out.dropAfterUse = true; // This is only used for cache client right now.. so we know this is an output file that can be dropped
			return true;
		}

		FileAccessor destinationFile(m_logger, casFile);
		if (!destinationFile.CreateWrite(false, DefaultAttributes(), readFileSize, m_tempPath.data))
			return false;

		u8* slot= m_bufferSlots.Pop();
		auto _ = MakeGuard([&](){ m_bufferSlots.Push(slot); });
		u64 left = readFileSize;

		while (left)
		{
			u32 toRead = u32(Min(left, BufferSlotSize));
			if (!ReadFile(m_logger, fileName, readHandle, slot, toRead))
				return false;
			if (!destinationFile.Write(slot, toRead))
				return false;

			left -= toRead;
		}

		if (!destinationFile.Close())
			return false;

		out.storedSize = readFileSize;
		return true;
	}

	bool StorageImpl::WriteCasFile(WriteCasResult& out, const tchar* fileName, const CasKey& casKey, bool useMapping, DeferredSource* deferredSource, bool allowFail)
	{
		UBA_ASSERT(IsCompressed(casKey) == m_storeCompressed);
		StringBuffer<> casFile;
		if (!StorageImpl::GetCasFileName(casFile, casKey))
			return false;
		if (FileExists(m_logger, casFile.data))
			return true;

		StringBuffer<> temp;
		temp.Append(fileName);
		if (CaseInsensitiveFs)
			temp.MakeLower();
		StringKey fileNameKey = ToStringKey(temp);
		return WriteCasFileNoCheck(out, fileNameKey, fileName, casKey, casFile.data, IsCompressed(casKey), useMapping, deferredSource, allowFail);
	}

	void StorageImpl::CasEntryAccessed(const CasKey& casKey)
	{
		SCOPED_READ_LOCK(m_casLookupLock, lookupLock);
		auto findIt = m_casLookup.find(casKey);
		if (findIt == m_casLookup.end())
			return;
		CasEntry& casEntry = findIt->second;
		lookupLock.Leave();
		CasEntryAccessed(casEntry);
	}

	bool StorageImpl::IsDisallowedPath(const tchar* fileName)
	{
		return false;
	}

	bool StorageImpl::DecompressMemoryToMemory(const u8* compressedData, u64 compressedSize, u8* writeData, u64 decompressedSize, const tchar* readHint, const tchar* writeHint)
	{
		DecompressWriter w { m_logger, m_bufferSlots, m_workManager, Stats() };
		return w.DecompressMemoryToMemory(compressedData, compressedSize, writeData, decompressedSize, readHint, writeHint);
	}

	bool StorageImpl::VerifyExisting(bool& outReturnValue, CasEntryWriteLock& casEntryLock, const CasKey& casKey, CasEntry& casEntry, StringBufferBase& casFile)
	{
		u64 outFileSize = 0;
		u32 outAttributes = 0;
		if (!FileExists(m_logger, casFile.data, &outFileSize, &outAttributes))
			return false;

		bool isBad = (outFileSize == 0 && casKey != EmptyFileKey);

		if (isBad)
		{
			#if !PLATFORM_WINDOWS
			if (!outAttributes)
				m_logger.Info(TC("Found file %s with attributes 0 which means it was never written fully. Deleting"), casFile.data);
			else
			#endif
				m_logger.Info(TC("Found file %s with size 0 which did not have the zero-size-caskey. Deleting"), casFile.data);

			if (!DeleteFileW(casFile.data))
			{
				outReturnValue = false;
				m_logger.Error(TC("Failed to delete %s. Clean cas folder and restart"), casFile.data);
				return false;
			}
		}
		else
		{
			UBA_ASSERT(!casEntry.verified || casEntry.size);
			casEntry.verified = true;
			casEntry.exists = true;
			casEntryLock.Leave();
			CasEntryWritten(casEntry, outFileSize);
			outReturnValue = true;
			return true;
		}

		return false;
	}


	bool StorageImpl::AddCasFile(StringKey fileNameKey, const tchar* fileName, const CasKey& casKey, bool deferCreation, u64 fileSize, u64 lastWriteTime, CasEntry** outEntry, bool dropCasAfterUse)
	{
		UBA_ASSERTF(IsCompressed(casKey) == m_storeCompressed, TC("CasKey compress mode must match storage compress mode. Compress mode Key: %i, Store: %i Cas: %s (%s)"), (IsCompressed(casKey) ? 1 : 0), (m_storeCompressed ? 1 : 0), CasKeyString(casKey).str, fileName);
		CasEntry& casEntry = GetOrCreateCasEntry(casKey);
		CasEntryAccessed(casEntry);

		CasEntryWriteLock casEntryLock(casEntry);

		if (casEntry.beingWritten)
		{
			m_logger.Info(TC("Cas %s is being written from network while being added by host (%s)"), CasKeyString(casKey).str, fileName);
			int waitCount = 0;
			while (casEntry.beingWritten)
			{
				CasKey key = casEntry.key;
				casEntryLock.Leave();
				Sleep(100);
				casEntryLock.Enter();

				if (++waitCount < 12*60*10)
					continue;
				return m_logger.Error(TC("Host waited more than 12 minutes for file %s (%s) to be written by network (Written %llu/%llu)"), CasKeyString(key).str, fileName);
			}
		}

		// File is requested by server itself to provide to someone remote. We have to un-drop the cas in case it was marked as dropped
		if (casEntry.dropped && m_allowUnDropOfCas && !outEntry)
		{
			m_logger.Debug(TC("Un-dropped cas for %s (%s)."), fileName, BytesToText(casEntry.size).str);
			casEntry.dropped = false;
		}

		if (casEntry.verified)
			if (casEntry.exists)
			{
				if (outEntry)
				{
					++casEntry.refCount;
					*outEntry = &casEntry;
				}
				return true;
			}

		casEntry.disallowed = IsDisallowedPath(fileName);

		StringBuffer<> casFile;
		if (!StorageImpl::GetCasFileName(casFile, casKey)) // Force to not use subclassed version
			return false;
	
		bool verifyReturnValue;
		if (!casEntry.verified && VerifyExisting(verifyReturnValue, casEntryLock, casKey, casEntry, casFile))
			return verifyReturnValue;

		if (deferCreation)
		{
			SCOPED_FUTEX(m_deferredCasCreationLookupLock, deferredLock);

			auto res2 = m_deferredCasCreationLookupByName.try_emplace(fileNameKey, casKey);
			if (!res2.second)
			{
				if (res2.first->second.key != casKey)
					return m_logger.Error(TC("Same file %s is registered with different caskey (%s vs %s)"), fileName, CasKeyString(res2.first->second.key).str, CasKeyString(casKey).str);
				return true;
			}
			DeferredSource& source = res2.first->second;
			source.manualDelete = dropCasAfterUse;
			source.fileName = fileName;
			source.fileSize = fileSize;
			source.lastWriteTime = lastWriteTime;

			auto res = m_deferredCasCreationLookup.try_emplace(casKey);

			DeferredCasCreation& dcc = res.first->second;
			++dcc.refCount;
			deferredLock.Leave();

			SCOPED_FUTEX(dcc.lock, dccLock);
			dcc.sources.emplace_back(fileNameKey);

			deferredLock.Enter();
			--dcc.refCount;

			if (outEntry)
			{
				++casEntry.refCount;
				*outEntry = &casEntry;
			}

			return true;
		}
		else
		{
			SCOPED_FUTEX(m_deferredCasCreationLookupLock, deferredLock);
			#if UBA_DEBUG
			auto findIt = m_deferredCasCreationLookupByName.find(fileNameKey);
			if (findIt != m_deferredCasCreationLookupByName.end() && findIt->second.key != casKey)
				return m_logger.Error(TC("Found deferred entry for %s with different cas key while writing cas. A ReportFileWrite is missing somewhere"), fileName);
			#endif

			// Another thread registered a deferred creation for this casKey between our HasCasFile and now — drain it and fall through to the immediate write (casEntryLock keeps it safe).
			auto findIt2 = m_deferredCasCreationLookup.find(casKey);
			if (findIt2 != m_deferredCasCreationLookup.end())
			{
				DeferredCasCreation& dcc = findIt2->second;
				++dcc.refCount;
				deferredLock.Leave();

				SCOPED_FUTEX(dcc.lock, dccLock);
				deferredLock.Enter();
				for (auto i = dcc.sources.begin(); i!=dcc.sources.end();)
				{
					auto findIt3 = m_deferredCasCreationLookupByName.find(*i);
					if (findIt3 != m_deferredCasCreationLookupByName.end() && findIt3->second.manualDelete)
					{
						++i;
						continue;
					}
					m_deferredCasCreationLookupByName.erase(*i);
					i = dcc.sources.erase(i);
				}
				if (!--dcc.refCount && dcc.sources.empty())
				{
					dccLock.Leave();
					m_deferredCasCreationLookup.erase(findIt2);
				}
			}
		}

		casEntry.verified = true;
		casEntry.exists = false;

		WriteCasResult res;
		if (!WriteCasFileNoCheck(res, fileNameKey, fileName, casKey, casFile.data, IsCompressed(casKey), false, nullptr, false))
			return false;
		casEntry.memoryHandle = res.memoryHandle;
		casEntry.memoryOffset = res.memoryOffset;
		casEntry.memorySize = res.storedSize;
		casEntry.dropped = res.dropAfterUse;
		casEntry.exists = true;

		if (outEntry)
		{
			++casEntry.refCount;
			*outEntry = &casEntry;
		}

		casEntryLock.Leave();

		if (!res.memoryHandle.IsValid())
			CasEntryWritten(casEntry, res.storedSize);

		return true;
	}

	void StorageImpl::TraverseAllCasFiles(const tchar* dir, const Function<void(const StringBufferBase& fullPath, const DirectoryEntry& e)>& func, bool allowParallel)
	{
		Atomic<u32> workLeft = 0;

		auto traverseCasFileDir = [&func, &workLeft, this](const StringView& casFileDir)
			{
				TraverseDir(m_logger, casFileDir,
					[&](const DirectoryEntry& e)
					{
						UBA_ASSERT(!IsDirectory(e.attributes));
						StringBuffer<> fullPath(casFileDir);
						fullPath.EnsureEndsWithSlash().Append(e.name);
						func(fullPath, e);
					});
				--workLeft;
			};

		TraverseDir(m_logger, ToView(dir),
			[&](const DirectoryEntry& e)
			{
				if (!IsDirectory(e.attributes))
					return;
				StringBuffer<> fullPath(dir);
				fullPath.EnsureEndsWithSlash().Append(e.name);
				++workLeft;

				if (allowParallel)
				{
					m_workManager.AddWork([&traverseCasFileDir, p = fullPath.ToString()](const WorkContext&) { traverseCasFileDir(p); }, 1, TC("TraverseCasFiles"));
				}
				else
				{
					traverseCasFileDir(fullPath);
				}
			});

		while (workLeft)
			m_workManager.DoWork();
	}

	void StorageImpl::TraverseAllCasFiles(const Function<void(const CasKey& key, u64 size)>& func, bool allowParallel)
	{
		StringBuffer<> casRoot;
		casRoot.Append(m_rootDir.data, m_rootDir.count - 1);
		TraverseAllCasFiles(casRoot.data, [&](const StringBufferBase& fullPath, const DirectoryEntry& e)
			{
				func(CasKeyFromString(e.name), e.size);
			}, allowParallel);
	}

	bool StorageImpl::CheckAllCasFiles(u64 checkContentOfFilesNewerThanTime)
	{
		u64 startTime = GetTime();
		u64 before = m_casTotalBytes;
		m_casTotalBytes = 0;
		// Need to scan all files to see so there are no orphans in the folders
		StringBuffer<> casRoot;
		casRoot.Append(m_rootDir.data, m_rootDir.count - 1);

		Atomic<bool> success = true;
		TraverseAllCasFiles(casRoot.data, [&](const StringBufferBase& fullPath, const DirectoryEntry& e)
			{
				CasKey casKey = CasKeyFromString(e.name);
				bool deleteFile = false;
				u64 size = e.size;

				// Do quick check on new files so the have good content (ignore files with zero size, they are tested further down)
				if (size && e.lastWritten >= checkContentOfFilesNewerThanTime)
				{
					FileAccessor fa(m_logger, fullPath.data);
					if (fa.OpenMemoryRead())
					{
						if (IsCompressed(casKey))
						{
							BinaryReader reader(fa.GetData(), 0, fa.GetSize());
							if (reader.GetLeft() < 12)
							{
								m_logger.Detail(TC("Corrupt cas. Is %llu bytes, must be at least 12 (%s)"), reader.GetLeft(), e.name);
								deleteFile = true;
							}
							else
							{
								reader.ReadU64(); // Decompressed size
								while (true)
								{
									if (reader.GetLeft() <= 8)
									{
										m_logger.Detail(TC("Corrupt cas. Missing beginning of block (%s)"), e.name);
										deleteFile = true;
										break;
									}

									u32 compressedBlockSize = reader.ReadU32();
									reader.ReadU32();

									if (!compressedBlockSize || compressedBlockSize > reader.GetLeft())
									{
										m_logger.Detail(TC("Corrupt cas. Bad block (%s)"), e.name);
										deleteFile = true;
										break;
									}

									reader.Skip(compressedBlockSize);

									if (!reader.GetLeft())
										break;
								}
							}
						}
						else
						{
							// TODO: Some simple validation of uncompressed file.. maybe even rehash and check? (since it is not many uncompressed files.. zero on server and a few on client)
						}
					}
					else
					{
						deleteFile = true;
					}
				}


				SCOPED_WRITE_LOCK(m_casLookupLock, lookupLock);
				auto insres = m_casLookup.try_emplace(casKey, casKey);
				CasEntry& casEntry = insres.first->second;
				casEntry.verified = true;
				casEntry.exists = true;

				m_casTotalBytes += size;
				if (insres.second)
				{
					casEntry.size = size;
					AttachEntry(casEntry);
				}
				else
				{
					UBA_ASSERT(casEntry.key == casKey);
					// We should probably delete this one.. something is wrong
					if (casEntry.size != 0 && casEntry.size != size && !deleteFile)
						m_logger.Detail(TC("Found cas entry which has a different size than what the table thought! Was %llu, is %llu (%s)"), casEntry.size, size, e.name);
					casEntry.size = size;
				}

				if (!size && casKey != ToCasKey(CasKeyHasher(), IsCompressed(casKey)))
				{
					m_logger.Detail(TC("Found file that has size 0 but does not have correct caskey (%s)"), e.name);
					deleteFile = true;
				}

				if (!deleteFile)
					return;

				DetachEntry(casEntry);
				m_casLookup.erase(insres.first);
				m_casTotalBytes -= size;
				lookupLock.Leave();

				if (DeleteFileW(fullPath.data))
					return;

				m_logger.Error(TC("Failed to delete file %s (%s)"), fullPath.data, LastErrorToText().data);
				success = false;

			}, true);

		if (!success)
			return false;

		u32 didNotExistCount = 0;
		// All files we saw is tagged as "handled") so let's see if there are cas entries we didn't find
		for (auto it = m_casLookup.begin(); it!=m_casLookup.end();)
		{
			CasEntry& casEntry = it->second;
			if (casEntry.verified)
			{
				++it;
				casEntry.verified = false; // Unhandle the entries to be able to evict later
				continue;
			}
			casEntry.size = 0;
			DetachEntry(casEntry);
			it = m_casLookup.erase(it);
			didNotExistCount++;
		}

		if (didNotExistCount)
			m_logger.Info(TC("Found %u cas entries that didn't have a file"), didNotExistCount);

		u64 duration = GetTime() - startTime;

		u64 after = m_casTotalBytes;
		if (before != after)
			m_logger.Info(TC("Corrected storage size from %s to %s in %s"), BytesToText(before).str, BytesToText(after).str, TimeToText(duration).str);
		else
			m_logger.Info(TC("Validated storage (size %s) in %s"), BytesToText(after).str, TimeToText(duration).str);
		m_casMaxBytes = m_casTotalBytes;
		return true;
	}

	void StorageImpl::HandleOverflow(UnorderedSet<CasKey>* outDeletedFiles)
	{
		if (!m_casCapacityBytes)
			return;
		u64 startTime = GetTime();
		u64 before = m_casTotalBytes;
		while (m_casTotalBytes > m_casCapacityBytes)
		{
			CasEntry* casEntry = m_oldestAccessed;
			if (!casEntry)
			{
				UBA_ASSERT(m_casLookup.empty());
				m_casTotalBytes = 0;
				break;
			}
			DropCasFile(casEntry->key, true, TC("HandleOverflow"));
			if (outDeletedFiles)
				outDeletedFiles->insert(casEntry->key);
			DetachEntry(*casEntry);
			m_casLookup.erase(casEntry->key);
		}
		m_overflowReported = false;
		u64 after = m_casTotalBytes;
		if (before != after)
			m_logger.Info(TC("Evicted %s from storage (%s). Estimated new storage is now %s (there might be files db is not aware of)"), BytesToText(before - after).str, TimeToText(GetTime() - startTime).str, BytesToText(after).str);
	}

	bool StorageImpl::DeleteIsRunningFile()
	{
		StringBuffer<256> isRunningName;
		isRunningName.Append(m_rootDir).Append(TCV(".isRunning"));
		if (DeleteFileW(isRunningName.data))
			return true;
		u32 lastError = GetLastError();
		if (lastError == ERROR_FILE_NOT_FOUND || lastError == ERROR_PATH_NOT_FOUND)
			return true;
		return m_logger.Warning(TC("Failed to delete %s (%s)"), isRunningName.data, LastErrorToText(lastError).data);
	}

	bool StorageImpl::TouchIsRunningFile()
	{
		StringBuffer<256> isRunningName;
		isRunningName.Append(m_rootDir).Append(TCV(".isRunning"));
		FileHandle fh = uba::CreateFileW(isRunningName.data, FILE_WRITE_ATTRIBUTES, FILE_SHARE_ALL, OPEN_EXISTING, FILE_FLAG_BACKUP_SEMANTICS);
		if (fh == InvalidFileHandle)
			return m_logger.Warning(TC("Failed to open %s to update lastwritetime (%s)"), isRunningName.data, LastErrorToText().data);
		bool success = SetFileLastWriteTime(fh, GetSystemTimeAsFileTime());
		if (!success)
			m_logger.Warning(TC("Failed to update lastwritetime on %s (%s)"), isRunningName.data, LastErrorToText().data);
		uba::CloseFile(isRunningName.data, fh);
		return success;
	}

	char g_fileMappingBackingEnv[512] = {};

	FileMappingBackend& GetFileMappingBackend(const StorageCreateInfo& info, Logger& logger)
	{
		#if PLATFORM_WINDOWS
		if (info.sideStepWineForSharedMemory && IsRunningWine())
		{
			static FileMappingBackend wineBackend;
			if (void* h = GetUbaWineModule())
				if (FileMapping_GetWineBackend(wineBackend, logger, (HMODULE)h))
				{
					logger.Debug(TC("Using wine filemapping backend"));
					return wineBackend;
				}
		}
		#endif
		static FileMappingBackend backend;
		return backend;
	}

	StorageImpl::StorageImpl(const StorageCreateInfo& info, const tchar* logPrefix)
	:	m_logger(info.writer, logPrefix)
	,	m_workManager(info.workManager)
	,	m_sharedMemory(m_logger, GetFileMappingBackend(info, m_logger))
	,	m_bufferSlots(BufferSlotSize)
	,	m_maxParallelCopyOrLinkBottleneck(info.maxParallelCopyOrLink)
	,	m_casDataBuffer(m_logger, m_sharedMemory)
	{
		m_casCapacityBytes = info.casCapacityBytes;
		m_fileEntryTableWipeThreshold = info.fileEntryTableWipeThreshold;
		m_deleteCasBatchSize = info.deleteCasBatchSize;
		m_storeCompressed = info.storeCompressed;
		m_manuallyHandleOverflow = info.manuallyHandleOverflow;
		m_asyncUnmapViewOfFile = info.asyncUnmapViewOfFile;
		m_allowDeleteVerified = info.allowDeleteVerified;
		m_allowDeferredDeletes = info.allowDeferredDeletes;
		m_skipCleanupOnShutdown = info.skipCleanupOnShutdown;
		m_allowUnDropOfCas = info.allowUnDropOfCas;
		m_writeToDisk = info.writeToDisk;
		if (!m_writeToDisk)
			m_casCapacityBytes = 0;

		m_casTableVersion = CasTableVersion;

		m_exclusiveMutex = info.exclusiveMutex;

		m_rootDir.count = GetFullPathNameW(info.rootDir, m_rootDir.capacity, m_rootDir.data, NULL);
		m_rootDir.Replace('/', PathSeparator).EnsureEndsWithSlash();

		// The mapping backing (memfd/shm/disk) is process-wide (setenv + pidfd + g_fileMappingBackingEnv),
		// so resolve once even if multiple StorageImpls construct in parallel. 1GB floor below which
		// /dev/shm can't back the working set.
		static Futex s_backingLock;
		static bool s_backingResolved = false;
		SCOPED_FUTEX(s_backingLock, backingLock);
#if PLATFORM_LINUX
		// Backing is auto-resolved to memfd at startup. If a later storage can't use memfd, force a
		// re-resolve to /dev/shm before its mappings (comm + on-demand file) are created.
		const char* curMemfd = getenv("UBA_FILE_MAPPING_MEMFD");
		if (!info.allowMemfd && curMemfd && *curMemfd == '1')
			s_backingResolved = false;
#endif
		if (!s_backingResolved)
		{
#if PLATFORM_LINUX
			if (!info.allowMemfd)
				setenv("UBA_FILE_MAPPING_MEMFD", "0", 1);
#endif
			GetFileMappingBackend(info, m_logger).ResolveBacking(m_logger, m_rootDir.data, 1ull*1024*1024*1024, g_fileMappingBackingEnv, sizeof_array(g_fileMappingBackingEnv));
			s_backingResolved = true;
		}
		backingLock.Leave();

		if (m_writeToDisk)
		{
			m_tempPath.Append(m_rootDir).Append(TCV("castemp"));
			CreateDirectory(m_tempPath.data);
			DeleteAllFiles(m_logger, m_tempPath.data, false);
			m_tempPath.EnsureEndsWithSlash();
		}

		m_rootDir.Append(TCV("cas")).EnsureEndsWithSlash();

		m_casCompressor = info.casCompressor;
		m_casCompressionLevel = info.casCompressionLevel;

		if (!m_sharedMemory.Init(info.sharedMemoryCapacity, info.sharedMemoryCommitSize, info.sharedMemoryTempFile, info.sharedMemoryTempFileSparse))
			FatalError(9887, TC("Failed to create shared memory"));

		if (!m_casDataBuffer.Init(TC("CasData"), 32ull*1024*1024*1024, 1ull*1024*1024))
			FatalError(9887, TC("Failed to initalize CasDataBuffer"));

		if (info.sharedMemoryInitialCommit)
			m_sharedMemory.Reserve(AlignUp(info.sharedMemoryInitialCommit, PageSize));
	}

	StorageImpl::~StorageImpl()
	{
		SaveCasTable(true, true);
		CloseMutex(m_exclusiveMutex);

		m_sharedMemory.Deinit(m_skipCleanupOnShutdown);
	}

	MutexHandle StorageImpl::GetExclusiveAccess(Logger& logger, const StringView& rootDir, bool reportError)
	{
		StringKey key = ToStringKeyNoCheck(rootDir.data, rootDir.count);
		KeyToString keyStr(key);
		MutexHandle exclusiveMutex = CreateMutexW(true, keyStr.data);
		u32 lastError = GetLastError();
		if (exclusiveMutex == InvalidMutexHandle)
		{
			if (reportError)
				logger.Error(TC("Failed to create mutex %s for path %s (%s)"), keyStr.data, rootDir.data, LastErrorToText(lastError).data);
			return InvalidMutexHandle;
		}

		if (lastError != ERROR_ALREADY_EXISTS)
			return exclusiveMutex;

		CloseMutex(exclusiveMutex);
		if (reportError)
			logger.Error(TC("Needs exclusive access to storage %s. Another process is running"), rootDir.data);
		return InvalidMutexHandle;
	}

	StorageImpl::FileEntry& StorageImpl::GetOrCreateFileEntry(const StringKey& fileNameKey)
	{
		SCOPED_READ_LOCK(m_fileTableLookupLock, lock);
		auto findIt = m_fileTableLookup.find(fileNameKey);
		if (findIt != m_fileTableLookup.end())
			return findIt->second;
		lock.Leave();
		SCOPED_WRITE_LOCK(m_fileTableLookupLock, lock2);
		return m_fileTableLookup.try_emplace(fileNameKey).first->second;
	}

	StorageImpl::CasEntry& StorageImpl::GetOrCreateCasEntry(const CasKey& casKey)
	{
		SCOPED_READ_LOCK(m_casLookupLock, lock);
		auto findIt = m_casLookup.find(casKey);
		if (findIt != m_casLookup.end())
			return findIt->second;
		lock.Leave();
		SCOPED_WRITE_LOCK(m_casLookupLock, lock2);
		return m_casLookup.try_emplace(casKey, casKey).first->second;
	}

	bool StorageImpl::GetFileEntry(FileEntry*& out, const tchar* file)
	{
		out = nullptr;
		StringBuffer<MaxPath> forKey;
		if (!FixPath(file, nullptr, 0, forKey))
			return m_logger.Error(TC("GetCasEntry - FixPath failed for %s"), file);
		if (CaseInsensitiveFs)
			forKey.MakeLower();
		StringKey fileNameKey = ToStringKey(forKey);

		SCOPED_READ_LOCK(m_fileTableLookupLock, lookupLock);
		auto findIt = m_fileTableLookup.find(fileNameKey);
		if (findIt == m_fileTableLookup.end())
			return false;
		out = &findIt->second;
		return true;
	}

	bool StorageImpl::LoadCasTable(bool logStats, bool alwaysCheckAllFiles, bool* outWasTerminated)
	{
		if (!m_writeToDisk)
			return true;

		SCOPED_FUTEX(m_casTableLoadSaveLock, loadSaveLock);
		if (m_casTableLoaded)
			return true;

		if (m_exclusiveMutex == InvalidMutexHandle)
		{
			m_exclusiveMutex = GetExclusiveAccess(m_logger, m_rootDir, true);
			if (m_exclusiveMutex == InvalidMutexHandle)
				return false;
		}

		CreateDirectory(m_rootDir.data);

		m_casTableLoaded = true;
		u64 startTime = GetTime();
		tchar isRunningName[256];
		TStrcpy_s(isRunningName, sizeof_array(isRunningName), m_rootDir.data);
		TStrcat_s(isRunningName, sizeof_array(isRunningName), TC(".isRunning"));
		bool wasTerminated = FileExists(m_logger, isRunningName);

		if (outWasTerminated)
			*outWasTerminated = wasTerminated;

		if (!wasTerminated)
		{
			FileAccessor isRunningFile(m_logger, isRunningName);
			if (!isRunningFile.CreateWrite(false, DefaultAttributes(), 0, m_tempPath.data) || !isRunningFile.Close())
				return m_logger.Error(TC("Failed to create temporary \".isRunning\" file"));
		}

		StringBuffer<> fileName;
		fileName.Append(m_rootDir).Append(TCV("casdb"));

		FileHandle fileHandle;
		if (!OpenFileSequentialRead(m_logger, fileName.data, fileHandle, false))
			return false;
		if (fileHandle == InvalidFileHandle)
			return true;

		auto fileGuard = MakeGuard([&](){ CloseFile(fileName.data, fileHandle); });

		u64 fileSize;
		if (!uba::GetFileSizeEx(fileSize, fileHandle))
			return m_logger.Error(TC("GetFileSize failed for %s (%s)"), fileName.data, LastErrorToText().data);

		if (fileSize < sizeof(u32))
			return m_logger.Warning(TC("CasTable file %s is corrupt (size: %u)"), fileName.data, fileSize);

		u8* buffer = new u8[fileSize];
		auto memGuard = MakeGuard([&](){ delete[] buffer; });

		if (!ReadFile(m_logger, fileName.data, fileHandle, buffer, fileSize))
			return false;

		BinaryReader reader(buffer);
		m_casTableVersion = reader.ReadU32();


		#if UBA_TRACK_IS_EXECUTABLE
		bool checkExecutableBit = true;
		if (m_casTableVersion == CasTableVersionWithoutIsExecutable)
		{
			// We don't want to upgrade cas table version automatically since this machine might swap between branches
			checkExecutableBit = false;
		}
		else
		#endif
			
		if (m_casTableVersion != CasTableVersion)
		{
			fileGuard.Execute();
			m_logger.Info(TC("New CasTable version v%u (Old %u). Deleting all cas files..."), CasTableVersion, m_casTableVersion);
			m_casTableVersion = CasTableVersion;
			DeleteAllCas();
			loadSaveLock.Leave();
			if (!SaveCasTable(false, false)) // Save it just to get the version down
				return false;
			return true;
		}

		u32 fileTableSize = reader.ReadU32();
		u32 casLookupSize = reader.ReadU32();

		{
			SCOPED_WRITE_LOCK(m_fileTableLookupLock, fileTableLock); // Note, this can be reached from InvalidateCachedFileInfo which is called directly from session, so use a lock

			UBA_ASSERT(m_casLookup.empty());
			UBA_ASSERT(m_fileTableLookup.empty());

			m_fileTableLookup.reserve(fileTableSize);
			for (u32 i=0; i!=fileTableSize; ++i)
			{
				StringKey fileNameKey = reader.ReadStringKey();
				if (reader.GetPosition() + 24 > fileSize)
				{
					m_fileTableLookup.clear();
					return m_logger.Warning(TC("CasTable file %s is corrupt"), fileName.data);
				}
				u64 size = reader.ReadU64();
				u64 lastWritten = reader.ReadU64();
				CasKey casKey = reader.ReadCasKey();
				if (casKey != CasKeyZero)
					casKey = AsCompressed(casKey, m_storeCompressed);
				else if (size || lastWritten)
					continue; // This should not happen

				auto insres = m_fileTableLookup.try_emplace(fileNameKey);
				FileEntry& fileEntry = insres.first->second;

				#if UBA_TRACK_IS_EXECUTABLE
				if (checkExecutableBit && IsExecutable(casKey))
				{
					casKey = AsExecutable(casKey, false);
					fileEntry.isExecutable = true;
				}
				#endif

				fileEntry.verified = false;
				fileEntry.size = size;
				fileEntry.lastWritten = lastWritten;
				fileEntry.casKey = casKey;
			}
		}

		m_casLookup.reserve(casLookupSize);
		CasEntry* prev = nullptr;
		while (true)
		{
			CasKey casKey = reader.ReadCasKey();
			if (casKey == CasKeyZero)
				break;
			auto insres = m_casLookup.try_emplace(casKey, casKey);
			if (!insres.second)
			{
				m_logger.Warning(TC("CasTable file %s is corrupt, it contains same cas key multiple times (%s)"), fileName.data, CasKeyString(casKey).str);
				m_fileTableLookup.clear();
				m_casLookup.clear();
				m_newestAccessed = nullptr;
				return false;
			}
			CasEntry& casEntry = insres.first->second;
			casEntry.size = reader.ReadU64();
			casEntry.exists = true;
			m_casTotalBytes += casEntry.size;

			if (prev)
			{
				prev->nextAccessed = &casEntry;
				casEntry.prevAccessed = prev;
			}
			else
				m_newestAccessed = &casEntry;
			prev = &casEntry;
		}
		m_oldestAccessed = prev;

		bool resave = false;
		if (wasTerminated)
		{
			u64 fileTime = 0;
			FileHandle fh = uba::CreateFileW(isRunningName, 0, FILE_SHARE_ALL, OPEN_EXISTING, FILE_FLAG_BACKUP_SEMANTICS);
			auto fhg = MakeGuard([&]() { uba::CloseFile(isRunningName, fh); });
			if (fh != InvalidFileHandle)
				if (!GetFileLastWriteTime(fileTime, fh))
					m_logger.Warning(TC("Failed to get lastwritetime of %s (%s)"), isRunningName, LastErrorToText().data);

			m_logger.Info(TC("Previous run was not gracefully shutdown. Reparsing cas directory %s to check for added/missing/corrupt files (Saved %s)"), m_rootDir.data, DateToText(fileTime).str);

			if (!CheckAllCasFiles(fileTime))
				return false;
			resave = true;

			if (fh != InvalidFileHandle)
				SetFileLastWriteTime(fh, GetSystemTimeAsFileTime());
		}
		else if (alwaysCheckAllFiles)
			CheckAllCasFiles();

		if (!m_manuallyHandleOverflow)
			HandleOverflow(nullptr);

		if (resave)
		{
			fileGuard.Execute();
			loadSaveLock.Leave();
			SaveCasTable(false, false);
		}

		if (logStats)
		{
			u64 duration = GetTime() - startTime;
			m_logger.Detail(TC("Database loaded from %s (v%u) in %s (contained %llu entries estimated to %s)"), fileName.data, m_casTableVersion, TimeToText(duration).str, m_casLookup.size(), BytesToText(m_casTotalBytes).str);
		}

		return true;
	}

	bool StorageImpl::SaveCasTable(bool deleteIsRunningfile, bool deleteDropped)
	{
		SCOPED_FUTEX(m_casTableLoadSaveLock, loadSaveLock);
		if (!m_casTableLoaded)
			return true;

		u64 startTime = GetTime();

		StringBuffer<256> fileName;
		fileName.Append(m_rootDir).Append(TCV("casdb"));

		StringBuffer<256> tempFileName;
		tempFileName.Append(fileName).Append(TCV(".tmp"));

		u32 deleteCount = 0;
		u32 fileTableSize = 0;
		u32 casTableSize = 0;

		{
			FileAccessor tempFile(m_logger, tempFileName.data);
			if (!tempFile.CreateWrite(false, DefaultAttributes(), 0, m_tempPath.data))
				return false;
			
			SCOPED_WRITE_LOCK(m_fileTableLookupLock, fileTableLock);
			SCOPED_WRITE_LOCK(m_casLookupLock, casLookupLock);
			SCOPED_FUTEX(m_accessLock, accessLock);

			Vector<u8> buffer;

			u64 casLookupEntrySize = sizeof(CasKey) + sizeof(u64);

			u64 headerSize = sizeof(u32)*3;
			u64 fileTableMaxWriteSize = m_fileTableLookup.size() * (sizeof(StringKey) + sizeof(u64) * 2 + sizeof(CasKey));
			u64 casLookupMaxWriteSize = m_casLookup.size() * casLookupEntrySize + sizeof(CasKey);

			buffer.reserve(headerSize + fileTableMaxWriteSize + casLookupMaxWriteSize);
			BinaryWriter writer(buffer.data(), 0, buffer.capacity());

			// Header
			writer.WriteU32(CasTableVersion);
			auto fileTableSizePtr = (u32*)writer.AllocWrite(4);
			auto casLookupSizePtr = (u32*)writer.AllocWrite(4);

			// File table

			// If we have a million entries in here we likely have lots of old entries we don't care about.
			// These accumulate over time and never clean up (since there is no way to know what entry belong to what file.
			// Best is to just wipe it when it gets too big and start from the beginning..
			// It will force all files cas to be recalculated but it should be really rare.
			if (m_fileEntryTableWipeThreshold && m_fileEntryTableWipeThreshold < m_fileTableLookup.size())
			{
				m_logger.Info(TC("File entry table larger than threshold (%llu/%u), wiping... (Threshold can be tweaked in config)"), m_casLookup.size(), m_fileEntryTableWipeThreshold);
			}
			else
			{
				#if UBA_TRACK_IS_EXECUTABLE
				bool useExecutableBit = m_casTableVersion != CasTableVersionWithoutIsExecutable;
				#endif

				for (auto& pair : m_fileTableLookup)
				{
					FileEntry& fileEntry = pair.second;
					CasKey casKey = fileEntry.casKey;
					//SCOPED_READ_LOCK(fileEntry.lock, entryLock); // TODO: Remove this? tsan complains about it
					if (casKey == CasKeyZero || fileEntry.lastWritten == 0 || fileEntry.isTemp || casKey == CasKeyInvalid)
						continue;

					#if UBA_TRACK_IS_EXECUTABLE
					if (fileEntry.isExecutable && useExecutableBit)
						casKey = AsExecutable(casKey, true);
					#endif


					writer.WriteStringKey(pair.first);
					writer.WriteU64(fileEntry.size);
					writer.WriteU64(fileEntry.lastWritten);
					writer.WriteCasKey(casKey);
					++fileTableSize;
				}
			}
			*fileTableSizePtr = fileTableSize;

			// Cas table
			CasEntry* last = nullptr;
			for (CasEntry* it = m_newestAccessed; it; it = it->nextAccessed)
			{
				last = it;
				CasEntry& casEntry = *it;
				if (casEntry.verified && !casEntry.exists)
					continue;
				if (deleteDropped && casEntry.dropped)
				{
					StringBuffer<512> casFileName;
					if (!StorageImpl::GetCasFileName(casFileName, casEntry.key))
						continue;
					if (DeleteFileW(casFileName.data))
						++deleteCount;
					if (m_casDataBuffer.IsIndependent(casEntry.memoryHandle)) // Log files etc could end up here
					{
						m_casDataBuffer.CloseIndependent(casEntry.memoryHandle, casEntry.memorySize, TC("SaveCasTable"));
						casEntry.memoryHandle = {};
					}
					continue;
				}

				if (writer.GetCapacityLeft() < casLookupEntrySize + sizeof(CasKey))
					return m_logger.Error(TC("This should not happen, somehow there are more valid entries in access list than lookup. (Lookup has %llu entries)"), m_casLookup.size());

				UBA_ASSERT(casEntry.key != CasKeyZero);
				writer.WriteCasKey(casEntry.key);
				writer.WriteU64(casEntry.size);
				++casTableSize;
			}
			writer.WriteCasKey(CasKeyZero);
			*casLookupSizePtr = casTableSize;

			if (!tempFile.Write(buffer.data(), writer.GetPosition()))
				return false;
			UBA_ASSERT(m_oldestAccessed == last); (void)last;
			if (!tempFile.Close())
				return false;
		}

		if (!MoveFileExW(tempFileName.data, fileName.data, MOVEFILE_REPLACE_EXISTING))
			return m_logger.Error(TC("Can't move file from %s to %s (%s)"), tempFileName.data, fileName.data, LastErrorToText().data);

		if (deleteIsRunningfile)
			DeleteIsRunningFile();
		else
			TouchIsRunningFile();

		if (m_overflowReported)
			m_logger.Info(TC("Session needs at least %s to not overflow."), BytesToText(m_casMaxBytes).str);

		u64 duration = GetTime() - startTime;
		m_logger.Detail(TC("Database saved in %s (%u file entries, %u cas entries. deleted %u dropped cas files)"), TimeToText(duration).str, fileTableSize, casTableSize, deleteCount);
		return true;
	}

	bool StorageImpl::CheckCasContent(Vector<TString>* badFiles)
	{
		StringBuffer<> casRoot;
		casRoot.Append(m_rootDir.data, m_rootDir.count - 1);

		u64 fileTimeNow = GetFileTimeAsSeconds(GetSystemTimeAsFileTime());
		auto writeTimeAgo = [&](StringBufferBase& out, u64 lastWritten)
			{
				u64 secondsAgoTotal = fileTimeNow - GetFileTimeAsSeconds(lastWritten);
				//u64 secondsAgo = secondsAgoTotal % 60;
				u64 daysAgo = secondsAgoTotal / (60 * 60 * 24);
				u64 hoursAgo = secondsAgoTotal / (60 * 60) % 24;
				u64 minutesAgo = (secondsAgoTotal / 60) % 60;
				out.Appendf(TC("%llud %02lluh %02llum"), daysAgo, hoursAgo, minutesAgo);
				//if (hoursAgo)
				//	timeStr.Appendf(TC(" %lluhours"), hoursAgo);
				//if (minutesAgo)
				//	timeStr.Appendf(TC(" %llumin"), minutesAgo);
			};

		m_logger.Info(TC("Traverse cas database..."));
		u32 entryCount = 0;
		u32 errorCount = 0;
		u64 newestWrittenError = 0;
		Futex lock;
		TraverseAllCasFiles(casRoot.data, [&](const StringBufferBase& fullPath, const DirectoryEntry& e)
			{
				++entryCount;
				m_workManager.AddWork([&, filePath = fullPath.ToString(), name = TString(e.name), lastWritten = e.lastWritten](const WorkContext&)
					{
						StringBuffer<> timeStr;
						writeTimeAgo(timeStr, lastWritten);

						
						//m_logger.Info(TC("Validating %s (%s ago)"), name.c_str(), timeStr.data);
						CasKey casKey = CasKeyFromString(name.c_str());

						// TODO: Add support for normalized files
						if (IsNormalized(casKey))
							return;

						auto reportError = MakeGuard([&]()
							{
								SCOPED_FUTEX(lock, l);
								++errorCount;
								if (lastWritten > newestWrittenError)
									newestWrittenError = lastWritten;
								if (badFiles)
									badFiles->push_back(filePath);
							});

						CasKey checkedKey = EmptyFileKey;
						if (IsCompressed(casKey))
						{
							FileAccessor file(m_logger, filePath.c_str());
							if (!file.OpenMemoryRead())
								return;
							if (u64 fileSize = file.GetSize())
							{
								u8* mem = file.GetData();
								u64 decompressedSize = *(u64*)mem;
								mem += sizeof(u64);
								u8* dest = new u8[decompressedSize];
								auto g = MakeGuard([dest]() { delete[] dest; });
								if (!DecompressMemoryToMemory(mem, fileSize, dest, decompressedSize, filePath.c_str(), TC("Memory")))
									return;
								checkedKey = CalculateCasKey(dest, decompressedSize, true);
							}
						}
						else
						{
							if (!CalculateCasKey(checkedKey, {}, filePath.c_str()))
								return;
							checkedKey = AsCompressed(checkedKey, false);
 						}
						if (casKey == checkedKey)
						{
							reportError.Cancel();
							return;
						}
						m_logger.Error(TC("Cas key and content mismatch for key %s (expected %s) (%s ago)"), CasKeyString(casKey).str, CasKeyString(checkedKey).str, timeStr.data);

					}, 1, TC("CheckCasContent"));
			});
		m_logger.Info(TC("Validating %u entries..."), entryCount);

		m_workManager.FlushWork();

		StringBuffer<> newestLastWrittenStr;
		writeTimeAgo(newestLastWrittenStr, newestWrittenError);

		if (errorCount == 0)
			m_logger.Info(TC("Done. No errors found"));
		else
			m_logger.Info(TC("Done. Found %u errors out of %u entries (Last written bad entry was %s)"), errorCount, entryCount, newestLastWrittenStr.data);
		return true;
	}

	bool StorageImpl::CheckFileTable(const tchar* searchPath)
	{
		m_logger.Info(TC("Searching %s to check files against file table..."), searchPath);

		List<TString> directories;
		directories.push_back(searchPath);

		Atomic<u32> foundFiles = 0;
		Atomic<u32> trackedFiles = 0;
		Atomic<u32> testedFiles = 0;
		Atomic<u32> errorCount = 0;

		u64 startTime = GetTime();
		
		while (!directories.empty())
		{
			TString dir = directories.front();
			directories.pop_front();
			TraverseDir(m_logger, dir, [&](const DirectoryEntry& e)
				{
					StringBuffer<> path(dir);
					path.EnsureEndsWithSlash().Append(e.name);
					if (CaseInsensitiveFs)
						path.MakeLower();

					if (IsDirectory(e.attributes))
					{
						if (Equals(e.name, TC("Content")))
							return;
						directories.push_back(path.data);
						return;
					}
					u64 lastWritten = e.lastWritten;
					u64 size = e.size;

					m_workManager.AddWork([&, p = path.ToString(), lastWritten, size](const WorkContext&)
						{
							++foundFiles;
							StringKey key = ToStringKey(p.data(), p.size());
							
							auto findIt = m_fileTableLookup.find(key);
							if (findIt == m_fileTableLookup.end())
								return;
							++trackedFiles;

							FileEntry& fe = findIt->second;
							if (fe.lastWritten != lastWritten || fe.size != size)
								return;
							++testedFiles;

							CasKey casKey;

							{
								FileAccessor file(m_logger, p.data());
								if (!file.OpenMemoryRead())
									return;
								if (file.GetSize() > sizeof(CompressedFileHeader) && ((CompressedFileHeader*)file.GetData())->IsValid())
									casKey = ((CompressedFileHeader*)file.GetData())->casKey;
							}
							if (casKey == CasKeyZero)
								if (!CalculateCasKey(casKey, {}, p.data()))
									m_logger.Warning(TC("Failed to calculate cas key for %s"), p.data());

							if (casKey != fe.casKey)
							{
								++errorCount;
								m_logger.Error(TC("CasKey mismatch for %s even though size and lastwritten were the same. Corrupt path table! (Correct: %s. Wrong: %s)"), p.data(), CasKeyString(casKey).str, CasKeyString(fe.casKey).str);
							}

						}, 1, TC("CheckFileTable"));
				});
		}
		m_workManager.FlushWork();

		m_logger.Info(TC("Done. %u errors found. Searched %u files where %u was tracked and %u matched table."), errorCount.load(), foundFiles.load(), trackedFiles.load(), testedFiles.load(), TimeToText(GetTime() - startTime).str);

		return errorCount == 0;
	}

	const tchar* StorageImpl::GetTempPath()
	{
		return m_tempPath.data;
	}

	u64 StorageImpl::GetStorageCapacity()
	{
		return m_casCapacityBytes;
	}

	u64 StorageImpl::GetStorageUsed()
	{
		return m_casTotalBytes;
	}

	bool StorageImpl::GetZone(StringBufferBase& out)
	{
		return false;
	}

	bool StorageImpl::Reset()
	{
		m_casLookup.clear();
		m_fileTableLookup.clear();
		m_newestAccessed = nullptr;
		m_oldestAccessed = nullptr;
		m_casTotalBytes = 0;
		m_casMaxBytes = 0;

		DeleteAllCas();
		return true;
	}

	bool StorageImpl::DeleteAllCas()
	{
		u32 deleteCount = 0;

		{
			Atomic<u32> atomicDeleteCount = 0;
			TraverseDir(m_logger, m_rootDir, [&](const DirectoryEntry& e)
				{
					if (!IsDirectory(e.attributes))
						return;
					m_workManager.AddWork([&, name = TString(e.name)](const WorkContext&)
						{
							StringBuffer<> fullPath;
							fullPath.Append(m_rootDir).Append(name);
							u32 deleteCountTemp = 0;
							DeleteAllFiles(m_logger, fullPath.data, true, &deleteCountTemp);
							atomicDeleteCount += deleteCountTemp;
						}, 1, TC("DeleteAllCas"));
				});
			m_workManager.FlushWork();
			deleteCount += atomicDeleteCount;
		}

		bool res = DeleteAllFiles(m_logger, m_rootDir.data, false, &deleteCount);
		m_logger.Info(TC("Deleted %u cas files"), deleteCount);
		m_dirCache.Clear();
		return res;
	}

	bool StorageImpl::RetrieveCasFile(RetrieveResult& out, const CasKey& casKey, StringKey keyHint, const tchar* hint, FileMappingBuffer* mappingBuffer, bool createIndependentMapping, u64 memoryMapAlignment, bool allowProxy, u32 clientId)
	{
		UBA_ASSERT(false);
		return false;
	}

	bool StorageImpl::AddRef(const CasKey& casKey)
	{
		SCOPED_READ_LOCK(m_casLookupLock, lookupLock);
		auto it = m_casLookup.find(casKey);
		if (it == m_casLookup.end())
			return false;
		CasEntry& casEntry = it->second;
		lookupLock.Leave();
		CasEntryWriteLock casEntryLock(casEntry);
		UBA_ASSERTF(casEntry.refCount < 65534, TC("CasEntry %s ref count wraparound?"), CasKeyString(casKey).data);
		bool isOk = casEntry.verified && casEntry.exists;
		if (!isOk)
			return false;
		++casEntry.refCount;
		return true;
	}

	bool StorageImpl::ReleaseRef(const CasKey& casKey, bool dropIfZero)
	{
		SCOPED_READ_LOCK(m_casLookupLock, lookupLock);
		auto it = m_casLookup.find(casKey);
		if (it == m_casLookup.end())
			return false;
		CasEntry& casEntry = it->second;
		lookupLock.Leave();
		CasEntryWriteLock casEntryLock(casEntry);
		ReleaseCasEntryNoLock(casEntry, dropIfZero);
		return true;
	}

	bool StorageImpl::VerifyAndGetCachedFileInfo(CachedFileInfo& out, StringKey fileNameKey, u64 verifiedLastWriteTime, u64 verifiedSize)
	{
		out.casKey = CasKeyZero;
		SCOPED_READ_LOCK(m_fileTableLookupLock, lookupLock);
		auto findIt = m_fileTableLookup.find(fileNameKey);
		if (findIt == m_fileTableLookup.end())
			return false;
		FileEntry& fileEntry = findIt->second;
		lookupLock.Leave();

		SCOPED_FUTEX(fileEntry.lock, entryLock);
		
		fileEntry.verified = fileEntry.lastWritten == verifiedLastWriteTime && fileEntry.size == verifiedSize && fileEntry.casKey != CasKeyInvalid;

		if (!fileEntry.verified)
			return false;
		out.casKey = fileEntry.casKey;
		return true;
	}

	bool StorageImpl::InvalidateCachedFileInfo(StringKey fileNameKey)
	{
		SCOPED_READ_LOCK(m_fileTableLookupLock, lookupLock);
		auto findIt = m_fileTableLookup.find(fileNameKey);
		if (findIt == m_fileTableLookup.end())
			return false;
		FileEntry& fileEntry = findIt->second;
		lookupLock.Leave();

		SCOPED_FUTEX(fileEntry.lock, entryLock);
		fileEntry.verified = false;
		fileEntry.casKey = CasKeyInvalid;
		return true;
	}

	bool StorageImpl::StoreCasFile(CasKey& out, const tchar* fileName, const CasKey& casKeyOverride, bool deferCreation)
	{
		return StoreCasFile(out, fileName, casKeyOverride, deferCreation, nullptr);
	}

	bool StorageImpl::StoreCasFile(CasKey& out, const tchar* fileName, const CasKey& casKeyOverride, bool deferCreation, CasEntry** outEntry)
	{
		StringBuffer<MaxPath> forKey;
		forKey.Append(fileName);
		if (CaseInsensitiveFs)
			forKey.MakeLower();
		StringKey fileNameKey = ToStringKey(forKey);

		bool dropCasAfterUse = false;

		FileEntry& fileEntry = GetOrCreateFileEntry(fileNameKey);
		SCOPED_FUTEX(fileEntry.lock, entryLock);

		if (fileEntry.verified)
		{
			UBA_ASSERT(fileEntry.casKey != CasKeyInvalid);

			CasKey verifiedKey = fileEntry.casKey;
			u64 verifiedSize = fileEntry.size;
			u64 verifiedLastWritten = fileEntry.lastWritten;
			entryLock.Leave();

			if (verifiedKey != CasKeyZero)
			{
				UBA_ASSERT(casKeyOverride == CasKeyZero || casKeyOverride == verifiedKey);
 				if (!AddCasFile(fileNameKey, fileName, verifiedKey, deferCreation, verifiedSize, verifiedLastWritten, outEntry, dropCasAfterUse))
					return false;
			}
			out = verifiedKey;
			return true;
		}
		fileEntry.verified = true;


		ExternalFileMapping externalMapping;
		auto externalMappingGuard = MakeGuard([&]() { m_sharedMemory.CloseHandle(m_logger, externalMapping.handle, TC("ExternalFileMapping")); });

		u64 fileSize = 0;
		u64 storedSize = 0;
		u64 lastWriteTime = 0;

		FileHandle fileHandle = InvalidFileHandle;
		auto fileGuard = MakeGuard([&](){ CloseFile(fileName, fileHandle); });

		if (m_externalFileMappingsProvider && m_externalFileMappingsProvider(externalMapping, fileNameKey, fileName))
		{
			fileSize = externalMapping.size;
			storedSize = externalMapping.storedSize;
			lastWriteTime = externalMapping.lastWriteTime;
			dropCasAfterUse = externalMapping.dropCasAfterUse;
			fileEntry.isTemp = true;
		}
		else
		{
			// Use OpenFile+GetFileInformationByHandle+close instead of Getting file attributes because it is actually faster on cloud setups (weirdly enough)
			if (!OpenFileSequentialRead(m_logger, fileName, fileHandle))
			{
				fileEntry.casKey = CasKeyZero;
				out = CasKeyZero;
				return true;
			}

			FileBasicInformation info;
			if (!GetFileBasicInformationByHandle(info, m_logger, fileName, fileHandle))
			{
				fileEntry.casKey = CasKeyZero;
				return m_logger.Error(TC("GetFileInformationByHandle failed on %s"), fileName);
			}
			fileSize = info.size;
			storedSize = fileSize;
			lastWriteTime = info.lastWriteTime;

			#if UBA_TRACK_IS_EXECUTABLE
			#if PLATFORM_WINDOWS
			fileEntry.isExecutable = ToView(fileName).EndsWith(TCV(".exe"));
			#else
			fileEntry.isExecutable = IsExecutable(info.attributes);
			#endif
			#endif
		}

		if (fileEntry.casKey != CasKeyZero)
		{
			if (casKeyOverride != CasKeyZero && casKeyOverride != fileEntry.casKey)
			{
				fileEntry.casKey = casKeyOverride;
				entryLock.Leave();
				if (!AddCasFile(fileNameKey, fileName, casKeyOverride, deferCreation, storedSize, lastWriteTime, outEntry, dropCasAfterUse))
					return false;
				out = casKeyOverride;
				return true;
			}
			if (fileSize == fileEntry.size && (lastWriteTime && lastWriteTime == fileEntry.lastWritten) && fileEntry.casKey != CasKeyInvalid)
			{
				CasKey existingKey = fileEntry.casKey;
				entryLock.Leave();
				if (!AddCasFile(fileNameKey, fileName, existingKey, deferCreation, storedSize, lastWriteTime, outEntry, dropCasAfterUse))
					return false;
				out = existingKey;
				return true;
			}
		}

		fileEntry.size = fileSize;
		fileEntry.lastWritten = lastWriteTime;
		if (casKeyOverride == CasKeyZero)
		{
			if (externalMapping.handle.IsValid())
			{
				MemoryMappingScope scope(m_logger, m_sharedMemory);
				u8* fileMem = scope.MapView(TC("StoreCasFile"), externalMapping.handle, externalMapping.offset, externalMapping.size, fileName);
				if (!fileMem)
					return false;
				if (fileSize > sizeof(CompressedFileHeader) && ((CompressedFileHeader*)fileMem)->IsValid())
					fileEntry.casKey = ((CompressedFileHeader*)fileMem)->casKey;
				else
					fileEntry.casKey = CalculateCasKey(fileMem, externalMapping.size, m_storeCompressed);
			}
			else
			{
				bool handled = false;
				if (fileSize >= sizeof(CompressedFileHeader) && g_globalRules.FileCanBeCompressed(ToView(fileName)))
				{
					CompressedFileHeader header(CasKeyZero);
					if (!ReadFile(m_logger, fileName, fileHandle, &header, sizeof(header)))
						return m_logger.Error(TC("Failed to read header of compressed file %s (%s)"), fileName, LastErrorToText().data);
					if (header.IsValid())
					{
						fileEntry.casKey = AsCompressed(header.casKey, m_storeCompressed);
						handled = true;
					}
					else
					{
						if (!SetFilePointer(m_logger, fileName, fileHandle, 0))
							return false;
					}
				}

				if (!handled)
					fileEntry.casKey = CalculateCasKey(fileName, fileHandle, fileSize, m_storeCompressed);
			}
		}
		else
			fileEntry.casKey = casKeyOverride;

		if (fileEntry.casKey == CasKeyZero)
			return false;

		CasKey newKey = fileEntry.casKey;
		entryLock.Leave();

		if (!AddCasFile(fileNameKey, fileName, newKey, deferCreation, storedSize, lastWriteTime, outEntry, dropCasAfterUse))
			return false;

		out = newKey;
		return true;
	}

	bool StorageImpl::IsFileVerified(const StringKey& fileNameKey)
	{
		SCOPED_READ_LOCK(m_fileTableLookupLock, lookupLock);
		auto findIt = m_fileTableLookup.find(fileNameKey);
		if (findIt == m_fileTableLookup.end())
			return false;
		FileEntry& fileEntry = findIt->second;
		lookupLock.Leave();
		SCOPED_FUTEX_READ(fileEntry.lock, entryLock);
		return fileEntry.verified;
	}

	void StorageImpl::ReportFileInfoWeak(const StringKey& fileNameKey, u64 verifiedLastWriteTime, u64 verifiedSize)
	{
		SCOPED_READ_LOCK(m_fileTableLookupLock, lookupLock);
		auto findIt = m_fileTableLookup.find(fileNameKey);
		if (findIt == m_fileTableLookup.end())
			return;
		FileEntry& fileEntry = findIt->second;
		lookupLock.Leave();

		SCOPED_FUTEX(fileEntry.lock, entryLock);
		if (fileEntry.verified)
			return;
		fileEntry.verified = fileEntry.lastWritten == verifiedLastWriteTime && fileEntry.size == verifiedSize && fileEntry.casKey != CasKeyInvalid;
	}

	bool StorageImpl::HasBeenSeen(const CasKey& casKey)
	{
		SCOPED_READ_LOCK(m_casLookupLock, lookupLock);
		return m_casLookup.find(casKey) != m_casLookup.end();
	}

	void StorageImpl::ReleaseCasEntry(CasEntry& casEntry, bool dropIfZero)
	{
		CasEntryWriteLock casEntryLock(casEntry);
		ReleaseCasEntryNoLock(casEntry, dropIfZero);
	}

	void StorageImpl::ReleaseCasEntryNoLock(CasEntry& casEntry, bool dropIfZero)
	{
		UBA_ASSERTF(casEntry.refCount > 0, TC("Reducing ref count on cas entry that already has ref count zero (%s)"), CasKeyString(casEntry.key).str);
		if (--casEntry.refCount)
			return;
		if (!casEntry.dropped || !dropIfZero)
			return;
		if (!casEntry.memoryHandle.IsValid() || !m_casDataBuffer.IsIndependent(casEntry.memoryHandle))
			return;
		SharedMemoryHandle memoryHandle = casEntry.memoryHandle;
		casEntry.memoryHandle = {};
		casEntry.verified = true;
		casEntry.exists = false;
		casEntry.dropped = false;

		m_casDataBuffer.CloseIndependent(memoryHandle, casEntry.memorySize, TC("ReleaseCasEntryNoLock"));
	}

	bool StorageImpl::StoreCasKey(CasKey& out, const tchar* fileName, const CasKey& casKeyOverride)
	{
		StringBuffer<MaxPath> forKey;
		forKey.Append(fileName);
		if (CaseInsensitiveFs)
			forKey.MakeLower();
		StringKey fileNameKey = ToStringKey(forKey);
		return StoreCasKey(out, fileNameKey, fileName, casKeyOverride);
	}

	bool StorageImpl::StoreCasKey(CasKey& out, const StringKey& fileNameKey, const tchar* fileName, const CasKey& casKeyOverride, bool showErrors)
	{
		FileEntry& fileEntry = GetOrCreateFileEntry(fileNameKey);

		SCOPED_FUTEX(fileEntry.lock, entryLock);
	
		if (fileEntry.verified)
		{
			out = fileEntry.casKey;
			return true;
		}
		fileEntry.verified = true;

		// Use OpenFile+GetFileInformationByHandle+close instead of Getting file attributes because it is actually faster on cloud setups (weirdly enough)
		FileHandle fileHandle;
		if (!OpenFileSequentialRead(m_logger, fileName, fileHandle, false, false, showErrors))
		{
			fileEntry.casKey = CasKeyZero;
			out = CasKeyZero;
			return false;
		}

		if (fileHandle == InvalidFileHandle)
		{
			fileEntry.casKey = CasKeyZero;
			out = CasKeyZero;
			return true;
		}

		auto fileGuard = MakeGuard([&](){ CloseFile(fileName, fileHandle); });

		FileBasicInformation info;
		if (!GetFileBasicInformationByHandle(info, m_logger, fileName, fileHandle))
		{
			fileEntry.casKey = CasKeyZero;
			return m_logger.Error(TC("GetFileInformationByHandle failed on %s"), fileName);
		}

		u64 fileSize = info.size;
		u64 lastWritten = info.lastWriteTime;

		if (fileEntry.casKey != CasKeyZero)
		{
			if (casKeyOverride != CasKeyZero && casKeyOverride != fileEntry.casKey)
			{
				fileEntry.casKey = casKeyOverride;
				out = fileEntry.casKey;
				return true;
			}
			if (fileSize == fileEntry.size && lastWritten == fileEntry.lastWritten && fileEntry.casKey != CasKeyInvalid)
			{
				out = fileEntry.casKey;
				return true;
			}
		}

		fileEntry.size = fileSize;
		fileEntry.lastWritten = lastWritten;
		if (casKeyOverride == CasKeyZero)
		{
			bool handled = false;
			if (fileSize >= sizeof(CompressedFileHeader) && g_globalRules.FileCanBeCompressed(ToView(fileName)))
			{
				CompressedFileHeader header(CasKeyZero);
				if (!ReadFile(m_logger, fileName, fileHandle, &header, sizeof(header)))
					return m_logger.Error(TC("Failed to read header of compressed file %s (%s)"), fileName, LastErrorToText().data);
				if (header.IsValid())
				{
					fileEntry.casKey = AsCompressed(header.casKey, m_storeCompressed);
					handled = true;
				}
				else
				{
					if (!SetFilePointer(m_logger, fileName, fileHandle, 0))
						return false;
				}
			}

			if (!handled)
				fileEntry.casKey = CalculateCasKey(fileName, fileHandle, fileSize, m_storeCompressed);
		}
		else
			fileEntry.casKey = casKeyOverride;

		if (fileEntry.casKey == CasKeyZero)
			return false;

		out = fileEntry.casKey;
		return true;
	}

	bool StorageImpl::StoreCasFileClient(CasKey& out, StringKey fileNameKey, const tchar* fileName, SharedMemoryHandle memoryHandle, u64 memoryOffset, u64 fileSize, const tchar* hint, bool keepMappingInMemory, bool storeCompressed)
	{
		UBA_ASSERT(false);
		return false;
	}

	bool StorageImpl::HasCasFile(const CasKey& casKey, CasEntry** out, bool useMapping, bool allowFail)
	{
		SCOPED_READ_LOCK(m_casLookupLock, lookupLock);
		auto it = m_casLookup.find(casKey);
		if (it == m_casLookup.end())
			return false;
		CasEntry& casEntry = it->second;
		lookupLock.Leave();
		CasEntryAccessed(casEntry);

		CasEntryWriteLock casEntryLock(casEntry);

		if (casEntry.verified && casEntry.exists)
		{
			if (out)
			{
				*out = &casEntry;
				++casEntry.refCount;
			}
			return true;
		}

		SCOPED_FUTEX(m_deferredCasCreationLookupLock, deferredLock);
		auto findIt = m_deferredCasCreationLookup.find(casKey);
		if (findIt == m_deferredCasCreationLookup.end())
			return false;
		DeferredCasCreation& dcc = findIt->second;
		++dcc.refCount;
		deferredLock.Leave();

		SCOPED_FUTEX(dcc.lock, dccLock);

		auto dccGuard = MakeGuard([&]()
			{
				deferredLock.Enter();
				for (auto i = dcc.sources.begin(); i!=dcc.sources.end();)
				{
					auto sourceIt = m_deferredCasCreationLookupByName.find(*i);
					if (sourceIt != m_deferredCasCreationLookupByName.end() && sourceIt->second.manualDelete)
					{
						++i;
						continue;
					}
					m_deferredCasCreationLookupByName.erase(*i);
					i = dcc.sources.erase(i);
				}
				if (!--dcc.refCount && dcc.sources.empty())
				{
					dccLock.Leave();
					m_deferredCasCreationLookup.erase(findIt);
				}
			});

		if (dcc.sources.empty())
			return false;

		StringKey fileNameKey = dcc.sources.front();
		deferredLock.Enter();
		auto sourceFindIt = m_deferredCasCreationLookupByName.find(fileNameKey);
		UBA_ASSERT(sourceFindIt != m_deferredCasCreationLookupByName.end());
		DeferredSource* source = &sourceFindIt->second;
		deferredLock.Leave();
		StringBuffer<MaxPath> deferredCreation(source->fileName);
		//u64 deferredFileSize = findIt->second.fileSize;

		if (casEntry.beingWritten)
			m_logger.Warning(TC("Deferred cas %s is being written from network. This should never happen (%s)"), CasKeyString(casKey).str, deferredCreation.data);

		// TODO: We need to validate that the file is still the same file.
		// It seems like there can be a race condition with ispc and .generated.h where file content changes
		// while .generated.h is in the deferred creation lookup.
		// .. we need to add verification for this and throw an error

		casEntry.verified = true;
		WriteCasResult res;
		if (!WriteCasFile(res, deferredCreation.data, casKey, useMapping, source, allowFail))
			return m_logger.Logf(allowFail?LogEntryType_Debug:LogEntryType_Error, TC("Failed to write deferred cas %s (%s)"), CasKeyString(casKey).str, deferredCreation.data).ToFalse();

		dccGuard.Execute();

		casEntry.dropped = res.dropAfterUse;

		// TODO: THIS MISMATCH WITH compressed vs uncompressed
		// What can happen is that we use an external memory mapping to calculate the caskey and then defer the creation of the cas
		// External memory mapping gets written to disk compressed
		// External memory mapping gets evicted after x seconds because it is not being used
		// Another agent wants the content of the caskey and now needs to re-read the file which is now compressed.
		//if (res.sourceStoredSize != deferredFileSize) // TODO: Should also check last written time
		//	m_logger.Warning(TC("Deferred cas %s was deferred with source file size %llu but was read with source file size %llu (%s)"), CasKeyString(casKey).str, deferredFileSize, res.sourceStoredSize, deferredCreation.data);

		bool hasMapping = false;
		if (res.memoryHandle.IsValid())
		{
			casEntry.memoryHandle = res.memoryHandle;
			casEntry.memoryOffset = res.memoryOffset;
			casEntry.memorySize = res.storedSize;
			hasMapping = true;
		}

		casEntry.exists = true;

		if (out)
		{
			*out = &casEntry;
			++casEntry.refCount;
		}
		else if (res.dropAfterUse)
		{
			m_logger.Warning(TC("Cas is set to drop-after-use but it is not being used in this call so it will leak %s (%s)"), CasKeyString(casKey).str, deferredCreation.data);
		}

		casEntryLock.Leave();

		if (!hasMapping)
			CasEntryWritten(casEntry, res.storedSize);
		return true;
	}

	bool StorageImpl::EnsureCasFile(const CasKey& casKey, const tchar* fileName)
	{
		CasEntry& casEntry = GetOrCreateCasEntry(casKey);
		CasEntryAccessed(casEntry);

		{
			CasEntryReadLock casEntryLock(casEntry);
			if (casEntry.verified)
			{
				if (casEntry.exists)
					return true;
				if (!fileName)
					return false;
			}
		}

		CasEntryWriteLock casEntryLock(casEntry);

		if (casEntry.verified)
		{
			if (casEntry.exists)
				return true;
			if (!fileName)
				return false;
		}

		StringBuffer<> casFile;
		if (!StorageImpl::GetCasFileName(casFile, casKey))
			return false;

		bool verifyReturnValue;
		if (VerifyExisting(verifyReturnValue, casEntryLock, casKey, casEntry, casFile))
			return verifyReturnValue;

		casEntry.exists = false;
		casEntry.verified = true;
		if (!fileName)
			return false;
		WriteCasResult res;
		if (!WriteCasFile(res, fileName, casKey, false, nullptr, false))
			return false;
		casEntry.exists = true;
		casEntryLock.Leave();
		CasEntryWritten(casEntry, res.storedSize);
		return true;
	}

	bool StorageImpl::GetCasFileName(StringBufferBase& out, const CasKey& casKey)
	{
		out.Append(m_rootDir.data).AppendHex(((const u8*)&casKey)[0]);
		if (m_writeToDisk)
			if (!CreateDirectory(out.data))
				return false;
		out.Append(PathSeparator).Append(CasKeyString(casKey).str);
		return true;
	}

	MappedView2 StorageImpl::MapView(const CasKey& casKey, const tchar* hint)
	{
		SCOPED_READ_LOCK(m_casLookupLock, lookupLock);
		auto findIt = m_casLookup.find(casKey);
		bool foundEntry = findIt != m_casLookup.end();
		if (!foundEntry)
		{
			m_logger.Error(TC("Can't find %s inside cas database (%s)"), CasKeyString(casKey).str, hint);
			return {};
		}
		CasEntry& casEntry = findIt->second;
		lookupLock.Leave();
		CasEntryWriteLock casEntryLock(casEntry);
		//if (!casEntry.verified)
		//{
		//	m_logger.Error(TC("Trying to use unverified mapping of %s (%s)"), CasKeyString(casKey).str, hint);
		//	return {{},0,0 };
		//}
		SharedMemoryHandle memoryHandle = casEntry.memoryHandle;
		u64 offset = casEntry.memoryOffset;
		u64 size = casEntry.memorySize;
		casEntryLock.Leave();

		MappedView2 res = m_casDataBuffer.MapView(memoryHandle, offset, size, hint);
		if (!res.memory)
			m_logger.Error(TC("Failed to map view for %s (%s)"), CasKeyString(casKey).str, hint);
		return res;
	}

	void StorageImpl::UnmapView(const MappedView& view, const tchar* hint)
	{
		m_casDataBuffer.UnmapView(view, hint);
	}

	SharedMemoryAllocator& StorageImpl::GetSharedMemory()
	{
		return m_sharedMemory;
	}

	bool StorageImpl::DropCasFile(const CasKey& casKey, bool forceDelete, const tchar* hint, bool reduceRefCount)
	{
		SCOPED_READ_LOCK(m_casLookupLock, lookupLock);
		auto findIt = m_casLookup.find(casKey);
		bool foundEntry = findIt != m_casLookup.end();
		if (!foundEntry)
		{
			if (forceDelete)
			{
				StringBuffer<> casFile;
				if (!StorageImpl::GetCasFileName(casFile, casKey))
					return false;
				if (DeleteFileW(casFile.data) == 0)
				{
					u32 lastError = GetLastError();
					if (lastError != ERROR_FILE_NOT_FOUND && lastError != ERROR_PATH_NOT_FOUND)
						return m_logger.Error(TC("Failed to drop cas %s (%s) (%s)"), casFile.data, hint, LastErrorToText(lastError).data);
				}
			}
			return true;
		}
		CasEntry& casEntry = findIt->second;
		lookupLock.Leave();
	
		CasEntryWriteLock casEntryLock(casEntry);
		casEntry.dropped = true;

		if (!forceDelete)
			return true;

		StringBuffer<> casFile;
		if (!StorageImpl::GetCasFileName(casFile, casKey))
			return false;
		if (reduceRefCount)
		{
			UBA_ASSERTF(casEntry.refCount > 0, TC("Reducing refcount of cas %s which already has 0 refcount (%s)"), casFile.data, hint);
			--casEntry.refCount;
		}

		// It could be that this cas is used somewhere and in that case we can't force delete it
		if (casEntry.refCount != 0)
			return false;//m_logger.Info(TC("Skipped dropping cas %s because refcount was %u and not 0 (%s)"), casFile.data, u32(casEntry.refCount), hint).ToFalse();

		//UBA_ASSERTF(casEntry.refCount == 0, TC("Force deleting cas %s with refcount %u (%s)"), casFile.data, casEntry.refCount, hint);
		u64 sizeDeleted = 0;

		if (casEntry.memoryHandle.IsValid())
		{
			if (m_casDataBuffer.IsIndependent(casEntry.memoryHandle))
			{
				m_casDataBuffer.CloseIndependent(casEntry.memoryHandle, casEntry.memorySize, TC("DropCasFile"));
				casEntry.memoryHandle = {};
			}
		}
		else
		{
			if (DeleteFileW(casFile.data) == 0)
			{
				u32 lastError = GetLastError();
				if (lastError != ERROR_FILE_NOT_FOUND && lastError != ERROR_PATH_NOT_FOUND)
					return m_logger.Error(TC("Failed to drop cas %s (%s) (%s)"), casFile.data, hint, LastErrorToText(lastError).data);
			}
			else
			{
				m_casDroppedBytes += casEntry.size;
				m_casDroppedCount++;
				//m_logger.Debug(TC("Evicted %s from cache (%llukb)"), casFile, casEntry.size/1024);
				sizeDeleted = casEntry.size;
			}
		}
		casEntry.verified = true;
		casEntry.exists = false;
		casEntry.dropped = false;
		casEntryLock.Leave();
	
		CasEntryDeleted(casEntry, sizeDeleted);

		return true;
	}

	bool StorageImpl::ReportBadCasFile(const CasKey& casKey)
	{
		DropCasFile(casKey, true, TC("BadCasFile"));
		return true;
	}

	bool StorageImpl::CalculateCasKey(CasKey& out, StringKey fileNameKey, const tchar* fileName)
	{
		ExternalFileMapping externalMapping;
		auto externalMappingGuard = MakeGuard([&]() { m_sharedMemory.CloseHandle(m_logger, externalMapping.handle, TC("ExternalFileMapping")); });

		if (m_externalFileMappingsProvider && m_externalFileMappingsProvider(externalMapping, fileNameKey, fileName))
		{
			MemoryMappingScope scope(m_logger, m_sharedMemory);
			u8* fileMem = scope.MapView(TC("CalculateCasKey"), externalMapping.handle, externalMapping.offset, externalMapping.size, fileName);
			if (!fileMem)
				return false;
			out = CalculateCasKey(fileMem, externalMapping.size, true);
			return out != CasKeyZero;
		}

		FileHandle fileHandle;
		if (!OpenFileSequentialRead(m_logger, fileName, fileHandle))
			return m_logger.Error(TC("[CalculateCasKey] OpenFileSequentialRead failed for %s (%s)"), fileName, LastErrorToText().data);

		auto fileGuard = MakeGuard([&](){ CloseFile(fileName, fileHandle); });

		u64 fileSize;
		if (!uba::GetFileSizeEx(fileSize, fileHandle))
			return m_logger.Error(TC("[CalculateCasKey] GetFileSize failed for %s (%s)"), fileName, LastErrorToText().data);

		out = CalculateCasKey(fileName, fileHandle, fileSize, true);
		return out != CasKeyZero;
	}

	bool StorageImpl::CopyOrLink(const CasKey& casKey, const tchar* destination, u32 fileAttributes, bool writeCompressed, u32 clientId, const FormattingFunc& formattingFunc, bool isTemp, bool allowHardLink)
	{
		UBA_ASSERT(casKey != CasKeyZero);
		UBA_ASSERT(fileAttributes);

		auto& stats = Stats();
		
		#if PLATFORM_WINDOWS
		BottleneckScope bs(m_maxParallelCopyOrLinkBottleneck, stats.copyOrLinkWait);
		#endif

		StringBuffer<MaxPath> forKey;
		forKey.Append(destination);
		if (CaseInsensitiveFs)
			forKey.MakeLower();
		StringKey key = ToStringKey(forKey);
		FileEntry& fileEntry = GetOrCreateFileEntry(key);

		TimerScope ts(stats.copyOrLink);

		CasKey actualKey = AsCompressed(casKey, writeCompressed);

		bool testCompressed = !writeCompressed;
		while (true)
		{
			CasEntry* casEntryPtr = nullptr;
			if (!HasCasFile(actualKey, &casEntryPtr, false, false))
			{
				if (!testCompressed)
				{
					SCOPED_READ_LOCK(m_casLookupLock, lookupLock);
					auto it = m_casLookup.find(casKey);
					bool found = it == m_casLookup.end();
					lookupLock.Leave();
					const tchar* extraInfo = TC("NeverExisted");
					if (found)
					{
						CasEntry& casEntry = it->second;
						CasEntryWriteLock casEntryLock(casEntry);
						extraInfo = casEntry.dropped ? TC("Dropped") : TC("InLookup");
					}

					return m_logger.Warning(TC("[CopyOrLink] Trying to copy cas %s to %s but can't find neither compressed or uncompressed version (%s)"), CasKeyString(actualKey).str, destination, extraInfo);
				}
				actualKey = AsCompressed(casKey, true);
				testCompressed = false;
				continue;
			}

			CasEntry& casEntry = *casEntryPtr;
			CasEntryReadLock casEntryLock(casEntry);
			UBA_ASSERTF(casEntry.verified, TC("CasEntry %s not verified for copy to %s"), CasKeyString(actualKey).str, destination);
			UBA_ASSERT(casEntry.exists);
			--casEntry.refCount; // Decrease after HasCasFile increase it

			auto done = [&](FileAccessor& destinationFile, u64 sizeOnDisk)
				{
					u64 lastWriteTime = 0;
					if (!destinationFile.Close(&lastWriteTime))
						return false;
					UBA_ASSERT(lastWriteTime);
					if (lastWriteTime)
					{
						fileEntry.casKey = casKey;
						fileEntry.lastWritten = lastWriteTime;
						fileEntry.size = sizeOnDisk;
						fileEntry.verified = true;
					}
					return true;
				};

			if (IsCompressed(actualKey))
			{
				FileHandle readHandle = InvalidFileHandle;
				auto rsg = MakeGuard([&](){ if (readHandle != InvalidFileHandle) CloseFile(nullptr, readHandle); });

				const u8* compressedData = nullptr;
				const u8* readData = nullptr;
				MappedView mappedView;
				auto mapViewGuard = MakeGuard([&](){ m_casDataBuffer.UnmapView(mappedView, destination); });

				StringBuffer<512> casFile;
				u64 decompressedSize;

				if (casEntry.memoryHandle.IsValid())
				{
					casFile.Append(CasKeyString(actualKey).str);
					mappedView = m_casDataBuffer.MapView(casEntry.memoryHandle, casEntry.memoryOffset, casEntry.memorySize, casFile.data);
					compressedData = mappedView.memory;
					if (!compressedData)
						return m_logger.Error(TC("[CopyOrLink] Failed to map view of mapping %s (%s)"), casFile.data, LastErrorToText().data);

					decompressedSize = *(u64*)compressedData;
					readData = compressedData + sizeof(u64);
				}
				else
				{
					if (!StorageImpl::GetCasFileName(casFile, actualKey))
						return false;

					if (!OpenFileSequentialRead(m_logger, casFile.data, readHandle))
						return m_logger.Error(TC("[CopyOrLink] Failed to open file %s for read (%s)"), casFile.data, LastErrorToText().data);

					if (!ReadFile(m_logger, casFile.data, readHandle, &decompressedSize, sizeof(u64)))
						return m_logger.Error(TC("[CopyOrLink] Failed to read first 8 bytes from compressed file %s (%s)"), casFile.data, LastErrorToText().data);
				}

				bool writeDirectlyToFile = casEntry.memoryHandle.IsValid() && false; // Experiment to try to fix bottlenecks on cloud
				bool useNoBuffering = writeDirectlyToFile && false; // Experiment to try to fix bottlenecks on cloud

				u32 writeFlags = useNoBuffering ? FILE_FLAG_NO_BUFFERING : 0;
				bool allowRead = !useNoBuffering;

				FileAccessor destinationFile(m_logger, destination);

				SCOPED_FUTEX(fileEntry.lock, entryLock);
				fileEntry.verified = false;
				fileEntry.isTemp = isTemp;

				u64 sizeOnDisk = decompressedSize;

				auto decompressToMemory = [&](u8* destinationMemory)
					{
						if (readData)
						{
							if (!DecompressMemoryToMemory(readData, mappedView.size, destinationMemory, decompressedSize, casFile.data, destination))
								return false;
						}
						else
						{
							if (!DecompressFileToMemory(CasKeyString(actualKey).str, readHandle, destinationMemory, decompressedSize, destination, 0))
								return false;
						}
						return true;
					};


				if (formattingFunc)
				{
					UBA_ASSERT(!writeCompressed);
					u8* slot = m_bufferSlots.Pop();
					auto _ = MakeGuard([&](){ m_bufferSlots.Push(slot); });

					UBA_ASSERT(decompressedSize < BufferSlotSize);
					if (!decompressToMemory(slot))
						return false;

					MemoryBlock block(TC("Formatting"), 5*1024*1024);

					if (!formattingFunc(block, slot, decompressedSize, destination))
						return false;

					if (!destinationFile.CreateWrite(false, DefaultAttributes(), block.writtenSize, m_tempPath.data))
						return false;
						
					if (!destinationFile.Write(block.memory, block.writtenSize))
						return false;
				}
				else if (writeCompressed)
				{
					u64 compressedFileSize = mappedView.size;
					if (!mappedView.memory)
						if (!GetFileSizeEx(compressedFileSize, readHandle))
							return m_logger.Error(TC("[CopyOrLink] Failed to get file size of compressed file %s (%s)"), casFile.data, LastErrorToText().data);
						
					sizeOnDisk = compressedFileSize + sizeof(CompressedFileHeader);

					if (!destinationFile.CreateMemoryWrite(false, fileAttributes, sizeOnDisk, m_tempPath.data))
						return false;
					u8* writePos = destinationFile.GetData();
					*(CompressedFileHeader*)writePos = CompressedFileHeader(casKey);
					writePos += sizeof(CompressedFileHeader);

					if (mappedView.memory)
					{
						TimerScope cts(stats.memoryCopy);
						FileMapping_CopyMem(writePos, mappedView.memory, compressedFileSize);
					}
					else
					{
						*(u64*)writePos = decompressedSize;
						writePos += sizeof(u64);
						if (!ReadFile(m_logger, casFile.data, readHandle, writePos, compressedFileSize - sizeof(u64)))
							return m_logger.Error(TC("[CopyOrLink] Failed to read compressed file %s (%s)"), casFile.data, LastErrorToText().data);
					}
						
				}
				else if (writeDirectlyToFile || !decompressedSize)
				{
					if (!destinationFile.CreateWrite(allowRead, writeFlags | fileAttributes, decompressedSize, m_tempPath.data))
						return false;
					if (decompressedSize)
					{
						DecompressWriter w { m_logger, m_bufferSlots, m_workManager, Stats() };
						if (!w.DecompressMemoryToFile(readData, destinationFile, decompressedSize, useNoBuffering))
							return false;
					}
				}
				else
				{
					if (!destinationFile.CreateMemoryWrite(allowRead, writeFlags | fileAttributes, decompressedSize, m_tempPath.data))
						return false;
					if (!decompressToMemory(destinationFile.GetData()))
						return false;
				}

				return done(destinationFile, sizeOnDisk);
			}

			if (casEntry.memoryHandle.IsValid())
			{
				SCOPED_FUTEX(fileEntry.lock, entryLock);
				fileEntry.verified = false;

				MappedView mappedView = m_casDataBuffer.MapView(casEntry.memoryHandle, casEntry.memoryOffset, casEntry.memorySize, TC(""));
				auto mapViewGuard = MakeGuard([&](){ m_casDataBuffer.UnmapView(mappedView, destination); });

				FileAccessor destinationFile(m_logger, destination);
				if (!destinationFile.CreateMemoryWrite(false, fileAttributes, mappedView.size, m_tempPath.data))
					return false;
				u8* writePos = destinationFile.GetData();
				TimerScope cts(stats.memoryCopy);
				FileMapping_CopyMem(writePos, mappedView.memory, mappedView.size);
				return done(destinationFile, mappedView.size);
			}

			StringBuffer<> casFile;
			if (!GetCasFileName(casFile, actualKey))
				return false;

			SCOPED_FUTEX(fileEntry.lock, entryLock);
			fileEntry.verified = false;

			bool firstTry = true;
			while (true)
			{
				bool success = false;

				#if !PLATFORM_MAC // For some reason creating links on macos causes trouble when they are exec/dylibs and being executed.. sometimes it is like the link behaves like a symlink.. but not always
				if (allowHardLink)
					success = CreateHardLinkW(destination, casFile.data);
				#endif

				if (!success)
				{
					u8* mem = nullptr;
					if (IsRunningDarling()) // Should also be used in WSL2
						mem = m_bufferSlots.Pop(); 
					success = uba::CopyFileW(casFile.data, destination, true, mem, BufferSlotSize);
					m_bufferSlots.Push(mem);
				}

				if (success)
				{
					#if !PLATFORM_WINDOWS
					if (fileAttributes & S_IXUSR)
					{
						struct stat destStat;
						int res = stat(destination, &destStat);
						UBA_ASSERTF(res == 0, TC("stat failed (%s) error: %s"), destination, strerror(errno));
						if ((destStat.st_mode & S_IXUSR) == 0)
						{
							res = chmod(destination, S_IRUSR | S_IWUSR | S_IXUSR); (void)res;
							UBA_ASSERTF(res == 0, TC("chmod failed (%s) error: %s"), destination, strerror(errno));
						}
					}
					#endif
					return true;
				}
				if (!firstTry)
					return m_logger.Error(TC("Failed link/copy %s to %s (%s)"), casFile.data, destination, LastErrorToText().data);

				firstTry = false;
				DeleteFileW(destination);
				continue;

				//SetFileTime(destination, DateTime.UtcNow);
			}
		}

	}

	bool StorageImpl::WritePlaceholder(const CasKey& casKey, const tchar* destination, u32 fileAttributes, u32 clientId)
	{
		// We need to populate FileEntry and set the correct caskey even if it is not what we are writing to disk
		// If we don't do this, then CacheClient will re-calculate the cas from the placeholder file and send wrong keys

		StringBuffer<MaxPath> forKey;
		forKey.Append(destination);
		if (CaseInsensitiveFs)
			forKey.MakeLower();
		StringKey key = ToStringKey(forKey);
		FileEntry& fileEntry = GetOrCreateFileEntry(key);
		SCOPED_FUTEX(fileEntry.lock, lock2);

		u64 lastWriteTime = 0;
		FileAccessor destinationFile(m_logger, destination);
		if (!destinationFile.CreateWrite() || !destinationFile.Close(&lastWriteTime))
			return false;

		fileEntry.casKey = casKey;
		fileEntry.lastWritten = lastWriteTime;
		fileEntry.size = 0;
		fileEntry.verified = true;
		fileEntry.isTemp = true;

		#if UBA_TRACK_IS_EXECUTABLE
		#if PLATFORM_WINDOWS
		fileEntry.isExecutable = ToView(destination).EndsWith(TCV(".exe"));
		#else
		fileEntry.isExecutable = IsExecutable(fileAttributes);
		#endif
		#endif
		return true;
	}

	bool StorageImpl::FakeCopy(const CasKey& casKey, const tchar* destination, u64 size, u64 lastWritten, u32 fileAttributes, bool deleteExisting)
	{
		if (deleteExisting)
			DeleteFileW(destination);

		StringBuffer<MaxPath> forKey;
		forKey.Append(destination);
		if (CaseInsensitiveFs)
			forKey.MakeLower();
		StringKey key = ToStringKey(forKey);
		FileEntry& fileEntry = GetOrCreateFileEntry(key);
		SCOPED_FUTEX(fileEntry.lock, lock2);
		fileEntry.casKey = casKey;
		fileEntry.lastWritten = lastWritten;
		fileEntry.size = size;
		fileEntry.verified = true;
		fileEntry.isTemp = deleteExisting; // For now, keep entries created by cache client downloads

		#if UBA_TRACK_IS_EXECUTABLE
		#if PLATFORM_WINDOWS
		fileEntry.isExecutable = ToView(destination).EndsWith(TCV(".exe"));
		#else
		fileEntry.isExecutable = IsExecutable(fileAttributes);
		#endif
		#endif
		return true;
	}

	void StorageImpl::ReportFileWrite(StringKey fileNameKey, const tchar* fileName)
	{
		// If a deferred cas creation is queued up while the source file is about to be modified we need to flush out the cas creation before modifying the file
		SCOPED_FUTEX_READ(m_deferredCasCreationLookupLock, deferredLock);
		auto findIt = m_deferredCasCreationLookupByName.find(fileNameKey);
		if (findIt == m_deferredCasCreationLookupByName.end())
			return;
		CasKey key = findIt->second.key;
		m_logger.Logf(IsDebug ? LogEntryType_Warning : LogEntryType_Info, TC("Flush cas creation of file %s at %s (This should be rare)"), CasKeyString(key).str, fileName);
		deferredLock.Leave();
		HasCasFile(key, nullptr, false, false);
	}
	/*
	void StorageImpl::PushStats(StorageStats* stats)
	{
		t_threadStats = stats;
	}

	void StorageImpl::PopStats(StorageStats* stats)
	{
		m_stats.calculateCasKey.Add(stats->calculateCasKey);
		m_stats.copyOrLink.Add(stats->copyOrLink);
		m_stats.copyOrLinkWait.Add(stats->copyOrLinkWait);
		m_stats.sendCas.Add(stats->sendCas);
		m_stats.recvCas.Add(stats->recvCas);
		m_stats.compressWrite.Add(stats->compressWrite);
		m_stats.compressSend.Add(stats->compressSend);
		m_stats.decompressRecv.Add(stats->decompressRecv);
		m_stats.decompressToMem.Add(stats->decompressToMem);
		m_stats.sendCasBytesRaw += stats->sendCasBytesRaw;
		m_stats.sendCasBytesComp += stats->sendCasBytesComp;
		m_stats.recvCasBytesRaw += stats->recvCasBytesRaw;
		m_stats.recvCasBytesComp += stats->recvCasBytesComp;
		m_stats.createCas.Add(stats->createCas);
		m_stats.createCasBytesRaw += stats->createCasBytesRaw;
		m_stats.createCasBytesComp += stats->createCasBytesComp;
		t_threadStats = nullptr;
	}
	*/
	StorageStats& StorageImpl::Stats()
	{
		if (StorageStats* s = StorageStats::GetCurrent())
			return *s;
		return m_stats;
	}

	void StorageImpl::AddStats(StorageStats& stats)
	{
		m_stats.Add(stats);
	}

	void StorageImpl::TraceUpdate(Trace& trace, u32 startRow)
	{
		#if 0
		m_casDataBuffer.TraceStatus(trace, startRow, TC("Cas"), 1);
		#endif

		#if 0
		SCOPED_FUTEX(m_deferredCasCreationLookupLock, lock);
		u64 byCas = m_deferredCasCreationLookup.size();
		u64 byName = m_deferredCasCreationLookupByName.size();
		lock.Leave();

		StringBuffer<> str;
		str.Appendf(TC("DeferredCount  Cas: %llu Name: %llu"), byCas, byName);
		trace.StatusUpdate(startRow, 7, str, LogEntryType_Info);
		#endif
	}

	void StorageImpl::PrintSummary(Logger& logger)
	{
		logger.Info(TC("  ------- Storage stats summary -------"));
		if (m_casLookup.empty())
		{
			logger.Info(TC("  Storage not loaded"));
			logger.Info(TC(""));
			return;
		}
		
		logger.Info(TC("  WorkMemoryBuffers    %6u %9s"), u32(m_bufferSlots.m_slots.size()), BytesToText(m_bufferSlots.m_slots.size() * BufferSlotSize).str);
		logger.Info(TC("  FileTable            %6u"), u32(m_fileTableLookup.size()));

		StorageStats& stats = Stats();
		logger.Info(TC("  CasDataBuffers       %6u %9s"), 1, BytesToText(m_casDataBuffer.GetUsed()).str);
		logger.Info(TC("  CasTable             %6u %9s"), u32(m_casLookup.size()), BytesToText(m_casTotalBytes).str);
		logger.Info(TC("     Dropped           %6u %9s"), m_casDroppedCount, BytesToText(m_casDroppedBytes).str);
		logger.Info(TC("     Evicted           %6u %9s"), m_casEvictedCount, BytesToText(m_casEvictedBytes).str);
		logger.Info(TC("     HandleOverflow    %6u %9s"), stats.handleOverflow.count.load(), TimeToText(stats.handleOverflow.time).str);
		stats.Print(logger);

		if (u64 deferredCount = m_deferredCasCreationLookup.size())
			logger.Info(TC("  DeferredCasSkipped   %6u"), u32(deferredCount));
		logger.Info(TC(""));
	}

	CasKey StorageImpl::CalculateCasKey(const u8* fileMem, u64 fileSize, bool storeCompressed)
	{
		StorageStats& stats = Stats();
		TimerScope ts(stats.calculateCasKey);
		return uba::CalculateCasKey(fileMem, fileSize, storeCompressed, &m_workManager, nullptr);
	}


	CasKey StorageImpl::CalculateCasKey(const tchar* fileName, FileHandle fileHandle, u64 fileSize, bool storeCompressed)
	{
		StorageStats& stats = Stats();
		TimerScope ts(stats.calculateCasKey);

		if (fileSize > BufferSlotSize) // Note that when filesize is larger than BufferSlotSize the hash becomes a hash of hashes
		{
			FileMappingHandle fileMapping = FileMapping_CreateFromFile(m_logger, fileHandle, PAGE_READONLY, fileSize, fileName);
			if (!fileMapping.IsValid())
			{
				m_logger.Error(TC("Failed to create file mapping for %s (%s)"), fileName, LastErrorToText().data);
				return CasKeyZero;
			}
			auto fmg = MakeGuard([&]() { FileMapping_Close(m_logger, fileMapping, fileName); });
			u8* fileData = FileMapping_Map(m_logger, fileMapping, FILE_MAP_READ, 0, fileSize);
			if (!fileData)
			{
				m_logger.Error(TC("Failed to map view of file mapping for %s (%s)"), fileName, LastErrorToText().data);
				return CasKeyZero;
			}
			auto udg = MakeGuard([&]() { 
					if (m_asyncUnmapViewOfFile)
						m_workManager.AddWork([this, fileData, fileSize, fn = TString(fileName)](const WorkContext&) { FileMapping_Unmap(m_logger, fileData, fileSize, fn.c_str()); }, 1, TC("UnmapFile"));
					else
						FileMapping_Unmap(m_logger, fileData, fileSize, fileName);
				});

			return uba::CalculateCasKey(fileData, fileSize, storeCompressed, &m_workManager, fileName);
		}

		CasKeyHasher hasher;
		u8* slot = m_bufferSlots.Pop();
		auto _ = MakeGuard([&](){ m_bufferSlots.Push(slot); });
		u64 left = fileSize;
		while (left)
		{
			u32 toRead = u32(Min(left, BufferSlotSize));
			if (!ReadFile(m_logger, fileName, fileHandle, slot, toRead))
				return CasKeyZero;
			hasher.Update(slot, toRead);
			left -= toRead;
		}

		return ToCasKey(hasher, storeCompressed);
	}

	bool StorageImpl::DecompressFileToMemory(const tchar* fileName, FileHandle fileHandle, u8* dest, u64 decompressedSize, const tchar* writeHint, u64 fileStartOffset)
	{
		DecompressWriter w { m_logger, m_bufferSlots, m_workManager, Stats() };
		return w.DecompressFileToMemory(fileName, fileHandle, dest, decompressedSize, writeHint, fileStartOffset);
	}

	bool StorageImpl::CreateDirectory(const tchar* dir, bool* outAlreadyExists)
	{
		return m_dirCache.CreateDirectory(m_logger, dir, outAlreadyExists);
	}

	bool StorageImpl::DeleteCasForFile(const tchar* file)
	{
		FileEntry* fileEntry;
		if (!GetFileEntry(fileEntry, file) || !fileEntry)
			return false;

		SCOPED_FUTEX(fileEntry->lock, entryLock);
		fileEntry->verified = false;
		CasKey casKey = fileEntry->casKey;
		entryLock.Leave();
		return DropCasFile(casKey, true, file);
	}

	bool StorageImpl::GetFileSize(u64& outSize, const CasKey& casKey, const tchar* hint)
	{
		SCOPED_READ_LOCK(m_casLookupLock, lookupLock);
		auto findIt = m_casLookup.find(casKey);
		if  (findIt == m_casLookup.end())
			return m_logger.Error(TC("GetCasEntry - Couldn't find cas entry for %s"), hint);
		CasEntry& casEntry = findIt->second;
		CasEntryReadLock casEntryLock(casEntry);
		if (!casEntry.memoryHandle.IsValid())
			return m_logger.Error(TC("Code path only supports mapped cas entries"));
		if (!IsCompressed(casEntry.key))
		{
			outSize = casEntry.memorySize;
			return true;
		}
		MemoryMappingScope scope(m_logger, m_sharedMemory);
		u8* fileMem = scope.MapView(TC("ReadFileSize"), casEntry.memoryHandle, casEntry.memoryOffset, 8, hint);
		if (!fileMem)
			return false;
		outSize = *(u64*)fileMem;
		return true;
	}

	bool StorageImpl::GetFileData(void* outData, const CasKey& casKey, const tchar* hint)
	{
		SCOPED_READ_LOCK(m_casLookupLock, lookupLock);
		auto findIt = m_casLookup.find(casKey);
		if  (findIt == m_casLookup.end())
			return m_logger.Error(TC("GetCasEntry - Couldn't find cas entry for %s"), hint);
		CasEntry& casEntry = findIt->second;
		CasEntryReadLock casEntryLock(casEntry);

		MemoryMappingScope scope(m_logger, m_sharedMemory);
		u8* fileMem = scope.MapView(TC("ReadFileSize"), casEntry.memoryHandle, casEntry.memoryOffset, casEntry.memorySize, hint);
		if (!fileMem)
			return false;
		if (!IsCompressed(casEntry.key))
		{
			memcpy(outData, fileMem, casEntry.memorySize);
			return true;
		}
		u64 decompressedSize = *(u64*)fileMem;
		fileMem += sizeof(u64);
		return DecompressMemoryToMemory(fileMem, casEntry.memorySize, (u8*)outData, decompressedSize, CasKeyString(casEntry.key).str, hint);
	}
}
