// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaStorageClient.h"
#include "UbaConfig.h"
#include "UbaDefinitions.h"
#include "UbaDirectoryIterator.h"
#include "UbaFileAccessor.h"
#include "UbaNetworkClient.h"
#include "UbaNetworkMessage.h"
#include "UbaNetworkBackendTcp.h"
#include "UbaStorageUtils.h"

#define UBA_REPORT_PROXYFETCH 0

namespace uba
{
	inline CasKey AsProxyCasKey(const CasKey& key)
	{
		CasKey newKey = key;
		#ifndef __clang_analyzer__
		u8 flagField = ((u8*)&key)[19];
		((u8*)&newKey)[19] = (flagField | IsProxyMask);
		#endif
		return newKey;
	}


	void StorageClientCreateInfo::Apply(const Config& config)
	{
		StorageCreateInfo::Apply(config);

		const ConfigTable* tablePtr = config.GetTable(TC("Storage"));
		if (!tablePtr)
			return;
		const ConfigTable& table = *tablePtr;
		table.GetValueAsBool(sendCompressed, TC("SendCompressed"));
		table.GetValueAsBool(allowProxy, TC("AllowProxy"));
		table.GetValueAsBool(sendOneBigFileAtTheTime, TC("SendOneBigAtTheTime"));
		table.GetValueAsBool(fileParallelSend, TC("FileParallelSend"));
		table.GetValueAsBool(checkExistsOnServer, TC("CheckExistsOnServer"));
		table.GetValueAsBool(resendCas, TC("ResendCas"));
		table.GetValueAsU32(proxyConnectionCount, TC("ProxyConnectionCount"));
	}

	StorageClient::StorageClient(const StorageClientCreateInfo& info)
	:	StorageImpl(info, TC("UbaStorageClient"))
	,	m_client(info.client)
	,	m_sendCompressed(info.sendCompressed)
	,	m_allowProxy(info.allowProxy)
	,	m_sendOneBigFileAtTheTime(info.sendOneBigFileAtTheTime)
	,	m_fileParallelSend(info.fileParallelSend)
	,	m_checkExistsOnServer(info.checkExistsOnServer)
	,	m_resendCas(info.resendCas)
	,	m_zone(info.zone)
	,	m_getProxyBackendCallback(info.getProxyBackendCallback)
	,	m_getProxyBackendUserData(info.getProxyBackendUserData)
	,	m_startProxyCallback(info.startProxyCallback)
	,	m_startProxyUserData(info.startProxyUserData)
	,	m_proxyConnectionCount(info.proxyConnectionCount)
	,	m_proxyPort(info.proxyPort)
	,	m_proxyAddress(info.proxyAddress)
	{
	}

	struct StorageClient::ProxyClient
	{
		ProxyClient(bool& outCtorSuccess, const NetworkClientCreateInfo& info) : client(outCtorSuccess, info, TC("UbaProxyClient")) {}
		~ProxyClient() { client.Disconnect(); }
		NetworkClient client;
		u32 refCount = 0;
	};

	bool StorageClient::Start()
	{
		m_client.RegisterOnConnected([this]()
			{
				StackBinaryWriter<2048> writer;
				NetworkMessage msg(m_client, ServiceId, StorageMessageType_Connect, writer);
				writer.WriteString(TC("Client"));
				writer.WriteU32(StorageNetworkVersion);
				writer.WriteBool(false); // Is Proxy
				writer.WriteU16(m_proxyPort);
				writer.WriteString(m_zone);
				writer.WriteU64(m_casTotalBytes);

				if (m_proxyAddress.empty())
				{
					TraverseNetworkAddresses(m_logger, [&](const StringBufferBase& addr)
						{
							writer.WriteString(addr);
							return false;
						});
				}
				else
				{
					writer.WriteString(m_proxyAddress);
				}
				writer.WriteString(TC(""));


				StackBinaryReader<1024> reader;
				if (!msg.Send(reader))
					return;

				m_storageServerUid = reader.ReadGuid();
				m_casCompressor = reader.ReadByte();
				m_casCompressionLevel = reader.ReadByte();
				if (reader.GetLeft())
				{
					m_resendCas = reader.ReadBool();
					if (m_resendCas)
						m_checkExistsOnServer = false;
				}

			});

		m_client.RegisterOnDisconnected([this]() { m_logger.isMuted = true; });
		return true;
	}

	StorageClient::~StorageClient()
	{
		delete m_proxyClient;
		for (auto& pair : m_localStorageFiles)
			m_sharedMemory.CloseHandle(m_logger, pair.second.casEntry.memoryHandle, pair.second.fileName.c_str());
	}

	bool StorageClient::IsUsingProxy()
	{
		SCOPED_FUTEX_READ(m_proxyClientLock, proxyLock);
		return m_proxyClient != nullptr;
	}

	void StorageClient::StopProxy()
	{
		SCOPED_FUTEX(m_proxyClientLock, proxyLock);
		if (m_proxyClient)
			m_proxyClient->client.Disconnect();
	}

	bool StorageClient::PopulateCasFromDirs(const DirVector& directories, u32 workerCount, const Function<bool()>& shouldExit)
	{
		if (directories.empty())
			return true;

		u64 start = GetTime();

		WorkManagerImpl workManager(workerCount, TC("UbaWrk/PoplCas"));
		bool success = true;
		UnorderedSet<u64> seenIds;
		Futex seenIdsLock;

		for (auto& dir : directories)
			success = PopulateCasFromDirsRecursive(dir.c_str(), workManager, seenIds, seenIdsLock, shouldExit) && success;
		workManager.FlushWork();

		if (u32 fileCount = u32(m_localStorageFiles.size()))
			m_logger.Info(TC("Prepopulated %u files to cas in %s"), fileCount, TimeToText(GetTime() - start).str);

		return success;
	}

	bool StorageClient::GetCasFileName(StringBufferBase& out, const CasKey& casKey)
	{
		SCOPED_READ_LOCK(m_localStorageFilesLock, tempLock);
		auto findIt = m_localStorageFiles.find(AsCompressed(casKey, false));
		if (findIt != m_localStorageFiles.end())
		{
			const LocalFile& localFile = findIt->second;
			if (localFile.casEntry.memoryHandle.IsValid())
			{
				GetMappingString(out, localFile.casEntry.memoryHandle, 0, false);
				return true;
			}

			if (!localFile.fileName.empty())
			{
				out.Append(localFile.fileName);
				return true;
			}
		}
		tempLock.Leave();

		return StorageImpl::GetCasFileName(out, casKey);
	}

	MappedView2 StorageClient::MapView(const CasKey& casKey, const tchar* hint)
	{
		SCOPED_READ_LOCK(m_localStorageFilesLock, tempLock);
		auto findIt = m_localStorageFiles.find(AsCompressed(casKey, false));
		bool isValid = findIt != m_localStorageFiles.end();
		tempLock.Leave();
		if (!isValid)
			return StorageImpl::MapView(casKey, hint);

		LocalFile& file = findIt->second;
		if (!file.casEntry.memoryHandle.IsValid())
			return StorageImpl::MapView(casKey, hint);

		MappedView2 view;
		view.handle = file.casEntry.memoryHandle;
		view.size = file.casEntry.size;
		view.offset = 0;
		view.isCompressed = false;
		return view;
	}

	bool StorageClient::GetZone(StringBufferBase& out)
	{
		if (m_zone.empty())
			return false;
		out.Append(m_zone);
		return true;
	}

	bool StorageClient::RetrieveCasFile(RetrieveResult& out, const CasKey& casKeyTmp, StringKey keyHint, const tchar* hint, FileMappingBuffer* mappingBuffer, bool createIndependentMapping, u64 memoryMapAlignment, bool allowProxy, u32 clientId)
	{
		CasKey casKey = casKeyTmp;

		bool shouldStore = mappingBuffer == nullptr;
		UBA_ASSERT(AsCompressed(casKey, false) != CasKeyZero);

		if (!m_writeToDisk && !mappingBuffer)
		{
			mappingBuffer = &m_casDataBuffer;
			casKey = AsCompressed(casKey, true);
		}

		out.casKey = casKey;
		out.size = InvalidValue;

		// Cas file might have been created by this client and in that case we can just reuse the file we just wrote. No need to fetch from server
		// This needs to be first so it doesnt end up in the cas table even though it is not in there. (otherwise it might be garbage collected)
		SCOPED_READ_LOCK(m_localStorageFilesLock, tempLock);
		auto findIt = m_localStorageFiles.find(AsCompressed(casKey, false));
		if (findIt != m_localStorageFiles.end())
		{
			LocalFile& lf = findIt->second;
			if (lf.casEntry.exists)
			{
				out.casKey = findIt->first;
				if (lf.casEntry.memoryHandle.IsValid())
				{
					out.size = lf.casEntry.size;
					out.view.handle = lf.casEntry.memoryHandle;
					out.view.size = lf.casEntry.size;
					out.view.isCompressed = false;
				}
				return true;
			}
		}
		tempLock.Leave();

		StorageStats& stats = Stats();
		CasEntry* casEntry = nullptr;
		auto casEntryLock = MakeGuard([&]() { if (casEntry) casEntry->internalLock.Leave(); });
		if (shouldStore)
		{
			TimerScope ts(stats.ensureCas);

			if (EnsureCasFile(casKey, nullptr))
				return true;

			SCOPED_READ_LOCK(m_casLookupLock, lock);
			casEntry = &m_casLookup.find(casKey)->second;
			lock.Leave();

			casEntry->internalLock.Enter();
			if (casEntry->verified && casEntry->exists)
				return true;

			if (casEntry->disallowed)
				return false;

			casEntry->dropped = false;  // In case this comes from a retry where previous cas was dropped
			casEntry->verified = true;
		}

		TimerScope ts2(stats.recvCas);

		StringBuffer<> casFile;
		GetCasFileName(casFile, casKey);

		u8* slot = m_bufferSlots.Pop();
		auto slotGuard = MakeGuard([&](){ m_bufferSlots.Push(slot); });

		MappedView mappedView;
		auto mvg = MakeGuard([&]() { if (mappingBuffer) mappingBuffer->UnmapView(mappedView, hint); });
		u8* writeMem = nullptr;

		u64 fileSize = 0;
		u64 actualSize = 0;
		u64 sizeOnDisk = 0;

		#if UBA_REPORT_PROXYFETCH
		bool proxyFetchSent = false;
		#endif

		while (true)
		{
			u8* readBuffer = nullptr;
			u8* readPosition = nullptr;

			u16 fetchId = 0;
			u32 responseSize = 0;
			bool isCompressed = false;
			bool sendEnd = false;
			u64 left = ~u64(0);

			u32 sizeOfFirstMessage = 0;


			NetworkClient* client = &m_client;
			ProxyClient* proxy = nullptr;

			bool wantsProxy = false;
			if (allowProxy && m_allowProxy)
			{
				SCOPED_FUTEX(m_proxyClientLock, proxyLock);
				while (true)
				{
					if (!m_proxyClient)
						break;

					if (m_proxyClient->client.IsConnected())
					{
						m_proxyClientKeepAliveTime = GetTime();
						++m_proxyClient->refCount;
						proxy = m_proxyClient;
						client = &proxy->client;
						break;
					}

					if (m_proxyClient && !m_proxyClient->refCount)
					{
						delete m_proxyClient;
						m_proxyClient = nullptr;
						break;
					}

					proxyLock.Leave();
					Sleep(200);
					proxyLock.Enter();
				}
				wantsProxy = proxy == nullptr && m_startProxyCallback;
			}

			auto pg = MakeGuard([&]()
				{
					if (proxy)
					{
						SCOPED_FUTEX(m_proxyClientLock, proxyLock);
						--proxy->refCount;
					}
				});

			{
				#if UBA_REPORT_PROXYFETCH
				if (proxy && !proxyFetchSent)
				{
					proxyFetchSent = true;
					StackBinaryWriter<1024> writer;
					NetworkMessage msg(m_client, ServiceId, StorageMessageType_ProxyFetchBegin, writer);
					writer.WriteCasKey(AsProxyCasKey(casKey));
					writer.WriteString(hint);
					StackBinaryReader<32> reader;
					msg.Send(reader); // Need to use reader just to make sure this reaches server before the ProxyFetchEnd
				}
				#endif

				StackBinaryWriter<1024> writer;
				NetworkMessage msg(*client, ServiceId, StorageMessageType_FetchBegin, writer);
				writer.WriteByte(wantsProxy ? 1 : 0);
				writer.WriteCasKey(casKey);
				if constexpr (StorageNetworkVersion >= 5)
					writer.WriteStringKey(keyHint);
				writer.WriteString(hint);
				BinaryReader reader(slot, 0, SendMaxSize);
				if (!msg.Send(reader))
				{
					if (proxy)
						continue;
					return m_logger.Error(TC("Failed to send fetch begin message for cas %s (%s). Error: %u"), casFile.data, hint, msg.GetError());
				}
				sizeOfFirstMessage = u32(reader.GetLeft());
				fetchId = reader.ReadU16();
				if (fetchId == 0)
					return m_logger.Error(TC("Failed to fetch cas %s (%s)"), casFile.data, hint);
				if (fetchId == FetchCasIdDisallowed)
				{
					m_logger.Error(TC("Disallowed cas %s (%s)"), casFile.data, hint); // Log first since Disconnect mute log
					if (casEntry)
						casEntry->disallowed = true;
					casEntryLock.Execute();
					if (proxy)
						proxy->client.Disconnect();
					m_client.Disconnect(false); // Can't flush since we might be in a job
					return false;
				}

				fileSize = reader.Read7BitEncoded();

				u8 flags = reader.ReadByte();

				if ((flags >> 2) & 1)
				{
					StringBuffer<> proxyHost;
					u16 proxyPort;
					bool isInProcessClient = false;

					if (reader.ReadBool()) // This will be true for only one message.. no need to guard it
					{
						proxyPort = reader.ReadU16();
						if (!m_startProxyCallback(m_startProxyUserData, proxyPort, m_storageServerUid))
						{
							// TODO: Tell server we failed
							m_logger.Warning(TC("Failed to create proxy server. This should never happen!"));
							continue;
						}
						proxyHost.Append(TCV("inprocess"));
						isInProcessClient = true;
					}
					else
					{
						reader.ReadString(proxyHost);
						proxyPort = reader.ReadU16();
					}

					u32 proxyClientId = reader.ReadU32();

					SCOPED_FUTEX(m_proxyClientLock, proxyLock2);
					if (m_proxyClient)
						continue;

					u64 startTime = GetTime();
					auto timeGuard = MakeGuard([&]
						{
							u64 deltaTime = GetTime() - startTime;
							if (deltaTime > MsToTime(10*1000))
								m_logger.Info(TC("Took %s to change proxy"), TimeToText(deltaTime).str);
						});


					auto createProxyClient = [&]()
						{
							NetworkClientCreateInfo ncci(m_logger.m_writer);
							ncci.workerCount = 0;
							bool ctorSuccess = true;
							proxy = new ProxyClient(ctorSuccess, ncci);
							m_proxyClient = proxy;
							allowProxy = true;

							auto disallowProxy = MakeGuard([&]() { m_proxyClient->client.Disconnect(); proxy = nullptr; allowProxy = false; });

							if (!ctorSuccess)
								return false;

							NetworkBackend& proxyBackend = m_getProxyBackendCallback(m_getProxyBackendUserData, proxyHost.data);

							u64 startTime = GetTime();
							if (!proxy->client.Connect(proxyBackend, proxyHost.data, proxyPort))
							{
								m_logger.Detail(TC("Connecting to proxy %s:%u for cas %s download failed! (%s) (%s)"), proxyHost.data, proxyPort, casFile.data, hint, TimeToText(GetTime() - startTime).str);
								return false;
							}
					
							u64 connectTime = GetTime() - startTime;
							if (connectTime > MsToTime(2000))
								m_logger.Info(TC("Took %s to connect to proxy %s:%u"), TimeToText(connectTime).str, proxyHost.data, proxyPort);

							// Send a message to the proxy just to validate that this is still part of this build
							{
								NetworkMessage proxyMsg(proxy->client, ServiceId, StorageMessageType_Connect, writer.Reset());

								writer.WriteString(TC("ProxyClient"));
								writer.WriteU32(StorageNetworkVersion);
								writer.WriteBool(isInProcessClient);
								StackBinaryReader<256> proxyReader;
								if (!proxyMsg.Send(proxyReader))
								{
									m_logger.Info(TC("Failed to send connect message to proxy %s:%u. Will ask storage server for new proxy"), proxyHost.data, proxyPort);
									return false;
								}
								if (proxyReader.ReadGuid() != m_storageServerUid)
								{
									m_logger.Info(TC("Proxy %s:%u is not the correct proxy anymore. Will ask storage server for new proxy"), proxyHost.data, proxyPort);
									return false;
								}
							}

							for (u32 i=1;i<m_proxyConnectionCount;++i)
								proxy->client.Connect(proxyBackend, proxyHost.data, proxyPort);

							disallowProxy.Cancel();
							++proxy->refCount;
							proxy->client.SetWorkTracker(m_client.GetWorkTracker());
							return true;
						};

					if (createProxyClient())
						continue;

					UBA_ASSERT(!isInProcessClient); // Should never fail to connect to itself
					
					
					// If failing to connect to proxy we report it as bad and hopefully get a new one from the server
					m_logger.Detail(TC("Reporting bad proxy %s:%u"), proxyHost.data, proxyPort);
					NetworkMessage reportMsg(m_client, ServiceId, StorageMessageType_ReportBadProxy, writer.Reset());
					writer.WriteU32(proxyClientId);

					StackBinaryReader<256> badProxyReader;
					if (!reportMsg.Send(badProxyReader))
						continue;

					if (!badProxyReader.GetLeft())
						continue; 

					proxyHost.Clear();

					if (badProxyReader.ReadBool()) // This will be true for only one message.. no need to guard it
					{
						proxyPort = badProxyReader.ReadU16();
						if (!m_startProxyCallback(m_startProxyUserData, proxyPort, m_storageServerUid))
						{
							// TODO: Tell server we failed
							m_logger.Warning(TC("Failed to create proxy server. This should never happen!"));
							continue;
						}
						proxyHost.Append(TCV("inprocess"));
						isInProcessClient = true;
					}
					else
					{
						badProxyReader.ReadString(proxyHost);
						proxyPort = badProxyReader.ReadU16();
					}
					
					// Destroy the 'old' new proxy client since we are creating another one in createProxyClient
					UBA_ASSERT(m_proxyClient->refCount == 0);
					delete m_proxyClient;
					createProxyClient();

					continue;
				}


				isCompressed = (flags >> 0) & 1;
				sendEnd = (flags >> 1) & 1;

				left = fileSize;

				responseSize = u32(reader.GetLeft());
				readBuffer = (u8*)reader.GetPositionData();
				readPosition = readBuffer;

				actualSize = fileSize;
				if (isCompressed)
					actualSize = *(u64*)readBuffer;
			}

			sizeOnDisk = IsCompressed(casKey) ? fileSize : actualSize;

			FileAccessor destinationFile(m_logger, casFile.data);

			u8* writePos = nullptr;
			bool isInitialized = false;

			// This is put in a function that is called in between sending and waiting for segment messages.
			// It is called in a few more places in case there are no segments
			auto initForWrite = [&]()
				{
					if (isInitialized)
						return true;
					isInitialized = true;

					if (mappingBuffer)
					{
						UBA_ASSERT(!writeMem || mappedView.size == sizeOnDisk);
						if (!writeMem)
						{
							mappedView = mappingBuffer->AllocAndMapView(sizeOnDisk, memoryMapAlignment, hint, createIndependentMapping);
							writeMem = mappedView.memory;
							if (!writeMem)
								return false;
						}
					}
					else
					{
						u32 extraFlags = DefaultAttributes();
						bool useOverlap = !IsRunningWine() && isCompressed == IsCompressed(casKey) && sizeOnDisk > 1024 * 1024 && EventIsNative;
						if (useOverlap)
							extraFlags |= FILE_FLAG_OVERLAPPED;
						if (!destinationFile.CreateWrite(false, extraFlags, sizeOnDisk, m_tempPath.data))
							return false;
					}
					writePos = writeMem;
					return true;
				};

			// This is here just to prevent server from getting a million messages at the same time.
			// In theory we could have 10 clients with 48 processes each where each one of the processes asks for a large file (64 messages in flight)
			// So worst case in that scenario would be 10*48*64 = 30000 messages.
			bool oneAtTheTime = false;//left > client->GetMessageMaxSize() * 2;
			if (oneAtTheTime)
				m_retrieveOneBatchAtTheTimeLock.Enter();
			auto oatg = MakeGuard([&]() { if (oneAtTheTime) m_retrieveOneBatchAtTheTimeLock.Leave(); });


			if (isCompressed == IsCompressed(casKey))
			{
				bool tryAgain = false;
				bool sendSegmentMessage = responseSize == 0;
				u32 readIndex = 0;
				while (left)
				{
					if (sendSegmentMessage)
					{
						if (fetchId == FetchCasIdDone)
							return m_logger.Error(TC("Cas content error. Server believes %s was only one segment but client sees more. Size: %llu Left to read: %llu ResponseSize: %u. (%s)"), hint, fileSize, left, responseSize, casFile.data);
						readBuffer = slot;
						u32 error;
						if (!SendBatchMessages(m_logger, *client, fetchId, readBuffer, BufferSlotSize, left, sizeOfFirstMessage, readIndex, responseSize, initForWrite, hint, &error))
						{
							if (proxy)
							{
								tryAgain = true;
								break;
							}
							return m_logger.Error(TC("Failed to send batched messages to server while retrieving cas %s (%s). Error: %u"), casFile.data, hint, error);
						}
					}
					else
					{
						sendSegmentMessage = true;
					}

					if (!initForWrite()) // In case no segments are fetched
						return false;

					if (!mappingBuffer)
					{
						if (!destinationFile.Write(readBuffer, responseSize, writePos - writeMem))
							return false;
						writePos += responseSize;
					}
					else
					{
						FileMapping_CopyMem(writePos, readBuffer, responseSize);
						writePos += responseSize;
					}

					UBA_ASSERT(left >= responseSize);
					left -= responseSize;
				}
				if (tryAgain)
					continue;
			}
			else
			{
				if (!isCompressed)
					return m_logger.Error(TC("Code path not implemented. Receiving non compressed cas %s and want to store it compressed (%s)"), casFile.data, hint);

				bool sendSegmentMessage = responseSize == 0;
				u64 leftUncompressed = actualSize;
				readBuffer += sizeof(u64); // Size is stored first
				u64 maxReadSize = BufferSlotHalfSize - sizeof(u64);

				if (actualSize)
				{
					u64 leftCompressed = fileSize - responseSize;
					u32 readIndex = 0;
					bool tryAgain = false;
					do
					{
						// There is a risk the compressed file we download is larger than half buffer
						u8* extraBuffer = nullptr;
						auto ebg = MakeGuard([&]() { delete[] extraBuffer; });

						// First read in a full decompressable block
						bool isFirstInBlock = true;
						u32 compressedSize = ~u32(0);
						u32 uncompressedSize = ~u32(0);
						left = 0;
						u32 overflow = 0;
						do
						{
							if (sendSegmentMessage)
							{
								if (fetchId == FetchCasIdDone)
									return m_logger.Error(TC("Cas content error (2). Server believes %s was only one segment but client sees more. UncompressedSize: %llu LeftUncompressed: %llu Size: %llu Left to read: %llu ResponseSize: %u. (%s)"), hint, actualSize, leftUncompressed, fileSize, left, responseSize, casFile.data);

								u64 capacity = maxReadSize - u64(readPosition - readBuffer);
								u64 writeCapacity = capacity;
								u8* writeDest = readPosition;
								if (capacity < sizeOfFirstMessage) // capacity left is less than the last message
								{
									UBA_ASSERT(!extraBuffer);
									extraBuffer = new u8[sizeOfFirstMessage];
									writeDest = extraBuffer;
									writeCapacity = sizeOfFirstMessage;
								}

								u32 error;
								if (!SendBatchMessages(m_logger, *client, fetchId, writeDest, writeCapacity, leftCompressed, sizeOfFirstMessage, readIndex, responseSize, initForWrite, hint, &error))
								{
									if (proxy)
									{
										tryAgain = true;
										break;
									}
									return m_logger.Error(TC("Failed to send batched messages to server while retrieving and decompressing cas %s. (%s) Error: %u"), casFile.data, hint, error);
								}

								if (extraBuffer)
								{
									memcpy(readPosition, extraBuffer, left);
									memmove(extraBuffer, extraBuffer + left, u32(responseSize - left));
									if (isFirstInBlock)
										return m_logger.Error(TC("Make static analysis happy. This should not be possible to happen (%s)"), casFile.data);
								}

								leftCompressed -= responseSize;
							}
							else
							{
								sendSegmentMessage = true;
							}

							if (isFirstInBlock)
							{
								if ((readPosition - readBuffer) + responseSize < sizeof(u32) * 2)
									return m_logger.Error(TC("Received less than minimum amount of data. Most likely corrupt cas file %s (Available: %u UncompressedSize: %llu LeftUncompressed: %llu)"), casFile.data, u32(readPosition - readBuffer), actualSize, leftUncompressed);

								isFirstInBlock = false;
								u32* blockSize = (u32*)readBuffer;
								compressedSize = blockSize[0];
								uncompressedSize = blockSize[1];
								readBuffer += sizeof(u32) * 2;
								maxReadSize = BufferSlotHalfSize - sizeof(u32) * 2;
								u32 read = (responseSize + u32(readPosition - readBuffer));
								//UBA_ASSERTF(read <= compressedSize, TC("Error in datastream fetching cas. Read size: %u CompressedSize: %u %s (%s)"), read, compressedSize, casFile.data, hint);
								if (read > compressedSize)
								{
									//UBA_ASSERT(!responseSize); // TODO: This has not really been tested
									left = 0;
									overflow = read - compressedSize;
									sendSegmentMessage = false;
								}
								else
								{
									left = compressedSize - read;
								}
								readPosition += responseSize;
							}
							else
							{
								readPosition += responseSize;
								if (responseSize > left)
								{
									overflow = responseSize - u32(left);
									UBA_ASSERTF(overflow < BufferSlotHalfSize, TC("Something went wrong. Overflow: %u responseSize: %u, left: %llu"), overflow, responseSize, left);
									if (overflow >= 8)
									{
										responseSize = 0;
										sendSegmentMessage = false;
									}
									left = 0;
								}
								else
								{
									left -= responseSize;
								}
							}
						} while (left);

						if (tryAgain)
							break;

						if (!initForWrite()) // In case no segments are fetched
							return false;

						// Then decompress
						{
							u8* decompressBuffer = slot + BufferSlotHalfSize;

							TimerScope ts(stats.decompressRecv);
							OO_SINTa decompLen = OodleLZ_Decompress(readBuffer, int(compressedSize), decompressBuffer, int(uncompressedSize));
							if (decompLen != uncompressedSize)
								return m_logger.Error(TC("Expected %u but got %i when decompressing %u bytes for file %s"), uncompressedSize, int(decompLen), compressedSize, hint);

							if (!mappingBuffer)
							{
								if (!destinationFile.Write(decompressBuffer, uncompressedSize, actualSize - leftUncompressed))
									return false;
							}
							else
							{
								FileMapping_CopyMem(writePos, decompressBuffer, uncompressedSize);
								writePos += uncompressedSize;
							}

							leftUncompressed -= uncompressedSize;
						}

						readBuffer = slot;
						maxReadSize = BufferSlotHalfSize;

						if (extraBuffer)
						{
							memcpy(readBuffer, extraBuffer, overflow);
							delete[] extraBuffer;
							extraBuffer = nullptr;
						}
						else
						{
							// Move overflow back to the beginning of the buffer and start the next block (if there is one)
							UBA_ASSERTF(readPosition - overflow >= readBuffer, TC("ReadPosition - overflow is before beginning of buffer (overflow: %u) for file %s"), overflow, hint);
							UBA_ASSERTF(readPosition <= readBuffer + BufferSlotHalfSize, TC("ReadPosition is outside readBuffer size (pos: %llu, overflow: %u) for file %s"), readPosition - readBuffer, overflow, hint);
							memmove(readBuffer, readPosition - overflow, overflow);
						}

						readPosition = readBuffer + overflow;
						if (overflow)
						{
							if (overflow < sizeof(u32) * 2) // Must always have the compressed and uncompressed size to be able to move on with logic above
								sendSegmentMessage = true;
							else
								responseSize = 0;
						}
					} while (leftUncompressed);

					if (tryAgain)
						continue;
				}
			}

			if (!initForWrite()) // In case no segments are fetched
				return false;

			if (sendEnd)
			{
				StackBinaryWriter<128> writer;
				NetworkMessage msg(*client, ServiceId, StorageMessageType_FetchEnd, writer);
				writer.WriteCasKey(casKey);
				if (!msg.Send() && !proxy)
					return false;
			}

			#if UBA_REPORT_PROXYFETCH
			if (proxyFetchSent)
			{
				StackBinaryWriter<1024> writer;
				NetworkMessage msg(m_client, ServiceId, StorageMessageType_ProxyFetchEnd, writer);
				writer.WriteCasKey(AsProxyCasKey(casKey));
				msg.Send();
			}
			#endif


			if (!mappingBuffer)
				if (!destinationFile.Close())
					return false;

			break;
		}

		if (shouldStore)
		{
			casEntry->memoryHandle = mappedView.handle;
			casEntry->memoryOffset = mappedView.offset;
			casEntry->memorySize = fileSize;

			casEntry->exists = true;
			casEntryLock.Execute();

			CasEntryWritten(*casEntry, sizeOnDisk);
		}
		else
		{
			out.view = mappedView;
			out.view.memory = nullptr;
			out.view.isCompressed = IsCompressed(casKey);
		}

		stats.recvCasBytesRaw += actualSize;
		stats.recvCasBytesComp += fileSize;

		out.size = actualSize;

		return true;
	}

	bool StorageClient::StoreCasFile(CasKey& out, const tchar* fileName, const CasKey& casKeyOverride, bool deferCreation)
	{
		UBA_ASSERTF(false, TC("This StoreCasFile function should not be used on the client side"));
		return true;
	}

	bool StorageClient::HasCasFile(const CasKey& casKey, CasEntry** out, bool useMapping, bool allowFail)
	{
		CasKey localKey = AsCompressed(casKey, false);
		SCOPED_READ_LOCK(m_localStorageFilesLock, lock);
		auto findIt = m_localStorageFiles.find(localKey);
		if (findIt != m_localStorageFiles.end())
		{
			LocalFile& localFile = findIt->second;
			if (localFile.casEntry.exists && localFile.casEntry.verified)
			{
				if (out)
				{
					++localFile.casEntry.refCount; // For good measure
					*out = &localFile.casEntry;
				}
				return true;
			}
		}
		lock.Leave();
		return StorageImpl::HasCasFile(casKey, out, useMapping, allowFail);
	}

	bool StorageClient::StoreCasFileClient(CasKey& out, StringKey fileNameKey, const tchar* fileName, SharedMemoryHandle memoryHandle, u64 memoryOffset, u64 fileSize, const tchar* hint, bool keepMappingInMemory, bool storeCompressed)
	{
		NetworkClient& client = m_client; // Don't use proxy

		out = CasKeyZero;

		bool isPersistentMapping = false;
		u8* fileMem = nullptr;

		FileAccessor source(m_logger, fileName);
		if (!memoryHandle.IsValid())
		{
			if (!source.OpenMemoryRead())
				return false;
			fileSize = source.GetSize();
			fileMem = source.GetData();
		}
		else
		{
			UBA_ASSERT(!memoryOffset);
			fileMem = m_sharedMemory.MapView(memoryHandle, hint);
			if (!fileMem)
				return m_logger.Error(TC("%s - MapViewOfFile failed (%s)"), fileName, LastErrorToText().data);
			isPersistentMapping = true;
		}

		auto unmapGuard = MakeGuard([&]()
			{
				if (isPersistentMapping)
					m_sharedMemory.UnmapView(memoryHandle, fileMem);
			});

		CasKey casKey = CalculateCasKey(fileMem, fileSize, storeCompressed);
		if (casKey == CasKeyZero)
			return false;

		SCOPED_WRITE_LOCK(m_localStorageFilesLock, lock);
		auto insres = m_localStorageFiles.try_emplace(AsCompressed(casKey, false));
		LocalFile& localFile = insres.first->second;

		if (keepMappingInMemory && isPersistentMapping && !localFile.casEntry.memoryHandle.IsValid())
		{
			localFile.casEntry.memoryHandle = m_sharedMemory.DuplicateHandle(memoryHandle, TC(""));
			localFile.casEntry.size = fileSize;
			localFile.casEntry.verified = true;
			localFile.casEntry.exists = true;
			localFile.casEntry.refCount = 1;
		}

		if (!isPersistentMapping && !localFile.casEntry.memoryHandle.IsValid() && localFile.fileName.empty())
		{
			localFile.casEntry.size = fileSize;
			localFile.casEntry.verified = true;
			localFile.casEntry.exists = true;
			localFile.casEntry.refCount = 1;
			localFile.fileName = fileName;
		}

		// Prevent same file from being sent multiple times from same client (it could still be sent from other clients)
		if (!m_resendCas)
		{
			if (localFile.hasBeenSent.IsCreated()) // If it is created it means that the file has either been transferred or is being transferred
			{
				lock.Leave();
				if (localFile.hasBeenSent.IsSet(30*1000)) // 30 seconds timeout, then we try to send it ourselves
				{
					out = casKey;
					return true;
				}
			}
			else
				localFile.hasBeenSent.Create(EventResetType_Manual);
		}
		lock.Leave();

		bool existsOnServer = false;

		if (m_checkExistsOnServer)
		{
			StackBinaryWriter<128> writer;
			NetworkMessage msg(client, ServiceId, StorageMessageType_ExistsOnServer, writer);
			writer.WriteCasKey(casKey);
			StackBinaryReader<128> reader;
			if (!msg.Send(reader))
				return false;
			existsOnServer = reader.ReadBool();
		}

		if (!existsOnServer)
		{
			if (storeCompressed)
			{
				FileSender sender { m_logger, m_client, m_bufferSlots, Stats(), m_sendOneAtTheTimeLock, m_casCompressor, m_casCompressionLevel };
				sender.m_sendOneBigFileAtTheTime = m_sendOneBigFileAtTheTime;
				sender.m_parallelSend = m_fileParallelSend;
				if (!sender.SendFileCompressed(casKey, fileName, fileMem, fileSize, hint))
					return false;
			}
			else
			{
				auto& stats = Stats();
				TimerScope ts(stats.sendCas);
				if (!SendFile(m_logger, m_client, &m_workManager, casKey, fileMem, fileSize, hint))
					return false;
				stats.sendCasBytesRaw += fileSize;
				stats.sendCasBytesComp += fileSize;
			}
		}

		if (!m_resendCas)
			localFile.hasBeenSent.Set();

		out = casKey;
		return true;
	}

	void StorageClient::Ping()
	{
		LOG_STALL_SCOPE(m_logger, 5, TC("StorageClient::Ping took more than %s"));
		SCOPED_FUTEX(m_proxyClientLock, lock);
		auto proxy = m_proxyClient;
		if (!proxy || !proxy->client.IsConnected())
			return;
		u64 now = GetTime();
		if (TimeToMs(now - m_proxyClientKeepAliveTime) < 30 * 1000)
			return;

		++proxy->refCount;
		lock.Leave();

		proxy->client.SendKeepAlive();
		u64 time = GetTime();
		m_proxyClientKeepAliveTime = now;

		u64 durationMs = TimeToMs(time - now);
		if (durationMs > 20 * 1000)
			m_logger.Info(TC("Took %llu seconds to ping proxy server"), durationMs/1000);


		lock.Enter();
		--proxy->refCount;
	}

	void StorageClient::PrintSummary(Logger& logger)
	{
		StorageImpl::PrintSummary(logger);
		if (m_proxyClient)
			m_proxyClient->client.PrintSummary(logger);
	}

	bool StorageClient::PopulateCasFromDirsRecursive(const tchar* dir, WorkManager& workManager, UnorderedSet<u64>& seenIds, Futex& seenIdsLock, const Function<bool()>& shouldExit)
	{
		if (shouldExit && shouldExit())
			return true;

		StringBuffer<> fullPath;
		fullPath.Append(dir).EnsureEndsWithSlash();
		u32 dirLen = fullPath.count;
		TraverseDir(m_logger, ToView(dir), [&](const DirectoryEntry& e)
			{
				fullPath.Resize(dirLen).Append(e.name);
				if (IsDirectory(e.attributes))
				{
					SCOPED_FUTEX(seenIdsLock, lock);
					if (!seenIds.insert(e.id).second)
						return;
					lock.Leave();
					workManager.AddWork([&, filePath = fullPath.ToString()](const WorkContext& context)
						{
							PopulateCasFromDirsRecursive(filePath.c_str(), workManager, seenIds, seenIdsLock, shouldExit);
						}, 1, TC("PopulateCasFromDirsRecursive"), ColorWork);
					return;
				}

				StringBuffer<> forKey;
				FixPath(fullPath.data, nullptr, 0, forKey);
				if (CaseInsensitiveFs)
					forKey.MakeLower();
				StringKey fileNameKey = ToStringKey(forKey);
				FileEntry& fileEntry = GetOrCreateFileEntry(fileNameKey);
				fileEntry.lock.Enter();
				if (e.size == fileEntry.size && e.lastWritten == fileEntry.lastWritten)
				{
					fileEntry.verified = true;
					fileEntry.casKey = AsCompressed(fileEntry.casKey, false); // TODO: Remove this when machines have flushed their db
					fileEntry.lock.Leave();

					SCOPED_WRITE_LOCK(m_localStorageFilesLock, lookupLock);
					auto insres = m_localStorageFiles.try_emplace(fileEntry.casKey);
					LocalFile& localFile = insres.first->second;
					if (insres.second)
					{
						localFile.casEntry.size = e.size;
						localFile.casEntry.verified = true;
						localFile.casEntry.exists = true;
						localFile.casEntry.refCount = 1;
						localFile.fileName = fullPath.data;
					}
					return;
				}

				workManager.AddWork([&, fe = &fileEntry, lw = e.lastWritten, s = e.size, filePath = fullPath.ToString()](const WorkContext& context)
					{
						auto feLockLeave = MakeGuard([fe]() { fe->lock.Leave(); });

						if (shouldExit && shouldExit())
							return;

						CasKey casKey;
						if (!CalculateCasKey(casKey, {}, filePath.c_str()))
						{
							m_logger.Error(TC("Failed to calculate cas key for %s"), filePath.c_str());
							return;
						}
						fe->size = s;
						fe->lastWritten = lw;
						fe->casKey = AsCompressed(casKey, false);
						fe->verified = true;
						feLockLeave.Execute();

						SCOPED_WRITE_LOCK(m_localStorageFilesLock, lookupLock);
						auto insres = m_localStorageFiles.try_emplace(fe->casKey);
						LocalFile& localFile = insres.first->second;
						if (insres.second)
						{
							localFile.casEntry.size = s;
							localFile.casEntry.verified = true;
							localFile.casEntry.exists = true;
							localFile.casEntry.refCount = 1;
							localFile.fileName = filePath;
						}

					}, 1, TC("PrepopulateCasFromFile"), ColorWork);
			});
		return true;
	}


}
