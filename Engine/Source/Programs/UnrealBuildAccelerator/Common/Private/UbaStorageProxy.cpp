// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaStorageProxy.h"
#include "UbaConfig.h"
#include "UbaFileAccessor.h"
#include "UbaNetworkClient.h"
#include "UbaNetworkMessage.h"
#include "UbaNetworkServer.h"
#include "UbaStorageClient.h"

namespace uba
{
	struct StorageProxy::SegmentMessage
	{
		SegmentMessage(StorageProxy& p, FileEntry& f, u8* readBuffer, u32 fi)
		:	proxy(p)
		,	file(f)
		,	message(p.m_client, ServiceId, StorageMessageType_FetchSegment, writer)
		,	reader(readBuffer, 0, SendMaxSize)
		,	fetchIndex(fi)
		{
			writer.WriteU16(file.fetchId);
			writer.WriteU32(fetchIndex + 1);
		}

		StorageProxy& proxy;
		FileEntry& file;
		StackBinaryWriter<16> writer;
		NetworkMessage message;
		BinaryReader reader;
		struct DeferredResponse { u32 clientId; u16 fetchId; MessageInfo info; };
		List<DeferredResponse> deferredResponses;
		u32 fetchIndex;
		bool done = false;
		bool error = false;
	};

	void StorageProxyCreateInfo::Apply(Config& config, const tchar* tableName)
	{
		const ConfigTable* tablePtr = config.GetTable(tableName);
		if (!tablePtr)
			return;
		const ConfigTable& table = *tablePtr;
		table.GetValueAsBool(useLocalStorage, TC("UseLocalStorage"));
	}

	StorageProxy::StorageProxy(const StorageProxyCreateInfo& info)
	:	m_server(info.server)
	,	m_client(info.client)
	,	m_localStorage(info.localStorage)
	,	m_logger(info.client.GetLogWriter(), TC("StorageProxy"))
	,	m_storageServerUid(info.storageServerUid)
	,	m_name(info.name)
	{
		m_useLocalStorage = info.useLocalStorage;

		m_server.RegisterOnClientDisconnected(0, [this](const Guid& clientUid, u32 clientId)
			{
				SCOPED_WRITE_LOCK(m_activeFetchesLock, lock);
				for (auto it=m_activeFetches.begin(); it!=m_activeFetches.end();)
				{
					if (it->second.clientId != clientId)
					{
						++it;
						continue;
					}
					PushId(it->first);
					it = m_activeFetches.erase(it);
				}
			});

		m_server.RegisterService(StorageServiceId,
			[this](const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer)
			{
				return HandleMessage(connectionInfo, workContext, messageInfo, reader, writer);
			},
			[](u8 messageType)
			{
				return ToString(StorageMessageType(messageType));
			}
		);

		m_client.RegisterOnDisconnected([this]() { m_logger.isMuted = true; });
	}

	StorageProxy::~StorageProxy()
	{
		m_server.UnregisterService(StorageServiceId);
		for (auto& kv : m_files)
			delete[] kv.second.memory;
	}

	void StorageProxy::PrintSummary()
	{
		LoggerWithWriter logger(m_logger.m_writer);
		logger.Info(TC("  -- Uba storage proxy stats summary --"));
		logger.Info(TC("  Total fetched           %6s"), BytesToText(0).str);
		logger.Info(TC("  Total provided          %6s"), BytesToText(0).str);
		logger.Info(TC(""));
	}

	u32 StorageProxy::GetActiveFetchCount()
	{
		SCOPED_READ_LOCK(m_activeFetchesLock, lock);
		return u32(m_activeFetches.size());
	}

	u16 StorageProxy::PopId()
	{
		if (m_availableIds.empty())
		{
			if (m_availableIdsHigh == 65534)
			{
				m_logger.Error(TC("OUT OF AVAILABLE IDs.. SHOULD NEVER HAPPEN!"));
				UBA_ASSERT(false);
			}
			return m_availableIdsHigh++;
		}
		u16 storeId = m_availableIds.back();
		m_availableIds.pop_back();
		return storeId;
	}

	void StorageProxy::PushId(u16 id)
	{
		m_availableIds.push_back(id);
	}

	bool StorageProxy::HandleMessage(const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer)
	{
		switch (messageInfo.type)
		{
		case StorageMessageType_Connect:
			return HandleConnect(connectionInfo, reader, writer);
		case StorageMessageType_FetchBegin:
			return HandleFetchBegin(connectionInfo, workContext, messageInfo, reader, writer);
		case StorageMessageType_FetchSegment:
			return HandleFetchSegment(connectionInfo, workContext, messageInfo, reader, writer);
		case StorageMessageType_FetchEnd:
			return true;
		default:
			return HandleDefault(messageInfo, reader, writer);
		}
	}

	bool StorageProxy::HandleConnect(const ConnectionInfo& connectionInfo, BinaryReader& reader, BinaryWriter& writer)
	{
		StringBuffer<> clientName;
		reader.ReadString(clientName);
		u32 clientVersion = reader.ReadU32();
		if (clientVersion != StorageNetworkVersion)
		{
			m_logger.Error(TC("Unsupported network version. Client: %u, Server: %u. Disconnecting"), clientVersion, StorageNetworkVersion);
			return false;
		}
		connectionInfo.SetStorageVersion(clientVersion);

		bool isInProcessClient = reader.ReadBool();
		if (isInProcessClient)
			m_inProcessClientId = connectionInfo.GetId();

		//m_logger.Info(TC("%s connected"), clientName.data);
		writer.WriteGuid(m_storageServerUid);
		return true;
	}

	struct StorageProxy::BeginMessage
	{
		BeginMessage(StorageProxy& p, FileEntry& f, const tchar* h) : proxy(p), file(f), hint(h), message(p.m_client, ServiceId, StorageMessageType_FetchBegin, writer) {}
		StorageProxy& proxy;
		FileEntry& file;
		TString hint;
		struct DeferredResponse { u32 clientId; MessageInfo info; };
		List<DeferredResponse> deferredResponses;
		StackBinaryWriter<1024> writer;
		StackBinaryReader<SendMaxSize> reader;
		NetworkMessage message;
	};

	bool StorageProxy::HandleFetchBegin(const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer)
	{
		u8 recvFlags = reader.ReadByte(); // Wants proxy
		UBA_ASSERT((recvFlags & 2) == 0);(void)recvFlags;

		CasKey casKey = reader.ReadCasKey();
		StringKey keyHint;
		if (connectionInfo.GetStorageVersion() >= 5)
			keyHint = reader.ReadStringKey();
		StringBuffer<> hint;
		reader.ReadString(hint);

		workContext.tracker.AddHint(StringView(hint).GetFileName());

		SCOPED_FUTEX(m_filesLock, filesLock);
		FileEntry& file = m_files.try_emplace(casKey).first->second;
		filesLock.Leave();

		u32 clientId = connectionInfo.GetId();

		SCOPED_FUTEX(file.lock, fileLock);
		if (file.memory || file.error)
		{
			fileLock.Leave();
			return SendFetchBeginResponse(clientId, file, writer);
		}

		UBA_ASSERT(!file.memory);
		if (!GetFileFromLocalStorage(clientId, casKey, file, fileLock))
			return false;

		if (file.memory || file.error)
		{
			fileLock.Leave();
			return SendFetchBeginResponse(clientId, file, writer);
		}

		if (file.beginMessage == nullptr)
		{
			file.casKey = casKey;
			file.beginMessage = new BeginMessage(*this, file, hint.data);

			file.trackId = m_client.TrackWorkStart(AsView(TC("ProxyFetch")), ColorWork);
			m_client.TrackWorkHint(file.trackId, StringView(hint).GetFileName());

			auto& writer2 = file.beginMessage->writer;
			writer2.WriteByte(2); // Does not want proxy but informs it is proxy
			writer2.WriteCasKey(casKey);
			if constexpr (StorageNetworkVersion >= 5)
				writer2.WriteStringKey(keyHint);
			writer2.WriteString(hint);
			writer2.WriteBytes(reader.GetPositionData(), reader.GetLeft());

			SCOPED_READ_LOCK(m_largeFileLock, largeFileLock);

			bool res = file.beginMessage->message.SendAsync(file.beginMessage->reader, [](bool error, void* userData)
				{
					auto m = (BeginMessage*)userData;
					m->proxy.m_server.AddWork([m, error](const WorkContext&) { m->proxy.HandleFetchBeginReceived(*m, error); }, 1, TC("ProxyFetchBegin"), ColorWork);
				}, file.beginMessage);

			if (!res)
				return false;
		}

		auto& deferredResponse = file.beginMessage->deferredResponses.emplace_back();
		deferredResponse.clientId = clientId;
		deferredResponse.info = messageInfo;
		messageInfo = {};
		return true;
	}

	void StorageProxy::HandleFetchBeginReceived(BeginMessage& m, bool error)
	{
		auto& file = m.file;
		u8* memory = nullptr;
		StringView hint(m.hint);

		auto sendResponses = MakeGuard([&]()
			{
				SCOPED_FUTEX(file.lock, fileLock);
				file.memory = memory;
				auto f = file.beginMessage;
				file.beginMessage = nullptr;
				UBA_ASSERTF(f, TC("No begin message connected to %s. Should not happen"), hint.data);
				if (!f)
					return;
				file.error = error;
				fileLock.Leave();

				StackBinaryWriter<SendMaxSize> writer;
				bool isFirst = true;
				for (auto& deferredResponse : f->deferredResponses)
				{
					writer.Reset();
					SendFetchBeginResponse(deferredResponse.clientId, file, writer, isFirst);
					m_server.SendResponse(deferredResponse.info, writer.GetData(), u32(writer.GetPosition()));
					isFirst = false;
				}
				delete f;
			});


		if (error || !m.message.ProcessAsyncResults(m.reader))
			return;

		u16 fetchId = m.reader.ReadU16();
		if (fetchId == 0)
		{
			error = true;
			m_logger.Error(TC("FetchBegin failed for cas file %s (%s)."), CasKeyString(file.casKey).str, hint.data);
			return;
		}
		if (fetchId == FetchCasIdDisallowed)
		{
			file.disallowed = true;
			error = true;
			m_logger.Error(TC("Disallowed download of cas file %s (%s)."), CasKeyString(file.casKey).str, hint.data);
			return;
		}

		u64 fileSize = m.reader.Read7BitEncoded();
		file.size = fileSize;

		u8 flags = m.reader.ReadByte();
		bool storeCompressed = (flags >> 0) & 1;
		bool sendEnd = (flags >> 1) & 1;
		u64 fetchedSize = m.reader.GetLeft();

		memory = new u8[fileSize];
		memcpy(memory, m.reader.GetPositionData(), fetchedSize);

		file.received = fetchedSize;
		file.fetchId = fetchId;
		file.sendEnd = sendEnd;
		file.storeCompressed = storeCompressed;

		if (sendEnd && fetchedSize == fileSize)
			SendEnd(file.casKey);

		if (file.received == file.size)
		{
			m_client.TrackWorkEnd(file.trackId);
			return;
		}

		u64 left = file.size - file.received;
		u64 segmentSize = m_client.GetMessageMaxSize() - 5; // This is server response size - header.. TODO: Should be taken from server
		u32 segmentCount = u32((left + segmentSize - 1) / segmentSize);
		file.segmentMessages.resize(segmentCount);
		for (u32 i=0; i!=segmentCount; ++i)
		{
			u64 offset = file.received + segmentSize * i;
			file.segmentMessages[i] = new SegmentMessage(*this, file, memory + offset, i);
		}

		sendResponses.Execute();

		// Move the additional messages to a job to be able to return this one quickly.
		m_server.AddWork([f = &file, segmentCount, this](const WorkContext&)
			{
				SCOPED_WRITE_LOCK(m_largeFileLock, lock);
				//TrackWorkScope tws(m_client, TC("SEGMENTS"));
				auto& file = *f;
				for (u32 i=0; i!=segmentCount; ++i)
				{
					auto mif = file.segmentMessages[i];

					bool res = mif->message.SendAsync(mif->reader, [](bool error, void* userData)
						{
							auto mif = (SegmentMessage*)userData;
							mif->error = error;
							mif->proxy.m_server.AddWork([mif](const WorkContext&) { mif->proxy.HandleFetchSegmentReceived(*mif); }, 1, TC("ProxyWaitMsg"), ColorWork);
						}, mif);
					if (!res)
					{
						// TODO: Don't leak mif
						mif->error = true;
					}
				}
			}, 1, TC("ProxySpawnMsg"), ColorWork);
	}


	bool StorageProxy::HandleFetchSegment(const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer)
	{
		u16 fetchId = reader.ReadU16();
		u32 fetchIndex = reader.ReadU32() - 1;

		SCOPED_READ_LOCK(m_activeFetchesLock, activeLock);
		auto findIt = m_activeFetches.find(fetchId);
		UBA_ASSERT(findIt != m_activeFetches.end());
		ActiveFetch& fetch = findIt->second;
		u32 clientId = fetch.clientId;
		activeLock.Leave();

		FileEntry& file = *fetch.file;
		SCOPED_FUTEX(file.lock, fileLock);
		if (file.error)
			return false;

		if (!file.available)
		{
			if (auto mif = file.segmentMessages[fetchIndex])
			{
				UBA_ASSERT(clientId == connectionInfo.GetId());
				mif->deferredResponses.push_back({clientId, fetchId, messageInfo});
				messageInfo = {};
				return true;
			}
		}
		fileLock.Leave();

		u64 headerSize = sizeof(u16) + Get7BitEncodedCount(file.size) + sizeof(u8);
		u64 firstFetchSize = m_client.GetMessageMaxSize() - m_client.GetMessageReceiveHeaderSize() - headerSize;
		u64 segmentSize = m_client.GetMessageMaxSize() - 5; // This is server response size - header.. TODO: Should be taken from server
		u64 offset = firstFetchSize + segmentSize * (fetchIndex);
		if (offset + segmentSize > file.size)
			segmentSize = file.size - offset;
		writer.WriteBytes(file.memory + offset, segmentSize);
		return UpdateFetch(fetch.clientId, fetchId, segmentSize);
	}

	void StorageProxy::HandleFetchSegmentReceived(SegmentMessage& mif)
	{
		auto& file = mif.file;
		if (mif.error)
		{
			SCOPED_FUTEX(file.lock, fileLock);
			file.error = true;
		}
		else
		{
			mif.message.ProcessAsyncResults(mif.reader);
		}

		SCOPED_FUTEX(file.lock, fileLock);

		UBA_ASSERT(file.segmentMessages[mif.fetchIndex] == &mif);
		file.segmentMessages[mif.fetchIndex] = nullptr;
		file.received += mif.reader.GetLeft();
		bool finished = file.received == file.size;
		if (finished)
			file.available = true;
		fileLock.Leave();

		if (finished)
		{
			m_client.TrackWorkEnd(file.trackId);
			SendEnd(file.casKey);
		}


		for (auto& r : mif.deferredResponses)
		{
			if (UpdateFetch(r.clientId, r.fetchId, mif.reader.GetLeft()) && !mif.error)
				m_server.SendResponse(r.info, mif.reader.GetPositionData(), u32(mif.reader.GetLeft()));
			else
				m_server.SendResponse(r.info, nullptr, 0);
		}

		delete &mif;
	}

	bool StorageProxy::HandleDefault(MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer)
	{
		StackBinaryWriter<1024> writer2;
		NetworkMessage msg(m_client, ServiceId, messageInfo.type, writer2);
		writer2.WriteBytes(reader.GetPositionData(), reader.GetLeft());

		StackBinaryReader<SendMaxSize> reader2;
		if (!msg.Send(reader2))
			return false;
		writer.WriteBytes(reader2.GetPositionData(), reader2.GetLeft());
		return true;
	}

	bool StorageProxy::GetFileFromLocalStorage(u32 clientId, const CasKey& casKey, FileEntry& file, ScopedFutex& fileLock)
	{
		if (!m_useLocalStorage || !m_localStorage || !IsCompressed(casKey) || !m_inProcessClientId || clientId == m_inProcessClientId)
			return true;

		// We need to leave this lock here since the in-process storage client might be asking for this file too and then we can end up in a deadlock
		fileLock.Leave();

		bool hasCas = m_localStorage->EnsureCasFile(casKey, nullptr);
		StringBuffer<> casFile;
		hasCas = hasCas && m_localStorage->GetCasFileName(casFile, casKey);

		// Enter lock again, and also check if another thread might have already handled this file while we looked if it existed in local storage
		fileLock.Enter();

		if (file.memory || !hasCas)
			return true;

		FileAccessor sourceFile(m_logger, casFile.data);
		if (!sourceFile.OpenMemoryRead())
			return true;

		u64 fileSize = sourceFile.GetSize();
		file.memory = new u8[fileSize];
		if (!file.memory)
			return false;
		file.error = false;
		file.size = fileSize;
		file.received = fileSize;
		file.storeCompressed = true;
		memcpy(file.memory, sourceFile.GetData(), fileSize);
		file.available = true;
		return true;
	}

	bool StorageProxy::SendFetchBeginResponse(u32 clientId, FileEntry& file, BinaryWriter& writer, bool writeData)
	{
		if (file.error)
		{
			if (file.disallowed)
			{
				writer.WriteU16(FetchCasIdDisallowed);
				return true;
			}
			else
			{
				writer.WriteU16(0);
				return false;
			}
		}
		u16 fetchId = FetchCasIdDone;

		u64 headerSize = sizeof(u16) + Get7BitEncodedCount(file.size) + sizeof(u8);
		u64 fetchedSize = Min(file.size, m_client.GetMessageMaxSize() - m_client.GetMessageReceiveHeaderSize() - headerSize);

		if (fetchedSize < file.size)
		{
			SCOPED_WRITE_LOCK(m_activeFetchesLock, lock);
			fetchId = PopId();
			auto res = m_activeFetches.try_emplace(fetchId);
			UBA_ASSERT(res.second);
			ActiveFetch& fetch = res.first->second;
			fetch.clientId = clientId;
			lock.Leave();

			fetch.fetchedSize = fetchedSize;
			fetch.file = &file;
		}

		u8 flags = 0;
		flags |= u8(file.storeCompressed) << 0;

		writer.WriteU16(fetchId);
		writer.Write7BitEncoded(file.size);
		writer.WriteByte(flags);

		if (writeData)
			writer.WriteBytes(file.memory, fetchedSize);
		else
			writer.AllocWrite(fetchedSize);
		return true;
	}

	bool StorageProxy::UpdateFetch(u32 clientId, u16 fetchId, u64 segmentSize)
	{
		SCOPED_WRITE_LOCK(m_activeFetchesLock, activeLock);
		auto findIt = m_activeFetches.find(fetchId);
		if (findIt == m_activeFetches.end())
		{
			// This can happen if we have async downloading and client is disconnected
			//m_logger.Info(TC("Failed to find active fetch with id %u"), fetchId);
			return false;
		}

		ActiveFetch& fetch = findIt->second;
		if (fetch.clientId != clientId)
		{
			// This can happen if we have async downloading and client is disconnected and new client have reused fetch id
			//m_logger.Info(TC("Active fetch %i has a different client id."), fetchId);
			return false;
		}

		fetch.fetchedSize += segmentSize;
		if (fetch.fetchedSize != fetch.file->size)
			return true;

		m_activeFetches.erase(findIt);
		PushId(fetchId);
		return true;
	}

	bool StorageProxy::SendEnd(const CasKey& key)
	{
		StackBinaryWriter<128> writer;
		NetworkMessage msg(m_client, ServiceId, StorageMessageType_FetchEnd, writer);
		writer.WriteCasKey(key);
		return msg.Send();
	}

}
