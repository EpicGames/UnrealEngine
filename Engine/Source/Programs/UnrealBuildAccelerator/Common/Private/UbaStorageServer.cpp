// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaStorageServer.h"
#include "UbaConfig.h"
#include "UbaFileAccessor.h"
#include "UbaNetworkServer.h"
#include "UbaTrace.h"

namespace uba
{
	void StorageServerCreateInfo::Apply(const Config& config)
	{
		StorageCreateInfo::Apply(config);
		const ConfigTable* tablePtr = config.GetTable(TC("Storage"));
		if (!tablePtr)
			return;
		const ConfigTable& table = *tablePtr;
		table.GetValueAsBool(writeReceivedCasFilesToDisk, TC("WriteReceivedCasFilesToDisk"));
		table.GetValueAsBool(allowHintAsFallback, TC("AllowHintAsFallback"));
		table.GetValueAsBool(createIndependentMappings, TC("CreateIndependentMappings"));
		table.GetValueAsBool(dropIndependentMappingsOnCopy, TC("DropIndependentMappingsOnCopy"));
	}

	StorageServer::StorageServer(const StorageServerCreateInfo& info)
	:	StorageImpl(info, TC("UbaStorageServer"))
	,	m_server(info.server)
	{
		m_zone = info.zone;
		m_allowHintAsFallback = info.allowHintAsFallback;
		m_writeReceivedCasFilesToDisk = info.writeReceivedCasFilesToDisk;
		m_createIndependentMappings = info.createIndependentMappings;
		m_dropIndependentMappingsOnCopy = info.dropIndependentMappingsOnCopy;

		if (!CreateGuid(m_uid))
			UBA_ASSERT(false);

		m_server.RegisterService(ServiceId,
			[this](const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer)
			{
				return HandleMessage(connectionInfo, workContext, messageInfo.type, reader, writer);
			},
			[](u8 messageType)
			{
				return ToString(StorageMessageType(messageType));
			}
		);

		m_server.RegisterOnClientConnected(ServiceId, [this](const Guid& clientUid, u32 clientId)
			{
				return LoadCasTable(true);
			});

		m_server.RegisterOnClientDisconnected(ServiceId, [this](const Guid& clientUid, u32 clientId)
			{
				OnDisconnected(clientId);
			});
	}

	StorageServer::~StorageServer()
	{
		WaitForActiveWork();
		UBA_ASSERT(m_waitEntries.empty());
		UBA_ASSERT(m_proxies.empty());
		m_server.UnregisterOnClientDisconnected(ServiceId);
		m_server.UnregisterService(ServiceId);
	}

	bool StorageServer::RegisterDisallowedPath(const tchar* path)
	{
		m_disallowedPaths.push_back(path);
		return true;
	}

	void StorageServer::WaitForActiveWork()
	{
		while (m_activeUnmap)
			m_server.DoAdditionalWork();

		// Need to make sure all wait entries are done before exiting.
		while (true)
		{
			SCOPED_FUTEX_READ(m_waitEntriesLock, lock);
			if (m_waitEntries.empty())
				break;
			lock.Leave();
			m_server.DoAdditionalWork();
		}

	}

	bool StorageServer::WaitForCasFile(const CasKey& casKey, const tchar* hint, u32 clientId)
	{
		CasKey actualKey = casKey;

		// This must be true because code handling storing files from remotes does not have decompression support
		// .. causing mix of compressed and non-compressed cas files with too many permutations
		UBA_ASSERT(m_storeCompressed);

		SCOPED_FUTEX(m_waitEntriesLock, waitLock);
		WaitEntry& waitEntry = m_waitEntries[actualKey];
		++waitEntry.refCount;
		waitLock.Leave();

		auto g = MakeGuard([&]()
			{
				SCOPED_FUTEX(m_waitEntriesLock, waitLock2);
				if (!--waitEntry.refCount)
					m_waitEntries.erase(actualKey);
			});

		if (HasCasFile(actualKey, nullptr, false, false))
			return true;

		u64 startTime = GetTime();
		u32 timeout = 0;
		while (!waitEntry.done.IsSet(timeout)) // TODO WaitMultipleObjects (additional work and Done)
		{
			u64 waited = GetTime() - startTime;
			u64 waitedMs = TimeToMs(waited);
			if (waitedMs > 500)
			{
				if (!m_server.IsConnectedSlow(clientId))
					return m_logger.Info(TC("Client with id %u disconnected while waiting for cas %s (%s)"), clientId, CasKeyString(casKey).str, hint).ToFalse();
				if (waitedMs > 4 * 60 * 1000) // 4 minutes timeout
					return m_logger.Info(TC("Timed out waiting %s for cas %s to be transferred from remote to storage (%s)"), TimeToText(waited, true).str, CasKeyString(casKey).str, hint).ToFalse();
			}

			timeout = m_server.DoAdditionalWork() ? 0 : 50;
		}
		return waitEntry.success;
	}

	bool StorageServer::GetZone(StringBufferBase& out)
	{
		if (m_zone.empty())
			return false;
		out.Append(m_zone);
		return true;
	}

	bool StorageServer::RetrieveCasFile(RetrieveResult& out, const CasKey& casKey, StringKey keyHint, const tchar* hint, FileMappingBuffer* mappingBuffer, bool createIndependentMapping, u64 memoryMapAlignment, bool allowProxy, u32 clientId)
	{
		UBA_ASSERT(!mappingBuffer);
		UBA_ASSERT(casKey != CasKeyZero);
		out.casKey = casKey;
		out.size = InvalidValue;
		return WaitForCasFile(casKey, hint, clientId);
	}

	bool StorageServer::CopyOrLink(const CasKey& casKey, const tchar* destination, u32 fileAttributes, bool writeCompressed, u32 clientId, const FormattingFunc& formattingFunc, bool isTemp, bool allowHardLink)
	{
		bool res = StorageImpl::CopyOrLink(casKey, destination, fileAttributes, writeCompressed, clientId, formattingFunc, isTemp, allowHardLink);
		if (clientId != 0)
			ReleaseActiveRefCount(casKey, clientId);
		return res;
	}

	bool StorageServer::WritePlaceholder(const CasKey& casKey, const tchar* destination, u32 fileAttributes, u32 clientId)
	{
		bool res = StorageImpl::WritePlaceholder(casKey, destination, fileAttributes, clientId);
		if (clientId != 0)
			ReleaseActiveRefCount(casKey, clientId);
		return res;
	}

	bool StorageServer::SkipWrite(const CasKey& casKey, u32 clientId)
	{
		if (clientId != 0)
			ReleaseActiveRefCount(casKey, clientId);
		return true;
	}

	void StorageServer::ActiveFetch::Release(StorageServer& server, const tchar* reason)
	{
		if (ownsMapping)
		{
			++server.m_activeUnmap;
			server.GetServer().AddWork([&server, mb = memoryBegin, mps = mappedView.size, fmp = fileMappingHandle, rfh = readFileHandle, reason](const WorkContext& context)
				{
					if (!FileMapping_Unmap(server.m_logger, mb, mps, reason))
						server.m_logger.Error(TC("%s - Failed to unmap memory at 0x%llx with size %llu (%s)"), reason, mb, mps, LastErrorToText().data);
					if (!FileMapping_Close(server.m_logger, fmp, reason))
						server.m_logger.Error(TC("%s - Failed to close file mapping %llu (%s)"), reason, fmp.ToU64(), LastErrorToText().data);
					if (!CloseFile(nullptr, rfh))
						server.m_logger.Error(TC("%s - Failed to close file (%s)"), reason, LastErrorToText().data);

					--server.m_activeUnmap;
				}, 1, TC("ActiveFetchRelease"));
		}
		else if (mappedView.handle.IsValid())
		{
			server.m_casDataBuffer.UnmapView(mappedView, reason);
		}
		else
		{
			if (!memoryBegin)
				server.m_logger.Warning(TC("This should not happen. It means there is a race between a fetch and a disconnect. Report to honk (%s)"), reason);
			server.m_bufferSlots.Push((u8*)memoryBegin);
			memoryBegin = nullptr;
		}

		server.ReleaseCasEntry(*casEntry, server.m_dropIndependentMappingsOnCopy);
	}

	void StorageServer::OnDisconnected(u32 clientId)
	{
		{
			Info info;
			{
				SCOPED_WRITE_LOCK(m_connectionInfoLock, lock);
				auto findIt = m_connectionInfo.find(clientId);
				bool found = findIt != m_connectionInfo.end();
				if (found)
				{
					info = std::move(findIt->second);
					m_connectionInfo.erase(findIt);
				}
			}

			if (!info.zone.empty())
			{
				StringBuffer<256> proxyName;
				GetProxyName(proxyName, info);
				if (!proxyName.IsEmpty())
				{
					auto proxyKey = ToStringKeyNoCheck(proxyName.data, proxyName.count);

					SCOPED_WRITE_LOCK(m_proxiesLock, proxiesLock);
					auto findIt2 = m_proxies.find(proxyKey);
					if (findIt2 != m_proxies.end())
					{
						ProxyEntry& e = findIt2->second;
						if (e.clientId == clientId || e.clientId == ~0u)
						{
							m_logger.Detail(TC("Proxy %s:%u for zone %s removed"), e.host.c_str(), e.port, e.zone.c_str());
							m_proxies.erase(findIt2->first);
						}
						else
						{
							e.failedClients.erase(clientId);
						}
					}
				}
			}
		}
		{
			struct StoreToRemove
			{
				FileAccessor* fileAccessor;
				CasEntry* casEntry;
				MappedView mappedView;
			};
			List<StoreToRemove> storesToRemove;

			SCOPED_WRITE_LOCK(m_activeStoresLock, lock);
			for (auto it=m_activeStores.begin(); it!=m_activeStores.end();)
			{
				ActiveStore& store = it->second;
				if (store.clientId != clientId)
				{
					++it;
					continue;
				}

				m_logger.Detail(TC("Cancelled store id %u because of disconnect of client with id %u"), u32(it->first), clientId);
				
				storesToRemove.push_back({store.fileAccessor, store.casEntry, store.mappedView});

				it = m_activeStores.erase(it);
			}
			lock.Leave();

			for (StoreToRemove& store : storesToRemove)
			{
				if (auto fa = store.fileAccessor)
				{
					const tchar* filename = fa->GetFileName();
					delete store.fileAccessor;
					free((void*)filename);
				}

				m_casDataBuffer.UnmapView(store.mappedView, TC("OnDisconnected"));
				if (m_casDataBuffer.IsIndependent(store.mappedView.handle))
					m_casDataBuffer.CloseIndependent(store.mappedView.handle, store.mappedView.size, TC("StoreDone"));

				CasEntryWriteLock casEntryLock(*store.casEntry);
				if (!store.casEntry->beingWritten)
					m_logger.Warning(TC("CasEntry does not have being written flag?")); // Searching for a bug

				store.casEntry->verified = false;
				store.casEntry->beingWritten = false;
				if (m_traceStore)
					m_trace->FileStoreEnd(clientId, store.casEntry->key);

				UpdateWaitEntries(store.casEntry->key, false);
			}
		}
		{
			SCOPED_WRITE_LOCK(m_activeFetchesLock, lock);
			for (auto it=m_activeFetches.begin(); it!=m_activeFetches.end();)
			{
				ActiveFetch& fetch = it->second;
				if (fetch.clientId != clientId)
				{
					++it;
					continue;
				}

				m_logger.Detail(TC("Cancelled fetch id %u because of disconnect of client with id %u"), u32(it->first), clientId);

				fetch.Release(*this, TC("OnDisconnected"));

				if (m_traceFetch)
					m_trace->FileFetchEnd(clientId, AsCompressed(fetch.casEntry->key, m_storeCompressed));

				it = m_activeFetches.erase(it);
			}
		}

		{
			SCOPED_WRITE_LOCK(m_activeRefCountsLock, refCountLock);
			auto findIt = m_activeRefCounts.find(clientId);
			if (findIt != m_activeRefCounts.end())
			{
				auto refCounts = std::move(findIt->second);
				m_activeRefCounts.erase(findIt);
				refCountLock.Leave();
				for (CasEntry* casEntry : refCounts)
					ReleaseCasEntry(*casEntry, m_dropIndependentMappingsOnCopy);
			}
		}
	}

	void StorageServer::ReleaseActiveRefCount(const CasKey& casKey, u32 clientId)
	{
		SCOPED_WRITE_LOCK(m_activeRefCountsLock, refCountLock);
		auto findIt = m_activeRefCounts.find(clientId);
		if (findIt == m_activeRefCounts.end())
			return;
		for (auto it=findIt->second.begin();it!=findIt->second.end(); ++it)
		{
			CasEntry* casEntry = *it;
			if (casEntry->key != casKey)
				continue;
			findIt->second.erase(it);
			refCountLock.Leave();
			ReleaseCasEntry(*casEntry, m_dropIndependentMappingsOnCopy);
			break;
		}
	}

	void StorageServer::UpdateWaitEntries(const CasKey& casKey, bool success)
	{
		SCOPED_FUTEX(m_waitEntriesLock, waitLock);
		auto waitFindIt = m_waitEntries.find(casKey);
		if (waitFindIt == m_waitEntries.end())
			return;
		WaitEntry& waitEntry = waitFindIt->second;
		waitEntry.success = success;
		waitEntry.done.Set();
	}

	bool StorageServer::IsDisallowedPath(const tchar* fileName)
	{
		for (auto& path : m_disallowedPaths)
			if (StartsWith(fileName, path.c_str()))
				return true;
		return false;
	}

	void StorageServer::SetTrace(Trace* trace, bool detailed)
	{
		m_trace = trace;
		m_traceFetch = detailed;
		m_traceStore = detailed;
	}

	bool StorageServer::HasProxy(u32 clientId)
	{
		SCOPED_READ_LOCK(m_proxiesLock, l);
		for (auto& kv : m_proxies)
			if (kv.second.clientId == clientId)
				return true;
		return false;
	}

	bool StorageServer::WaitForWritten(CasEntry& casEntry, CasEntryWriteLock& casEntryLock, const ConnectionInfo& connectionInfo, const tchar* hint)
	{
		int waitCount = 0;
		while (true)
		{
			if (!casEntry.beingWritten)
				return true;
			CasKey key = casEntry.key;
			casEntryLock.Leave();
			Sleep(100);
			auto g = MakeGuard([&]() { casEntryLock.Enter(); });

			if (connectionInfo.ShouldDisconnect())
				return false;

			if (++waitCount < 12*60*10)
				continue;

			// Something went wrong.. should not take 12 minutes to write a file

			SCOPED_READ_LOCK(m_activeStoresLock, activeLock);
			for (auto& kv : m_activeStores)
			{
				if (kv.second.casEntry != &casEntry)
					continue;
				ActiveStore& as = kv.second;
				return m_logger.Error(TC("Client %u waited more than 12 minutes for file %s (%s) to be written by client %u (Written %llu/%llu)"), connectionInfo.GetId(), CasKeyString(key).str, hint, as.clientId, as.totalWritten.load(), as.fileSize);
			}
			return m_logger.Error(TC("Client %u waited more than 12 minutes for file %s (%s) to be written but there are no active writes. This should not be possible!"), connectionInfo.GetId(), CasKeyString(key).str, hint);
		}
	}

	bool StorageServer::VerifyExists(bool& outExists, CasEntry& casEntry, CasEntryWriteLock& casEntryLock, const CasKey& casKey)
	{
		outExists = false;
		StringBuffer<> casFile;
		if (!GetCasFileName(casFile, casKey))
			return false;
		u64 outFileSize = 0;
		if (FileExists(m_logger, casFile.data, &outFileSize))
		{
			if (outFileSize == 0 && casKey != EmptyFileKey)
			{
				m_logger.Warning(TC("Found file %s with size 0 which did not have the zero-size-caskey. Deleting"), casFile.data);
				if (!uba::DeleteFileW(casFile.data))
					return m_logger.Error(TC("Failed to delete %s. Clean cas folder and restart"), casFile.data);
				casEntry.exists = false;
				casEntry.verified = true;
			}
			else
			{
				casEntry.verified = true;
				outExists = true;
				casEntryLock.Leave();
				CasEntryWritten(casEntry, outFileSize);
			}
		}
		else
		{
			casEntry.exists = false;
			casEntry.verified = true;
		}
		return true;
	}

	bool StorageServer::GetProxy(BinaryWriter& writer, u32 clientId, bool writeCasHeader)
	{
		SCOPED_READ_LOCK(m_connectionInfoLock, lock);
		auto findIt = m_connectionInfo.find(clientId);
		if (findIt == m_connectionInfo.end())
			return m_logger.Error(TC("Couldn't find connection info for client %u"), clientId);
		Info& info = findIt->second;
		lock.Leave();

		if (info.zone.empty())
			return false;

		// Zone logic is a bit special to be able to handle AWS setups.
		// AWS has zones named a,b,c etc in the end. us-east-1a, us-east-1b etc.
		// If host is also in AWS, then we want different proxies per zone but if host is not in AWS, we want one proxy for all zones in a region

		StringBuffer<256> proxyName;
		GetProxyName(proxyName, info);
		if (proxyName.IsEmpty())
			return false;

		auto writeHeader = [&]()
			{
				if (writeCasHeader)
				{
					writer.WriteU16(u16(~0));
					writer.Write7BitEncoded(0);
					writer.WriteByte(1 << 2);
				}
			};

		auto proxyKey = ToStringKeyNoCheck(proxyName.data, proxyName.count);
		SCOPED_WRITE_LOCK(m_proxiesLock, proxiesLock);
		ProxyEntry& proxy = m_proxies[proxyKey];
		if (proxy.clientId == ~0u)
		{
			if (proxy.failedClients.contains(clientId))
				return false;

			proxy.clientId = clientId;
			proxy.host = info.internalAddress;
			proxy.port = info.proxyPort;
			proxy.zone = proxyName.data;

			m_logger.Detail(TC("%s:%u (client %u) is assigned as proxy for zone %s"), proxy.host.c_str(), proxy.port, clientId, proxy.zone.c_str());

			writeHeader();
			writer.WriteBool(true);
			if (m_trace)
				m_trace->ProxyCreated(proxy.clientId, proxyName.data);
		}
		else
		{
			const tchar* proxyHost = proxy.host.c_str();
			if (clientId == proxy.clientId)
				proxyHost = TC("inprocess");

			writeHeader();
			writer.WriteBool(false);
			writer.WriteString(proxyHost);
			if (m_trace)
				m_trace->ProxyUsed(clientId, proxyName.data);
		}

		writer.WriteU16(proxy.port);
		writer.WriteU32(proxy.clientId);
		return true;
	}

	void StorageServer::GetProxyName(StringBufferBase& out, const Info& info)
	{
		if (!m_zone.empty() && info.zone.size() == m_zone.size() && Equals(m_zone.c_str(), info.zone.c_str(), u64(m_zone.size() - 1)))
		{
			// Host is inside AWS and same region.... if aws availability zone is different than host, then we use proxy
			if (m_zone != info.zone)
				out.Append(info.zone.c_str());
		}
		else if (m_zone != info.zone)
		{
			// We remove last character from zone to make sure all AWS availability zones in the same region use same proxy if host zone is outside AWS
			out.Append(info.zone.c_str(), info.zone.size() - 1);
		}
	}

	bool StorageServer::HandleMessage(const ConnectionInfo& connectionInfo, const WorkContext& workContext, u8 messageType, BinaryReader& reader, BinaryWriter& writer)
	{
		ActiveStore* firstStore = nullptr;
		ActiveStore tempStore;

		switch (messageType)
		{
			case StorageMessageType_Connect:
			{
				StringBuffer<> clientName;
				reader.ReadString(clientName);
				u32 clientVersion = reader.ReadU32();
				if (clientVersion < 4 || clientVersion > 5)
					return m_logger.Error(TC("Unsupported network version. Client: %u, Server: %u. Disconnecting"), clientVersion, StorageNetworkVersion);
				connectionInfo.SetStorageVersion(clientVersion);

				if (reader.ReadBool()) // is proxy
					return m_logger.Error(TC("Proxy is sending connect message. This path is not implemented"));
				u16 proxyPort = reader.ReadU16();
				SCOPED_WRITE_LOCK(m_connectionInfoLock, lock);
				Info& info = m_connectionInfo[connectionInfo.GetId()];
				info.zone = reader.ReadString();
				info.storageSize = reader.ReadU64();
				info.internalAddress = reader.ReadString();
				info.proxyPort = proxyPort;

				writer.WriteGuid(m_uid);
				writer.WriteByte(m_casCompressor);
				writer.WriteByte(m_casCompressionLevel);
				bool resendCas = m_dropIndependentMappingsOnCopy;
				writer.WriteBool(resendCas);
				return true;
			}

			case StorageMessageType_FetchBegin:
			{
				CasKey casKey;
				StringBuffer<> hint;

				// TODO: Remove this when we have tracked down the issue where clients time out
				u32 todoRemoveMeIndex = 0;
				u64 todoRemoveMeLastTime = GetTime();
				u64 todoRemoveMeLongestIndex = 0;
				auto todoRemoveMe = [&](int index)
					{
						u64 t = GetTime();
						u64 delta = t - todoRemoveMeLastTime;
						todoRemoveMeLastTime = t;
						if (delta <= todoRemoveMeLongestIndex)
							return;
						todoRemoveMeIndex = index;
						todoRemoveMeLongestIndex = delta;
					};

				auto timeoutGuard = MakeGuard([&, timeoutStartTime = GetTime()]()
					{
						u64 timeSpentMs = TimeToMs(GetTime() - timeoutStartTime);
						if (timeSpentMs > 8 * 60 * 1000)
						{
							// Took more than 5 minutes to respond
							m_logger.Warning(TC("Took more than 8 minutes to respond to FetchBegin to client %u (Hang code %u, cas %s, hint %s)"), connectionInfo.GetId(), todoRemoveMeIndex, CasKeyString(casKey).str, hint.data);
						}
					});

				u8 recvFlags = reader.ReadByte();
				bool wantsProxy = (recvFlags & 1) != 0;
				if (wantsProxy) // Wants proxy
				{
					todoRemoveMe(1);
					if (GetProxy(writer, connectionInfo.GetId(), true))
						return true;
				}

				todoRemoveMe(2);

				u64 start = GetTime();
				casKey = reader.ReadCasKey();
				StringKey keyHint = {};
				if (connectionInfo.GetStorageVersion() >= 5)
					keyHint = reader.ReadStringKey();

				reader.ReadString(hint);
				bool isKnownInput = hint.Equals(TCV("KnownInput"));

				StringBuffer<16> temp;
				StringView logHint = hint;
				if (isKnownInput)
				{
					temp.Append(CasKeyString(casKey).SubStr(0, 10));
					logHint = temp;
				}

				workContext.tracker.AddHint(logHint.GetFileName());

				casKey = AsCompressed(casKey, m_storeCompressed);

				bool detailedTrace = m_traceFetch;
				if (detailedTrace)
					m_trace->FileFetchBegin(connectionInfo.GetId(), casKey, logHint);

				auto traceFetchGuard = MakeGuard([&](){ if (detailedTrace) m_trace->FileFetchEnd(connectionInfo.GetId(), casKey); });


				CasEntry* casEntryPtr = nullptr;
				bool allowFail = false;
				bool has = HasCasFile(casKey, &casEntryPtr, false, allowFail); // HasCasFile writes deferred cas entries if in queue. Also increase CasEntry refcount

				todoRemoveMe(3);

				if (!has)
				{
					todoRemoveMe(4);
					if (!EnsureCasFile(casKey, nullptr) && m_allowHintAsFallback)
					{
						// Last resort.. use hint to load file into cas (hint should be renamed since it is now a critical parameter)
						// We better check the caskey first to make sure it is matching on the server

						StringKey fileNameKey = keyHint;
						if (keyHint == StringKeyZero)
						{
							// If known input without keyHint we have no way of finding cas.
							// This should be fine though because there will be a later request with the name later on
							if (isKnownInput)
							{
								m_logger.Info(TC("Server did not find cas %s for %s in file table lookup"), CasKeyString(casKey).str, hint.data);
								writer.WriteU16(0);
								return true;
							}

							fileNameKey = CaseInsensitiveFs ? ToStringKeyLower(hint) : ToStringKey(hint);
						}

						CasKey checkedCasKey;
						{
							SCOPED_READ_LOCK(m_fileTableLookupLock, lookupLock);
							auto findIt = m_fileTableLookup.find(fileNameKey);
							if (findIt != m_fileTableLookup.end())
							{
								FileEntry& fileEntry = findIt->second;
								lookupLock.Leave();
								SCOPED_FUTEX_READ(fileEntry.lock, entryLock);
								if (fileEntry.verified)
									checkedCasKey = fileEntry.casKey;
							}
						}
						if (checkedCasKey == CasKeyZero)
						{
							m_logger.Info(TC("Server did not find cas %s for %s in file table lookup. Recalculating cas key"), CasKeyString(casKey).str, hint.data);
							if (!CalculateCasKey(checkedCasKey, fileNameKey, hint.data))
							{
								m_logger.Error(TC("FetchBegin failed for cas file %s (%s) requested by client %u. Can't calculate cas key for file"), CasKeyString(casKey).str, hint.data, connectionInfo.GetId());
								writer.WriteU16(0);
								return false;
							}
						}

						if (AsCompressed(checkedCasKey, m_storeCompressed) != casKey)
						{
							m_logger.Error(TC("FetchBegin failed for cas file %s (%s) requested by client %u. File on disk has different cas %s"), CasKeyString(casKey).str, hint.data, connectionInfo.GetId(), CasKeyString(checkedCasKey).str);
							writer.WriteU16(0);
							return false;
						}

						bool deferCreation = false;
						bool dropCasAfterUse = false;
						if (!AddCasFile(fileNameKey, hint.data, casKey, deferCreation, InvalidValue, 0, nullptr, dropCasAfterUse))
						{
							m_logger.Error(TC("FetchBegin failed for cas file %s (%s) requested by client %u. Can't add cas file to database"), CasKeyString(casKey).str, hint.data, connectionInfo.GetId());
							writer.WriteU16(0);
							return true;
						}
					}
					SCOPED_WRITE_LOCK(m_casLookupLock, lookupLock);
					auto findIt = m_casLookup.find(casKey);
					if (findIt == m_casLookup.end())
					{
						writer.WriteU16(0);
						return true;
					}
					casEntryPtr = &findIt->second;
				}

				CasEntry& casEntry = *casEntryPtr;
				CasEntryWriteLock casEntryLock(casEntry);
				if (!has) // HasCasFile increase refcount, otherwise we have to do it manually
					++casEntry.refCount;
				if (!casEntry.refCount)
					m_logger.Error(TC("CasEntry refcount wrap-around during fetch. This is bad, report to uba devs"));
				casEntryLock.Leave();
				auto rlg = MakeGuard([&]() { casEntryLock.Enter(); ReleaseCasEntryNoLock(casEntry, m_dropIndependentMappingsOnCopy); });

				if (!casEntry.exists)
				{
					m_logger.Info(TC("Client %u failed FetchBegin for cas file %s (%s). Cas was deleted"), connectionInfo.GetId(), CasKeyString(casKey).str, hint.data);
					writer.WriteU16(0);
					return true;
				}

				todoRemoveMe(5);

				if (casEntry.disallowed)
				{
					writer.WriteU16(FetchCasIdDisallowed);
					m_logger.Warning(TC("Client %u is asking for cas content of file that is not allowed to be transferred (%s)"), connectionInfo.GetId(), hint.data);
					return true;
				}
				
				StringBuffer<512> casFile;
				FileHandle readFileHandle = InvalidFileHandle;
				auto rfg = MakeGuard([&]()
					{
						todoRemoveMe(14);
						if (!CloseFile(nullptr, readFileHandle))
							m_logger.Error(TC("Failed to close file %s (%s)"), casFile.data, LastErrorToText().data);
					});
				u64 fileSize = 0;
				const u8* memoryBegin = nullptr;
				const u8* memoryPos = nullptr;
				bool ownsMapping = false;

				FileMappingHandle fileMappingHandle;
				MappedView mappedView;
				auto mvg = MakeGuard([&]()
					{
						if (ownsMapping)
						{
							todoRemoveMe(12);
							if (mappedView.memory)
								if (!FileMapping_Unmap(m_logger, mappedView.memory, fileSize, TC("FetchBegin")))
									m_logger.Error(TC("Failed to unmap memory %s at 0x%llx with size %llu (%s)"), hint.data, mappedView.memory, fileSize, LastErrorToText().data);
							if (fileMappingHandle.IsValid())
								if (!FileMapping_Close(m_logger, fileMappingHandle, hint.data))
									m_logger.Error(TC("Failed to close file mapping for %s with handle %llu (%s)"), hint.data, fileMappingHandle.ToU64(), LastErrorToText().data);
						}
						else
						{
							todoRemoveMe(13);
							m_casDataBuffer.UnmapView(mappedView, TC("FetchBegin"));
						}
					});

				todoRemoveMe(6);

				bool useFileMapping = casEntry.memoryHandle.IsValid();
				if (useFileMapping)
				{
					mappedView = m_casDataBuffer.MapView(casEntry.memoryHandle, casEntry.memoryOffset, casEntry.memorySize, CasKeyString(casKey).str);
					memoryBegin = mappedView.memory;
					fileSize = casEntry.memorySize;
					if (!memoryBegin)
						return m_logger.Error(TC("Failed to map memory map for %s. Will use file handle instead (%s)"), CasKeyString(casKey).str, LastErrorToText().data);
					memoryPos = memoryBegin;
				}
				else
				{
					GetCasFileName(casFile, casKey);
					if (!OpenFileSequentialRead(m_logger, casFile.data, readFileHandle))
					{
						writer.WriteU16(0);
						return true;
					}

					if (!uba::GetFileSizeEx(fileSize, readFileHandle))
						return m_logger.Error(TC("GetFileSizeEx failed on file %s (%s)"), casFile.data, LastErrorToText().data);

					if (fileSize > BufferSlotSize)
					{
						ownsMapping = true;
						fileMappingHandle = FileMapping_CreateFromFile(m_logger, readFileHandle, PAGE_READONLY, fileSize, casFile.data);
						if (!fileMappingHandle.IsValid())
							return m_logger.Error(TC("Failed to create file mapping of %s (%s)"), casFile.data, LastErrorToText().data);
						u64 offset = memoryPos - memoryBegin;
						mappedView.memory = FileMapping_Map(m_logger, fileMappingHandle, FILE_MAP_READ, 0, fileSize);
						if (!mappedView.memory)
							return m_logger.Error(TC("Failed to map memory of %s with size %llu (%s)"), casFile.data, fileSize, LastErrorToText().data);
						memoryBegin = mappedView.memory;
						memoryPos = memoryBegin + offset;
						useFileMapping = true;
					}
				}

				todoRemoveMe(7);

				if (detailedTrace)
					m_trace->FileFetchSize(connectionInfo.GetId(), casKey, fileSize);
				else if (m_trace)
					m_trace->FileFetchLight(connectionInfo.GetId(), casKey, fileSize);

				u64 left = fileSize;

				u16* fetchId = (u16*)writer.AllocWrite(sizeof(u16));
				*fetchId = 0;
				writer.Write7BitEncoded(fileSize);
				u8 flags = 0;
				flags |= u8(m_storeCompressed) << 0;
				flags |= u8(m_traceFetch)    << 1;
				writer.WriteByte(flags);

				u64 capacityLeft = writer.GetCapacityLeft();
				u32 toWrite = u32(Min(left, capacityLeft));
				void* writeBuffer = writer.AllocWrite(toWrite);

				todoRemoveMe(8);

				if (useFileMapping)
				{
					memcpy(writeBuffer, memoryPos, toWrite);
					memoryPos += toWrite;
				}
				else if (toWrite == left)
				{
					if (!ReadFile(m_logger, casFile.data, readFileHandle, writeBuffer, toWrite))
					{
						UBA_ASSERTF(false, TC("ReadFile failed for file %s trying to read %u bytes (%s)"), casFile.data, toWrite, LastErrorToText().data); // Implement
						return m_logger.Error(TC("Failed to read file %s (%s) (1)"), casFile.data, LastErrorToText().data);
					}
				}
				else
				{
					u8* buffer = m_bufferSlots.Pop();
					memoryBegin = buffer;
					memoryPos = memoryBegin;
					u32 toRead = u32(Min(left, BufferSlotSize));
 					if (!ReadFile(m_logger, casFile.data, readFileHandle, buffer, toRead))
					{
						UBA_ASSERT(false); // Implement
						return m_logger.Error(TC("Failed to read file %s (%s) (2)"), casFile.data, LastErrorToText().data);
					}
					memcpy(writeBuffer, memoryPos, toWrite);
					memoryPos += toWrite;

					if (!CloseFile(casFile.data, readFileHandle))
						m_logger.Error(TC("Failed to close file %s (%s)"), casFile.data, LastErrorToText().data);
					readFileHandle = InvalidFileHandle;
				}

				todoRemoveMe(9);

				u64 actualSize = fileSize;
				if (m_storeCompressed)
					actualSize = *(u64*)writeBuffer;

				StorageStats& stats = Stats();
				stats.sendCasBytesComp += fileSize;
				stats.sendCasBytesRaw += actualSize;

				left -= toWrite;

				traceFetchGuard.Cancel();

				if (!left)
				{
					*fetchId = FetchCasIdDone;
					u64 sendCasTime = GetTime() - start;
					stats.sendCas += Timer{sendCasTime, 1};
					return true;
				}

				todoRemoveMe(10);

				rlg.Cancel();
				mvg.Cancel();
				rfg.Cancel();

				*fetchId = PopId();

				SCOPED_WRITE_LOCK(m_activeFetchesLock, lock);
				auto insres = m_activeFetches.try_emplace(*fetchId);
				UBA_ASSERT(insres.second);
				ActiveFetch& fetch = insres.first->second;
				fetch.clientId = connectionInfo.GetId();
				lock.Leave();

				todoRemoveMe(11);

				mappedView.size = fileSize;

				fetch.readFileHandle = readFileHandle;
				fetch.fileMappingHandle = fileMappingHandle;
				fetch.mappedView = mappedView;
				fetch.ownsMapping = ownsMapping;
				fetch.memoryBegin = memoryBegin;
				fetch.memoryPos = memoryPos;
				fetch.left = left;
				fetch.casEntry = &casEntry;
				fetch.sendCasTime = GetTime() - start;

				#if UBA_TRACK_FETCH_STORE_HINTS
				fetch.hint = logHint.ToString();
				#endif

				return true;
			}
			case StorageMessageType_FetchSegment:
			{
				u64 start = GetTime();
				u16 fetchId = reader.ReadU16();
				u32 fetchIndex = reader.ReadU32();

				SCOPED_READ_LOCK(m_activeFetchesLock, lock);
				auto findIt = m_activeFetches.find(fetchId);
				if (findIt == m_activeFetches.end())
					return m_logger.Detail(TC("Can't find active fetch %u, disconnected client? (fetch index %u, client id %u uid %s)"), fetchId, fetchIndex, connectionInfo.GetId(), GuidToString(connectionInfo.GetUid()).str).ToFalse();
				ActiveFetch& fetch = findIt->second; // This is safe because OnDisconnected can't happen while work is active
				UBA_ASSERT(fetch.clientId == connectionInfo.GetId());
				lock.Leave();

				UBA_ASSERT(fetchIndex);
				const u8* pos = fetch.memoryPos + (fetchIndex-1) * writer.GetCapacityLeft();
				u64 toWrite = writer.GetCapacityLeft();
				u64 readOffset = pos - fetch.memoryBegin;
				u64 viewSize = fetch.mappedView.size;
				if (readOffset + toWrite > viewSize)
				{
					if (readOffset >= viewSize)
						return m_logger.Detail(TC("Client is asking for more data than file contains (readoffset %llu, size %llu, fetch index %u, client id %u uid %s)"), readOffset, viewSize, fetchIndex, connectionInfo.GetId(), GuidToString(connectionInfo.GetUid()).str).ToFalse();
					toWrite = viewSize - readOffset;
				}
				memcpy(writer.AllocWrite(toWrite), pos, toWrite);

				fetch.sendCasTime += GetTime() - start;

				bool isDone = fetch.left.fetch_sub(toWrite) == toWrite;
				if (!isDone)
					return true;

				fetch.Release(*this, TC("FetchDone"));

				u64 sendCasTime = fetch.sendCasTime;
				SCOPED_WRITE_LOCK(m_activeFetchesLock, activeLock);
				m_activeFetches.erase(fetchId);
				activeLock.Leave();
				PushId(fetchId);

				Stats().sendCas += Timer{sendCasTime, 1};
				return true;
			}

			case StorageMessageType_FetchEnd:
			{
				CasKey key = reader.ReadCasKey();
				if (m_traceFetch)
					m_trace->FileFetchEnd(connectionInfo.GetId(), AsCompressed(key, m_storeCompressed));
				return true;
			}

			case StorageMessageType_ExistsOnServer:
			{
				CasKey casKey = reader.ReadCasKey();
				//UBA_ASSERT(IsCompressed(casKey));
				SCOPED_WRITE_LOCK(m_casLookupLock, lookupLock);
				auto casInsres = m_casLookup.try_emplace(casKey, casKey);
				CasEntry& casEntry = casInsres.first->second;
				lookupLock.Leave();

				CasEntryWriteLock casEntryLock(casEntry);
				
				if (!WaitForWritten(casEntry, casEntryLock, connectionInfo, TC("UNKNOWN")))
					return false;

				bool exists = casEntry.verified && casEntry.exists;

				if (!exists && casEntry.exists)
					if (!VerifyExists(exists, casEntry, casEntryLock, casKey))
						return false;

				UBA_ASSERTF(!m_dropIndependentMappingsOnCopy, TC("Not supported... ref count is increased in next"));

				writer.WriteBool(exists);
				return true;
			}

			case StorageMessageType_StoreBegin:
			{
				u32 clientId = connectionInfo.GetId();

				u64 start = GetTime();
				CasKey casKey = reader.ReadCasKey();
				u64 fileSize = reader.ReadU64();
				u64 actualSize = reader.ReadU64();
				//UBA_ASSERT(IsCompressed(casKey));
				StringBuffer<> hint;
				reader.ReadString(hint);

				SCOPED_WRITE_LOCK(m_casLookupLock, lookupLock);
				auto casInsres = m_casLookup.try_emplace(casKey, casKey);
				CasEntry& casEntry = casInsres.first->second;
				lookupLock.Leave();

				auto failGuard = MakeGuard([&]() { ReleaseActiveRefCount(casEntry.key, clientId); });
				auto success = [&]() { failGuard.Cancel(); return true; };

				CasEntryWriteLock casEntryLock(casEntry);

				// We need to take ref count on cas entry early since there are a bunch of sections
				// that temporarily release cas entry lock where file could actually be dropped
				{
					++casEntry.refCount;
					if (!casEntry.refCount)
						m_logger.Error(TC("CasEntry %s refcount wrap-around during store. This is bad, report to uba devs (%s)"), CasKeyString(casKey).str, hint.data);
					SCOPED_WRITE_LOCK(m_activeRefCountsLock, refCountLock);
					m_activeRefCounts[clientId].push_back(&casEntry);
				}

				{
					// Check if there is a deferred creation of this file, if there is we can just as well just use the local file
					SCOPED_FUTEX(m_deferredCasCreationLookupLock, deferredLock);
					auto findIt = m_deferredCasCreationLookup.find(casKey);
					if (findIt != m_deferredCasCreationLookup.end())
					{
						deferredLock.Leave();
						casEntryLock.Leave();
						bool allowFail = true; // We can afford to fail here since we are about to receive the cas file
						if (HasCasFile(casKey, nullptr, false, allowFail))
						{
							writer.WriteU16(u16(~0));
							writer.WriteBool(m_traceStore);
							m_logger.Debug(TC("Client %u Store request of %s which already exists in casdb (was in deferred list) (%s)"), clientId, CasKeyString(casKey).str, hint.data);
							UpdateWaitEntries(casEntry.key, true);
							return success();
						}
						casEntryLock.Enter();
					}
				}

				if (!casEntry.verified)
				{
					if (bool exists = casEntry.exists)
					{
						if (!VerifyExists(exists, casEntry, casEntryLock, casKey))
							return false;
						if (exists)
						{
							// Unless writeReceivedCasFilesToDisk is set this is extremely unlikely to happen. "PerModuleInline.gen.cpp" seems to be the only one (they are supposed to all be the same)
							writer.WriteU16(u16(~0));
							writer.WriteBool(m_traceStore);
							m_logger.Debug(TC("Client %u Store request of %s which already exists in casdb (%s)"), clientId, CasKeyString(casKey).str, hint.data);
							UpdateWaitEntries(casEntry.key, true);
							return success();
						}
					}
					else
						casEntry.verified = true;
				}
				else
				{
					if (!WaitForWritten(casEntry, casEntryLock, connectionInfo, hint.data))
						return false;

					if (casEntry.exists)
					{
						casEntryLock.Leave();
						CasEntryAccessed(casEntry);
						writer.WriteU16(u16(~0));
						writer.WriteBool(m_traceStore);
						//m_logger.Debug(TC("Client %u Store request of %s which was just added to casdb (%s)"), clientId, CasKeyString(casKey).str, hint.data);
						UpdateWaitEntries(casEntry.key, true);
						return success();
					}
				}
				if (!fileSize)
				{
					m_logger.Error(TC("Client %u Store is of 0 size (%s)"), clientId, hint.data);
					casEntry.verified = false;
					return false;
				}

				MappedView mappedView;
				FileAccessor* fileAccessor = nullptr;
				
				if (m_writeReceivedCasFilesToDisk)
				{
					StringBuffer<> casKeyName;
					GetCasFileName(casKeyName, casKey);
					
					const tchar* filename = TStrdup(casKeyName.data);
					fileAccessor = new FileAccessor(m_logger, filename);
					if (!fileAccessor->CreateMemoryWrite(false, DefaultAttributes(), fileSize, m_tempPath.data))
					{
						delete fileAccessor;
						free((void*)filename);

						m_logger.Error(TC("Failed to create cas file %s (%s)"), casKeyName.data, hint.data);
						casEntry.verified = false;
						return false;
					}

					#ifdef __clang_analyzer__ // Seems clang analyzer gets lost
					free((void*)filename);
					#endif

					mappedView.memory = fileAccessor->GetData();
				}
				else
				{
					mappedView = m_casDataBuffer.AllocAndMapView(fileSize, 1, CasKeyString(casKey).str, m_createIndependentMappings);
					if (!mappedView.memory)
					{
						casEntry.verified = false;
						return false;
					}
					casEntry.dropped = true; // Set to dropped because this will never hit disk
				}

				if (casEntry.beingWritten)
					return m_logger.Error(TC("Cas is being written while starting to write. %s (%s)"), CasKeyString(casKey).str, hint.data);

				casEntry.beingWritten = true;

				firstStore = &tempStore;

				//*(u64*)mappedView.memory = fileSize; // This code cause sigbus on mac. don't know why because address is always aligned.
				memcpy(mappedView.memory, &fileSize, sizeof(u64)); 

				firstStore->casEntry  = &casEntry;
				firstStore->fileSize = fileSize;
				firstStore->actualSize = actualSize;
				firstStore->mappedView = mappedView;
				firstStore->fileAccessor = fileAccessor;
				firstStore->recvCasTime = GetTime() - start;

				#if UBA_TRACK_FETCH_STORE_HINTS
				firstStore->hint = hint.ToString();
				#endif

				if (m_trace)
					m_trace->FileStoreBegin(clientId, casKey, fileSize, hint, m_traceStore);

				success();

				[[fallthrough]];
			}
			case StorageMessageType_StoreSegment:
			{
				u64 start = GetTime();

				u16 storeId = 0;
				u64 memOffset = 0;
				ActiveStore* activeStoreTemp = firstStore;
				if (!firstStore) // If this is set we are a continuation from StorageMessageType_Begin
				{
					storeId = reader.ReadU16();
					memOffset = reader.ReadU64();
					SCOPED_READ_LOCK(m_activeStoresLock, activeLock);
					auto storeIt = m_activeStores.find(storeId);
					if (storeIt == m_activeStores.end())
						return m_logger.Error(TC("Can't find active store %u, disconnected client?"), storeId);
					activeStoreTemp = &storeIt->second;
					if (activeStoreTemp->clientId != connectionInfo.GetId())
						return m_logger.Error(TC("Client id not matching for active store %u, disconnected client? (%u vs %u)"), storeId, activeStoreTemp->clientId, connectionInfo.GetId());
				}
				ActiveStore& activeStore = *activeStoreTemp;

				u64 toRead = reader.GetLeft();
				u64 fileSize = activeStore.fileSize;

				if (fileSize < memOffset + toRead)
					return m_logger.Error(TC("Trying to write data to cas outside of size (Size %llu, writing %llu at offset %llu"), fileSize, memOffset, toRead);

				reader.ReadBytes(activeStore.mappedView.memory + memOffset, toRead);
				
				u64 time2 = GetTime();
				activeStore.recvCasTime += time2 - start;

				u64 totalWritten = activeStore.totalWritten.fetch_add(toRead) + toRead;
				if (totalWritten == fileSize)
				{
					m_casDataBuffer.UnmapView(activeStore.mappedView, TC("StoreDone"));

					if (activeStore.fileAccessor)
					{
						bool success = activeStore.fileAccessor->Close();
						const tchar* filename = activeStore.fileAccessor->GetFileName();
						delete activeStore.fileAccessor;
						free((void*)filename);
						if (!success)
							return m_logger.Error(TC("REVISIT THIS!"));
					}

					CasEntry& casEntry = *activeStore.casEntry;
					{
						CasEntryWriteLock casEntryLock(casEntry);
						casEntry.memoryHandle = activeStore.mappedView.handle;
						casEntry.memoryOffset = activeStore.mappedView.offset;
						casEntry.memorySize = totalWritten;
						casEntry.exists = true;
						casEntry.beingWritten = false;
					}

					bool isPersistentStore = m_writeReceivedCasFilesToDisk;
					if (isPersistentStore)
						CasEntryWritten(casEntry, totalWritten);

					activeStore.recvCasTime += GetTime() - time2;

					StorageStats& stats = Stats();
					stats.recvCas += Timer{activeStore.recvCasTime, 1};
					stats.recvCasBytesComp += activeStore.fileSize;
					stats.recvCasBytesRaw += activeStore.actualSize;

					UpdateWaitEntries(casEntry.key, true);

					if (!firstStore)
					{
						SCOPED_WRITE_LOCK(m_activeStoresLock, activeLock);
						m_activeStores.erase(storeId);
						activeLock.Leave();
						PushId(storeId);
					}
					else
					{
						writer.WriteU16(0);
						writer.WriteBool(m_traceStore);
					}

					return true;
				}

				if (firstStore)
				{
					storeId = PopId();
					UBA_ASSERT(storeId != 0);
					writer.WriteU16(storeId);
					writer.WriteBool(m_traceStore);

					SCOPED_WRITE_LOCK(m_activeStoresLock, activeLock);
					auto insres = m_activeStores.try_emplace(storeId);
					UBA_ASSERT(insres.second);
					ActiveStore& s = insres.first->second;
					s.clientId = connectionInfo.GetId();
					activeLock.Leave();

					s.fileSize = firstStore->fileSize;
					s.actualSize = firstStore->actualSize;
					s.mappedView = firstStore->mappedView;
					s.fileAccessor = firstStore->fileAccessor;
					s.casEntry = firstStore->casEntry;
					s.totalWritten = firstStore->totalWritten.load();
					s.recvCasTime = firstStore->recvCasTime.load();

					#if UBA_TRACK_FETCH_STORE_HINTS
					s.hint = firstStore->hint;
					#endif

				}
				return true;
			}
			case StorageMessageType_StoreEnd:
			{
				CasKey key = reader.ReadCasKey();
				if (m_traceStore)
					m_trace->FileStoreEnd(connectionInfo.GetId(), key);
				return true;
			}
			case StorageMessageType_ProxyFetchBegin:
			{
				CasKey casKey = reader.ReadCasKey();
				StringBuffer<> hint;
				reader.ReadString(hint);
				if (m_trace)
					m_trace->FileFetchBegin(connectionInfo.GetId(), casKey, hint);
				return true;
			}
			case StorageMessageType_ProxyFetchEnd:
			{
				CasKey casKey = reader.ReadCasKey();
				if (m_trace)
					m_trace->FileFetchEnd(connectionInfo.GetId(), casKey);
				return true;
			}
			case StorageMessageType_ReportBadProxy:
			{
				u32 proxyClientId = reader.ReadU32();

				SCOPED_WRITE_LOCK(m_proxiesLock, lock);
				for (auto it=m_proxies.begin(); it!=m_proxies.end(); ++it)
				{
					ProxyEntry& e = it->second;
					if (e.clientId != proxyClientId)
						continue;
					m_logger.Detail(TC("Proxy %u (%s:%u) for zone %s removed (bad proxy reported by client %u)"), proxyClientId, e.host.c_str(), e.port, e.zone.c_str(), connectionInfo.GetId());
					e.failedClients.emplace(proxyClientId);
					e.clientId = ~0u;
					break;
				}
				lock.Leave();

				GetProxy(writer, connectionInfo.GetId(), false);
				return true;
			}
			case StorageMessageType_ReportBadCas:
			{
				CasKey casKey = reader.ReadCasKey();
				TString file = reader.ReadString();

				// TODO: Implement this, should drop the file
				m_logger.Warning(TC("Client reported bad cas %s (%s). Todo: Validate cas and delete it if it is bad"), CasKeyString(casKey).str, file.c_str());
				return true;
			}
		}
		UBA_ASSERT(false);
		return false;
	}

	u16 StorageServer::PopId()
	{
		SCOPED_FUTEX(m_availableIdsLock, lock);
		if (m_availableIds.empty())
		{
			if (m_availableIdsHigh == 65534)
			{
				m_logger.Error(TC("OUT OF AVAILABLE IDs.. SHOULD NEVER HAPPEN!"));
				UBA_ASSERT(false);
			}
			return m_availableIdsHigh++;
		}
		u16 storeId = m_availableIds.back();
		m_availableIds.pop_back();
		return storeId;
	}

	void StorageServer::PushId(u16 id)
	{
		SCOPED_FUTEX(m_availableIdsLock, lock);
		m_availableIds.push_back(id);
	}

}
