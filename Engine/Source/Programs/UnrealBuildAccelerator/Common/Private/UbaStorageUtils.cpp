// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaStorageUtils.h"
#include "UbaCompressedFileHeader.h"
#include "UbaFileMapping.h"
#include "UbaNetworkMessage.h"
#include "UbaStorage.h"
#include "UbaFileAccessor.h"
#include "UbaStats.h"
#include "UbaWorkManager.h"
#include <oodle2.h>

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	constexpr u32 MaxWorkItemsPerAction2 = 128; // Cap this to not starve other things


	#define OODLE_COMPRESSORS \
		OODLE_COMPRESSOR(Selkie) \
		OODLE_COMPRESSOR(Mermaid) \
		OODLE_COMPRESSOR(Kraken) \
		OODLE_COMPRESSOR(Leviathan) \

	#define OODLE_COMPRESSION_LEVELS \
		OODLE_COMPRESSION_LEVEL(None) \
		OODLE_COMPRESSION_LEVEL(SuperFast) \
		OODLE_COMPRESSION_LEVEL(VeryFast) \
		OODLE_COMPRESSION_LEVEL(Fast) \
		OODLE_COMPRESSION_LEVEL(Normal) \
		OODLE_COMPRESSION_LEVEL(Optimal1) \
		OODLE_COMPRESSION_LEVEL(Optimal2) \
		OODLE_COMPRESSION_LEVEL(Optimal3) \
		OODLE_COMPRESSION_LEVEL(Optimal4) \
		OODLE_COMPRESSION_LEVEL(Optimal5) \

	////////////////////////////////////////////////////////////////////////////////////////////////////

	u8 GetCompressor(const tchar* str)
	{
		#define OODLE_COMPRESSOR(x) if (Equals(str, TC(#x))) return u8(OodleLZ_Compressor_##x);
		OODLE_COMPRESSORS
		#undef OODLE_COMPRESSOR
		return DefaultCompressor;
	}

	u8 GetCompressionLevel(const tchar* str)
	{
		#define OODLE_COMPRESSION_LEVEL(x) if (Equals(str, TC(#x))) return u8(OodleLZ_CompressionLevel_##x);
		OODLE_COMPRESSION_LEVELS
		#undef OODLE_COMPRESSION_LEVEL
		return DefaultCompressionLevel;
	}


	CasKey CalculateCasKey(const u8* fileMem, u64 fileSize, bool storeCompressed, WorkManager* workManager, const tchar* hint)
	{
		CasKeyHasher hasher;

		if (fileSize == 0)
			return ToCasKey(hasher, storeCompressed);

		#ifndef __clang_analyzer__

		if (fileSize > BufferSlotSize) // Note that when filesize is larger than BufferSlotSize the hash becomes a hash of hashes
		{
			struct WorkRec
			{
				Atomic<u64> refCount = 2;
				Atomic<u64> counter = 0;
				Atomic<u64> doneCounter = 0;
				const u8* fileMem = nullptr;
				u64 workCount = 0;
				u64 fileSize = 0;
				bool error = false;
				Vector<CasKey> keys;
				EventSlim done;
			};

			u32 workCount = u32((fileSize + BufferSlotSize - 1) / BufferSlotSize);

			WorkRec* rec = new WorkRec();
			rec->fileMem = fileMem;
			rec->workCount = workCount;
			rec->fileSize = fileSize;
			rec->keys.resize(workCount);
			rec->done.Create(EventResetType_Manual);

			auto work = [rec](const WorkContext& context)
			{
				while (true)
				{
					u64 index = rec->counter++;
					if (index >= rec->workCount)
					{
						if (!--rec->refCount)
							delete rec;
						return 0;
					}

					u64 startOffset = BufferSlotSize*index;
					u64 toRead = Min(BufferSlotSize, rec->fileSize - startOffset);
					const u8* slot = rec->fileMem + startOffset;
					CasKeyHasher hasher;
					hasher.Update(slot, toRead);
					rec->keys[index] = ToCasKey(hasher, false);

					if (++rec->doneCounter == rec->workCount)
						rec->done.Set();
				}
			};

			u32 workerCount = 0;
			if (workManager)
			{
				workerCount = Min(workCount, workManager->GetWorkerCount()-1); // We are a worker ourselves
				workerCount = Min(workerCount, MaxWorkItemsPerAction2); // Cap this to not starve other things
				rec->refCount += workerCount;
				workManager->AddWork(WorkFunction(work), workerCount, TC("CalculateKey"));
			}

			{
				TrackWorkScope tws;
				work({tws});
			}
			rec->done.IsSet();

			hasher.Update(rec->keys.data(), rec->keys.size()*sizeof(CasKey));

			bool error = rec->error;

			if (!--rec->refCount)
				delete rec;

			if (error)
				return CasKeyZero;
		}
		else
		{
			hasher.Update(fileMem, fileSize);
		}

		#endif // __clang_analyzer__

		return ToCasKey(hasher, storeCompressed);
	}

	bool SendBatchMessages(Logger& logger, NetworkClient& client, u16 fetchId, u8* slot, u64 capacity, u64 left, u32 messageMaxSize, u32& readIndex, u32& responseSize, const Function<bool()>& runInWaitFunc, const tchar* hint, u32* outError, const Function<bool(u64)>& progressFunc)
	{
		responseSize = 0;

		if (outError)
			*outError = 0;

		struct Entry
		{
			Entry(u8* slot, u32 i, u32 messageMaxSize) : reader(slot + i * messageMaxSize, 0, SendMaxSize), done(EventResetType_Manual) {}
			NetworkMessage message;
			BinaryReader reader;
			EventSlim done;
		};

		u64 sendCountCapacity = capacity / messageMaxSize;
		u64 sendCount = left/messageMaxSize;

		if (left <= capacity)
		{
			if (sendCount * messageMaxSize < left)
				++sendCount;
		}
		else
		{
			if (sendCount > sendCountCapacity)
				sendCount = sendCountCapacity;
			else if (sendCount < sendCountCapacity && (left - sendCount * messageMaxSize) > 0)
				++sendCount;

			UBA_ASSERT(sendCount);
		}

		u64 entriesMem[sizeof(Entry) * 8];
		Entry* entries = (Entry*)entriesMem;

		if (sizeof(Entry)*sendCount > sizeof(entriesMem))
		{
			entries = (Entry*)malloc(sizeof(Entry)*sendCount);
			UBA_ASSERT(u64(entries) % alignof(Entry) == 0);
		}

		bool success = true;
		u32 error = 0;
		u32 inFlightCount = u32(sendCount);
		for (u32 i=0; i!=sendCount; ++i)
		{
			auto& entry = *new (entries + i) Entry(slot, i, messageMaxSize);
			StackBinaryWriter<32> writer;
			entry.message.Init(client, StorageServiceId, StorageMessageType_FetchSegment, writer);
			writer.WriteU16(fetchId);
			writer.WriteU32(readIndex + i + 1);
			if (entry.message.SendAsync(entry.reader, [](bool error, void* userData) { ((EventSlim*)userData)->Set(); }, &entry.done))
				continue;
			error = entry.message.GetError();
			entry.~Entry();
			inFlightCount = i;
			success = false;
			break;
		}

		if (runInWaitFunc)
		{
			if (!runInWaitFunc())
			{
				success = false;
				if (!error)
					error = 100;
			}
		}

		u32 timeOutTimeMs = 20*60*1000;

		for (u32 i=0; i!=inFlightCount; ++i)
		{
			Entry& entry = entries[i];
			if (!entry.done.IsSet(timeOutTimeMs))
			{
				logger.Error(TC("SendBatchMessages timed out after 20 minutes getting async message response (%u/%u) This timeout will cause a crash. Received %llu bytes so far. FetchId: %u (%s)"), i, inFlightCount, responseSize, fetchId, hint);
				timeOutTimeMs = 10;
			}
			if (!entry.message.ProcessAsyncResults(entry.reader))
			{
				if (!error)
					error = entry.message.GetError();
				success = false;
			}
			else
				responseSize += u32(entry.reader.GetLeft());

			if (success && progressFunc)
			{
				if (!progressFunc(responseSize))
				{
					success = false;
					if (!error)
						error = 100;
				}
			}
		}

		for (u32 i=0; i!=inFlightCount; ++i)
			entries[i].~Entry();

		if (entries != (Entry*)entriesMem)
			free(entries);

		readIndex += u32(sendCount);

		if (outError)
			*outError = error;

		return success;
	}

	UBA_NOINLINE bool SendFileBeginMessage(NetworkClient& client, const CasKey& casKey, const u8* sourceData, u64 sourceSize, const tchar* hint, StackBinaryReader<128>& reader, u64& outToWrite)
	{
		StackBinaryWriter<SendMaxSize> writer;
		NetworkMessage msg(client, StorageServiceId, StorageMessageType_StoreBegin, writer);
		writer.WriteCasKey(casKey);
		writer.WriteU64(sourceSize);
		writer.WriteU64(sourceSize);
		writer.WriteString(hint);

		u64 toWrite = Min(sourceSize, writer.GetCapacityLeft());
		writer.WriteBytes(sourceData, toWrite);

		outToWrite = toWrite;

		return msg.Send(reader);
	}

	UBA_NOINLINE bool SendFileSegmentMessage(NetworkClient& client, u16 storeId, u64 sendPos, const u8* data, u64 dataSize)
	{
		StackBinaryWriter<SendMaxSize> writer;
		NetworkMessage msg(client, StorageServiceId, StorageMessageType_StoreSegment, writer);
		writer.WriteU16(storeId);
		writer.WriteU64(sendPos);
		writer.WriteBytes(data, dataSize);
		return msg.Send();
	}

	bool SendFile(Logger& logger, NetworkClient& client, WorkManager* workManager, const CasKey& casKey, const u8* sourceMem, u64 sourceSize, const tchar* hint)
	{
		UBA_ASSERT(casKey != CasKeyZero);

		const u8* readData = sourceMem;

		u16 storeId = 0;
		bool sendEnd = false;
		u64 sendLeft = sourceSize;
		u64 sendPos = 0;

		auto sendEndMessage = [&]()
			{
				if (!sendEnd)
					return true;
				StackBinaryWriter<32> writer;
				NetworkMessage msg(client, StorageServiceId, StorageMessageType_StoreEnd, writer);
				writer.WriteCasKey(casKey);
				return msg.Send();
			};

		if (sendLeft)
		{
			StackBinaryReader<128> reader;
			u64 toWrite = 0;
			if (!SendFileBeginMessage(client, casKey, readData, sendLeft, hint, reader, toWrite))
				return false;

			readData += toWrite;
			sendLeft -= toWrite;
			sendPos += toWrite;

			storeId = reader.ReadU16();
			sendEnd = reader.ReadBool();

			if (sendLeft == 0)
				return sendEndMessage();

			if (!storeId) // Zero means error
				return logger.Error(TC("Server failed to start storing file %s (%s)"), CasKeyString(casKey).str, hint);

			if (storeId == u16(~0)) // File already exists on server
			{
				//logger.Info(TC("Server already has file %s (%s)"), CasKeyString(casKey).str, hint);
				return sendEndMessage();
			}
		}

		u64 messageHeader = client.GetMessageHeaderSize();
		u64 messageCapacity = client.GetMessageMaxSize() - (messageHeader + sizeof(u16) + sizeof(u64));
		u64 sendCount = (sendLeft + messageCapacity - 1) / messageCapacity;

		u32 connectionCount = client.GetConnectionCount();
		if (workManager && connectionCount > 1) // Use parallel send.. to make sure all sockets send buffers are fully utilized
		{
			IndexContainer container(sendCount);

			Atomic<bool> error = false;

			workManager->ParallelFor(connectionCount - 1, container, [&](const WorkContext&, auto& it)
				{
					u64 sendPos2 = sendPos + it.index * messageCapacity;
					const u8* writeData = readData + it.index * messageCapacity;
					u64 toWrite = messageCapacity;
					if (it.index == sendCount - 1)
						toWrite = sendLeft % messageCapacity;
					if (!SendFileSegmentMessage(client, storeId, sendPos2, writeData, toWrite))
						error = true;
				}, TCV("SendFile"));

			if (error)
				return false;
		}
		else
		{
			while (sendLeft)
			{
				u64 toWrite = Min(sendLeft, messageCapacity);
				if (!SendFileSegmentMessage(client, storeId, sendPos, readData, toWrite))
					return false;
				readData += toWrite;
				sendLeft -= toWrite;
				sendPos += toWrite;
			}
		}

		return sendEndMessage();
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool FileSender::SendFileCompressed(const CasKey& casKey, const tchar* fileName, const u8* sourceMem, u64 sourceSize, const tchar* hint)
	{
		UBA_ASSERT(casKey != CasKeyZero);

		NetworkClient& client = m_client;

		TimerScope ts(m_stats.sendCas);

		u64 firstMessageOverHead = (sizeof(CasKey) + sizeof(u64)*2 + GetStringWriteSize(hint, TStrlen(hint)));

		u64 messageHeader = client.GetMessageHeaderSize();
		u64 messageHeaderMaxSize = messageHeader + firstMessageOverHead;

		MemoryBlock memoryBlock(TC("FileSender"), sourceSize + messageHeaderMaxSize + 1024);
		{
			const u8* uncompressedData = sourceMem;
			u8* compressBufferStart = memoryBlock.memory + messageHeaderMaxSize;
			u8* compressBuffer = compressBufferStart;
			u64 totalWritten = messageHeaderMaxSize; // Make sure there is room for msg header in the memory since we are using it to send
			u64 left = sourceSize;

			compressBuffer += 8;
			totalWritten += 8;
			memoryBlock.Allocate(totalWritten, 1, hint);

			u64 diff = u64(OodleLZ_GetCompressedBufferSizeNeeded((OodleLZ_Compressor)m_casCompressor, BufferSlotHalfSize)) - BufferSlotHalfSize;
			u64 maxUncompressedBlock = BufferSlotHalfSize - diff - totalWritten - 8; // 8 bytes block header

			OodleLZ_CompressOptions oodleOptions = *OodleLZ_CompressOptions_GetDefault();
			while (left)
			{
				u32 uncompressedBlockSize = (u32)Min(left, maxUncompressedBlock);

				u64 reserveSize = totalWritten + uncompressedBlockSize + diff + 8;
				if (reserveSize > memoryBlock.committedSize)
				{
					u64 toAllocate = reserveSize - memoryBlock.writtenSize;
					memoryBlock.Allocate(toAllocate, 1, hint);
				}

				u8* destBuf = compressBuffer;
				u32 compressedBlockSize;
				{
					TimerScope cts(m_stats.compressSend);
					compressedBlockSize = (u32)OodleLZ_Compress((OodleLZ_Compressor)m_casCompressor, uncompressedData, (int)uncompressedBlockSize, destBuf + 8, (OodleLZ_CompressionLevel)m_casCompressionLevel, &oodleOptions);
					if (compressedBlockSize == OODLELZ_FAILED)
						return m_logger.Error(TC("Failed to compress %u bytes at %llu for %s (%s) (%s) (uncompressed size: %llu)"), uncompressedBlockSize, totalWritten, fileName, CasKeyString(casKey).str, hint, sourceSize);
				}

				*(u32*)destBuf =  u32(compressedBlockSize);
				*(u32*)(destBuf+4) =  u32(uncompressedBlockSize);

				u32 writeBytes = u32(compressedBlockSize) + 8;

				totalWritten += writeBytes;
				memoryBlock.writtenSize = totalWritten;

				left -= uncompressedBlockSize;
				uncompressedData += uncompressedBlockSize;
				compressBuffer += writeBytes;
			}

			*(u64*)compressBufferStart = sourceSize;
		}


		u8* readData = memoryBlock.memory + messageHeaderMaxSize;
		u64 fileSize = memoryBlock.writtenSize - messageHeaderMaxSize;

		u16 storeId = 0;
		bool sendEnd = false;
		u64 sendLeft = fileSize;
		u64 sendPos = 0;

		auto sendEndMessage = [&]()
			{
				if (!sendEnd)
					return true;
				StackBinaryWriter<128> writer;
				NetworkMessage msg(client, StorageServiceId, StorageMessageType_StoreEnd, writer);
				writer.WriteCasKey(casKey);
				return msg.Send();
			};

		bool hasSendOneAtTheTimeLock = false;
		auto lockGuard = MakeGuard([&]() { if (hasSendOneAtTheTimeLock) m_sendOneAtTheTimeLock.Leave(); });

		{
			u64 writerStartOffset = messageHeader + firstMessageOverHead;
			BinaryWriter writer(readData - writerStartOffset, 0, client.GetMessageMaxSize());
			NetworkMessage msg(client, StorageServiceId, StorageMessageType_StoreBegin, writer);
			writer.WriteCasKey(casKey);
			writer.WriteU64(fileSize);
			writer.WriteU64(sourceSize);
			writer.WriteString(hint);
			u64 capacityLeft = writer.GetCapacityLeft();
			u64 toWrite = Min(sendLeft, capacityLeft);
			writer.AllocWrite(toWrite);
			sendLeft -= toWrite;
			sendPos += toWrite;

			if (sendLeft != 0 && m_sendOneBigFileAtTheTime)
			{
				m_sendOneAtTheTimeLock.Enter();
				hasSendOneAtTheTimeLock = true;
			}
			StackBinaryReader<128> reader;
			if (!msg.Send(reader))
				return false;
			storeId = reader.ReadU16();
			sendEnd = reader.ReadBool();
		}

		if (sendLeft != 0)
		{
			if (!storeId) // Zero means error
				return m_logger.Error(TC("Server failed to start storing file %s (%s)"), CasKeyString(casKey).str, hint);
			if (storeId == u16(~0)) // File already exists on server
				return sendEndMessage(); //m_logger.Info(TC("Server already has file %s (%s)"), CasKeyString(casKey).str, hint);

			u64 writerHeaderSize = messageHeader + (sizeof(u16) + sizeof(u64));
			u64 writerCapacity = client.GetMessageMaxSize() - writerHeaderSize;
			u64 sendCount = (sendLeft + writerCapacity - 1) / writerCapacity;

			auto SendSegment = [&](u64 index)
				{
					u64 segmentPos = sendPos + writerCapacity*index;
					u8* writerStart = readData + segmentPos - writerHeaderSize;
					u8 prevData[32];
					memcpy(prevData, writerStart, writerHeaderSize);

					BinaryWriter writer(writerStart, 0, client.GetMessageMaxSize());
					NetworkMessage msg(client, StorageServiceId, StorageMessageType_StoreSegment, writer);
					writer.WriteU16(storeId);
					writer.WriteU64(segmentPos);
					u64 toWrite = Min(fileSize - segmentPos, writerCapacity);
					writer.AllocWrite(toWrite);
					bool res = msg.Send();
					memcpy(writerStart, prevData, writerHeaderSize);
					return res;
				};

			// If we have more than one connection we can use parallel for to send to multiple connections at the same time.
			u32 connectionCount = m_parallelSend && sendCount > 1 ? client.GetConnectionCount() : 1;
			if (connectionCount > 1)
			{
				// We have to leapfrog every other message since the SendSegment modifies the memory just before the sent segment
				for (u32 i=0;i!=2; ++i)
				{
					Atomic<bool> error;
					IndexContainer container((sendCount+1)/2);
					m_client.ParallelFor(connectionCount - 1, container, [&](const WorkContext&, auto& it)
					{
						u32 index = it.index*2+i;
						if (index < sendCount)
							if (!SendSegment(index))
								error = true;
					}, TCV("SendCompressedFile"));
					if (error)
						return false;
				}
			}
			else
			{
				for (u64 i=0; i!=sendCount; ++i)
					if (!SendSegment(i))
						return false;
			}
		}

		m_stats.sendCasBytesRaw += sourceSize;
		m_stats.sendCasBytesComp += fileSize;
		m_bytesSent = fileSize;

		return sendEndMessage();
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool FileFetcher::RetrieveFile(Logger& logger, NetworkClient& client, const CasKey& casKey, const tchar* destination, bool writeCompressed, MemoryBlock* destinationMem, u32 attributes)
	{
		TimerScope ts(m_stats.recvCas);
		u8* slot = m_bufferSlots.Pop();
		auto sg = MakeGuard([&](){ m_bufferSlots.Push(slot); });

		u64 fileSize = 0;
		u64 actualSize = 0;

		u8* readBuffer = nullptr;
		u8* readPosition = nullptr;

		u16 fetchId = 0;
		u32 responseSize = 0;
		bool isCompressed = false;
		bool sendEnd = false;

		u32 sizeOfFirstMessage = 0;

		{
			StackBinaryWriter<1024> writer;
			NetworkMessage msg(client, StorageServiceId, StorageMessageType_FetchBegin, writer);
			writer.WriteBool(false);
			writer.WriteCasKey(casKey);
			writer.WriteStringKey({});
			writer.WriteString(destination);
			BinaryReader reader(slot + (writeCompressed ? sizeof(CompressedFileHeader) : 0), 0, SendMaxSize); // Create some space in front of reader to write obj file header there if destination is compressed
			if (!msg.Send(reader))
				return logger.Error(TC("Failed to send fetch begin message for cas %s (%s). Error: %u"), CasKeyString(casKey).str, destination, msg.GetError());
			sizeOfFirstMessage = u32(reader.GetLeft());
			fetchId = reader.ReadU16();
			if (fetchId == 0)
			{
				logger.Logf(m_errorOnFail ? LogEntryType_Error : LogEntryType_Detail, TC("Failed to fetch cas %s (%s)"), CasKeyString(casKey).str, destination);
				return false;
			}

			fileSize = reader.Read7BitEncoded();

			u8 flags = reader.ReadByte();

			isCompressed = (flags >> 0) & 1;
			sendEnd = (flags >> 1) & 1;

			responseSize = u32(reader.GetLeft());
			readBuffer = (u8*)reader.GetPositionData();
			readPosition = readBuffer;

			actualSize = fileSize;
			if (isCompressed)
				actualSize = *(u64*)readBuffer;
		}

		sizeOnDisk = writeCompressed ? (sizeof(CompressedFileHeader) + fileSize) : actualSize;

		FileAccessor destinationFile(logger, destination);

		bool useFileMapping = sizeOnDisk > 0;
		u8* fileMappingMem = nullptr;

		if (!destinationMem)
		{
			if (useFileMapping)
			{
				if (!destinationFile.CreateMemoryWrite(false, attributes, sizeOnDisk))
					return false;
				fileMappingMem = destinationFile.GetData();
			}
			else
			{
				if (!destinationFile.CreateWrite(false, attributes, sizeOnDisk, m_tempPath.data))
					return false;
			}
		}

		u64 destOffset = 0;

		auto ReportBadCas = [&]()
			{
				StackBinaryWriter<1024> writer;
				NetworkMessage msg(client, StorageServiceId, StorageMessageType_ReportBadCas, writer);
				writer.WriteCasKey(casKey);
				writer.WriteString(destination);
				StackBinaryReader<8> reader;
				msg.Send(reader);
			};


		auto WriteDestination = [&](const void* source, u64 sourceSize)
			{
				if (fileMappingMem)
				{
					TimerScope ts(m_stats.memoryCopy);
					FileMapping_CopyMem(fileMappingMem + destOffset, source, sourceSize);
					destOffset += sourceSize;
				}
				else if (destinationMem)
				{
					TimerScope ts(m_stats.memoryCopy);
					void* mem = destinationMem->Allocate(sourceSize, 1, TC(""));
					memcpy(mem, source, sourceSize);
				}
				else
				{
					if (!destinationFile.Write(source, sourceSize, destOffset))
						return false;
					destOffset += sourceSize;
				}
				return true;
			};

		u32 readIndex = 0;

		if (writeCompressed)
		{
			u8* source = slot + BufferSlotHalfSize;
			u8* lastSource = readBuffer;
			u64 lastResponseSize = responseSize;

			lastSource -= sizeof(CompressedFileHeader);
			lastResponseSize += sizeof(CompressedFileHeader);
			new (lastSource) CompressedFileHeader{ casKey };

			if (fileMappingMem)
			{
				auto writePrev = [&]() { bool res = WriteDestination(lastSource, lastResponseSize); sg.Execute(); return res;};
				if (u64 leftCompressed = fileSize - responseSize)
				{
					u64 destOffset2 = destOffset + lastResponseSize;
					u32 error;
					if (!SendBatchMessages(logger, client, fetchId, fileMappingMem + destOffset2, leftCompressed, leftCompressed, sizeOfFirstMessage, readIndex, responseSize, writePrev, destination, &error))
						return logger.Error(TC("Failed to send batched messages to server while retrieving cas %s to %s. Error: %u"), CasKeyString(casKey).str, destination, error);
					UBA_ASSERT(responseSize == leftCompressed);
				}
				else if (!writePrev())
					return false;
			}
			else
			{
				auto writePrev = [&]() { return WriteDestination(lastSource, lastResponseSize); };

				u64 leftCompressed = fileSize - responseSize;
				while (leftCompressed)
				{
					if (fetchId == u16(~0))
						return logger.Error(TC("Cas content error (2). Server believes %s was only one segment but client sees more. "));//UncompressedSize: %llu LeftUncompressed: %llu Size: %llu Left to read: %llu ResponseSize: %u. (%s)"), destination, actualSize, leftUncompressed, fileSize, left, responseSize, CasKeyString(casKey).str);

					u32 error;
					if (!SendBatchMessages(logger, client, fetchId, source, BufferSlotHalfSize, leftCompressed, sizeOfFirstMessage, readIndex, responseSize, writePrev, destination, &error))
						return logger.Error(TC("Failed to send batched messages to server while retrieving cas %s to %s. Error: %u"), CasKeyString(casKey).str, destination, error);

					lastSource = source;
					lastResponseSize = responseSize;
					source = source == slot ? slot + BufferSlotHalfSize : slot;

					leftCompressed -= responseSize;
				}
				if (!writePrev())
					return false;
			}
		}
		else if (actualSize && (fileMappingMem || destinationMem))
		{
			u64 fileSize2 = fileSize - sizeof(u64);
			readBuffer += sizeof(u64); // Size is stored first
			responseSize -= sizeof(u64);

			u8* destMem = fileMappingMem;

			if (!destMem)
				destMem = (u8*)destinationMem->Allocate(actualSize, 1, TC(""));

			u64 initialRead = 0;
			u32 compressedSize = 0;
			u32 decompressedSize = 0;
			u64 nextOffset = 0;
			void* decoderMem = nullptr;
			u64 decoderMemSize = 0;

			auto decompressAndWrite = [&](u64 pos)
				{
					u64 offset = pos + initialRead;
					while (true)
					{
						if (!compressedSize && offset >= nextOffset + 8)
						{
							compressedSize =  ((u32*)readBuffer)[0];
							decompressedSize = ((u32*)readBuffer)[1];
							readBuffer += 8;
							nextOffset += compressedSize + 8;
						}

						if (offset < nextOffset)
							return true;

						TimerScope ts2(m_stats.decompressRecv);
						OO_SINTa decompLen = OodleLZ_Decompress(readBuffer, int(compressedSize), destMem, int(decompressedSize), OodleLZ_FuzzSafe_Yes, OodleLZ_CheckCRC_No, OodleLZ_Verbosity_None, NULL, 0, NULL, NULL, decoderMem, decoderMemSize);
						if (decompLen != decompressedSize)
						{
							ReportBadCas();
							return logger.Error(TC("Expected %u but got %i when decompressing %u bytes for file %s"), decompressedSize, int(decompLen), compressedSize, destination);
						}

						readBuffer += compressedSize;
						destMem += decompressedSize;
						compressedSize = 0;

						if (offset < nextOffset + 8)
							break;
					}
					return true;
				};

			if (u64 leftCompressed = fileSize2 - responseSize)
			{
				initialRead = responseSize;

				u8* compressedData = readBuffer;

				u64 sizeThatFitsInSlot = BufferSlotSize - (readBuffer - slot);
				bool useAllocation = fileSize2 > sizeThatFitsInSlot;
				if (useAllocation)
				{
					compressedData = (u8*)malloc(fileSize2);
					memcpy(compressedData, readBuffer, responseSize);
					
					sg.Execute();
					//decoderMem = slot;
					//decoderMemSize = BufferSlotSize;
				}
				auto mbg = MakeGuard([&]() { if (useAllocation) free(compressedData); });

				readBuffer = compressedData;

				u32 error;
				if (!SendBatchMessages(logger, client, fetchId, compressedData + responseSize, leftCompressed, leftCompressed, sizeOfFirstMessage, readIndex, responseSize, {}, destination, &error, decompressAndWrite))
					return logger.Error(TC("Failed to send batched messages to server while retrieving cas %s to %s. Error: %u"), CasKeyString(casKey).str, destination, error);
			}
			else if (!decompressAndWrite(responseSize))
				return false;
		}
		else if (actualSize)
		{
			bool sendSegmentMessage = responseSize == 0;
			u64 leftUncompressed = actualSize;
			readBuffer += sizeof(u64); // Size is stored first
			u64 maxReadSize = BufferSlotHalfSize - sizeof(u64);

			u8* decompressBuffer = slot + BufferSlotHalfSize;
			u32 lastDecompressSize = 0;
			auto tryWriteDecompressed = [&]()
				{
					if (!lastDecompressSize)
						return true;
					u32 toWrite = lastDecompressSize;
					lastDecompressSize = 0;
					return WriteDestination(decompressBuffer, toWrite);
				};

			u64 leftCompressed = fileSize - responseSize;
			do
			{
				// There is a risk the compressed file we download is larger than half buffer
				u8* extraBuffer = nullptr;
				auto ebg = MakeGuard([&]() { delete[] extraBuffer; });

				// First read in a full decompressable block
				bool isFirstInBlock = true;
				u32 compressedSize = ~u32(0);
				u32 decompressedSize = ~u32(0);
				u32 left = 0;
				u32 overflow = 0;
				do
				{
					if (sendSegmentMessage)
					{
						if (fetchId == u16(~0))
						{
							ReportBadCas();
							return logger.Error(TC("Cas content error (2). Server believes %s was only one segment but client sees more. UncompressedSize: %llu LeftUncompressed: %llu Size: %llu Left to read: %llu ResponseSize: %u. (%s)"), destination, actualSize, leftUncompressed, fileSize, left, responseSize, CasKeyString(casKey).str);
						}

						u64 capacity = maxReadSize - u64(readPosition - readBuffer);
						u64 writeCapacity = capacity;
						u8* writeDest = readPosition;
						if (capacity < sizeOfFirstMessage) // capacity left is less than the last message
						{
							UBA_ASSERT(!extraBuffer);
							extraBuffer = new u8[sizeOfFirstMessage];
							writeDest = extraBuffer;
							writeCapacity = sizeOfFirstMessage;
						}
					
						u32 error;
						if (!SendBatchMessages(logger, client, fetchId, writeDest, writeCapacity, leftCompressed, sizeOfFirstMessage, readIndex, responseSize, tryWriteDecompressed, destination, &error))
							return logger.Error(TC("Failed to send batched messages to server while retrieving and decompressing cas %s to %s. Error: %u"), CasKeyString(casKey).str, destination, error);

						if (extraBuffer)
						{
							memcpy(readPosition, extraBuffer, left);
							memmove(extraBuffer, extraBuffer + left, u32(responseSize - left));
							if (isFirstInBlock)
								return logger.Error(TC("Make static analysis happy. This should not be possible to happen (%s)"), CasKeyString(casKey).str);
						}

						leftCompressed -= responseSize;
					}
					else
					{
						sendSegmentMessage = true;
					}

					if (isFirstInBlock)
					{
						if (readPosition - readBuffer < sizeof(u32) * 2)
						{
							ReportBadCas();
							return logger.Error(TC("Received less than minimum amount of data. Most likely corrupt cas file %s (Available: %u UncompressedSize: %llu LeftUncompressed: %llu)"), CasKeyString(casKey).str, u32(readPosition - readBuffer), actualSize, leftUncompressed);
						}
						isFirstInBlock = false;
						u32* blockSize = (u32*)readBuffer;
						compressedSize = blockSize[0];
						decompressedSize = blockSize[1];
						readBuffer += sizeof(u32) * 2;
						maxReadSize = BufferSlotHalfSize - sizeof(u32) * 2;
						u32 read = (responseSize + u32(readPosition - readBuffer));
						//UBA_ASSERTF(read <= compressedSize, TC("Error in datastream fetching cas. Read size: %u CompressedSize: %u %s (%s)"), read, compressedSize, CasKeyString(casKey).str, destination);
						if (read > compressedSize)
						{
							//UBA_ASSERT(!responseSize); // TODO: This has not really been tested
							left = 0;
							overflow = read - compressedSize;
							sendSegmentMessage = false;
						}
						else
						{
							left = compressedSize - read;
						}
						readPosition += responseSize;
					}
					else
					{
						readPosition += responseSize;
						if (responseSize > left)
						{
							overflow = responseSize - u32(left);
							UBA_ASSERTF(overflow < BufferSlotHalfSize, TC("Something went wrong. Overflow: %u responseSize: %u, left: %u"), overflow, responseSize, left);
							if (overflow >= 8)
							{
								responseSize = 0;
								sendSegmentMessage = false;
							}
							left = 0;
						}
						else
						{
							if (left < responseSize)
								return logger.Error(TC("Something went wrong. Left %u, Response: %u (%s)"), left, responseSize, destination);
							left -= responseSize;
						}
					}
				} while (left);


				// Second, decompress
				while (true)
				{
					tryWriteDecompressed();

					{
						TimerScope ts2(m_stats.decompressRecv);
						OO_SINTa decompLen = OodleLZ_Decompress(readBuffer, int(compressedSize), decompressBuffer, int(decompressedSize));
						if (decompLen != decompressedSize)
						{
							ReportBadCas();
							return logger.Error(TC("Expected %u but got %i when decompressing %u bytes for file %s"), decompressedSize, int(decompLen), compressedSize, destination);
						}
					}

					lastDecompressSize = decompressedSize;
					leftUncompressed -= decompressedSize;

					constexpr bool decompressMultiple = true; // This does not seem to be a win.. it batches more but didn't save any time

					if (!decompressMultiple)
						break;

					if (overflow < 8)
						break;
					u8* nextBlock = readBuffer + compressedSize;
					u32* blockSize = (u32*)nextBlock;
					u32 compressedSize2 = blockSize[0];
					if (overflow < compressedSize2 + 8)
						break;
					readBuffer += compressedSize + 8;

					decompressedSize = blockSize[1];
					compressedSize = compressedSize2;
					overflow -= compressedSize + 8;
				}

				// Move overflow back to the beginning of the buffer and start the next block (if there is one)
				readBuffer = slot;
				maxReadSize = BufferSlotHalfSize;

				if (extraBuffer)
				{
					memcpy(readBuffer, extraBuffer, overflow);
					delete[] extraBuffer;
					extraBuffer = nullptr;
				}
				else
				{
					// Move overflow back to the beginning of the buffer and start the next block (if there is one)
					UBA_ASSERTF(readPosition - overflow >= readBuffer, TC("ReadPosition - overflow is before beginning of buffer (overflow: %u) for file %s"), overflow, destination);
					UBA_ASSERTF(readPosition <= readBuffer + BufferSlotHalfSize, TC("ReadPosition is outside readBuffer size (pos: %llu, overflow: %u) for file %s"), readPosition - readBuffer, overflow, destination);
					memmove(readBuffer, readPosition - overflow, overflow);
				}

				readPosition = readBuffer + overflow;
				if (overflow)
				{
					if (overflow < sizeof(u32) * 2) // Must always have the compressed and uncompressed size to be able to move on with logic above
						sendSegmentMessage = true;
					else
						responseSize = 0;
				}
			} while (leftUncompressed);

			if (!tryWriteDecompressed())
				return false;
		}

		if (sendEnd)
		{
			StackBinaryWriter<128> writer;
			NetworkMessage msg(client, StorageServiceId, StorageMessageType_FetchEnd, writer);
			writer.WriteCasKey(casKey);
			if (!msg.Send())
				return false;
		}

		if (!destinationMem)
			if (!destinationFile.Close(&lastWritten))
				return false;

		bytesReceived = fileSize;

		m_stats.recvCasBytesRaw += actualSize;
		m_stats.recvCasBytesComp += fileSize;

		return true;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	CompressWriter::CompressWriter(Logger& l, SharedMemoryAllocator& a, BufferCache& bs, WorkManager& wm, StorageStats& s, u8 cc, u8 ccl, bool avof)
	:	m_logger(l)
	,	m_allocator(a)
	,	m_bufferSlots(bs)
	,	m_workManager(wm)
	,	m_stats(s)
	,	m_casCompressor(cc)
	,	m_casCompressionLevel(ccl)
	,	m_asyncUnmapViewOfFile(avof)
	{
	}

	bool CompressWriter::CompressToFile(u64& outCompressedSize, const tchar* from, FileHandle readHandle, u8* readMem, u64 fileSize, const tchar* toFile, const void* header, u64 headerSize, u64 lastWriteTime, const tchar* tempPath)
	{
		FileAccessor destinationFile(m_logger, toFile);
		if (!destinationFile.CreateWrite(false, DefaultAttributes(), 0, tempPath))
			return false;
		if (headerSize)
			if (!destinationFile.Write(header, headerSize))
				return false;

		LinearWriterFile destination(destinationFile);
		if (!CompressFromMemOrFile(destination, from, readHandle, readMem, fileSize))
			return false;
		if (lastWriteTime)
			if (!SetFileLastWriteTime(destinationFile.GetHandle(), lastWriteTime))
				return m_logger.Error(TC("Failed to set file time on filehandle for %s"), toFile);
		if (!destinationFile.Close())
			return false;
		outCompressedSize = destination.GetWritten() + headerSize;
		return true;
	}

	bool CompressWriter::CompressToMapping(SharedMemoryHandle& outMappingHandle, u64& outMappingSize, u8* readMem, u64 fileSize, const tchar* hint)
	{
		u64 mappingSize = fileSize + 1024;

		SharedMemoryHandle destMapping = m_allocator.CreateHandle(m_logger, hint);
		m_allocator.ExtendMemory(destMapping, mappingSize, hint, false);
		auto destGuard = MakeGuard([&]() { m_allocator.CloseHandle(m_logger, destMapping, hint); });
		u8* compressedData = m_allocator.MapView(destMapping, hint, SharedMemoryMapType_ReadWrite);
		if (!compressedData)
			return false;
		auto unmapGuard = MakeGuard([&]() { m_allocator.UnmapView(destMapping, compressedData); });

		LinearWriterMem destination(compressedData, mappingSize);
		if (!CompressFromMemOrFile(destination, hint, InvalidFileHandle, readMem, fileSize))
			return false;

		unmapGuard.Execute();
		destGuard.Cancel();

		u64 written = destination.GetWritten();
		m_allocator.TrimMemory(destMapping, written);

		outMappingHandle = destMapping;
		outMappingSize = written;
		return true;
	}

	bool CompressWriter::CompressToMappingFromFile(SharedMemoryHandle& outMappingHandle, u64& outMappingSize, FileHandle readHandle, u64 fileSize, const tchar* hint)
	{
		u64 mappingSize = fileSize + 1024;

		SharedMemoryHandle destMapping = m_allocator.CreateHandle(m_logger, hint);
		m_allocator.ExtendMemory(destMapping, mappingSize, hint, false);
		auto destGuard = MakeGuard([&]() { m_allocator.CloseHandle(m_logger, destMapping, hint); });
		u8* compressedData = m_allocator.MapView(destMapping, hint, SharedMemoryMapType_ReadWrite);
		if (!compressedData)
			return false;
		auto unmapGuard = MakeGuard([&]() { m_allocator.UnmapView(destMapping, compressedData); });

		LinearWriterMem destination(compressedData, mappingSize);
		if (!CompressFromMemOrFile(destination, hint, readHandle, nullptr, fileSize))
			return false;

		unmapGuard.Execute();
		destGuard.Cancel();

		u64 written = destination.GetWritten();
		m_allocator.TrimMemory(destMapping, written);

		outMappingHandle = destMapping;
		outMappingSize = written;
		return true;
	}

	bool CompressWriter::CompressFromMem(LinearWriter& destination, u32 workCount, const u8* uncompressedData, u64 fileSize, u64 maxUncompressedBlock, u64& totalWritten)
	{
		#ifndef __clang_analyzer__

		struct WorkRec
		{
			WorkRec() = delete;
			WorkRec(const WorkRec&) = delete;
			WorkRec(u32 wc)
			{
				workCount = wc;

				// For some very unknown reason ASAN on linux triggers on the "new[]" call when doing delete[]
				// while doing it manually works properly. Will stop investigating this and move on
				#if PLATFORM_WINDOWS
				events = new EventSlim[workCount];
				#else
				events = (EventSlim*)aligned_alloc(alignof(EventSlim), sizeof(EventSlim)*workCount);
				for (auto i=0;i!=workCount; ++i)
					new (events + i) EventSlim();
				#endif
			}
			~WorkRec()
			{
				#if PLATFORM_WINDOWS
				delete[] events;
				#else
				for (auto i=0;i!=workCount; ++i)
					events[i].~EventSlim();
				aligned_free(events);
				#endif
			}
			Logger* logger;
			LinearWriter* destination;
			BufferCache* bufferSlots;
			Atomic<u64> refCount;
			Atomic<u64> compressCounter = 0;
			EventSlim* events;
			const u8* uncompressedData = nullptr;
			u64 written = 0;
			u64 workCount = 0;
			u64 maxUncompressedBlock = 0;
			u64 fileSize = 0;
			u8 casCompressor = 0;
			u8 casCompressionLevel = 0;
			bool error = false;
		};

		WorkRec* recPtr = new WorkRec(workCount);
		WorkRec& rec = *recPtr;
		rec.logger = &m_logger;
		rec.destination = &destination;
		rec.bufferSlots = &m_bufferSlots;
		rec.uncompressedData = uncompressedData;
		rec.maxUncompressedBlock = maxUncompressedBlock;
		rec.fileSize = fileSize;
		rec.casCompressor = m_casCompressor;
		rec.casCompressionLevel = m_casCompressionLevel;

		for (u32 i=0; i!=workCount; ++i)
			rec.events[i].Create(EventResetType_Manual);

		TimerScope cts(m_stats.compressWrite);

		auto& kernelStats = KernelStats::GetCurrent();

		auto work = [recPtr, &kernelStats](const WorkContext&)
		{
			auto& rec = *recPtr;
			KernelStatsScope kss(kernelStats);

			u8* slot = nullptr;
			u8* compressSlotBuffer = nullptr;

			auto exitGuard = MakeGuard([&]()
				{
					if (slot)
						rec.bufferSlots->Push(slot);
					if (!--rec.refCount)
						delete recPtr;
				});

			while (true)
			{
				u64 index = rec.compressCounter++;
				if (index >= rec.workCount)
					return;

				if (!compressSlotBuffer)
				{
					slot = rec.bufferSlots->Pop();
					compressSlotBuffer = slot + BufferSlotHalfSize;
				}

				u64 startOffset = rec.maxUncompressedBlock*index;
				const u8* uncompressedDataSlot = rec.uncompressedData + startOffset;
				OO_SINTa uncompressedBlockSize = (OO_SINTa)Min(rec.maxUncompressedBlock, rec.fileSize - startOffset);
				OO_SINTa compressedBlockSize;
				{
					void* scratchMem = slot;
					u64 scratchSize = BufferSlotHalfSize;
					TimerScope kts(kernelStats.memoryCompress);
					compressedBlockSize = OodleLZ_Compress((OodleLZ_Compressor)rec.casCompressor, uncompressedDataSlot, uncompressedBlockSize, compressSlotBuffer + 8, (OodleLZ_CompressionLevel)rec.casCompressionLevel, NULL, NULL, NULL, scratchMem, scratchSize);
					if (compressedBlockSize == OODLELZ_FAILED)
					{
						rec.logger->Error(TC("Failed to compress %llu bytes for %s"), u64(uncompressedBlockSize), rec.destination->GetHint());
						rec.error = true;
						return;
					}
					kernelStats.memoryCompress.bytes += compressedBlockSize;
				}
				*(u32*)compressSlotBuffer =  u32(compressedBlockSize);
				*(u32*)(compressSlotBuffer+4) =  u32(uncompressedBlockSize);

				if (index)
					rec.events[index-1].IsSet();

				u32 writeBytes = u32(compressedBlockSize) + 8;

				if (!rec.destination->Write(compressSlotBuffer, writeBytes))
					rec.error = true;

				rec.written += writeBytes;
				if (index < rec.workCount)
					rec.events[index].Set();
			}
		};

		u32 workerCount = Min(workCount, m_workManager.GetWorkerCount());
		workerCount = Min(workerCount, MaxWorkItemsPerAction2);

		rec.refCount = workerCount + 1; // We need to keep refcount up 1 to make sure it is not deleted before we read rec->written
		m_workManager.AddWork(WorkFunction(work), workerCount-1, TC("Compress")); // We are a worker ourselves
		{
			TrackWorkScope tws;
			work({tws});
		}
		rec.events[rec.workCount - 1].IsSet();

		totalWritten += rec.written;
		bool error = rec.error;

		if (!--rec.refCount)
			delete recPtr;

		if (error)
			return false;
		#endif // __clang_analyzer__
		return true;
	}

	bool CompressWriter::CompressFromMemOrFile(LinearWriter& destination, const tchar* from, FileHandle readHandle, u8* readMem, u64 fileSize)
	{
		if (!destination.Write(&fileSize, sizeof(u64))) // Store file size first in compressed file
			return false;
		u64 totalWritten = sizeof(u64);

		StorageStats& stats = m_stats;

		u64 diff = (u64)OodleLZ_GetCompressedBufferSizeNeeded((OodleLZ_Compressor)m_casCompressor, BufferSlotHalfSize) - BufferSlotHalfSize;
		u64 maxUncompressedBlock = BufferSlotHalfSize - diff - 8; // 8 bytes for the little header
		u32 workCount = u32((fileSize + maxUncompressedBlock - 1) / maxUncompressedBlock);

		u64 left = fileSize;

		if (workCount > 1)
		{
			if (!readMem)
			{
				FileMappingHandle fileMapping = FileMapping_CreateFromFile(m_logger, readHandle, PAGE_READONLY, fileSize, from);
				if (!fileMapping.IsValid())
					return m_logger.Error(TC("Failed to create file mapping for %s (%s)"), from, LastErrorToText().data);

				auto fmg = MakeGuard([&]() { FileMapping_Close(m_logger, fileMapping, from); });
				u8* uncompressedData = FileMapping_Map(m_logger, fileMapping, FILE_MAP_READ, 0, fileSize);
				if (!uncompressedData)
					return m_logger.Error(TC("Failed to map view of file mapping for %s (%s)"), from, LastErrorToText().data);

				auto udg = MakeGuard([&]()
					{
						if (m_asyncUnmapViewOfFile)
							m_workManager.AddWork([uncompressedData, fileSize, f = TString(from), loggerPtr = &m_logger](const WorkContext&) { FileMapping_Unmap(*loggerPtr, uncompressedData, fileSize, f.c_str()); }, 1, TC("UnmapFile"));
						else
							FileMapping_Unmap(m_logger, uncompressedData, fileSize, from);
					});

				if (!CompressFromMem(destination, workCount, uncompressedData, fileSize, maxUncompressedBlock, totalWritten))
					return false;
			}
			else
			{
				if (!CompressFromMem(destination, workCount, readMem, fileSize, maxUncompressedBlock, totalWritten))
					return false;
			}
		}
		else
		{
			u8* slot = m_bufferSlots.Pop();
			auto _ = MakeGuard([&](){ m_bufferSlots.Push(slot); });
			u8* uncompressedData = slot;
			u8* compressBuffer = slot + BufferSlotHalfSize;

			auto& memoryCompressTime = KernelStats::GetCurrent().memoryCompress;

			TimerScope cts(stats.compressWrite);
			while (left)
			{
				u64 uncompressedBlockSize = Min(left, maxUncompressedBlock);
				
				void* scratchMem = nullptr;
				u64 scratchSize = 0;

				if (readMem)
				{
					scratchMem = uncompressedData;
					scratchSize = BufferSlotHalfSize;
					uncompressedData = readMem + fileSize - left;
				}
				else
				{
					if (!ReadFile(m_logger, from, readHandle, uncompressedData, uncompressedBlockSize))
						return false;
					scratchMem = uncompressedData + uncompressedBlockSize;
					scratchSize = BufferSlotHalfSize - uncompressedBlockSize;
				}
				u8* destBuf = compressBuffer;
				OO_SINTa compressedBlockSize;
				{
					TimerScope kts(memoryCompressTime);
					compressedBlockSize = OodleLZ_Compress((OodleLZ_Compressor)m_casCompressor, uncompressedData, (OO_SINTa)uncompressedBlockSize, destBuf + 8, (OodleLZ_CompressionLevel)m_casCompressionLevel, NULL, NULL, NULL, scratchMem, scratchSize);
					if (compressedBlockSize == OODLELZ_FAILED)
						return m_logger.Error(TC("Failed to compress %llu bytes for %s"), uncompressedBlockSize, from);
					memoryCompressTime.bytes += compressedBlockSize;
				}

				*(u32*)destBuf =  u32(compressedBlockSize);
				*(u32*)(destBuf+4) =  u32(uncompressedBlockSize);

				u32 writeBytes = u32(compressedBlockSize) + 8;
				if (!destination.Write(destBuf, writeBytes))
					return false;

				totalWritten += writeBytes;

				left -= uncompressedBlockSize;
			}
		}

		stats.createCasBytesRaw += fileSize;
		stats.createCasBytesComp += totalWritten;
		return true;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	LinearWriterFile::LinearWriterFile(FileAccessor& f) : file(f) {}
	bool LinearWriterFile::Write(const void* data, u64 dataLen) { written += dataLen; return file.Write(data, dataLen); }
	u64 LinearWriterFile::GetWritten() { return written; }
	const tchar* LinearWriterFile::GetHint() { return file.GetFileName(); }

	////////////////////////////////////////////////////////////////////////////////////////////////////

	LinearWriterMem::LinearWriterMem(u8* d, u64 c) : pos(d), size(0), capacity(c) {}
	u64 LinearWriterMem::GetWritten() { return size; }
	bool LinearWriterMem::Write(const void* data, u64 dataLen)
	{
		u64 newSize = size + dataLen;
		if (newSize > capacity)
			return false;
		memcpy(pos, data, dataLen);
		pos += dataLen;
		size = newSize;
		return true;
	}
	const tchar* LinearWriterMem::GetHint() { return TC(""); }

	////////////////////////////////////////////////////////////////////////////////////////////////////

	static OO_SINTa DecompressBlock(const u8* src, u32 srcLen, u8* dst, u32 dstLen, bool& outCrashed)
	{
#if PLATFORM_WINDOWS && UBA_DEBUG
		g_expectingAccessViolation = true;
		__try
		{
#endif
			void* decoderMem = nullptr;
			u64 decoderMemSize = 0;
			OO_SINTa res = OodleLZ_Decompress(src, (OO_SINTa)srcLen, dst, (OO_SINTa)dstLen, OodleLZ_FuzzSafe_Yes, OodleLZ_CheckCRC_No, OodleLZ_Verbosity_None, NULL, 0, NULL, NULL, decoderMem, decoderMemSize);
#if PLATFORM_WINDOWS && UBA_DEBUG
			g_expectingAccessViolation = false;
			return res;
		}
		__except(EXCEPTION_EXECUTE_HANDLER)
		{
			g_expectingAccessViolation = false;
			outCrashed = true;
			return 0;
		}
#else
			return res;
#endif
	}

	bool DecompressWriter::DecompressMemoryToMemory(const u8* compressedData, u64 compressedSize, u8* writeData, u64 decompressedSize, const tchar* readHint, const tchar* writeHint)
	{
		UBA_ASSERTF(compressedData, TC("DecompressMemoryToMemory got readmem nullptr (%s)"), readHint);
		UBA_ASSERTF(writeData, TC("DecompressMemoryToMemory got writemem nullptr (%s)"), writeHint);

		StorageStats& stats = m_stats;

		if (decompressedSize > BufferSlotSize * 4) // Arbitrary size threshold. We want to at least catch the pch here
		{
			struct WorkRec
			{
				Logger* logger;
				const tchar* readHint;
				const tchar* writeHint;
				Atomic<u64> refCount = 2;
				const u8* readPos = nullptr;
				u8* writePos = nullptr;

				Futex lock;
				u64 decompressedSize = 0;
				u64 decompressedLeft = 0;
				u64 written = 0;
				EventSlim done;
				Atomic<bool> error = false;
			};

			WorkRec* rec = new WorkRec();
			rec->logger = &m_logger;
			rec->readHint = readHint;
			rec->writeHint = writeHint;
			rec->readPos = compressedData;
			rec->writePos = writeData;
			rec->decompressedSize = decompressedSize;
			rec->decompressedLeft = decompressedSize;
			rec->done.Create(EventResetType_Manual);

			auto work = [rec](const WorkContext&)
				{
					u64 lastWritten = 0;
					while (true)
					{
						SCOPED_FUTEX(rec->lock, lock);
						rec->written += lastWritten;
						if (!rec->decompressedLeft)
						{
							if (rec->written == rec->decompressedSize)
								rec->done.Set();
							lock.Leave();
							if (!--rec->refCount)
								delete rec;
							return;
						}
						const u8* readPos = rec->readPos;
						u8* writePos = rec->writePos;
						u32 compressedBlockSize = ((const u32*)readPos)[0];
						u32 decompressedBlockSize = ((const u32*)readPos)[1];

						if (decompressedBlockSize == 0 || decompressedBlockSize > rec->decompressedSize)
						{
							bool f = false;
							if (rec->error.compare_exchange_strong(f, true))
								rec->logger->Warning(TC("Decompressed block size %u is invalid. Decompressed file is %u (%s -> %s)"), decompressedBlockSize, rec->decompressedSize, rec->readHint, rec->writeHint);
							if (!--rec->refCount)
								delete rec;
							rec->done.Set();
							return;
						}

						readPos += sizeof(u32) * 2;
						rec->decompressedLeft -= decompressedBlockSize;
						rec->readPos = readPos + compressedBlockSize;
						rec->writePos += decompressedBlockSize;
						lock.Leave();

						bool crashed = false;
						OO_SINTa decompLen = DecompressBlock(readPos, compressedBlockSize, writePos, decompressedBlockSize, crashed);
						if (decompLen != decompressedBlockSize)
						{
							bool f = false;
							if (rec->error.compare_exchange_strong(f, true))
							{
								if (crashed)
									rec->logger->Warning(TC("Access violation decompressing %u bytes to %u bytes (corrupt compressed block) (%s -> %s)"), compressedBlockSize, decompressedBlockSize, rec->readHint, rec->writeHint);
								else
									rec->logger->Warning(TC("Expecting to be able to decompress %u bytes to %u bytes but got %llu (%s -> %s)"), compressedBlockSize, decompressedBlockSize, decompLen, rec->readHint, rec->writeHint);
							}
							if (!--rec->refCount)
								delete rec;
							rec->done.Set();
							return;
						}
						lastWritten = u64(decompLen);
					}
				};

			u32 workCount = u32(decompressedSize / BufferSlotSize) + 1;
			u32 workerCount = Min(workCount, m_workManager.GetWorkerCount() - 1); // We are a worker ourselves
			workerCount = Min(workerCount, MaxWorkItemsPerAction2); // Cap this to not starve other things
			rec->refCount += workerCount;
			m_workManager.AddWork(WorkFunction(work), workerCount, TC("DecompressMemToMem"));

			TimerScope ts(stats.decompressToMem);
			{
				TrackWorkScope tws;
				work({tws});
			}
			rec->done.IsSet();
			bool success = !rec->error;
			if (!success)
				while (rec->refCount > 1)
					Sleep(10);

			if (!--rec->refCount)
				delete rec;
			return success;
		}
		else
		{
			const u8* readPos = compressedData;
			u8* writePos = writeData;

			u64 left = decompressedSize;
			while (left)
			{
				u32 compressedBlockSize = ((const u32*)readPos)[0];
				if (!compressedBlockSize)
					break;
				u32 decompressedBlockSize = ((const u32*)readPos)[1];
				if (decompressedBlockSize == 0 || decompressedBlockSize > left)
					return m_logger.Warning(TC("Decompressed block size %u is invalid. Decompressed file is %u (%s -> %s)"), decompressedBlockSize, decompressedSize, readHint, writeHint);
				readPos += sizeof(u32) * 2;

				TimerScope ts(stats.decompressToMem);
				bool crashed = false;
				OO_SINTa decompLen = DecompressBlock(readPos, compressedBlockSize, writePos, decompressedBlockSize, crashed);
				if (crashed)
					return m_logger.Warning(TC("Access violation decompressing %u to %u bytes at pos %llu (corrupt compressed block). File compressed size: %llu Decompressed size: %llu (%s -> %s)"), compressedBlockSize, decompressedBlockSize, decompressedSize - left, compressedSize, decompressedSize, readHint, writeHint);
				if (decompLen != decompressedBlockSize)
					return m_logger.Warning(TC("Expecting to be able to decompress %u to %u bytes at pos %llu but got %llu. File compressed size: %llu Decompressed size: %llu (%s -> %s)"), compressedBlockSize, decompressedBlockSize, decompressedSize - left, decompLen, compressedSize, decompressedSize, readHint, writeHint);
				writePos += decompressedBlockSize;
				readPos += compressedBlockSize;
				left -= decompressedBlockSize;
			}
		}
		return true;
	}

	bool DecompressWriter::DecompressMemoryToFile(const u8* compressedData, FileAccessor& destination, u64 decompressedSize, bool useNoBuffering)
	{
		StorageStats& stats = m_stats;
		const u8* readPos = compressedData;

		u8* slot = m_bufferSlots.Pop();
		auto _ = MakeGuard([&](){ m_bufferSlots.Push(slot); });

		u64 left = decompressedSize;
		u64 overflow = 0;
		while (left)
		{
			u32 compressedBlockSize = ((u32*)readPos)[0];
			if (!compressedBlockSize)
				break;
			u32 decompressedBlockSize = ((u32*)readPos)[1];

			readPos += sizeof(u32)*2;

			OO_SINTa decompLen;
			{
				TimerScope ts(stats.decompressToMem);
				decompLen = OodleLZ_Decompress(readPos, (OO_SINTa)compressedBlockSize, slot + overflow, (OO_SINTa)decompressedBlockSize);
			}
			UBA_ASSERT(decompLen == decompressedBlockSize);
			
			u64 available = overflow + u64(decompLen);

			if (left - available > 0 && available < BufferSlotHalfSize)
			{
				overflow += u64(decompLen);
				readPos += compressedBlockSize;
				continue;
			}

			if (useNoBuffering)
			{
				u64 writeSize = AlignUp(available - 4096 + 1, 4096);

				if (!destination.Write(slot, writeSize))
					return false;

				overflow = available - writeSize;
				readPos += compressedBlockSize;
				left -= writeSize;

				if (overflow == left)
				{
					if (!destination.Write(slot + writeSize, 4096))
						return false;
					break;
				}

				memcpy(slot, slot + writeSize, overflow);
			}
			else
			{
				u64 writeSize = available;
				if (!destination.Write(slot, writeSize))
					return false;
				readPos += compressedBlockSize;
				left -= writeSize;
				overflow = 0;
			}
		}

		if (useNoBuffering)
			if (!SetEndOfFile(m_logger, destination.GetFileName(), destination.GetHandle(), decompressedSize))
				return false;
		return true;
	}

	bool DecompressWriter::DecompressFileToMemory(const tchar* fileName, FileHandle fileHandle, u8* dest, u64 decompressedSize, const tchar* writeHint, u64 fileStartOffset)
	{
		if (decompressedSize > BufferSlotSize*4) // Arbitrary size threshold. We want to at least catch the pch here
		{
			u64 compressedSize;
			if (!uba::GetFileSizeEx(compressedSize, fileHandle))
				return m_logger.Error(TC("GetFileSize failed for %s (%s)"), fileName, LastErrorToText().data);
			FileMappingHandle fileMapping = FileMapping_CreateFromFile(m_logger, fileHandle, PAGE_READONLY, compressedSize, fileName);
			if (!fileMapping.IsValid())
				return m_logger.Error(TC("Failed to create file mapping for %s (%s)"), fileName, LastErrorToText().data);
			auto fmg = MakeGuard([&]() { FileMapping_Close(m_logger, fileMapping, fileName); });
			u8* fileData = FileMapping_Map(m_logger, fileMapping, FILE_MAP_READ, 0, compressedSize);
			if (!fileData)
				return m_logger.Error(TC("Failed to map view of file mapping for %s (%s)"), fileName, LastErrorToText().data);
			auto udg = MakeGuard([&]() { FileMapping_Unmap(m_logger, fileData, compressedSize, fileName); });
			
			u8* readPos = fileData + 8 + fileStartOffset;
			if (!DecompressMemoryToMemory(readPos, compressedSize, dest, decompressedSize, fileName, writeHint))
				return false;
		}
		else
		{
			StorageStats& stats = m_stats;
			u8* slot = m_bufferSlots.Pop();
			auto _ = MakeGuard([&]() { m_bufferSlots.Push(slot); });

			void* decoderMem = slot + BufferSlotHalfSize;
			u64 decoderMemSize = BufferSlotHalfSize;

			u64 bytesRead = 8; // We know size has already been read

			u8* readBuffer = slot;
			u8* writePos = dest;
			u64 left = decompressedSize;
			bool sizeRead = false;
			u32 sizes[2];

			while (left)
			{
				if (!sizeRead)
				{
					if (!ReadFile(m_logger, fileName, fileHandle, sizes, sizeof(u32) * 2))
					{
						u64 compressedSize;
						if (!uba::GetFileSizeEx(compressedSize, fileHandle))
							return m_logger.Error(TC("GetFileSize failed for %s (%s)"), fileName, LastErrorToText().data);
						if (bytesRead + 8 > compressedSize)
							return m_logger.Error(TC("File %s corrupt. Tried to read 8 bytes. File is smaller than expected (Read: %llu, Size: %llu)"), fileName, bytesRead, compressedSize);
						return false;
					}
					bytesRead += 8;
				}

				u32 compressedBlockSize = sizes[0];
				u32 decompressedBlockSize = sizes[1];

				u32 toRead = compressedBlockSize;

				sizeRead = false;

				if (left > decompressedBlockSize && toRead < BufferSlotHalfSize - sizeof(u32) * 2)
				{
					toRead += sizeof(u32) * 2;
					sizeRead = true;
				}

				if (!ReadFile(m_logger, fileName, fileHandle, readBuffer, toRead))
				{
					u64 compressedSize;
					if (!uba::GetFileSizeEx(compressedSize, fileHandle))
						return m_logger.Error(TC("GetFileSize failed for %s (%s)"), fileName, LastErrorToText().data);
					if (bytesRead + toRead > compressedSize)
						return m_logger.Error(TC("File %s corrupt. Compressed block size (%u) is larger than what is left of file (%llu)"), fileName, toRead, compressedSize - bytesRead);
					return false;
				}
				bytesRead += toRead;

				if (sizeRead)
					memcpy(sizes, readBuffer + compressedBlockSize, sizeof(u32) * 2);

				TimerScope ts(stats.decompressToMem);
				OO_SINTa decompLen = OodleLZ_Decompress(readBuffer, (OO_SINTa)compressedBlockSize, writePos, (OO_SINTa)decompressedBlockSize, OodleLZ_FuzzSafe_Yes, OodleLZ_CheckCRC_No, OodleLZ_Verbosity_None, NULL, 0, NULL, NULL, decoderMem, decoderMemSize);
				if (decompLen != decompressedBlockSize)
					return m_logger.Error(TC("Failed to decompress data from file %s at pos %llu"), fileName, decompressedSize - left);
				writePos += decompressedBlockSize;
				left -= decompressedBlockSize;
			}
		}
		return true;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}