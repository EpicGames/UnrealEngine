// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaThread.h"
#include "UbaDefinitions.h"
#include "UbaEnvironment.h"
#include "UbaPlatform.h"
#include "UbaUnorderedMap.h"

#if PLATFORM_WINDOWS
#include <tlhelp32.h>
#elif PLATFORM_LINUX
#include <dirent.h>
#include <execinfo.h>
#include <sys/user.h>
#include <sys/uio.h>
#define REG_SP regs.rsp
#else
#include <mach/mach.h>
#include <mach/thread_act.h>
#endif

#define UBA_TRACK_THREADS 0

namespace uba
{
	bool AlternateThreadGroupAffinity(void* nativeThreadHandle)
	{
#if PLATFORM_WINDOWS
		int processorGroupCount = GetProcessorGroupCount();
		if (processorGroupCount <= 1)
			return true;
		static Atomic<int> processorGroupCounter;
		u16 processorGroup = u16((processorGroupCounter++) % processorGroupCount);

		u32 groupProcessorCount = ::GetActiveProcessorCount(processorGroup);

		GROUP_AFFINITY groupAffinity = {};
		groupAffinity.Mask = ~0ull >> (int)(64 - groupProcessorCount);
		groupAffinity.Group = processorGroup;
		return ::SetThreadGroupAffinity(nativeThreadHandle, &groupAffinity, NULL);
#else
		return true;
#endif
	}

#if PLATFORM_WINDOWS
	// Query CPU set IDs grouped by processor group. CPU set IDs are opaque ULONGs — round-trip,
	// never construct. Returns false if CpuSet APIs are unavailable (pre-Win10-1607).
	static bool QueryCpuSetsByGroup(Vector<Vector<ULONG>>& outIdsPerGroup)
	{
		outIdsPerGroup.clear();

		ULONG bytesNeeded = 0;
		GetSystemCpuSetInformation(nullptr, 0, &bytesNeeded, GetCurrentProcess(), 0);
		if (bytesNeeded == 0)
			return false;

		Vector<u8> buf(bytesNeeded);
		if (!GetSystemCpuSetInformation((PSYSTEM_CPU_SET_INFORMATION)buf.data(), bytesNeeded, &bytesNeeded, GetCurrentProcess(), 0))
			return false;

		int groupCount = GetProcessorGroupCount();
		outIdsPerGroup.resize(groupCount);

		for (u8* p = buf.data(), *end = p + bytesNeeded; p < end; )
		{
			auto* info = (PSYSTEM_CPU_SET_INFORMATION)p;
			if (info->Type == CpuSetInformation)
			{
				u16 g = info->CpuSet.Group;
				if (g < groupCount)
					outIdsPerGroup[g].push_back(info->CpuSet.Id);
			}
			p += info->Size; // entries are variable-length
		}
		return true;
	}

	// Cache CPU set topology — doesn't change at runtime. Empty outer vector = CpuSet APIs unavailable.
	static const Vector<Vector<ULONG>>& GetCpuSetsByGroup()
	{
		static Vector<Vector<ULONG>> cache = []() {
			Vector<Vector<ULONG>> v;
			QueryCpuSetsByGroup(v);
			return v;
		}();
		return cache;
	}

	// All CpuSet IDs flat across groups, cached. Saves DistributeProcessSoft a per-call alloc+memcpy.
	static const Vector<ULONG>& GetAllCpuSetIds()
	{
		static Vector<ULONG> cache = []() {
			Vector<ULONG> v;
			for (const auto& g : GetCpuSetsByGroup())
				v.insert(v.end(), g.begin(), g.end());
			return v;
		}();
		return cache;
	}
#endif

	bool DistributeProcessSoft(void* nativeProcessHandle, void* nativeMainThreadHandle)
	{
#if PLATFORM_WINDOWS
		int groupCount = GetProcessorGroupCount();
		if (groupCount <= 1)
			return true;

		const auto& allIds = GetAllCpuSetIds();
		if (allIds.empty())
			return false;

		HANDLE proc = (HANDLE)nativeProcessHandle;
		if (!::SetProcessDefaultCpuSets(proc, allIds.data(), (ULONG)allIds.size()))
			return false;

		if (nativeMainThreadHandle)
		{
			// Explicit wide CpuSet selection on the main thread — without this it keeps the
			// primary-group GROUP_AFFINITY from CreateProcess, which worker threads then inherit
			// and get clamped to one group regardless of process default CpuSets.
			::SetThreadSelectedCpuSets((HANDLE)nativeMainThreadHandle, allIds.data(), (ULONG)allIds.size());

			// Soft hint: round-robin the main thread's ideal processor across groups for placement.
			static Atomic<int> counter;
			int groupIdx = (counter++) % groupCount;
			PROCESSOR_NUMBER ideal = {};
			ideal.Group = (USHORT)groupIdx;
			ideal.Number = 0;
			::SetThreadIdealProcessorEx((HANDLE)nativeMainThreadHandle, &ideal, NULL);
		}
		return true;
#else
		(void)nativeProcessHandle;
		(void)nativeMainThreadHandle;
		return true;
#endif
	}

	Futex g_allThreadsLock;
	Thread* g_firstThread;

	Thread::Thread()
	{
	}

	Thread::Thread(Function<u32()>&& func, const tchar* description)
	{
		Start(std::move(func), description);
	}

	Thread::~Thread()
	{
		Wait();
	}

	void Thread::Start(Function<u32()>&& f, const tchar* description)
	{
		// Most threads use 260-300kb stack space (mostly network messages)
		// And biggest seen use 560kb (two network messages or so)
		// So let's set stack size to 1mb
		u64 stackSize = 1024*1024;

		m_func = std::move(f);
#if PLATFORM_WINDOWS
		m_handle = CreateThread(NULL, stackSize, [](LPVOID p) -> DWORD { return ((Thread*)p)->m_func(); }, this, 0, NULL);
		UBA_ASSERT(m_handle);
		if (!m_handle)
			return;
		if (description)
			SetThreadDescription(m_handle, description);
		AlternateThreadGroupAffinity(m_handle);
#else
		int err = 0;

		m_finished.Create(EventResetType_Manual);
		static_assert(sizeof(pthread_t) <= sizeof(m_handle), "");
		auto& pth = *(pthread_t*)&m_handle;

		pthread_attr_t tattr;
		err = pthread_attr_init(&tattr);

		// Have no idea if this is needed but play safe for now (intel macs are becoming very uncommon)
		#if PLATFORM_MAC && !defined(__arm__) && !defined(__arm64__)
		stackSize = 4 * 1024 * 1024;
		#endif

		err = pthread_attr_setstacksize(&tattr, stackSize);

		err = pthread_create(&pth, &tattr, [](void* p) -> void*
			{
				// Ignore sigint. Uba is designed for someone else to cancel it
				sigset_t set;
				sigemptyset(&set);
				sigaddset(&set, SIGINT);
				pthread_sigmask(SIG_BLOCK, &set, NULL);		

				#if UBA_DISABLE_PTHREAD_CANCELLATIONS
				int old;
				pthread_setcancelstate(PTHREAD_CANCEL_DISABLE, &old);
				#endif

				auto& t = *(Thread*)p;
				int res = t.m_func();
				t.m_finished.Set();
				
				#if UBA_DISABLE_PTHREAD_CANCELLATIONS
				pthread_setcancelstate(old, NULL);
				#endif

				return (void*)(uintptr_t)res;
			}, this);
		UBA_ASSERT(err == 0); (void)err;
		if (!description)
			description = "UbaUnknown";
		#if PLATFORM_MAC
		pthread_setname_np(description);
		#else
		pthread_setname_np(pth, description);
		#endif

		err = pthread_attr_destroy(&tattr);
		UBA_ASSERT(err == 0); (void)err;
#endif

		#if UBA_TRACK_THREADS
		SCOPED_FUTEX(g_allThreadsLock, lock);
		printf("Adding THREAD %llx\n", (u64)*(pthread_t*)&m_handle);
		m_next = g_firstThread;
		if (m_next)
			m_next->m_prev = this;
		g_firstThread = this;
		#endif
	}

	bool Thread::Wait(u32 milliseconds, Event* wakeupEvent)
	{
		SCOPED_READ_LOCK(m_funcLock, readLock);
		if (!m_handle)
			return true;

		auto removeThread = [this]()
			{
				#if UBA_TRACK_THREADS
				SCOPED_FUTEX(g_allThreadsLock, lock);
				printf("REMOVING THREAD %llx\n", (u64)*(pthread_t*)&m_handle);
				if (m_next)
					m_next->m_prev = m_prev;
				if (m_prev)
					m_prev->m_next = m_next;
				else if (g_firstThread == this)
					g_firstThread = m_next;
				m_next = nullptr;
				m_prev = nullptr;
				#endif
			};

#if PLATFORM_WINDOWS // Optimization, not needed in initial implementation
		if (wakeupEvent && EventIsNative)
		{
			HANDLE h[] = { m_handle, wakeupEvent->GetHandle() };
			DWORD res = WaitForMultipleObjects(2, h, false, milliseconds);
			if (res == WAIT_OBJECT_0 + 1 || res == WAIT_TIMEOUT)
				return false;
		}
		else
		{
			UBA_ASSERTF(GetCurrentThreadId() != GetThreadId(m_handle), TC("Thread is trying to wait on itself. This will end up in infinite hang"));

			if (WaitForSingleObject(m_handle, milliseconds) == WAIT_TIMEOUT)
				return false;
		}
		removeThread();
#else
		if (!m_finished.IsSet(milliseconds))
			return false;
		removeThread();
		int* ptr = 0;
		int res = pthread_join(*(pthread_t*)&m_handle, (void**)&ptr);
		UBA_ASSERT(res == 0);
#endif

		readLock.Leave();

		SCOPED_WRITE_LOCK(m_funcLock, lock);
		if (!m_handle)
			return true;

		#if PLATFORM_WINDOWS
		CloseHandle(m_handle);
		#endif

		m_func = {};
		m_handle = nullptr;
		return true;
	}

	bool Thread::IsInside() const
	{
#if PLATFORM_WINDOWS
		return GetCurrentThreadId() == GetThreadId(m_handle);
#else
		return pthread_self() == *(pthread_t*)&m_handle;
#endif
	}

	bool TraverseAllThreads(const TraverseThreadFunc& func, const TraverseThreadErrorFunc& errorFunc)
	{
#if PLATFORM_WINDOWS
		StringBuffer<256> error;
		auto reportError = [&](HANDLE hThread, const tchar* call) { errorFunc(error.Clear().Appendf(TC("%s failed for thread %llu (%s)"), call, u64(hThread), LastErrorToText().data)); };

#if UBA_TRACK_THREADS
		DWORD currentThreadId = GetCurrentThreadId();
		SCOPED_FUTEX(g_allThreadsLock, lock);
		for (Thread* t=g_firstThread; t; t=t->m_next)
		{
			HANDLE hThread = t->m_handle;
			if (currentThreadId == GetThreadId(hThread))
				continue;
			if (SuspendThread(hThread) == -1)
			{
				reportError(hThread, TC("SuspendThread"));
				continue;
			}
			auto rtg = MakeGuard([&]() { ResumeThread(hThread); });
			CONTEXT ctx;
			memset(&ctx, 0, sizeof(CONTEXT));
			ctx.ContextFlags = CONTEXT_FULL;
			if (!GetThreadContext(hThread, &ctx))
			{
				reportError(hThread, TC("GetThreadContext"));
				continue;
			}
			func(0, &ctx);
		}
#else
		DWORD pid = GetCurrentProcessId();
		DWORD tid = GetCurrentThreadId();
		HANDLE hSnapshot = CreateToolhelp32Snapshot(TH32CS_SNAPTHREAD, 0);
		if (hSnapshot == INVALID_HANDLE_VALUE)
			return false;
		auto sg = MakeGuard([&]() { CloseHandle(hSnapshot); });
		UnorderedSet<CasKey> handledCallstacks;
		THREADENTRY32 te32 = { sizeof(THREADENTRY32) };
		if (!Thread32First(hSnapshot, &te32))
			return false;
		do
		{
			if (te32.th32OwnerProcessID != pid || te32.th32ThreadID == tid || te32.th32ThreadID == 0)
				continue;
			HANDLE hThread = OpenThread(THREAD_GET_CONTEXT | THREAD_SET_CONTEXT | THREAD_SUSPEND_RESUME | THREAD_QUERY_INFORMATION, FALSE, te32.th32ThreadID);
			if (!hThread)
			{
				reportError(hThread, TC("OpenThread"));
				continue;
			}
			auto tg = MakeGuard([&]() { CloseHandle(hThread); });
			if (SuspendThread(hThread) == -1)
			{
				reportError(hThread, TC("SuspendThread"));
				continue;
			}
			auto rtg = MakeGuard([&]() { ResumeThread(hThread); });
			PWSTR threadDesc = nullptr;
			GetThreadDescription(hThread, &threadDesc);
			auto tdg = MakeGuard([&]() { LocalFree(threadDesc); });
			CONTEXT ctx = {};
			ctx.ContextFlags = CONTEXT_ALL;
			if (!GetThreadContext(hThread, &ctx))
			{
				reportError(hThread, TC("GetThreadContext"));
				continue;
			}
			void* callstack[100];
			u32 callstackCount = GetCallstack(callstack, 100, 1, &ctx);
			func(te32.th32ThreadID, callstack, callstackCount, threadDesc);
		}
		while (Thread32Next(hSnapshot, &te32));
#endif
		return true;
#elif PLATFORM_LINUX

		// TODO: None of these approaches work.
		// signal path will break if thread is in certain system calls.. and ptrace does not work either :-/

		static Event s_ev(EventResetType_Auto);
		static const TraverseThreadFunc* s_func;
		s_func = &func;

		struct sigaction sa;
		memset(&sa, 0, sizeof(sa));
		sa.sa_flags = SA_SIGINFO;
		sa.sa_sigaction = [](int sig, siginfo_t* info, void* context)
			{
				void* callstack[100];
				u32 callstackCount = GetCallstack(callstack, 100, 1, context);
				(*s_func)(syscall(SYS_gettid), callstack, callstackCount, nullptr);
				s_ev.Set();
			};
		sigaction(SIGUSR1, &sa, NULL);

		DIR* dir = opendir("/proc/self/task");
		if (!dir)
			return false;

		pid_t currentTid = syscall(SYS_gettid);
		struct dirent *entry;
		while ((entry = readdir(dir)) != NULL)
		{
			pid_t tid = (pid_t)atoi(entry->d_name);
			if (tid <= 0 || tid == currentTid)
				continue;
			kill(tid, SIGUSR1);
			s_ev.IsSet();
		}

		#if 0
		pid_t parentPid = getpid();

		char message[100] = "Original message in parent.";
 
		// We need to fork to be able to use ptrace on threads
		pid_t child = fork();
		if (child < 0)
		{
			perror("fork");
			return false;
		}
		
		if (child == 0) // The child
		{
			StringBuffer<128>().Appendf("/proc/%u/task", parentPid);

			auto g = MakeGuard([] { exit(EXIT_SUCCESS); });
			DIR* dir = opendir(StringBuffer<128>().Appendf("/proc/%u/task", parentPid).data);
			if (!dir)
				return false;

			struct dirent *entry;
			while ((entry = readdir(dir)) != NULL)
			{
				pid_t tid = (pid_t)atoi(entry->d_name);
				if (tid <= 0)
					continue;

				if (ptrace(PTRACE_ATTACH, tid, NULL, NULL) == -1)
				{
					perror("ptrace attach");
					continue;
				}

				int status;
				waitpid(tid, &status, 0); // Wait until the thread stops.


				struct user_regs_struct regs;
				if (ptrace(PTRACE_GETREGS, tid, NULL, &regs) == -1)
				{
					perror("ptrace getregs");
					return false;
				}
				
				printf("Callstack for thread %d:\n", tid);
				printf("RIP: %llx\n", regs.rip);

				// Start from the current base pointer.
				unsigned long long fp = regs.rbp;
				int frame = 0;

				#define MAX_FRAMES 64
				while (fp && frame < MAX_FRAMES) {
					// Read the saved frame pointer (first word) and return address (second word)
					unsigned long long next_fp, ret_addr;
					errno = 0;
					next_fp = ptrace(PTRACE_PEEKDATA, tid, (void *)fp, NULL);
					if (errno != 0) break;
					ret_addr = ptrace(PTRACE_PEEKDATA, tid, (void *)(fp + sizeof(unsigned long long)), NULL);
					if (errno != 0) break;
					printf("  Frame %d: ret_addr = %llx (fp = %llx)\n", frame, ret_addr, fp);
					fp = next_fp;
					frame++;
				}
				printf("\n");








				if (ptrace(PTRACE_DETACH, tid, NULL, NULL) == -1) {
					perror("ptrace detach");
					return false;
				}

				const char* new_msg = "Hello from child!";

				struct iovec local[1];
				local[0].iov_base = (void *)new_msg;
				local[0].iov_len  = strlen(new_msg) + 1;  // Include the null terminator

				// Set up the remote iovec.
				// The parent's view of the address is the same as the child's copy after fork.
				struct iovec remote[1];
				remote[0].iov_base = message;
				remote[0].iov_len  = strlen(new_msg) + 1;

				// Use process_vm_writev to write the new message into the parent's memory.
				ssize_t nwritten = process_vm_writev(getppid(), local, 1, remote, 1, 0);
			}

			closedir(dir);

		}
		else // The parent
		{
			while (message[0] == 'O')
			{
				Sleep(500);
			}
		}
		#endif

#else
		task_t task;
		kern_return_t kr = task_for_pid(mach_task_self(), getpid(), &task);
		if (kr != KERN_SUCCESS)
			return false;

		thread_act_array_t threads;
		mach_msg_type_number_t thread_count;
		kr = task_threads(task, &threads, &thread_count);
		if (kr != KERN_SUCCESS)
			return false;
		auto tsg = MakeGuard([&] { vm_deallocate(mach_task_self(), (vm_address_t)threads, thread_count * sizeof(thread_t)); });

		for (int threadIt = 0; threadIt < thread_count; threadIt++)
		{
			auto thread = threads[threadIt];

			if (thread == mach_thread_self())
				continue;

			thread_suspend(thread);
			auto g = MakeGuard([&] { thread_resume(thread); });

			uint64_t pc = 0, fp = 0;

			#if defined(__x86_64__)
				x86_thread_state64_t state;
				mach_msg_type_number_t count = x86_THREAD_STATE64_COUNT;
				kr = thread_get_state(thread, x86_THREAD_STATE64, (thread_state_t)&state, &count);
				if (kr != KERN_SUCCESS)
					return false;
				pc = state.__rip;
				fp = state.__rbp;
			#elif defined(__arm64__)
				arm_thread_state64_t state;
				mach_msg_type_number_t count = ARM_THREAD_STATE64_COUNT;
				kr = thread_get_state(thread, ARM_THREAD_STATE64, (thread_state_t)&state, &count);
				if (kr != KERN_SUCCESS)
					return false;
				pc = state.__pc;
				fp = state.__fp;
			#else
				#error "Unsupported architecture"
			#endif
		
			void* callstack[100];
			u32 callstackCount = 0;
			for (int i = 0; i < 32 && fp; i++) {
				uint64_t *stack = (uint64_t *)fp;
				uint64_t return_addr = stack[1];
				fp = stack[0];
				if (!fp)
					break;
				if (i > 0)
					callstack[callstackCount++] = (void*)return_addr;
			}
			g.Execute();

			func(thread, callstack, callstackCount, nullptr);
		}
#endif
#if !PLATFORM_WINDOWS
		return true;
#endif
	}
}
