// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaTrace.h"
#include "UbaFileAccessor.h"
#include "UbaProcessHandle.h"
#include "UbaStats.h"

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////
	//
	// Lock-free trace writer
	//
	// Writes proceed without serializing on a global mutex. Each entry precomputes its
	// exact on-wire size, atomically reserves a slot in the trace ring (m_memoryReserved),
	// fills the slot, then releases an in-flight counter (m_inflightWriters). When the
	// counter drops to zero, the dropper publishes the new size header at offset 0.
	//
	// The size header MUST stay accurate for the visualizer: every byte from 4 to header
	// must be a fully-written entry. Two pieces ensure that:
	//
	//   * Successful writers never publish a value that exceeds m_firstAbortPos. Aborted
	//     writers (capacity overshoot or commit failure) only mark their position as a
	//     hard ceiling — they never publish.
	//
	//   * Each writer snapshots m_memoryReserved BEFORE its inflight decrement. If the
	//     decrement returns prev==1 (we were the last in-flight), every reservation in
	//     [0, snapshot) belongs to a writer that has already completed by the time we
	//     publish, so all those bytes are filled.
	//
	// Backstop publishing: under steady write pressure the inflight count never hits
	// zero on a writer's exit, so the header would lag. SessionServer::TraceSessionUpdate
	// calls PublishIfQuiet() every tick to advance the header during quiet windows.
	//
	////////////////////////////////////////////////////////////////////////////////////////////////////

	Trace::Trace(LogWriter& logWriter)
	:	m_logger(logWriter)
	,	m_channel(m_logger)
	{
	}

	Trace::~Trace()
	{
		FreeMemory();
	}

	struct Trace::TraceWriter : BinaryWriter
	{
		TraceWriter(Trace& trace, u64 size) : BinaryWriter(nullptr, 0, 0), m_trace(trace), m_size(size)
		{
			if (!trace.m_memoryBegin)
				return;

			// Increment in-flight first, then reserve. If anyone observes our reservation,
			// they also observe our presence in inflight.
			trace.m_inflightWriters.fetch_add(1, std::memory_order_acq_rel);

			u64 myStart = trace.m_memoryReserved.fetch_add(size, std::memory_order_acq_rel);
			u64 myEnd = myStart + size;

			if (myEnd > trace.m_memoryCapacity)
			{
				m_myStart = myStart;
				m_aborted = true;
				return;
			}

			// Fast path: already committed.
			if (myEnd > trace.m_memoryCommitted.load(std::memory_order_acquire))
			{
				SCOPED_FUTEX(trace.m_memoryLock, lock);
				if (myEnd > trace.m_memoryCommitted.load(std::memory_order_relaxed))
				{
					if (!trace.CommitTo(myEnd))
					{
						m_myStart = myStart;
						m_aborted = true;
						return;
					}
				}
			}

			Reset(trace.m_memoryBegin, myStart, myEnd);
			m_myStart = myStart;
			m_myEnd = myEnd;
			m_isValid = true;
		}

		~TraceWriter()
		{
			if (m_aborted)
			{
				// Mark our position as the hard ceiling for any future publish.
				AtomicMin(m_trace.m_firstAbortPos, m_myStart);
				m_trace.m_inflightWriters.fetch_sub(1, std::memory_order_acq_rel);
				return;
			}

			if (!m_isValid)
				return; // Ctor never incremented inflight (m_memoryBegin was null).

			UBA_ASSERTF(GetPosition() == m_myEnd, TC("Trace entry size mismatch: precomputed end %llu, actual position %llu"), m_myEnd, GetPosition());

			// Snapshot reserved before releasing inflight. If prev==1, no one else was
			// in flight, so every slot in [0, snapshot) has been completed.
			u64 reservedSnap = m_trace.m_memoryReserved.load(std::memory_order_acquire);

			u32 prev = m_trace.m_inflightWriters.fetch_sub(1, std::memory_order_acq_rel);
			if (prev == 1)
			{
				u64 firstAbort = m_trace.m_firstAbortPos.load(std::memory_order_acquire);
				u64 desired = reservedSnap < firstAbort ? reservedSnap : firstAbort;
				m_trace.PublishHeaderTo(desired);
			}
		}

		bool IsValid() const { return m_isValid; }

		TraceWriter(const TraceWriter&) = delete;
		void operator=(const TraceWriter&) = delete;

		Trace& m_trace;
		u64 m_size = 0;
		u64 m_myStart = 0;
		u64 m_myEnd = 0;
		bool m_isValid = false;
		bool m_aborted = false;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool Trace::StartWriteAndThread(const tchar* namedTrace, u64 traceMemCapacity, bool logStart, const tchar* channelName)
	{
		if (logStart)
		{
			if (namedTrace)
				m_logger.Info(TC("---- Starting trace: %s ----"), namedTrace);
			else
				m_logger.Info(TC("---- Starting trace ----"));
		}

		if (!StartWrite(namedTrace, traceMemCapacity, channelName))
			return false;

		StringBuffer<256> buf;
		if (!GetComputerNameW(buf))
			buf.Append(TCV("LOCAL"));
		SessionAdded(0, {}, buf, {});

		StringBuffer<> systemInfo;
		GetSystemInfo(m_logger, systemInfo);
		SessionInfo(0, systemInfo);

		StartThread();
		return true;
	}

	void Trace::StartThread()
	{
		m_threadEvent.Create(EventResetType_Manual);
		m_thread.Start([this]() { ThreadLoop(); return 0; }, TC("UbaTraceLoop"));
	}

	void Trace::StopThread()
	{
		m_threadEvent.Set();
		m_thread.Wait();
	}

	void Trace::SetThreadUpdateCallback(Function<void()>&& callback, u32 traceIntervalMs)
	{
		SCOPED_FUTEX(m_threadCallbackLock, lock);
		m_threadCallback = callback;
		m_threadIntervalMs = traceIntervalMs;
	}

	bool Trace::StartWrite(const tchar* namedTrace, u64 traceMemCapacity, const tchar* channelName)
	{
		m_memoryCapacity = traceMemCapacity;
		const tchar* mappingName = namedTrace;
		#if !PLATFORM_WINDOWS
		mappingName = nullptr;
		#endif
		m_memoryHandle = FileMapping_Create(m_logger, PAGE_READWRITE|SEC_RESERVE, m_memoryCapacity, mappingName, TC("Trace"));
		if (!m_memoryHandle.IsValid())
			return false;
		if (GetLastError() != ERROR_ALREADY_EXISTS)
		{
			m_memoryBegin = FileMapping_Map(m_logger, m_memoryHandle, FILE_MAP_WRITE, 0, m_memoryCapacity);
		}

		if (!m_memoryBegin)
		{
			m_logger.Warning(TC("Failed to map view of trace mapping '%s' (%s)"), namedTrace, LastErrorToText().data);
			FileMapping_Close(m_logger, m_memoryHandle, TC("Trace"));
			m_memoryHandle = {};
			return false;
		}

		m_memoryReserved.store(0, std::memory_order_relaxed);
		m_memoryCommitted.store(0, std::memory_order_relaxed);
		m_firstAbortPos.store(~u64(0), std::memory_order_relaxed);
		m_inflightWriters.store(0, std::memory_order_relaxed);
		m_startTime = GetTime();
		u64 systemStartTimeUs = GetSystemTimeUs();

		// Initial entry: 4 bytes reserved for the size header at offset 0, followed by
		// version, pid, system start, frequency, start time. All fields are precomputable.
		u64 initialSize = 4 + 4 + 4 + Get7BitEncodedCount(systemStartTimeUs) + Get7BitEncodedCount(GetFrequency()) + Get7BitEncodedCount(m_startTime);
		{
			TraceWriter writer(*this, initialSize);
			if (!writer.IsValid())
				return false;
			writer.AllocWrite(4); // Skip the size-header bytes; PublishHeaderTo writes them.
			writer.WriteU32(TraceVersion);
			writer.WriteU32(GetCurrentProcessId());
			writer.Write7BitEncoded(systemStartTimeUs);
			writer.Write7BitEncoded(GetFrequency());
			writer.Write7BitEncoded(m_startTime);
		}


		if (namedTrace)
		{
			#if !PLATFORM_WINDOWS
			StringBuffer<64> uidName;
			GetMappingHandleName(uidName, m_memoryHandle.internal);
			namedTrace = uidName.data;
			#endif

			m_namedTrace = namedTrace;
			if (channelName && *channelName && m_channel.Init(channelName))
				m_channel.Write(namedTrace);
		}
		return true;
	}

	bool Trace::Write(const tchar* writeFileName, bool writeSummary)
	{
		if (!m_memoryBegin)
			return true;

		if (!writeFileName || !*writeFileName)
			return true;

		FileAccessor traceFile(m_logger, writeFileName);
		if (!traceFile.CreateWrite(false, DefaultAttributes(), 0, nullptr))
			return false;

		u64 fileSize = GetPublishedSize();
		if (!traceFile.Write(m_memoryBegin, fileSize))
			return false;

		if (writeSummary)
		{
			StackBinaryWriter<32> summaryWriter;
			summaryWriter.WriteByte(TraceType_Summary);
			summaryWriter.Write7BitEncoded(GetTime() - m_startTime);
			if (!traceFile.Write(summaryWriter.GetData(), summaryWriter.GetPosition()))
				return false;
			fileSize += summaryWriter.GetPosition();
		}

		if (!traceFile.Close())
			return false;
		m_logger.Info(TC("Trace written to file %s with size %s"), writeFileName, BytesToText(fileSize).str);
		return true;
	}

	bool Trace::StopWrite(const tchar* writeFileName)
	{
		if (!m_memoryBegin)
			return true;
		auto g = MakeGuard([this]() { FreeMemory(); });

		if (m_channel.IsInitialized())
			m_channel.Write(TC(""), m_namedTrace.c_str());

		{
			u64 t = GetTime() - m_startTime;
			u64 size = 1 + Get7BitEncodedCount(t);
			TraceWriter writer(*this, size);
			if (!writer.IsValid())
				return false;
			writer.WriteByte(TraceType_Summary);
			writer.Write7BitEncoded(t);
		}

		return Write(writeFileName, false); // Don't write summary, already included in memory
	}

	u32 Trace::TrackWorkStart(const StringView& desc, const Color& color)
	{
		u32 workId = m_workCounter++;
		WorkBegin(workId, desc, color);
		return workId;
	}

	void Trace::TrackWorkHint(u32 id, const StringView& hint, u64 startTime)
	{
		WorkHint(id, hint, startTime);
	}

	void Trace::TrackWorkEnd(u32 id)
	{
		WorkEnd(id);
	}

	void Trace::WriteSessionSummary(const Function<void(Logger& logger)>& summaryFunc)
	{
		StackBinaryWriter<256*1024> writer;
		WriteSummaryText(writer, summaryFunc);
		SessionSummary(0, writer.GetData(), writer.GetPosition());
	}

	void Trace::WriteSummaryText(BinaryWriter& writer, const Function<void(Logger& logger)>& summaryFunc)
	{
		struct SummaryLogWriter : public LogWriter
		{
			SummaryLogWriter(BinaryWriter& w) : writer(w) {}
			virtual void BeginScope() override {}
			virtual void EndScope() override {}
			virtual void Log(LogEntryType type, const tchar* str, u32 strLen, const tchar* prefix, u32 prefixLen) override
			{
				writer.WriteString(str, strLen);
				++count;
			}

			BinaryWriter& writer;
			u32 count = 0;
		};
		u32* lineCount = (u32*)writer.AllocWrite(4);

		SummaryLogWriter logWriter(writer);
		LoggerWithWriter logger(logWriter, TC(""));

		summaryFunc(logger);

		*lineCount = logWriter.count;
	}

	float Trace::UpdateCpuLoad()
	{
		u64 totalTime = 0;
		u64 idleTime = 0;
		if (!GetCpuTime(totalTime, idleTime))
			return m_cpuLoad;

		u64 totalTimeSinceLastTime = totalTime - m_previousTotalCpuTime;
		u64 idleTimeSinceLastTime = idleTime - m_previousIdleCpuTime;

		float cpuLoad = 1.0f - ((totalTimeSinceLastTime > 0) ? (float(idleTimeSinceLastTime) / float(totalTimeSinceLastTime)) : 0);

		m_previousTotalCpuTime = totalTime;
		m_previousIdleCpuTime = idleTime;

		// TODO: This is the wrong solution.. but can't repro the bad values some people get
		if (cpuLoad >= 0 && cpuLoad <= 1.0f)
			m_cpuLoad = cpuLoad;

		return m_cpuLoad;
	}

	void Trace::ThreadLoop()
	{
		while (true)
		{
			SCOPED_FUTEX_READ(m_threadCallbackLock, lock);
			if (m_threadCallback)
				m_threadCallback();
			else
			{
				float cpuLoad = UpdateCpuLoad();
				u64 memAvail;
				u64 memTotal;
				GetMemoryInfo(m_logger, memAvail, memTotal);
				SessionUpdate(0, 0, 0, 0, 0, memAvail, memTotal, cpuLoad);
			}
			u32 threadIntervalMs = m_threadIntervalMs;
			lock.Leave();


			if (m_threadEvent.IsSet(threadIntervalMs))
				break;
		}
	}

	void Trace::FreeMemory()
	{
		if (m_memoryBegin)
		{
			FileMapping_Unmap(m_logger, m_memoryBegin, m_memoryCapacity, TC("Trace"));
			m_memoryBegin = nullptr;
		}
		if (m_memoryHandle.IsValid())
		{
			FileMapping_Close(m_logger, m_memoryHandle, TC("Trace"));
			m_memoryHandle = {};
		}
	}

	bool Trace::CommitTo(u64 endPosition)
	{
		// Caller holds m_memoryLock.
		if (!m_memoryBegin)
			return false;

		u64 currentCommitted = m_memoryCommitted.load(std::memory_order_relaxed);
		u64 needed = AlignUp(endPosition, 64*1024);
		if (currentCommitted >= needed)
			return true;

		void* address = m_memoryBegin + currentCommitted;
		u64 toCommit = needed - currentCommitted;
		if (!FileMapping_Commit(m_logger, address, toCommit))
		{
			m_logger.Warning(TC("Failed to commit memory for trace (End: %llu Capacity: %llu, Already Committed: %llu, Needed: %llu): %s"), endPosition, m_memoryCapacity, currentCommitted, needed, LastErrorToText().data);
			return false;
		}
		LockMemory(m_logger, address, toCommit, TC("Trace"));
		m_memoryCommitted.store(needed, std::memory_order_release);
		return true;
	}

	void Trace::PublishHeaderTo(u64 desired)
	{
		if (!m_memoryBegin)
			return;
		std::atomic_ref<u32> header(*(u32*)m_memoryBegin);
		u32 want = u32(desired);
		u32 cur = header.load(std::memory_order_relaxed);
		while (cur < want)
		{
			if (header.compare_exchange_weak(cur, want, std::memory_order_release, std::memory_order_relaxed))
				break;
		}
	}

	void Trace::PublishIfQuiet()
	{
		if (!m_memoryBegin)
			return;

		// Snapshot reserved BEFORE checking inflight. If a writer arrives between the
		// snapshot and the check, it'll show up in inflight and we'll bail. If a writer
		// completes between the snapshot and the check, it'll have published already and
		// our CAS-max in PublishHeaderTo can't regress.
		u64 reservedSnap = m_memoryReserved.load(std::memory_order_acquire);

		if (m_inflightWriters.load(std::memory_order_acquire) != 0)
			return;

		u64 firstAbort = m_firstAbortPos.load(std::memory_order_acquire);
		u64 desired = reservedSnap < firstAbort ? reservedSnap : firstAbort;
		PublishHeaderTo(desired);
	}

	u32 Trace::GetPublishedSize()
	{
		if (!m_memoryBegin)
			return 0;
		PublishIfQuiet();
		return std::atomic_ref<u32>(*(u32*)m_memoryBegin).load(std::memory_order_acquire);
	}

	u32 Trace::AddString(StringView string)
	{
		if (!m_memoryBegin)
			return 0;

		SCOPED_FUTEX(m_stringsLock, lock);
		auto insres = m_strings.try_emplace(ToStringKeyNoCheck(string.data, string.count));
		if (insres.second)
		{
			insres.first->second = u32(m_strings.size() - 1);
			u64 size = 1 + GetStringWriteSize(string.data, string.count);
			TraceWriter writer(*this, size);
			if (!writer.IsValid())
				return 0;
			writer.WriteByte(TraceType_String);
			writer.WriteString(string);
		}
		return insres.first->second;
	}

	#define BEGIN_TRACE_ENTRY(x, payloadSize) \
		BEGIN_TRACE_ENTRY_IMPL(x, payloadSize, true)

	#define BEGIN_TRACE_ENTRY_NOTIME(x, payloadSize) \
		BEGIN_TRACE_ENTRY_IMPL(x, payloadSize, false)

	#define BEGIN_TRACE_ENTRY_IMPL(x, payloadSize, writeTime) \
		if (!m_memoryBegin) \
			return; \
		u64 _entryTime = writeTime ? (GetTime() - m_startTime) : 0; \
		u64 _entrySize = 1 + (writeTime ? Get7BitEncodedCount(_entryTime) : 0) + (payloadSize); \
		TraceWriter writer(*this, _entrySize); \
		if (!writer.IsValid()) \
			return; \
		writer.WriteByte(TraceType_##x); \
		if (writeTime) \
			writer.Write7BitEncoded(_entryTime);

	void Trace::SessionAdded(u32 sessionId, u32 clientId, StringView name, StringView machineId)
	{
		u64 payload = GetStringWriteSize(name.data, name.count) + GetStringWriteSize(machineId.data, machineId.count) + Get7BitEncodedCount(clientId) + 4;
		BEGIN_TRACE_ENTRY(SessionAdded, payload);
		writer.WriteString(name);
		writer.WriteString(machineId);
		writer.Write7BitEncoded(clientId);
		writer.WriteU32(sessionId);
	}

	void Trace::SessionInfo(u32 sessionId, StringView info)
	{
		u64 payload = 4 + GetStringWriteSize(info.data, info.count);
		BEGIN_TRACE_ENTRY(SessionInfo, payload);
		writer.WriteU32(sessionId);
		writer.WriteString(info);
	}

	void Trace::SessionUpdate(u32 sessionId, u32 connectionCount, u64 send, u64 recv, u64 lastPing, u64 memAvail, u64 memTotal, float cpuLoad)
	{
		u64 payload = Get7BitEncodedCount(sessionId) + Get7BitEncodedCount(connectionCount) + Get7BitEncodedCount(send) + Get7BitEncodedCount(recv) + Get7BitEncodedCount(lastPing) + Get7BitEncodedCount(memAvail) + Get7BitEncodedCount(memTotal) + 4;
		BEGIN_TRACE_ENTRY(SessionUpdate, payload);
		writer.Write7BitEncoded(sessionId);
		writer.Write7BitEncoded(connectionCount);
		writer.Write7BitEncoded(send);
		writer.Write7BitEncoded(recv);
		writer.Write7BitEncoded(lastPing);
		writer.Write7BitEncoded(memAvail);
		writer.Write7BitEncoded(memTotal);
		writer.WriteU32(*(u32*)&cpuLoad);
	}

	void Trace::SessionNotification(u32 sessionId, const tchar* text)
	{
		u64 textLen = TStrlen(text);
		u64 payload = 4 + GetStringWriteSize(text, textLen);
		BEGIN_TRACE_ENTRY(SessionNotification, payload);
		writer.WriteU32(sessionId);
		writer.WriteString(text, textLen);
	}

	void Trace::SessionSummary(u32 sessionId, const u8* data, u64 dataSize)
	{
		u64 payload = 4 + dataSize;
		BEGIN_TRACE_ENTRY(SessionSummary, payload);
		writer.WriteU32(sessionId);
		writer.WriteBytes(data, dataSize);
	}

	void Trace::SessionDisconnect(u32 sessionId)
	{
		BEGIN_TRACE_ENTRY(SessionDisconnect, 4);
		writer.WriteU32(sessionId);
	}

	void Trace::ProcessAdded(u32 sessionId, u32 processId, StringView description, StringView breadcrumbs)
	{
		BinaryWriter::PreparedLongString prepared(breadcrumbs, 20);
		u64 payload = 4 + 4 + GetStringWriteSize(description.data, description.count) + prepared.GetWriteSize();
		BEGIN_TRACE_ENTRY(ProcessAdded, payload);
		writer.WriteU32(sessionId);
		writer.WriteU32(processId);
		writer.WriteString(description);
		writer.WriteLongString(prepared);
	}

	void Trace::ProcessEnvironmentUpdated(u32 processId, StringView reason, u32 prevExitCode, const u8* prevStats, u64 prevStatsSize, StringView breadcrumbs)
	{
		BinaryWriter::PreparedLongString prepared(breadcrumbs, 20);
		u64 payload = 4 + GetStringWriteSize(reason.data, reason.count) + prevStatsSize + prepared.GetWriteSize();
		BEGIN_TRACE_ENTRY(ProcessEnvironmentUpdated, payload);
		writer.WriteU32(processId);
		writer.WriteString(reason);
		//writer.WriteU32(prevExitCode); // TODO: Add this on next trace version update
		writer.WriteBytes(prevStats, prevStatsSize);
		writer.WriteLongString(prepared);
	}

	void Trace::ProcessExited(u32 processId, u32 exitCode, const u8* data, u64 dataSize, const Vector<ProcessLogLine>& logLines)
	{
		// Cap loglines to 100 (matches the previous behavior). Precompute size to fit
		// exactly that many lines plus the terminator byte.
		u64 logSize = 0;
		u32 lineCount = 0;
		for (auto& line : logLines)
		{
			if (lineCount >= 100)
				break;
			++lineCount;
			logSize += 1 + GetStringWriteSize(line.text.data(), line.text.size());
		}
		u64 payload = 4 + 4 + dataSize + logSize + 1; // trailing 255 terminator
		BEGIN_TRACE_ENTRY(ProcessExited, payload);
		writer.WriteU32(processId);
		writer.WriteU32(exitCode);
		writer.WriteBytes(data, dataSize);

		u32 emitted = 0;
		for (auto& line : logLines)
		{
			if (emitted >= lineCount)
				break;
			++emitted;
			writer.WriteByte(line.type);
			writer.WriteString(line.text);
		}
		writer.WriteByte(255);
	}

	void Trace::ProcessExited(u32 processId, u32 exitCode)
	{
		StackBinaryWriter<1024> writer;
		ProcessStats().Write(writer);
		SessionStats().Write(writer);
		StorageStats().Write(writer);
		KernelStats().Write(writer);
		ProcessExited(processId, exitCode, writer.GetData(), writer.GetPosition(), {});
	}

	void Trace::ProcessReturned(u32 processId, StringView reason)
	{
		u64 payload = 4 + GetStringWriteSize(reason.data, reason.count);
		BEGIN_TRACE_ENTRY(ProcessReturned, payload);
		writer.WriteU32(processId);
		writer.WriteString(reason);
	}

	void Trace::ProcessAddBreadcrumbs(u32 processId, StringView breadcrumbs, bool deleteOld)
	{
		BinaryWriter::PreparedLongString prepared(breadcrumbs, 20);
		u64 payload = 4 + prepared.GetWriteSize() + 1;
		BEGIN_TRACE_ENTRY(ProcessBreadcrumbs, payload);
		writer.WriteU32(processId);
		writer.WriteLongString(prepared);
		writer.WriteBool(deleteOld);
	}

	void Trace::ProcessRace(u32 processId, u32 remoteProcess)
	{
		BEGIN_TRACE_ENTRY(ProcessRace, 4 + 4);
		writer.WriteU32(processId);
		writer.WriteU32(remoteProcess);
	}

	void Trace::ProxyCreated(u32 clientId, const tchar* proxyName)
	{
		u64 nameLen = TStrlen(proxyName);
		u64 payload = Get7BitEncodedCount(clientId) + GetStringWriteSize(proxyName, nameLen);
		BEGIN_TRACE_ENTRY(ProxyCreated, payload);
		writer.Write7BitEncoded(clientId);
		writer.WriteString(proxyName, nameLen);
	}

	void Trace::ProxyUsed(u32 clientId, const tchar* proxyName)
	{
		u64 nameLen = TStrlen(proxyName);
		u64 payload = Get7BitEncodedCount(clientId) + GetStringWriteSize(proxyName, nameLen);
		BEGIN_TRACE_ENTRY(ProxyUsed, payload);
		writer.Write7BitEncoded(clientId);
		writer.WriteString(proxyName, nameLen);
	}

	void Trace::FileFetchLight(u32 clientId, const CasKey& key, u64 fileSize)
	{
		u64 payload = Get7BitEncodedCount(clientId) + Get7BitEncodedCount(fileSize);
		BEGIN_TRACE_ENTRY(FileFetchLight, payload);
		writer.Write7BitEncoded(clientId);
		writer.Write7BitEncoded(fileSize);
	}

	void Trace::FileFetchBegin(u32 clientId, const CasKey& key, StringView hint)
	{
		u32 stringIndex = AddString(hint);
		u64 payload = Get7BitEncodedCount(clientId) + 20 + Get7BitEncodedCount(stringIndex);
		BEGIN_TRACE_ENTRY(FileFetchBegin, payload);
		writer.Write7BitEncoded(clientId);
		writer.WriteCasKey(key);
		writer.Write7BitEncoded(stringIndex);
	}

	void Trace::FileFetchSize(u32 clientId, const CasKey& key, u64 fileSize)
	{
		u64 payload = Get7BitEncodedCount(clientId) + 20 + Get7BitEncodedCount(fileSize);
		BEGIN_TRACE_ENTRY(FileFetchSize, payload);
		writer.Write7BitEncoded(clientId);
		writer.WriteCasKey(key);
		writer.Write7BitEncoded(fileSize);
	}

	void Trace::FileFetchEnd(u32 clientId, const CasKey& key)
	{
		u64 payload = Get7BitEncodedCount(clientId) + 20;
		BEGIN_TRACE_ENTRY(FileFetchEnd, payload);
		writer.Write7BitEncoded(clientId);
		writer.WriteCasKey(key);
	}

	void Trace::FileStoreBegin(u32 clientId, const CasKey& key, u64 size, StringView hint, bool detailed)
	{
		if (detailed)
		{
			u32 stringIndex = AddString(hint);
			u64 payload = Get7BitEncodedCount(clientId) + 20 + Get7BitEncodedCount(size) + Get7BitEncodedCount(stringIndex);
			BEGIN_TRACE_ENTRY(FileStoreBegin, payload);
			writer.Write7BitEncoded(clientId);
			writer.WriteCasKey(key);
			writer.Write7BitEncoded(size);
			writer.Write7BitEncoded(stringIndex);
		}
		else
		{
			u64 payload = Get7BitEncodedCount(clientId) + Get7BitEncodedCount(size);
			BEGIN_TRACE_ENTRY(FileStoreLight, payload);
			writer.Write7BitEncoded(clientId);
			writer.Write7BitEncoded(size);
		}
	}

	void Trace::FileStoreEnd(u32 clientId, const CasKey& key)
	{
		u64 payload = Get7BitEncodedCount(clientId) + 20;
		BEGIN_TRACE_ENTRY(FileStoreEnd, payload);
		writer.Write7BitEncoded(clientId);
		writer.WriteCasKey(key);
	}

	void Trace::WorkBegin(u32 workIndex, StringView desc, const Color& color)
	{
		u32 stringIndex = AddString(desc);
		u64 payload = Get7BitEncodedCount(workIndex) + Get7BitEncodedCount(stringIndex) + 4;
		BEGIN_TRACE_ENTRY(WorkBegin, payload);
		writer.Write7BitEncoded(workIndex);
		writer.Write7BitEncoded(stringIndex);
		writer.WriteU32(color);
	}

	void Trace::WorkHint(u32 workIndex, StringView hint, u64 startTime)
	{
		u32 stringIndex = AddString(hint);
		u64 startVal = startTime ? (startTime - m_startTime) : 0;
		u64 payload = Get7BitEncodedCount(workIndex) + Get7BitEncodedCount(stringIndex) + Get7BitEncodedCount(startVal);
		BEGIN_TRACE_ENTRY(WorkHint, payload);
		writer.Write7BitEncoded(workIndex);
		writer.Write7BitEncoded(stringIndex);
		writer.Write7BitEncoded(startVal);
	}

	void Trace::WorkEnd(u32 workIndex)
	{
		u64 payload = Get7BitEncodedCount(workIndex);
		BEGIN_TRACE_ENTRY(WorkEnd, payload);
		writer.Write7BitEncoded(workIndex);
	}

	void Trace::ProgressUpdate(u32 processesTotal, u32 processesDone, u32 errorCount)
	{
		u64 payload = Get7BitEncodedCount(processesTotal) + Get7BitEncodedCount(processesDone) + Get7BitEncodedCount(errorCount);
		BEGIN_TRACE_ENTRY(ProgressUpdate, payload);
		writer.Write7BitEncoded(processesTotal);
		writer.Write7BitEncoded(processesDone);
		writer.Write7BitEncoded(errorCount);
	}

	void Trace::StatusUpdate(u32 statusRow, u32 statusColumn, StringView statusText, LogEntryType statusType, StringView statusLink)
	{
		u64 payload = Get7BitEncodedCount(statusRow) + Get7BitEncodedCount(statusColumn) + GetStringWriteSize(statusText.data, statusText.count) + 1 + GetStringWriteSize(statusLink.data, statusLink.count);
		BEGIN_TRACE_ENTRY(StatusUpdate, payload);
		writer.Write7BitEncoded(statusRow);
		writer.Write7BitEncoded(statusColumn);
		writer.WriteString(statusText);
		writer.WriteByte(statusType);
		writer.WriteString(statusLink);
	}

	u32 Trace::StringAdd(StringView string)
	{
		return AddString(string);
	}

	void Trace::StatusUpdate2(u32 statusRow, u32 statusColumn, u32 stringId)
	{
		u64 payload = Get7BitEncodedCount(statusRow) + Get7BitEncodedCount(statusColumn) + Get7BitEncodedCount(stringId);
		BEGIN_TRACE_ENTRY(StatusUpdate2, payload);
		writer.Write7BitEncoded(statusRow);
		writer.Write7BitEncoded(statusColumn);
		writer.Write7BitEncoded(stringId);
	}

	void Trace::DriveUpdate(const tchar drive, u8 busyPercent, u32 readCount, u64 readBytes, u32 writeCount, u64 writeBytes)
	{
		u64 payload = 1 + 1 + Get7BitEncodedCount(readCount) + Get7BitEncodedCount(readBytes) + Get7BitEncodedCount(writeCount) + Get7BitEncodedCount(writeBytes);
		BEGIN_TRACE_ENTRY_NOTIME(DriveUpdate, payload);
		writer.WriteByte((u8)drive);
		writer.WriteByte(busyPercent);
		writer.Write7BitEncoded(readCount);
		writer.Write7BitEncoded(readBytes);
		writer.Write7BitEncoded(writeCount);
		writer.Write7BitEncoded(writeBytes);
	}

	void Trace::RemoteExecutionDisabled()
	{
		BEGIN_TRACE_ENTRY(RemoteExecutionDisabled, 0);
	}

	void Trace::SchedulerUpdate(bool spawnPaused)
	{
		BEGIN_TRACE_ENTRY(SchedulerUpdate, 1);
		writer.WriteBool(spawnPaused);
	}

	void Trace::SchedulerKillProcess(u32 processId, StringView reason)
	{
		u32 stringIndex = AddString(reason);
		u64 payload = 4 + Get7BitEncodedCount(stringIndex);
		BEGIN_TRACE_ENTRY(SchedulerKillProcess, payload);
		writer.WriteU32(processId);
		writer.Write7BitEncoded(stringIndex);
	}

	void Trace::CacheBeginFetch(u32 fetchId, const tchar* description)
	{
		u64 descLen = TStrlen(description);
		u64 payload = Get7BitEncodedCount(fetchId) + GetStringWriteSize(description, descLen);
		BEGIN_TRACE_ENTRY(CacheBeginFetch, payload);
		writer.Write7BitEncoded(fetchId);
		writer.WriteString(description, descLen);
	}

	void Trace::CacheEndFetch(u32 fetchId, bool success, const u8* data, u64 dataSize)
	{
		u64 payload = Get7BitEncodedCount(fetchId) + 1 + dataSize;
		BEGIN_TRACE_ENTRY(CacheEndFetch, payload);
		writer.Write7BitEncoded(fetchId);
		writer.WriteBool(success);
		writer.WriteBytes(data, dataSize);
	}

	void Trace::CacheBeginWrite(u32 processId)
	{
		u64 payload = Get7BitEncodedCount(processId);
		BEGIN_TRACE_ENTRY(CacheBeginWrite, payload);
		writer.Write7BitEncoded(processId);
	}

	void Trace::CacheEndWrite(u32 processId, bool success, const u8* data, u64 dataSize)
	{
		u64 payload = Get7BitEncodedCount(processId) + 1 + dataSize;
		BEGIN_TRACE_ENTRY(CacheEndWrite, payload);
		writer.Write7BitEncoded(processId);
		writer.WriteBool(success);
		writer.WriteBytes(data, dataSize);
	}

	void Trace::CacheSummary(const u8* data, u64 dataSize)
	{
		BEGIN_TRACE_ENTRY(CacheSummary, dataSize);
		writer.WriteBytes(data, dataSize);
	}

	void Trace::TaskBegin(u32 processId, StringView description, StringView details, const Color& color)
	{
		u64 payload = Get7BitEncodedCount(processId) + GetStringWriteSize(description.data, description.count) + GetStringWriteSize(details.data, details.count) + 4;
		BEGIN_TRACE_ENTRY(TaskBegin, payload);
		writer.Write7BitEncoded(processId);
		writer.WriteString(description);
		writer.WriteString(details);
		writer.WriteU32(color);
	}

	void Trace::TaskHint(u32 processId, StringView hint)
	{
		u64 payload = Get7BitEncodedCount(processId) + GetStringWriteSize(hint.data, hint.count);
		BEGIN_TRACE_ENTRY(TaskHint, payload);
		writer.Write7BitEncoded(processId);
		writer.WriteString(hint);
	}

	void Trace::TaskEnd(u32 processId, bool success)
	{
		u64 payload = Get7BitEncodedCount(processId) + 1;
		BEGIN_TRACE_ENTRY(TaskEnd, payload);
		writer.Write7BitEncoded(processId);
		writer.WriteBool(success);
	}

	void Trace::TableUpdate(u32 name, u32 rowCharWidth, u32 rowMaxCount, const u32* items, u32 itemCount)
	{
		u64 itemsSize = 0;
		for (u32 i = 0; i != itemCount; ++i)
			itemsSize += Get7BitEncodedCount(items[i]);
		u64 payload = Get7BitEncodedCount(name) + Get7BitEncodedCount(rowCharWidth) + Get7BitEncodedCount(rowMaxCount) + Get7BitEncodedCount(itemCount) + itemsSize;
		BEGIN_TRACE_ENTRY(TableUpdate, payload);
		writer.Write7BitEncoded(name);
		writer.Write7BitEncoded(rowCharWidth);
		writer.Write7BitEncoded(rowMaxCount);
		writer.Write7BitEncoded(itemCount);
		for (u32 i=0; i!=itemCount; ++i)
			writer.Write7BitEncoded(items[i]);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
