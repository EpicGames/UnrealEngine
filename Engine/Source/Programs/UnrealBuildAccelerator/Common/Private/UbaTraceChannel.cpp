// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaTraceChannel.h"
#include "UbaFile.h"
#include "UbaFileMapping.h"
#include "UbaTrace.h"

#if PLATFORM_WINDOWS
#include <tlhelp32.h>
#include <psapi.h>
#endif

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	TraceChannel::TraceChannel(Logger& logger) : m_logger(logger)
	{
	}

	bool TraceChannel::Init(const tchar* channelName)
	{
	#if UBA_USE_TRACE_CHANNEL_FILE
		StringBuffer<> fileName;

		#if PLATFORM_WINDOWS
		fileName.Append(TCV("d:\\temp\\"));
		#else
		const char* tmpdir = getenv("TMPDIR");
		if (!tmpdir || tmpdir[0] == '\0')
			tmpdir = "/tmp";
		fileName.Append(tmpdir);
		#endif

		m_channelFile = fileName.EnsureEndsWithSlash().Append(TCV("UbaTrace_")).Append(channelName).ToString();
		m_listenerFile = StringBuffer<>().Append(m_channelFile.c_str()).Append(TCV(".listener")).ToString();

	#else
		m_channelName = channelName;
		StringBuffer<245> channelMutex;
		channelMutex.Append(TCV("Uba")).Append(channelName).Append(TCV("Channel"));
		m_memHandle = FileMapping_Create(m_logger, PAGE_READWRITE, 256, channelName, TC("TraceChannel"));
		if (!m_memHandle.IsValid())
		{
			m_logger.Error(TC("Failed to create file mapping %s for trace channel (%s)"), channelName, LastErrorToText().data);
			return false;
		}
		bool isCreator = GetLastError() != ERROR_ALREADY_EXISTS;

		auto mhg = MakeGuard([&]() { FileMapping_Close(m_logger, m_memHandle, TC("TraceChannel")); m_memHandle = {}; });

		m_mem = FileMapping_Map(m_logger, m_memHandle, FILE_MAP_WRITE, 0, 256);
		if (!m_mem)
		{
			m_logger.Error(TC("Failed to map file mapping for uba trace channel"));
			return false;
		}

		if (isCreator)
			*(tchar*)m_mem = 0;

		auto mg = MakeGuard([&]() { FileMapping_Unmap(m_logger, m_mem, 256, channelMutex.data); m_mem = nullptr; });

		channelMutex.Append(channelName).Append(TCV("Mutex"));
		m_mutex = CreateMutexW(false, channelMutex.data);
		if (m_mutex == InvalidMutexHandle)
			return false;

		mg.Cancel();
		mhg.Cancel();
	#endif
		return true;
	}

	void TraceChannel::Deinit()
	{
		#if UBA_USE_TRACE_CHANNEL_FILE
		#else
		if (m_mem)
			FileMapping_Unmap(m_logger, m_mem, 256, TC("TraceChannel"));
		if (m_memHandle.IsValid())
			FileMapping_Close(m_logger, m_memHandle, TC("TraceChannel"));
		if (m_mutex != InvalidMutexHandle)
			CloseMutex(m_mutex);
		if (m_listenerMutex != InvalidMutexHandle)
			CloseMutex(m_listenerMutex);
		m_mem = nullptr;
		m_memHandle = {};
		m_mutex = InvalidMutexHandle;
		m_listenerMutex = InvalidMutexHandle;
		#endif
	}

	TraceChannel::~TraceChannel()
	{
		Deinit();
	}

#if UBA_USE_TRACE_CHANNEL_FILE
	template<typename Function>
	bool UpdateTraceFile(Logger& logger, StringView channelFile, const tchar* traceName, const tchar* ifMatching, const Function& onPotentialValidTrace)
	{
		u8 writeBuffer[4096];
		BinaryWriter writer(writeBuffer, 0, sizeof(writeBuffer));
		bool modified = false;

#if PLATFORM_WINDOWS
		DWORD currentProcessId = GetCurrentProcessId();
		FILETIME createTime, exitTime, kernelTime, userTime;
		if (!GetProcessTimes(GetCurrentProcess(), &createTime, &exitTime, &kernelTime, &userTime))
			return logger.Error(TC("Failed to get current process process times"));
		u64 currentProcessCreateTime = ToLargeInteger(createTime.dwHighDateTime, createTime.dwLowDateTime).QuadPart
		if (*traceName)
		{
			writer.Write7BitEncoded(currentProcessId);
			writer.Write7BitEncoded(ToLargeInteger(createTime.dwHighDateTime, createTime.dwLowDateTime).QuadPart);
			writer.WriteString(traceName);
			modified = true;
		}
#else
		pid_t currentProcessId = getpid();
		u64 currentProcessCreateTime = 0;
#endif

		if (*traceName)
		{
			writer.Write7BitEncoded(currentProcessId);
			writer.Write7BitEncoded(currentProcessCreateTime);
			writer.WriteString(traceName);
			modified = true;
		}

		u32 createDisp = CREATE_NEW;
		u32 retryCount = 0;
		while (true)
		{
			FileHandle file = CreateFileW(channelFile.data, GENERIC_READ|GENERIC_WRITE, 0, createDisp, DefaultAttributes());
			if (file == InvalidFileHandle)
			{
				if (createDisp == CREATE_NEW && GetLastError() == ERROR_FILE_EXISTS)
				{
					createDisp = OPEN_EXISTING;
					continue;
				}
				if (retryCount > 10)
					return logger.Warning(TC("Failed to create/open trace channel file %s. No trace will be available (%s)"), channelFile.data, LastErrorToText().data);
				++retryCount;
				Sleep(100);
				continue;
			}
			auto g = MakeGuard([&]() { CloseFile(channelFile.data, file); });

			u64 fileSize = 0;
			if (!GetFileSizeEx(fileSize, file))
				return logger.Error(TC("Failed to get file size for %s"), channelFile.data);

			u8 readBuffer[4096];
			if (fileSize > sizeof(readBuffer))
				return logger.Error(TC("TraceChannel file %s too big"), channelFile.data);

			if (!ReadFile(logger, channelFile.data, file, readBuffer, fileSize))
				return false;

			BinaryReader reader(readBuffer, 0, fileSize);
			while (reader.GetLeft())
			{
				auto mg = MakeGuard([&]() { modified = true; });
				u64 processId;
				u64 processCreateTime;
				StringBuffer<> trace;

				if (!reader.TryRead7BitEncoded(processId))
					break;
				if (!reader.TryRead7BitEncoded(processCreateTime))
					break;
				if (!reader.TryReadString(trace))
					break;

				if (processId == currentProcessId)
				{
					if (ifMatching && trace.Equals(ifMatching))
					{
						if (!*traceName)
							continue;
						if (!trace.Equals(traceName))
							mg.Execute();
						trace.Clear().Append(traceName);
					}
				}
				else
				{
#if PLATFORM_WINDOWS
					HANDLE p = OpenProcess(PROCESS_QUERY_LIMITED_INFORMATION, TRUE, u32(processId));
					if (p == NULL)
						continue;
					auto pg = MakeGuard([p](){ CloseHandle(p); });
					if (!GetProcessTimes(p, &createTime, &exitTime, &kernelTime, &userTime))
						continue;
					u64 processCreateTime2 = ToLargeInteger(createTime.dwHighDateTime, createTime.dwLowDateTime).QuadPart;
					if (processCreateTime != processCreateTime2)
						continue;
#else
					if (kill(processId, 0) != 0)
						continue;
#endif
				}

				mg.Cancel();

				writer.Write7BitEncoded(processId);
				writer.Write7BitEncoded(processCreateTime);
				writer.WriteString(trace);

				onPotentialValidTrace(trace);
			}

			if (!modified)
				return true;

			u32 newFileSize = u32(writer.GetPosition());

			if (!SetFilePointer(logger, channelFile.data, file, 0))
				return false;

			if (!WriteFile(logger, channelFile.data, file, writeBuffer, newFileSize))
				return false;

			if (!SetEndOfFile(logger, channelFile.data, file, newFileSize))
				return false;

			return true;
		}
	}
#endif

	bool TraceChannel::Write(const tchar* traceName, const tchar* ifMatching)
	{
		#if UBA_USE_TRACE_CHANNEL_FILE
		return UpdateTraceFile(m_logger, m_channelFile, traceName, ifMatching, [](StringView trace) {});
		#else
		WaitForSingleObject((HANDLE)m_mutex, INFINITE);
		auto g = MakeGuard([this]() { ReleaseMutex(m_mutex); });
		if (ifMatching)
			if (!Equals((tchar*)m_mem, ifMatching))
				return true;
		TStrcpy_s((tchar*)m_mem, 256, traceName);
		return true;
		#endif
	}

	bool TraceChannel::Read(StringBufferBase& outTraceName)
	{
		#if UBA_USE_TRACE_CHANNEL_FILE
		struct stat info;
		if (stat(m_channelFile.c_str(), &info) == -1)
			return false;
		if (info.st_mtimespec.tv_nsec == m_lastModified.tv_nsec && info.st_mtimespec.tv_sec == m_lastModified.tv_sec)
			return true;
		m_lastModified = info.st_mtimespec;

		return UpdateTraceFile(m_logger, m_channelFile, TC(""), TC(""), [&](StringView trace)
			{
				if (!outTraceName.count)
					outTraceName.Append(trace);
			});
		#else
		WaitForSingleObject((HANDLE)m_mutex, INFINITE);
		outTraceName.Append((tchar*)m_mem);
		ReleaseMutex(m_mutex);
		return true;
#endif
	}

	bool TraceChannel::IsInitialized()
	{
#if UBA_USE_TRACE_CHANNEL_FILE
		return !m_channelFile.empty();
#else
		return m_mutex != InvalidMutexHandle;
#endif
	}

	bool TraceChannel::StartListening()
	{
#if UBA_USE_TRACE_CHANNEL_FILE
		if (m_listenerFile.empty())
			return false;
		FileHandle file = CreateFileW(m_listenerFile.c_str(), GENERIC_WRITE, 0, CREATE_ALWAYS, DefaultAttributes());
		if (file == InvalidFileHandle)
			return m_logger.Warning(TC("Failed to create listener pidfile %s (%s)"), m_listenerFile.c_str(), LastErrorToText().data);
		auto g = MakeGuard([&]() { CloseFile(m_listenerFile.c_str(), file); });
#if PLATFORM_WINDOWS
		u32 pid = (u32)GetCurrentProcessId();
#else
		u32 pid = (u32)getpid();
#endif
		return WriteFile(m_logger, m_listenerFile.c_str(), file, &pid, sizeof(pid));
#else
		if (m_channelName.empty())
			return false;
		StringBuffer<256> name;
		name.Append(TCV("Uba")).Append(m_channelName.c_str()).Append(TCV("Listener"));
		m_listenerMutex = CreateMutexW(false, name.data);
		return m_listenerMutex != InvalidMutexHandle;
#endif
	}

	bool TraceChannel::IsListenerActive()
	{
#if UBA_USE_TRACE_CHANNEL_FILE
		if (m_listenerFile.empty())
			return false;
		FileHandle file = CreateFileW(m_listenerFile.c_str(), GENERIC_READ, FILE_SHARE_READ | FILE_SHARE_WRITE, OPEN_EXISTING, DefaultAttributes());
		if (file == InvalidFileHandle)
			return false;
		auto g = MakeGuard([&]() { CloseFile(m_listenerFile.c_str(), file); });
		u32 pid = 0;
		if (!ReadFile(m_logger, m_listenerFile.c_str(), file, &pid, sizeof(pid)))
			return false;
		if (pid == 0)
			return false;
#if PLATFORM_WINDOWS
		HANDLE h = OpenProcess(PROCESS_QUERY_LIMITED_INFORMATION, FALSE, pid);
		if (h == NULL)
			return false;
		CloseHandle(h);
		return true;
#else
		// kill(pid, 0) sends no signal; it just probes whether the pid resolves to a live process.
		return kill((pid_t)pid, 0) == 0;
#endif
#else
		if (m_channelName.empty())
			return false;
		StringBuffer<256> name;
		name.Append(TCV("Uba")).Append(m_channelName.c_str()).Append(TCV("Listener"));
		HANDLE h = OpenMutexW(SYNCHRONIZE, FALSE, name.data);
		if (h == NULL)
			return false;
		CloseHandle(h);
		return true;
#endif
	}

	static OwnerInfo InternalGetOwnerInfo()
	{
		static tchar buffer[260];
		*buffer = 0;

		OwnerInfo info { buffer, 0 };

		StringBuffer<32> ownerPidStr;
		ownerPidStr.count = GetEnvironmentVariableW(TC("UBA_OWNER_PID"), ownerPidStr.data, ownerPidStr.capacity);
		if (ownerPidStr.count)
		{
			GetEnvironmentVariableW(TC("UBA_OWNER_ID"), buffer, sizeof_array(buffer));
			ownerPidStr.Parse(info.pid);
			return info;
		}

		#if PLATFORM_WINDOWS
		HANDLE snapshotHandle = CreateToolhelp32Snapshot(TH32CS_SNAPPROCESS, 0);
		if (snapshotHandle == INVALID_HANDLE_VALUE)
			return info;

		PROCESSENTRY32 pe = { 0 };
		pe.dwSize = sizeof(PROCESSENTRY32);
		UnorderedMap<u32, u32> pidToParent;
		if (Process32First(snapshotHandle, &pe))
		{
			do
			{
				pidToParent[pe.th32ProcessID] = pe.th32ParentProcessID;
			}
			while (Process32Next(snapshotHandle, &pe));
		}
		CloseHandle(snapshotHandle);

		u32 pid = ::GetCurrentProcessId();
		while (true)
		{
			auto findIt = pidToParent.find(pid);
			if (findIt == pidToParent.end())
				break;
			pid = findIt->second;
			pidToParent.erase(findIt);

			HANDLE parentHandle = OpenProcess(PROCESS_QUERY_INFORMATION | PROCESS_VM_READ, FALSE, pid);
			if (parentHandle == NULL)
				break;
			tchar moduleName[260];
			DWORD len = GetModuleFileNameExW(parentHandle, 0, moduleName, MAX_PATH);
			CloseHandle(parentHandle);
			if (!len)
				break;
			if (!Contains(moduleName, L"devenv.exe"))
				continue;
			TStrcpy_s(buffer, MAX_PATH, L"vs");
			info.pid = pid;
			break;
		}
		#endif

		return info;
	}

	const OwnerInfo& GetOwnerInfo()
	{
		static OwnerInfo info = InternalGetOwnerInfo();
		return info;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}