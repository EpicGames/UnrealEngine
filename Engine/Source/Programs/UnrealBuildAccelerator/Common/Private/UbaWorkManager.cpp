// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaWorkManager.h"
#include "UbaBottleneck.h"
#include "UbaEvent.h"
#include "UbaPlatform.h"
#include "UbaThread.h"

namespace uba
{
	thread_local TrackWorkScope* t_trackWorkScope;

	TrackWorkScope::TrackWorkScope()
	:	tracker(nullptr)
	,	id(0)
	{
		outerScope = t_trackWorkScope;
		t_trackWorkScope = this;
	}

	TrackWorkScope::TrackWorkScope(WorkTracker& t, StringView desc, const Color& color)
	:	tracker(&t)
	,	id(t.TrackWorkStart(desc, color))
	{
		outerScope = t_trackWorkScope;
		t_trackWorkScope = this;
	}

	void TrackWorkScope::AddHint(StringView hint, u64 startTime)
	{
		if (tracker)
			tracker->TrackWorkHint(id, hint, startTime);
	}

	TrackWorkScope::~TrackWorkScope()
	{
		t_trackWorkScope = outerScope;
		if (tracker)
			tracker->TrackWorkEnd(id);
	}

	TrackWorkScope* TrackWorkScope::GetActiveScope()
	{
		return t_trackWorkScope;
	}

	u32 WorkManager::TrackWorkStart(const StringView& desc, const Color& color)
	{
		if (auto t = m_workTracker.load())
			return t->TrackWorkStart(desc, color);
		return 0;
	}

	void WorkManager::TrackWorkHint(u32 id, const StringView& hint, u64 startTime)
	{
		if (auto t = m_workTracker.load())
			return t->TrackWorkHint(id, hint, startTime);
	}

	void WorkManager::TrackWorkEnd(u32 id)
	{
		if (auto t = m_workTracker.load())
			return t->TrackWorkEnd(id);
	}

	void WorkManager::SetWorkTracker(WorkTracker* workTracker)
	{
		m_workTracker = workTracker;
	}

	WorkQueue::~WorkQueue()
	{
		for (u32 i=0; i != sizeof_array(first); ++i)
		{
			Work* w = first[i];
			while (w)
			{
				Work* n = w->next;
				delete w;
				w = n;
			}
		}
	}

	void WorkQueue::Enqueue(WorkFunction&& func, u32 count, const tchar* desc, const Color& color, WorkPriority priority, Bottleneck* bottleneck, bool trackWork)
	{
		UBA_ASSERT(*desc);
		SCOPED_FUTEX(lock, lk);
		u32 lastIndex = count - 1;
		for (u32 i = 0; i != count; ++i)
		{
			Work* work;
			if (free)
			{
				work = free;
				free = free->next;
			}
			else
				work = new Work();
		
			work->next = nullptr;

			if (i == lastIndex)
				work->func = std::move(func);
			else
				work->func = func;

			work->bottleneck = bottleneck;
			if (trackWork)
				work->desc = desc;

			Work*& l = last[priority];

			if (!l)
			{
				l = work;
				first[priority] = work;
			}
			else
			{
				l->next = work;
				l = work;
			}
		}
	}

	Work* WorkQueue::Dequeue(BottleneckTicket& ticket)
	{
		SCOPED_FUTEX(lock, l);
		for (u32 i=0; i != sizeof_array(first); ++i)
		{
			Work* p = nullptr;
			Work* w = first[i];
			while (w)
			{
				if (w->bottleneck && !w->bottleneck->TryEnter())
				{
					p = w;
					w = w->next;
					continue;
				}

				if (!w->next)
				{
					last[i] = p;
				}
				if (p)
				{
					p->next = w->next;
				}
				else
				{
					first[i] = w->next;
				}

				return w;
			}
		}
		return nullptr;
	}

	void WorkQueue::Finish(Work* work, BottleneckTicket& ticket)
	{
		Bottleneck* bottleneck = work->bottleneck;
		work->bottleneck = nullptr;
		work->func = {};
		work->desc.clear();

		SCOPED_FUTEX(lock, l);
		work->next = free;
		free = work;
		l.Leave();

		if (bottleneck)
			bottleneck->Leave(ticket);
	}

	bool WorkQueue::IsEmpty()
	{
		SCOPED_FUTEX(lock, l);
		return IsEmptyNoLock();
	}

	bool WorkQueue::IsEmptyNoLock()
	{
		for (u32 i=0; i != sizeof_array(first); ++i)
			if (first[i])
				return false;
		return true;
	}


	struct WorkManagerImpl::Worker
	{
		Worker(WorkManagerImpl& manager, const tchar* workerDesc) : m_workAvailable(EventResetType_Auto)
		{
			m_thread.Start([&]() { ThreadWorker(manager); return 0; }, workerDesc);
		}
		~Worker()
		{
			m_thread.Wait();
		}

		void Stop()
		{
			m_loop = false;
			m_workAvailable.Set();
		}

		void ThreadWorker(WorkManagerImpl& manager)
		{
			BottleneckTicket ticket;
			WorkQueue& workQueue = manager.m_workQueue;

			while (true)
			{
				if (!m_workAvailable.IsSet())
					break;
				if (!m_loop)
					break;

				while (true)
				{
					while (true)
					{
						Work* work = workQueue.Dequeue(ticket);
						if (!work)
							break;

						#if UBA_TRACK_WORK
						TrackWorkScope tws(manager, work->desc);
						#else
						TrackWorkScope tws;
						#endif
						work->func({tws, &m_thread});
						workQueue.Finish(work, ticket);
					}

					SCOPED_FUTEX(manager.m_availableWorkersLock, lock1);
					SCOPED_FUTEX_READ(workQueue.lock, lock2);
					if (!workQueue.IsEmptyNoLock())
						continue;

					manager.PushWorkerNoLock(this);
					break;
				}
			}
		}

		Worker* m_nextWorker = nullptr;
		Worker* m_prevWorker = nullptr;
		Atomic<bool> m_loop = true;
		EventSlim m_workAvailable;
		Thread m_thread;
	};

	WorkManagerImpl::WorkManagerImpl(u32 maxWorkerCount, const tchar* workerDesc, u32 precreateWorkerCount)
	:	m_maxWorkerCount(maxWorkerCount)
	,	m_workerDesc(workerDesc)
	{
		for (u32 i = 0; i != precreateWorkerCount; ++i)
		{
			auto& worker = m_workers.emplace_back(*this, m_workerDesc.c_str());
			PushWorkerNoLock(&worker);
		}
		m_activeWorkerCount = 0;
	}

	WorkManagerImpl::~WorkManagerImpl()
	{
		for (auto& worker : m_workers)
			worker.Stop();
		m_workers.clear();
	}


	void WorkManagerImpl::AddWork(WorkFunction&& work, u32 count, const tchar* desc, const Color& color, WorkPriority priority, Bottleneck* bottleneck)
	{
		m_workQueue.Enqueue(std::move(work), count, desc, color, priority, bottleneck, m_workTracker != nullptr);

		SCOPED_FUTEX(m_availableWorkersLock, lock2);
		while (count--)
		{
			Worker* worker = PopWorkerNoLock();
			if (!worker)
				break;
			worker->m_workAvailable.Set();
		}
	}

	u32 WorkManagerImpl::GetWorkerCount()
	{
		return m_maxWorkerCount;
	}

	void WorkManagerImpl::PushWorkerNoLock(Worker* worker)
	{
		if (m_firstAvailableWorker)
			m_firstAvailableWorker->m_prevWorker = worker;
		worker->m_prevWorker = nullptr;
		worker->m_nextWorker = m_firstAvailableWorker;
		m_firstAvailableWorker = worker;
		--m_activeWorkerCount;
	}

	void WorkManagerImpl::DoWork(u32 count)
	{
		BottleneckTicket ticket;

		while (count--)
		{
			Work* work = m_workQueue.Dequeue(ticket);
			if (!work)
				break;
			#if UBA_TRACK_WORK
			TrackWorkScope tws(*this, work->desc);
			#else
			TrackWorkScope tws;
			#endif
			work->func({tws});

			m_workQueue.Finish(work, ticket);
		}
	}

	bool WorkManagerImpl::FlushWork(u32 timeoutMs)
	{
		u64 startTime = GetTime();
		auto hasTimedOut = [&]() { return timeoutMs ? (timeoutMs < TimeToMs(GetTime() - startTime)) : false; };

		while (true)
		{
			SCOPED_FUTEX_READ(m_workQueue.lock, lock);
			if (m_workQueue.IsEmptyNoLock())
				break;
			lock.Leave();
			if (hasTimedOut())
				return false;
			Sleep(5);
		}

		while (m_activeWorkerCount)
		{
			if (hasTimedOut())
				return false;
			Sleep(5);
		}
		return true;
	}

	WorkManagerImpl::Worker* WorkManagerImpl::PopWorkerNoLock()
	{
		Worker* worker = m_firstAvailableWorker;
		if (!worker)
		{
			if (m_workers.size() == m_maxWorkerCount)
				return nullptr;
			Worker& newWorker = m_workers.emplace_back(*this, m_workerDesc.c_str());
			++m_activeWorkerCount;
			return &newWorker;
		}
		m_firstAvailableWorker = worker->m_nextWorker;
		if (m_firstAvailableWorker)
			m_firstAvailableWorker->m_prevWorker = nullptr;
		++m_activeWorkerCount;
		return worker;
	}
}
