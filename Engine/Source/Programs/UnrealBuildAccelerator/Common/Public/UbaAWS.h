// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaFileAccessor.h"
#include "UbaNetworkBackend.h"
#include "UbaStringBuffer.h"

#define UBA_USE_CLOUD !PLATFORM_MAC

#if UBA_USE_CLOUD

namespace uba
{
	class Cloud
	{
	public:
		struct Provider
		{
			const tchar* name;
			const char* tokenCommand;
			const char* tokenImds;
			const char* tokenHeader;
			const tchar* tokenPrefix;
			const char* instanceIdImds;
			const char* instanceLifeCycleImds;
			const char* autoScalingLifeCycleStateImds;
			const char* availabilityZoneImds;
			const char* maintenanceEventImds;
		};

		static constexpr Provider g_providers[]
		{
			{
				TC("AWS"),
				"PUT",
				"latest/api/token",
				"X-aws-ec2-metadata-token-ttl-seconds: 21600\r\n",
				TC("X-aws-ec2-metadata-token: "),
				"latest/meta-data/instance-id",
				"latest/meta-data/instance-life-cycle",
				"latest/meta-data/autoscaling/target-lifecycle-state",
				"latest/meta-data/placement/availability-zone",
				nullptr, // maintenance
			},
			{
				TC("GCP"),
				"POST",
				"computeMetadata/v1/instance/service-accounts/default/identity?audience=https://example.com",
				"Accept: */*\r\nMetadata-Flavor: Google\r\nContent-Length: 0\r\n",
				TC("Metadata-Flavor: Google\r\nAuthorization: Bearer "),
				"computeMetadata/v1/instance/id",
				nullptr, // Life cycle
				nullptr, // Autoscaling
				"computeMetadata/v1/instance/zone",
				"computeMetadata/v1/instance/maintenance-event",
			}
		};


		static constexpr char g_imdsHost[]						= "169.254.169.254";
		static constexpr char g_imdsAutoScalingLifeCycleState[] = "latest/meta-data/autoscaling/target-lifecycle-state";
		static constexpr char g_imdsSpotInstanceAction[]		= "latest/meta-data/spot/instance-action";

		bool QueryToken(Logger& logger, const tchar* rootDir)
		{
			for (auto& provider : g_providers)
			{
				if (IsNotCloud(logger, rootDir, provider.name))
					continue;

				HttpConnection http;
				http.SetConnectTimeout(200);

				u32 statusCode = 0;

				//logger.Warning(TC("%hs TOKEN PUT START"), provider.name);
				
				StringBuffer<1024> token;
				token.Append(provider.tokenPrefix);

				if (!http.Query(logger, provider.tokenCommand, token, statusCode, g_imdsHost, provider.tokenImds, provider.tokenHeader, 250))
				{
					WriteIsNotCloud(logger, rootDir, provider.name);
					continue;
				}
				token.Append("\r\n");

				#if PLATFORM_WINDOWS
				char tokenString[1024];
				size_t tokenStringLen;
				wcstombs_s(&tokenStringLen, tokenString, sizeof_array(tokenString), token.data, _TRUNCATE);
				m_tokenString = tokenString;
				#else
				m_tokenString = token.data;
				#endif

				m_provider = &provider;

				return true;
			}

			return false;
		}

		bool QueryInformation(Logger& logger, StringBufferBase& outExtraInfo, const tchar* rootDir)
		{
			if (!QueryToken(logger, rootDir))
				return false;

			HttpConnection http;
			http.SetConnectTimeout(200);

			u32 statusCode = 0;

			//logger.Warning(TC("%hs TOKEN PUT START"), provider.name);
				
			StringBuffer<512> instanceId;
			if (!http.Query(logger, "GET", instanceId, statusCode, g_imdsHost, m_provider->instanceIdImds, m_tokenString.c_str()))
				return false;

			outExtraInfo.Append(TCV(", ")).Append(m_provider->name).Append(TCV(": ")).Append(instanceId);

			if (m_provider->instanceLifeCycleImds)
			{
				StringBuffer<32> instanceLifeCycle;
				if (http.Query(logger, "GET", instanceLifeCycle, statusCode, g_imdsHost, m_provider->instanceLifeCycleImds, m_tokenString.c_str()))
				{
					outExtraInfo.Append(' ').Append(instanceLifeCycle);
					m_isSpot = instanceLifeCycle.Contains(TC("spot"));
				}
			}

			if (m_provider->autoScalingLifeCycleStateImds)
			{
				StringBuffer<32> autoscaling;
				if (http.Query(logger, "GET", autoscaling, statusCode, g_imdsHost, m_provider->autoScalingLifeCycleStateImds, m_tokenString.c_str()) && statusCode == 200)
				{
					outExtraInfo.Append(m_isSpot ? '/' : ' ').Append(TCV("autoscale"));
					m_isAutoscaling = true;
				}
			}

			if (!QueryAvailabilityZone(logger, nullptr))
				return false;
			return true;
		}

		bool QueryAvailabilityZone(Logger& logger, const tchar* rootDir)
		{
			if (rootDir && !QueryToken(logger, rootDir))
				return false;
			if (!m_provider->availabilityZoneImds)
				return false;

			HttpConnection http;
			http.SetConnectTimeout(200);

			StringBuffer<128> availabilityZone;
			u32 statusCode = 0;
			if (!http.Query(logger, "GET", availabilityZone, statusCode, g_imdsHost, m_provider->availabilityZoneImds, m_tokenString.c_str()))
				return false;

			const tchar* zone = availabilityZone.data;
			if (const tchar* lastSlash = TStrrchr(zone, '/'))
				zone = lastSlash + 1;

			m_availabilityZone = zone;
			return true;
		}

		// Returns true if we _know_ we are not in this cloud provider
		bool IsNotCloud(Logger& logger, const tchar* rootDir, const tchar* provider)
		{
			StringBuffer<512> file;
			file.Append(rootDir).EnsureEndsWithSlash().Append(TCV(".isNot")).Append(provider);
			if (FileExists(logger, file.data))
				return true;
			return false;
		}

		bool WriteIsNotCloud(Logger& logger, const tchar* rootDir, const tchar* provider)
		{
			StringBuffer<512> file;
			file.Append(rootDir).EnsureEndsWithSlash().Append(TCV(".isNot")).Append(provider);
			FileAccessor f(logger, file.data);
			if (!f.CreateWrite())
				return false;
			if (!f.Close())
				return false;
			return false;
		}

		const tchar* GetProvider() const
		{
			if (!m_provider)
				return TC("");
			return m_provider->name;
		}

		bool IsTerminating(Logger& logger, StringBufferBase& outReason, u64& outTerminationTimeMs)
		{
			HttpConnection http;

			outTerminationTimeMs = 0;
			if (m_isSpot)
			{
				StringBuffer<1024> content;
				u32 statusCode = 0;
				if (http.Query(logger, "GET", content, statusCode, g_imdsHost, g_imdsSpotInstanceAction, m_tokenString.c_str()) && statusCode == 200)
				{
					outReason.Append(TCV("AWS spot instance interruption"));
					return true;
				}
			}

			if (m_isAutoscaling)
			{
				StringBuffer<1024> content;
				u32 statusCode = 0;
				if (http.Query(logger, "GET", content, statusCode, g_imdsHost, g_imdsAutoScalingLifeCycleState, m_tokenString.c_str()) && statusCode == 200)
				{
					//if (!content.Equals(L"InService"))
					//{
					//	wprintf(L"AWSACTION: AUTOSCALE: %ls\n", content.data);
					//}

					if (!content.Contains(TC("InService"))) // AWS can return "InServiceI" as well?
					{
						//wprintf(L"AWSACTION: AUTOSCALE REBALANCING!!!! (%ls)\n", content.data);
						outReason.Append(TCV("AWS autoscale rebalancing"));
						return true;
					}
				}
			}

			if (m_provider && m_provider->maintenanceEventImds)
			{
				StringBuffer<1024> content;
				u32 statusCode = 0;
				if (http.Query(logger, "GET", content, statusCode, g_imdsHost, m_provider->maintenanceEventImds, m_tokenString.c_str()) && statusCode == 200)
				{
					if (!content.IsEmpty() && !content.Equals(TCV("NONE")))
					{
						outReason.Append(TCV("Google cloud instance interruption (")).Append(content).Append(')');
						return true;
					}
				}
			}
			return false;
		}

		const tchar* GetAvailabilityZone()
		{
			return m_availabilityZone.c_str();
		}

		TString m_availabilityZone;
		std::string m_tokenString;
		const Provider* m_provider = nullptr;
		bool m_isSpot = false;
		bool m_isAutoscaling = false;
	};
}

#endif
