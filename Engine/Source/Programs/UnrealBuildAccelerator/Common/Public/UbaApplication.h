// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaEnvironment.h"
#include "UbaStringBuffer.h"

namespace uba
{
	inline bool ExpandEnvironmentVariables(StringBufferBase& str, const Function<bool(const tchar*)>& errorFunc)
	{
		StringBuffer<> expandedDir;
		u64 offset = 0;
		while (true)
		{
			const tchar* begin = str.First('%', offset);
			if (!begin)
			{
				expandedDir.Append(str.data + offset);
				str.Clear().Append(expandedDir);
				break;
			}
			u64 beginOffset = begin - str.data;
			const tchar* end = str.First('%', beginOffset + 1);
			if (!end)
				return errorFunc(TC("Missing closing % for environment variable in dir path"));
			u64 endOffset = end - str.data;
			StringBuffer<256> var;
			var.Append(begin + 1, endOffset - beginOffset - 1);
			StringBuffer<> value;
			value.count = GetEnvironmentVariableW(var.data, value.data, value.capacity);
			if (!value.count)
			{
				StringBuffer<256> err;
				err.Appendf(TC("Can't find environment variable %s used in dir path"), var.data);
				return errorFunc(err.data);
			}

			expandedDir.Append(str.data + offset, beginOffset - offset).Append(value);
			offset = endOffset + 1;
		}
		return true;
	}
}