// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaEvent.h"
#include "UbaPlatform.h"
#include "UbaSynchronization.h"
#include "UbaTimer.h"

namespace uba
{
	struct BottleneckTicket
	{
		Event ev;
		BottleneckTicket* next = nullptr;
	};

	struct Bottleneck
	{
		Bottleneck(u32 mc) : m_maxCount(mc) {}

		void Enter(BottleneckTicket& ticket, Timer& timer)
		{
			if (TryEnter())
				return;

			SCOPED_FUTEX(m_lock, lock);
			if (TryEnter())
				return;

			if (!m_first)
				m_first = &ticket;
			else
				m_last->next = &ticket;
			m_last = &ticket;

			if (!ticket.ev.IsCreated())
				ticket.ev.Create(EventResetType_Auto);

			lock.Leave();

			TimerScope ts(timer);
			ticket.ev.IsSet();
		}

		bool TryEnter()
		{
			u32 ac = m_activeCount.load(std::memory_order_relaxed);
			while (ac < m_maxCount)
				if (m_activeCount.compare_exchange_weak(ac, ac + 1, std::memory_order_acquire, std::memory_order_relaxed))
					return true;
			return false;
		}

		void Leave(BottleneckTicket& ticket)
		{
			SCOPED_FUTEX(m_lock, lock);
			BottleneckTicket* first = m_first;
			if (!first)
			{
				UBA_ASSERT(m_activeCount.load(std::memory_order_relaxed) > 0);
				m_activeCount.fetch_sub(1, std::memory_order_release);
				return;
			}
			m_first = first->next;
			if (!m_first)
				m_last = nullptr;
			first->ev.Set();
		}

		u32 GetMaxCount() const { return m_maxCount; }

	private:
		Futex m_lock;
		BottleneckTicket* m_first = nullptr;
		BottleneckTicket* m_last = nullptr;
		Atomic<u32> m_activeCount = 0;
		u32 m_maxCount;
	};

	struct BottleneckScope
	{
		BottleneckScope(Bottleneck& b, Timer& timer) : bottleneck(b) { b.Enter(ticket, timer); }
		~BottleneckScope() { bottleneck.Leave(ticket); }

		Bottleneck& bottleneck;
		BottleneckTicket ticket;
	};

}
