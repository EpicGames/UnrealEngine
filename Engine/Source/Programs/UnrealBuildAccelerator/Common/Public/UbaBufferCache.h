// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBinaryReaderWriter.h"

#define UBA_CHECK_BUFFER_SLOTS 0

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct BufferCache
	{
		inline u8* Pop();
		inline void Push(u8* slot);

		inline BufferCache(u32 bufferSize);
		inline ~BufferCache();

		u32 m_bufferSize;
		Futex m_slotsLock;
		Vector<u8*> m_slots;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct BinaryWriterUsingCache : BinaryWriter
	{
		BinaryWriterUsingCache(BufferCache& cache) : BinaryWriter(cache.Pop(), 0, cache.m_bufferSize), m_cache(cache) {}
		~BinaryWriterUsingCache() { m_cache.Push(m_begin); }
		BufferCache& m_cache;
	};

	struct BinaryReaderUsingCache : BinaryReader
	{
		BinaryReaderUsingCache(BufferCache& cache) : BinaryReader(cache.Pop(), 0, cache.m_bufferSize), m_cache(cache) {}
		~BinaryReaderUsingCache() { m_cache.Push(const_cast<u8*>(m_begin)); }
		BufferCache& m_cache;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	u8* BufferCache::Pop()
	{
		SCOPED_FUTEX(m_slotsLock, lock);
		if (!m_slots.empty())
		{
			auto back = m_slots.back();
			m_slots.pop_back();
			return back;
		}
		#if UBA_CHECK_BUFFER_SLOTS
		u8* res = (u8*)malloc(m_bufferSize + 12);
		res += 8;
		*(u32*)(res - 4) = 0xdeadbeef;
		*(u32*)(res + m_bufferSize) = 0xdeadbeef;
		return res;
		#else
		return (u8*)malloc(m_bufferSize);
		#endif
	}

	void BufferCache::Push(u8* slot)
	{
		if (!slot)
			return;

		#if UBA_CHECK_BUFFER_SLOTS
		UBA_ASSERT(*(u32*)(slot - 4) == 0xdeadbeef);
		UBA_ASSERT(*(u32*)(slot + m_bufferSize) == 0xdeadbeef);
		#endif

		SCOPED_FUTEX(m_slotsLock, lock);
		m_slots.push_back(slot);
	}

	BufferCache::BufferCache(u32 bufferSize) : m_bufferSize(bufferSize)
	{
	}

	BufferCache::~BufferCache()
	{
		#if UBA_CHECK_BUFFER_SLOTS
		for (u8* slot : m_slots)
			free(slot-8);
		#else
		for (u8* slot : m_slots)
			free(slot);
		#endif
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}