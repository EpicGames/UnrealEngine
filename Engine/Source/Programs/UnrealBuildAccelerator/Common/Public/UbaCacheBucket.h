// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaCacheEntry.h"
#include "UbaCompactTables.h"

namespace uba
{
	class StorageServer;
	class WorkManager;

	struct CacheBucket
	{
		CacheBucket(u64 id, u32 version);

		ReaderWriterLock m_cacheEntryLookupLock;
		UnorderedMap<CasKey, CacheEntries> m_cacheEntryLookup;

		CompactPathTable m_pathTable;
		CompactCasKeyTable m_casKeyTable;

		u64 m_id;
		u64 totalEntryCount = 0;
		u64 totalEntrySize = 0;
		Atomic<bool> hasDeletedEntries = false;
		Atomic<bool> needsSave = false;

		// Times are in file time from creation of cache database
		Atomic<u64> lastSavedTime = 0;
		Atomic<u64> lastUsedTime = 0;
		u64 oldestUsedTime = 0; 

		u32 index = ~0u;

		struct MaintenanceContext;
		MaintenanceContext* m_maintenanceContext = nullptr;

		struct LoadStats
		{
			Atomic<u32> totalPathTableSize;
			Atomic<u32> totalCasKeyTableSize;
			Atomic<u64> totalCacheEntryCount;
		};

		bool Load(Logger& logger, BinaryReader& reader, u32 databaseVersion, LoadStats& outStats, StorageServer& storage);
		bool Validate(Logger& logger, WorkManager& workManager);
	};
}