// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBufferCache.h"
#include "UbaHash.h"
#include "UbaLogger.h"
#include "UbaMap.h"
#include "UbaProcessHandle.h"
#include "UbaRootPaths.h"
#include "UbaStats.h"
#include "UbaUnorderedMap.h"

#define UBA_CACHE_USE_SEND_MAX_BUFFER_CACHE PLATFORM_MAC // To prevent stack overflow

namespace uba
{
	class CompactCasKeyTable;
	class CompactPathTable;
	class Config;
	class NetworkClient;
	class RootPaths;
	class Session;
	class StorageImpl;
	enum MessagePriority : u8;
	struct CacheFetchStats;
	struct CacheSendStats;
	struct CasKey;
	struct ProcessStartInfo;
	struct StorageStats;
	struct StringView;
	struct TrackWorkScope;
	using RootsHandle = u64;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct CacheClientCreateInfo
	{
		CacheClientCreateInfo(LogWriter& w, StorageImpl& st, NetworkClient& c, Session& se) : writer(w), storage(st), client(c), session(se) {}
		LogWriter& writer;
		StorageImpl& storage;
		NetworkClient& client;
		Session& session;

		void Apply(Config& config, const tchar* tableName = TC("CacheClient"));

		bool reportCacheKey = false;
		bool reportMissReason = false; // Report the reason no matching cache entry was found
		bool useDirectoryPreparsing = true; // This is used to minimize syscalls. GetFileAttributes can be very expensive on cloud machines and we can enable this to minimize syscall count
		bool validateCacheWritesInput = false; // Set to true to validate cas of all input files before sent to cache
		bool validateCacheWritesOutput = false; // Set to true to validate cas of all output files before sent to cache
		bool useRoots = true; // Set this to false to allow paths that are not under roots and to not fix them up
		bool useCacheHit = true; // Set this to false to ignore found cache hits.. this is for debugging/testing only
		bool useMappingForCreatedCas = true; // Set to this to false to make sure cas created when sending cas are stored on disk
		const tchar* hint = TC(""); // Hint will show up in cache server log
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct CacheResult
	{
		bool hit = false;
		Vector<ProcessLogLine> logLines;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class CacheClient
	{
	public:
		CacheClient(const CacheClientCreateInfo& info);
		~CacheClient();

		bool RegisterPathHash(const tchar* path, const CasKey& hash);

		bool WriteToCache(u32 bucketId, const ProcessStartInfo& info, const u8* inputs, u64 inputsSize, const u8* outputs, u64 outputsSize, const u8* logLines, u64 logLinesSize, u32 processId = 0);
		bool WriteToCache(const RootPaths& rootPaths, u32 bucketId, const ProcessStartInfo& info, const u8* inputs, u64 inputsSize, const u8* outputs, u64 outputsSize, const u8* logLines, u64 logLinesSize, u32 processId = 0);
		bool FetchFromCache(CacheResult& outResult, RootsHandle rootsHandle, u32 bucketId, const ProcessStartInfo& info);
		bool FetchFromCache(CacheResult& outResult, const RootPaths& rootPaths, u32 bucketId, const ProcessStartInfo& info);
		bool RequestServerShutdown(const tchar* reason);

		bool ExecuteCommand(Logger& logger, const tchar* command, const tchar* destinationFile = nullptr, const tchar* additionalInfo = nullptr);

		inline MutableLogger& GetLogger() { return m_logger; }
		inline NetworkClient& GetClient() { return m_client; }
		inline StorageImpl& GetStorage() { return m_storage; }
		inline Session& GetSession() { return m_session; }

		void Disable();

		void TraceSummary();

		struct Bucket;

		struct FetchContext
		{
			FetchContext(const ProcessStartInfo& info, Session& session, RootsHandle rootsHandle, MessagePriority priority);
			FetchContext(const ProcessStartInfo& info, const RootPaths& rootPaths);

			const ProcessStartInfo& info;

			RootPaths rootPaths;
			Bucket* bucket = nullptr;
			Vector<u32> casKeyOffsets;
			MessagePriority priority;
		};

		bool FetchEntryFromCache(CacheResult& outResult, FetchContext& context, u32 bucketId);
		bool FetchFilesFromCache(CacheResult& outResult, FetchContext& context, bool writePlaceholders);

		u64 RegisterSpeculativeOutputs(FetchContext& context);
		void UnregisterSpeculativeOutputs(u64 id);

	private:
		struct FetchScope;

		u64 MakeId(u32 bucketId);

		// No inlining because they use lots of stack space
		UBA_NOINLINE bool SendPathTable(Bucket& bucket, CacheSendStats& stats, u32 requiredPathTableSize);
		UBA_NOINLINE bool SendCasTable(Bucket& bucket, CacheSendStats& stats, u32 requiredCasTableSize);
		UBA_NOINLINE bool SendCacheEntry(TrackWorkScope& tws, Bucket& bucket, CacheSendStats& stats, const RootPaths& rootPaths, const CasKey& cmdKey, const Map<u32, u32>& inputsStringToCasKey, const Map<u32, u32>& outputsStringToCasKey, const u8* logLines, u64 logLinesSize, const ProcessStartInfo& info);
		UBA_NOINLINE bool SendCacheEntryMessage(BinaryReader& out, Bucket& bucket, CacheSendStats& stats, const CasKey& cmdKey, const Map<u32, u32>& inputsStringToCasKey, const Map<u32, u32>& outputsStringToCasKey, const u8* logLines, u64 logLinesSize);
		UBA_NOINLINE bool FetchCasTable(TrackWorkScope& tws, Bucket& bucket, CacheFetchStats& stats, u32 requiredCasTableOffset);
		UBA_NOINLINE bool FetchPathTable(TrackWorkScope& tws, Bucket& bucket, CacheFetchStats& stats, u32 requiredPathTableOffset);
		UBA_NOINLINE bool FetchFile(Bucket& bucket, const RootPaths& rootPaths, const ProcessStartInfo& info, CacheFetchStats& cacheStats, StorageStats& storageStats, u32 casKeyOffset, bool writePlaceholders);
		template<typename TableType> bool FetchCompactTable(u32 bucketId, TableType& table, u32 requiredTableOffset, u8 messageType);
		bool HasEnoughCasData(Bucket& bucket, u32 requiredCasTableOffset);
		bool HasEnoughPathData(Bucket& bucket, u32 requiredPathTableOffset);
		UBA_NOINLINE bool ReportUsedEntry(Vector<ProcessLogLine>& outLogLines, bool ownedLogLines, Bucket& bucket, const CasKey& cmdKey, u32 entryId);
		bool PopulateLogLines(Vector<ProcessLogLine>& outLogLines, const u8* mem, u64 memLen);

		UBA_NOINLINE CasKey GetCmdKey(const RootPaths& rootPaths, const ProcessStartInfo& info, bool report, u32 bucketId);
		bool DevirtualizePath(StringBufferBase& inOut, RootsHandle rootsHandle);
		bool ShouldNormalize(const StringBufferBase& path);

		bool GetLocalPathAndCasKey(Bucket& bucket, const RootPaths& rootPaths, StringBufferBase& outPath, CasKey& outKey, CompactCasKeyTable& casKeyTable, CompactPathTable& pathTable, u32 offset);
		UBA_NOINLINE void PreparseDirectory(const StringKey& fileNameKey, const StringBufferBase& filePath);

		MutableLogger m_logger;
		StorageImpl& m_storage;
		NetworkClient& m_client;
		Session& m_session;
		bool m_reportCacheKey;
		bool m_reportMissReason;
		bool m_useDirectoryPreParsing;
		bool m_validateCacheWritesInput;
		bool m_validateCacheWritesOutput;
		bool m_useRoots;
		bool m_useCacheHit;
		bool m_useMappingForCreatedCas;

		Atomic<bool> m_connected;

		Futex m_bucketsLock;
		UnorderedMap<u32, Bucket> m_buckets;

		Futex m_sendOneAtTheTimeLock;

		Futex m_directoryPreparserLock;
		struct PreparedDir { Futex lock; Atomic<bool> done = false; };
		UnorderedMap<StringKey, PreparedDir> m_directoryPreparser;

		struct PathHash { TString path; CasKey hash; };
		Vector<PathHash> m_pathHashes;

		Futex m_speculativeCasKeysLock;
		UnorderedMap<StringKey, CasKey> m_speculativeCasKeys;
		UnorderedMap<u64, Vector<StringKey>> m_speculativeOutputsById;
		u64 m_nextSpeculativeOutputId = 1;

		#if UBA_CACHE_USE_SEND_MAX_BUFFER_CACHE
		BufferCache m_sendBufferSlots;
		#endif

		CacheClient(const CacheClient&) = delete;
		CacheClient& operator=(const CacheClient&) = delete;
		struct DowngradedLogger;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////
}