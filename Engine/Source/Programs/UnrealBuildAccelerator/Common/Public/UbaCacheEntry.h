// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBinaryReaderWriter.h"
#include "UbaList.h"
#include "UbaLogger.h"
#include "UbaMap.h"
#include "UbaUnorderedMap.h"

namespace uba
{
	class CompactCasKeyTable;
	class StorageImpl;
	template<typename Key, typename Value> struct HashMap2;

	enum LogLinesType : u8
	{
		LogLinesType_Empty,
		LogLinesType_Shared,
		LogLinesType_Owned,
	};

	struct CacheEntry
	{
		// Times are in file time from creation of cache database
		u64 creationTime = 0;
		u64 lastUsedTime = 0;
		
		u32 id = 0;
		LogLinesType logLinesType = LogLinesType_Empty;

		Vector<u8> sharedInputCasKeyOffsetRanges;
		Vector<u8> extraInputCasKeyOffsets;
		Vector<u8> outputCasKeyOffsets;
		Vector<u8> logLines;
	};

	struct CacheEntries
	{
		ReaderWriterLock lock;
		List<CacheEntry> entries;
		Vector<u8> sharedInputCasKeyOffsets;
		Vector<u8> sharedLogLines;
		u32 idCounter = 0;
		u32 primaryId = ~0u; // Id of entry that shared offsets was made from
		UnorderedSet<u32> inputsThatAreOutputs;

		u64 GetSharedSize();
		u64 GetEntrySize(CacheEntry& entry, u32 clientVersion, bool toDisk);
		u64 GetTotalSize(u32 clientVersion, bool toDisk);
		bool Write(BinaryWriter& writer, u32 clientVersion, bool toDisk);
		bool ReadFromDisk(Logger& logger, BinaryReader& reader, u32 databaseVersion, StorageImpl& storage, CompactCasKeyTable& table);
		void BuildInputs(CacheEntry& entry, const Set<u32>& inputs);
		void UpdateEntries(Logger& logger, const HashMap2<u32, u32>& oldToNewCasKeyOffset, Vector<u32>& temp, Vector<u8>& temp2, Vector<u8>& temp3);
		bool Validate(Logger& logger);

		void Flatten(Vector<u8>& out, const CacheEntry& entry);
		void Flatten(Vector<u32>& out, const CacheEntry& entry, const Vector<u8>& sharedOffsets);

		template<typename Container>
		void BuildInputsT(CacheEntry& entry, const Container& sortedInputs, bool populateShared, Vector<u8>& temp);

		template<typename Container>
		void BuildRangesFromExcludedT(CacheEntry& entry, const Container& sortedExcludedInputs);

		void PopulateInputsThatAreOutputs(const Vector<u8>& inputData, StorageImpl& storage, CompactCasKeyTable& table);
	};


	struct CacheEntriesTraverser
	{
		CacheEntriesTraverser(BinaryReader& r) : reader(r)
		{
			entryCount = reader.ReadU16();
			if (!reader.GetLeft())
				return;
			u64 sharedSize = r.Read7BitEncoded();
			sharedInputOffsets = r.GetPositionData();
			reader.Skip(sharedSize);
		}

		template<typename Func>
		bool TraverseEntryInputs(const Func& func)
		{
			lastId = u32(reader.Read7BitEncoded());
			u64 extraSize = reader.Read7BitEncoded();
			BinaryReader extraReader(reader.GetPositionData(), 0, extraSize);
			reader.Skip(extraSize);
			u64 rangeSize = reader.Read7BitEncoded();
			BinaryReader rangeReader(reader.GetPositionData(), 0, rangeSize);
			reader.Skip(rangeSize);

			while (extraReader.GetLeft())
				if (!func(u32(extraReader.Read7BitEncoded())))
					return false;

			while (rangeReader.GetLeft())
			{
				u64 begin = rangeReader.Read7BitEncoded();
				u64 end = rangeReader.Read7BitEncoded();
				BinaryReader rangeReader2(sharedInputOffsets + begin, 0, end - begin);
				while (rangeReader2.GetLeft())
					if (!func(u32(rangeReader2.Read7BitEncoded())))
						return false;
			}
			return true;
		}

		template<typename Func>
		bool TraverseEntryOutputs(const Func& func)
		{
			u64 outSize = reader.Read7BitEncoded();
			BinaryReader outReader(reader.GetPositionData(), 0, outSize);
			reader.Skip(outSize);
			while (outReader.GetLeft())
				if (!func(u32(outReader.Read7BitEncoded())))
					return false;
			return true;
		}

		void SkipEntryOutputs()
		{
			u64 outSize = reader.Read7BitEncoded();
			reader.Skip(outSize);
		}

		BinaryReader& reader;
		const u8* sharedInputOffsets = nullptr;
		u32 entryCount = 0;
		u32 lastId = 0;
	};
}
