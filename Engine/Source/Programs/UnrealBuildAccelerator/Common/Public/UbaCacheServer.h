// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaHash.h"
#include "UbaLogger.h"
#include "UbaMap.h"
#include "UbaStringBuffer.h"
#include "UbaUnorderedMap.h"

namespace uba
{
	class CompactCasKeyTable;
	class Config;
	class NetworkServer;
	class StorageServer;
	struct BinaryReader;
	struct BinaryWriter;
	struct CacheBucket;
	struct CacheEntry;
	struct CacheEntries;
	struct ConnectionInfo;

	struct CacheServerCreateInfo
	{
		CacheServerCreateInfo(StorageServer& s, const tchar* rd, LogWriter& w = g_consoleLogWriter) : storage(s), rootDir(rd), logWriter(w) {}

		// Storage server
		StorageServer& storage;

		// Root dir
		const tchar* rootDir = nullptr;

		// Log writer
		LogWriter& logWriter;

		// Will check cache entry inputs of they depend on cas files that have been deleted
		bool checkInputsForDeletedCas = true;

		// The time cache entries will stay around after they were last used in hours (defaults to two days)
		// Set to zero to never expire
		u64 expirationTimeSeconds = 2*24*60*60;

		// The amount of reserved memory used per bucket when doing maintenance
		u64 maintenanceReserveSize = 256ull * 1024 * 1024;

		// Max size of cas bucket. When within 2mb it will start decreasing expiry time by one hour
		u64 bucketCasTableMaxSize = 32ull * 1024 * 1024;

		void Apply(Config& config);
	};

	class CacheServer
	{
	public:
		CacheServer(const CacheServerCreateInfo& info);
		~CacheServer();

		bool Load(bool validateBuckets);
		bool Save();

		void SetForceFullMaintenance();

		void PrintStatusLine(const tchar* additionalInfo);
		bool RunMaintenance(bool force, bool allowSave, const Function<bool()>& shouldExit);

		bool ShouldShutdown();

	private:
		using Bucket = CacheBucket;

		bool RunMaintenanceInternal(const Function<bool()>& shouldExit, bool allowSave);
		bool SaveBucket(Bucket& bucket, Vector<u8>& temp);
		bool SaveNoLock();
		bool SaveDbNoLock();
		bool DeleteEmptyBuckets();
		void OnDisconnected(u32 clientId);

		struct Connection;
		struct ConnectionBucket;

		ConnectionBucket& GetConnectionBucket(const ConnectionInfo& connectionInfo, BinaryReader& reader, u32* outClientVersion = nullptr);
		Bucket& GetBucket(BinaryReader& reader, const tchar* reason);
		Bucket& GetBucket(u64 id, const tchar* reason, bool addCommon = true);
		u32 GetBucketWorkerCount();

		bool HandleMessage(const ConnectionInfo& connectionInfo, u8 messageType, BinaryReader& reader, BinaryWriter& writer);
		bool HandleStoreEntry(ConnectionBucket& bucket, BinaryReader& reader, BinaryWriter& writer, u32 clientVersion, u32 clientId);
		bool HandleStoreEntryDone(const ConnectionInfo& connectionInfo, BinaryReader& reader);
		bool HandleFetchPathTable(BinaryReader& reader, BinaryWriter& writer);
		bool HandleFetchCasTable(BinaryReader& reader, BinaryWriter& writer);
		bool HandleFetchPathTable2(BinaryReader& reader, BinaryWriter& writer);
		bool HandleFetchCasTable2(BinaryReader& reader, BinaryWriter& writer);
		bool HandleFetchEntries(BinaryReader& reader, BinaryWriter& writer, u32 clientId);
		bool HandleReportUsedEntry(BinaryReader& reader, BinaryWriter& writer, u32 clientId);
		bool HandleExecuteCommand(BinaryReader& reader, BinaryWriter& writer);

		MutableLogger m_logger;
		NetworkServer& m_server;
		StorageServer& m_storage;

		StringBuffer<MaxPath> m_rootDir;

		Atomic<u32> m_addsSinceMaintenance = 0;
		Atomic<u64> m_cacheKeyFetchCount = 0;
		Atomic<u64> m_cacheKeyHitCount = 0;
		Atomic<bool> m_isRunningMaintenance = false;
		Atomic<bool> m_bucketIsOverflowing = false;
		Atomic<bool> m_forceMaintenance = false;

		Futex m_bucketsLock;
		Map<u64, Bucket> m_buckets;

		Futex m_connectionsLock;
		Map<u32, Connection> m_connections;

		UnorderedSet<CasKey> m_trackedDeletes;

		Atomic<bool> m_shutdownRequested = false;
		Atomic<u64> m_totalEntryCount = 0;

		u64 m_maintenanceReserveSize = 0;
		u64 m_bucketCasTableMaxSize = 0;
		u64 m_creationTime = 0;
		u64 m_bootTime = 0;
		u64 m_lastMaintenance = 0;
		u64 m_longestMaintenance = 0;
		u64 m_expirationTimeSeconds = 0;
		u32 m_peakConnectionCount = 0;
		bool m_dbfileDirty = false;

		bool m_checkInputsForDeletedCas = true;

		bool m_shouldWipe = false;
		bool m_forceAllSteps = false;

		StringKey m_statusLineKey;

		CacheServer(const CacheServer&) = delete;
		CacheServer& operator=(const CacheServer&) = delete;
	};
}