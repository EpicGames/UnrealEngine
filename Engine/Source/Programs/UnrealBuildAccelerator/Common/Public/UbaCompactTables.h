// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaHash.h"
#include "UbaHashMap.h"
#include "UbaUnorderedMap.h"

namespace uba
{
	class Logger;
	struct BinaryReader;
	struct BinaryWriter;
	struct StringKey;

	static constexpr u32 PathTableMaxSize = 128*1024*1024; // Max size for string memory
	static constexpr u32 CasKeyTableMaxSize = 256*1024*1024;

	class CompactPathTable
	{
	public:
		CompactPathTable(bool caseSensitive, u64 reservePathCount, u64 reserveSegmentCount, u32 version);

		u32 Add(const tchar* str, u64 strLen);
		u32 Add(const tchar* str, u64 strLen, u32& outRequiredPathTableSize);
		u32 AddNoLock(const tchar* str, u64 strLen);

		struct AddContext;
		u32 AddNoLock(AddContext& context, u32 offset);

		void AddCommonStringSegments();

		bool GetString(StringBufferBase& out, u64 offset) const;
		bool TryGetString(Logger& logger, StringBufferBase& out, u64 offset) const;

		u8* GetMemory();
		u32 GetSize() const;
		u32 GetCommonSize();
		bool GetCaseInsensitive();
		u32 GetVersion();

		bool ReadMem(BinaryReader& reader, bool populateLookup);
		u8* BeginCommit(u64 size);
		void EndCommit(u8* data, u64 written);
		void Swap(CompactPathTable& other);

		u64 GetPathCount() { return m_offsets.Size(); }
		u64 GetSegmentCount() { return m_segmentOffsets.Size(); }

		template<typename Func>
		bool TraversePaths(const Func& func) const;

		void InitMem();

		enum { MaxSegments = 48 }; // This number is arbitrary.. don't know how many folders/dots deep something can be. We want it to be as low as possible to catch corrupt tables
		struct AddContext
		{
			const CompactPathTable& fromTable;
			StringBuffer<> path;
			u32 fromOffsets[MaxSegments] = { 0 };
			u32 toOffsets[MaxSegments] = { 0 };
			u32 pathOffsets[MaxSegments] = { 0 };
			u32 offsetsCount = 1;
		};

	private:
		u32 InternalAdd(const tchar* str, const tchar* stringKeyString, u64 strLen);
		u32 InternalAddWithNewSegment(u32 offset, u32 parentOffset, bool separator, const StringView& segment);
		void InternalAddWithExistingSegment(u32 offset, u32 parentOffset, bool separator, u32 segmentOffset);

		mutable Futex m_lock;
		MemoryBlock m_pathTableMem;
		HashMap<StringKey, u32, true> m_offsets;
		HashMap<StringKey, u32, true> m_segmentOffsets;
		u32 m_version = 0;
		u32 m_commonSize = 0;
		bool m_caseInsensitive = true;

		CompactPathTable(const CompactPathTable&) = delete;
		void operator=(const CompactPathTable&) = delete;
	};

	class CompactCasKeyTable
	{
	public:
		CompactCasKeyTable(u64 reserveOffsetsCount = 1024);
		~CompactCasKeyTable();

		u32 Add(const CasKey& casKey, u64 stringOffset, u32& outRequiredCasTableSize);
		u32 AddNoLock(const CasKey& casKey, u64 stringOffset);

		template<typename Func>
		void TraverseOffsets(const CasKey& casKey, const Func& func) const;

		void GetKey(CasKey& outKey, u64 offset) const;
		bool GetPathAndKey(StringBufferBase& outPath, CasKey& outKey, const CompactPathTable& pathTable, u64 offset) const;

		u8* GetMemory();
		u32 GetSize();

		void ReadMem(BinaryReader& reader, bool populateLookup);
		u8* BeginCommit(u64 size);
		void EndCommit(u8* data, u64 written);
		void Swap(CompactCasKeyTable& other);

		void Debug(const CompactPathTable& pathTable);

		u64 GetKeyCount() { return m_offsets.Size(); }

	private:
		u32* InternalAdd(const CasKey& casKey, u64 stringOffset, bool& outAdded);

		Futex m_lock;
		MemoryBlock m_casKeyTableMem;

		struct StringAndKey
		{
			u32 stringOffset;
			u32 casKeyOffset;
		};

		struct Value
		{
			union
			{
				StringAndKey single;
				StringAndKey* array;
				UnorderedMap<u32, u32>* stringLookup;
			};
			u32 count; // If count is 1, single is set, otherwise array is allocated up to 8 elements and contains two offsets per entry. After 8 it turns to a stringlookup
		};

		HashMap<CasKey, Value, true> m_offsets;

		CompactCasKeyTable(const CompactCasKeyTable&) = delete;
		void operator=(const CompactCasKeyTable&) = delete;
	};

	template<typename Func>
	bool CompactPathTable::TraversePaths(const Func& func) const
	{
		for (auto i = m_offsets.ValuesBegin(), e = m_offsets.ValuesEnd(); i!=e; ++i)
		{
			StringBuffer path;
			if (!GetString(path, *i))
				return false;
			func(path);
		}
		return true;
	}

	template<typename Func>
	void CompactCasKeyTable::TraverseOffsets(const CasKey& casKey, const Func& func) const
	{
		const Value* valuePtr = m_offsets.Find(casKey);
		if (!valuePtr)
			return;
		const Value& value = *valuePtr;
		if (value.count == 1)
			func(value.single.casKeyOffset);
		else if (value.count <= 16)
		{
			for (u32 i=0, e=value.count; i!=e; ++i)
				func(value.array[i].casKeyOffset);
		}
		else
		{
			for (auto& kv : *value.stringLookup)
				func(kv.second);
		}
	}
}
