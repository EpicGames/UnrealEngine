// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaMap.h"
#include "UbaVector.h"
#include "UbaStringBuffer.h"
#include "UbaUnorderedMap.h"

namespace uba
{
	class Config;
	class Logger;


	class ConfigTable
	{
	public:
		bool GetValueAsString(const tchar*& out, const tchar* key) const;
		bool GetValueAsString(TString& out, const tchar* key) const;
		bool GetValueAsU32(u32& out, const tchar* key) const;
		bool GetValueAsU64(u64& out, const tchar* key) const;
		bool GetValueAsBytes(u64& out, const tchar* key) const;
		bool GetValueAsInt(int& out, const tchar* key) const;
		bool GetValueAsBool(bool& out, const tchar* key) const;

		const ConfigTable* GetTable(const tchar* name) const;

		template<typename Func /* = void(const tchar* name, const ConfigTable&) */> 
		void IterateTables(const Func& func) const { for (auto& kv : m_tables) func(kv.first.c_str(), kv.second); }

		ConfigTable& AddTable(const tchar* name);

		void AddValue(const tchar* key, int value);
		void AddValue(const tchar* key, u32 value);
		void AddValue(const tchar* key, u64 value);
		void AddValue(const tchar* key, bool value);
		void AddValue(const tchar* key, const tchar* str);

		bool SaveToText(Logger& logger, Vector<char>& outText) const;
		bool LoadFromText(Logger& logger, const char* text, u64 textLen);

	private:
		ConfigTable* m_parent = nullptr;
		enum ValueType { ValueType_Value, ValueType_String };
		struct Value { ValueType type; TString string; };
		Map<TString, Value> m_values;
		UnorderedMap<TString, ConfigTable> m_tables;
		friend Config;
	};


	class Config : public ConfigTable
	{
	public:
		bool LoadFromFile(Logger& logger, const tchar* configFile);
		bool IsLoaded() const;

		bool SaveToFile(Logger& logger, const tchar* configFile) const;

		bool m_isLoaded = false;
	};
}
