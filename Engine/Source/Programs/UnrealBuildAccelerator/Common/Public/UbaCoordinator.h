// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaLogWriter.h"

namespace uba
{
	struct CoordinatorCreateInfo
	{
		const tchar* workDir = nullptr;
		const tchar* binariesDir = nullptr;
		u32 maxCoreCount = 0;
		bool logging = false;
	};

	class Coordinator
	{
	public:
		using AddClientCallback = bool(void* userData, const tchar* ip, u16 port, const tchar* crypto);
		virtual void SetAddClientCallback(AddClientCallback* callback, void* userData) = 0;
		virtual void SetTargetCoreCount(u32 count) = 0;

		using UpdateStatusCallback = void(void* userData, const tchar* status);
		virtual void SetUpdateStatusCallback(UpdateStatusCallback* callback, void* userData) {}

		using LogCallback = void(void* userData, LogEntryType type, const tchar* str);
		virtual void SetLogCallback(LogCallback* callback, void* userData) {}

		virtual bool RequestCacheServer(tchar* outEndpoint, u32 outEndpointCapacity, Guid& outSessionKey, bool& outWriteAccess) { return false; }

		// Tells the coordinator that no more agents should be requested. Subsequent
		// SetTargetCoreCount calls become no-ops. Called before the coordinator-update
		// loop is shut down so a final in-flight SetTargetCoreCount returns immediately
		// instead of spawning new agents whose connect we'll then have to abort.
		virtual void Stop() {}
	};

	// This is how the function signatures creating/destroying the coordinator module needs to exported
	using UbaCreateCoordinatorFunc = Coordinator*(const CoordinatorCreateInfo& info);
	using UbaDestroyCoordinatorFunc = void(Coordinator* coordinator);
}