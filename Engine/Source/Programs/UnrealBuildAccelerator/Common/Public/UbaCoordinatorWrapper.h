// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaCoordinator.h"
#include "UbaFile.h"
#include "UbaNetworkServer.h"
#include "UbaScheduler.h"
#include "UbaSessionServer.h"
#include "UbaStringBuffer.h"

#if !PLATFORM_WINDOWS
#include <dlfcn.h>
#define GetProcAddress dlsym
#define LoadLibrary(name) dlopen(name, RTLD_LAZY);
#define LoadLibraryError dlerror()
#define HMODULE void*
#else
#define LoadLibraryError LastErrorToText().data
#endif

namespace uba
{
	class CoordinatorWrapper
	{
	public:
		bool Create(Logger& logger, const tchar* coordinatorType, const CoordinatorCreateInfo& info, NetworkBackend& networkBackend, NetworkServer& networkServer, Trace& trace, u32 traceRow = 10)
		{
			UbaCreateCoordinatorFunc* createCoordinator = nullptr;

			if (!*coordinatorType)
				return false;

			StringBuffer<> coordinatorBin(info.binariesDir);
			coordinatorBin.EnsureEndsWithSlash();

			#if PLATFORM_WINDOWS
			coordinatorBin.Append(TCV("UbaCoordinator")).Append(coordinatorType).Append(TCV(".dll"));
			#elif PLATFORM_MAC
			coordinatorBin.Append(TCV("libUbaCoordinator")).Append(coordinatorType).Append(TCV(".dylib"));
			#else
			coordinatorBin.Append(TCV("libUbaCoordinator")).Append(coordinatorType).Append(TCV(".so"));
			#endif

			HMODULE coordinatorModule = LoadLibrary(coordinatorBin.data);
			if (!coordinatorModule)
				return logger.Error(TC("Failed to load coordinator binary %s (%s)"), coordinatorBin.data, LoadLibraryError);

			if (!coordinatorModule)
				return logger.Error(TC("Failed to load coordinator binary %s (%s)"), coordinatorBin.data, LastErrorToText().data);
			createCoordinator = (UbaCreateCoordinatorFunc*)(void*)GetProcAddress(coordinatorModule,"UbaCreateCoordinator");
			if (!createCoordinator)
				return logger.Error(TC("Failed to find UbaCreateCoordinator function inside %s (%s)"), coordinatorBin.data, LastErrorToText().data);
			m_destroyCoordinator = (UbaDestroyCoordinatorFunc*)(void*)GetProcAddress(coordinatorModule, "UbaDestroyCoordinator");
			if (!m_destroyCoordinator)
				return logger.Error(TC("Failed to find UbaDestroyCoordinator function inside %s (%s)"), coordinatorBin.data, LastErrorToText().data);

			m_coordinator = createCoordinator(info);
			if (!m_coordinator)
				return false;

			m_traceRow = traceRow;
			m_coordinator->SetUpdateStatusCallback([](void* userData, const tchar* status)
				{
					auto& cw = *(CoordinatorWrapper*)userData;
					cw.m_trace->StatusUpdate(cw.m_traceRow, 6, ToView(status), LogEntryType_Info, {});
				}, this);

			m_coordinator->SetAddClientCallback([](void* userData, const tchar* ip, u16 port, const tchar* crypto)
				{
					auto& cw = *(CoordinatorWrapper*)userData;
					return cw.m_networkServer->AddClient(*cw.m_networkBackend, ip, port);
				}, this);

			m_coordinator->SetLogCallback([](void* userData, LogEntryType type, const tchar* str)
				{
					auto& logger = *(Logger*)userData;
					logger.Log(type, str, TStrlen(str));
				}, &logger);

			m_networkBackend = &networkBackend;
			m_networkServer = &networkServer;
			m_trace = &trace;
			m_trace->StatusUpdate(m_traceRow, 1, ToView(coordinatorType), LogEntryType_Info, {});

			if (info.maxCoreCount)
				RequestHelpers(info.maxCoreCount);
			return true;
		}

		bool IsCreated()
		{
			return m_coordinator != nullptr;
		}

		void SetScheduler(Scheduler* scheduler)
		{
			m_scheduler = scheduler;
		}

		bool RequestCacheServer(StringBufferBase& outEndpoint, Guid& outSessionKey, bool& outWriteAccess)
		{
			UBA_ASSERT(m_coordinator);
			if (!m_coordinator->RequestCacheServer(outEndpoint.data, outEndpoint.capacity, outSessionKey, outWriteAccess))
				return false;
			outEndpoint.count = TStrlen(outEndpoint.data);
			return true;
		}

		bool RequestHelpers(u32 maxCoreCount)
		{
			if (!m_coordinator)
				return false;
			if (m_loopCoordinator.IsCreated())
				return false;
			m_loopCoordinator.Create(EventResetType_Manual);
			m_coordinatorThread.Start([this, mcc = maxCoreCount]() { ThreadUpdate(mcc); return 0; }, TC("UbaCoordWrap"));
			return true;
		}

		void ThreadUpdate(u32 maxCoreCount)
		{
			do
			{
				u32 coreCount = maxCoreCount;
				if (m_scheduler)
					coreCount = Min(m_scheduler->GetProcessCountThatCanRunRemotelyNow(), maxCoreCount);

				m_coordinator->SetTargetCoreCount(coreCount); // Need to be called all the time to request more helpers
			}
			while (!m_loopCoordinator.IsSet(2000));
		}

		void Destroy()
		{
			if (!m_coordinator)
				return;
			// Tell the coordinator to stop accepting new SetTargetCoreCount work BEFORE we
			// signal the update thread to exit. Otherwise its in-flight iteration can spawn
			// fresh agents whose connect/handshake we'd then have to cancel on the way out.
			m_coordinator->Stop();
			if (m_loopCoordinator.IsCreated())
			{
				m_loopCoordinator.Set();
				m_coordinatorThread.Wait();
			}
			m_destroyCoordinator(m_coordinator);
			m_coordinator = nullptr;
		}

		Coordinator* m_coordinator = nullptr;
		NetworkBackend* m_networkBackend = nullptr;
		NetworkServer* m_networkServer = nullptr;
		Trace* m_trace;
		Scheduler* m_scheduler = nullptr;
		UbaDestroyCoordinatorFunc* m_destroyCoordinator = nullptr;
		Event m_loopCoordinator;
		Thread m_coordinatorThread;
		u32 m_traceRow = 0;
	};
}