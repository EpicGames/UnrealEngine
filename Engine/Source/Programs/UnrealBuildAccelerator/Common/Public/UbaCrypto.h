// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBase.h"

namespace uba
{
	class Logger;

	enum CryptoKey : u64 {};
	inline constexpr CryptoKey InvalidCryptoKey = (CryptoKey)0;

	class Crypto
	{
	public:
		static CryptoKey CreateKey(Logger& logger, const u8* key128);
		static CryptoKey DuplicateKey(Logger& logger, CryptoKey original);
		static void DestroyKey(CryptoKey key);

		static bool Encrypt(Logger& logger, CryptoKey key, u8* data, u32 size, Guid& inOutInitVector);
		static bool Decrypt(Logger& logger, CryptoKey key, u8* data, u32 size, Guid& inOutInitVector);
	};

	bool CryptoFromString(u8* out, u32 outSize, const tchar* str);
}
