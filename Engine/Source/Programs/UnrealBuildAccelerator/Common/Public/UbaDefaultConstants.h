// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBase.h"

namespace uba
{
	// Default constants
	static constexpr u16 DefaultPort = 1345;
	static constexpr u16 DefaultStorageProxyPort = DefaultPort + 1;
	static constexpr u16 DefaultCachePort = DefaultPort + 2;
	static constexpr u32 SendDefaultSize = 256*1024;
	static constexpr u32 DefaultNetworkReceiveTimeoutSeconds = 10*60;
	static constexpr u32 DefaultNetworkSendTimeoutSeconds = 10*60;


	static constexpr u32 ProcessPriority_Normal = 0x00000020;
	static constexpr u32 ProcessPriority_BelowNormal = 0x00004000;

	static constexpr u32 DefaultMemoryWorkingSet = 512*1024*1024;
}
