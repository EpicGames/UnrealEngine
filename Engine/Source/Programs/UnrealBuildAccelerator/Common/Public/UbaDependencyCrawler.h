// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaApplicationRules.h"
#include "UbaBottleneck.h"
#include "UbaHash.h"
#include "UbaLogger.h"
#include "UbaPathUtils.h"
#include "UbaProcessUtils.h"
#include "UbaUnorderedMap.h"
#include "UbaWorkManager.h"

#if UBA_DEBUG
#define UBA_LOG_DEVIRTUALIZATION_ERROR(x, ...) m_logger.Info(TC(x), __VA_ARGS__)
#else
#define UBA_LOG_DEVIRTUALIZATION_ERROR(x, ...) do { } while(false)
#endif

#define UBA_CRAWLER_DEBUG_PRINT(str1, str2) //wprintf(TC("") str1 ": %s\n", str2);

// Note
// DependencyCrawler is quite Unreal specific. It might work in different environments but is based on using response (.rsp) files.
// It does parse some defines to figure out custom paths but code should also work without this
// Ideally it should be a full fledged preprocessor parser but that would also not make it possible to run this in parallel

namespace uba
{
	class DependencyCrawler
	{
	public:
		inline DependencyCrawler(Logger& logger, WorkManager& workManager, u32 bottleneckMax = ~0u);

		using FileExistsFunc = Function<bool(const StringView& fileName, u32& outAttr)>;
		using FileFunc = Function<void(const StringView& file, bool isDirectory)>;
		using TraverseFilesFunc = Function<void(const StringView& path, const FileFunc& fileFunc)>;

		using AccessFileFunc = Function<bool(const void* data, u64 dataSize)>;
		using DevirtualizePathFunc = Function<bool(StringBufferBase& inOut)>;
		using CreateFileFunc = Function<bool(TrackWorkScope& tracker, const StringView& fileName, const AccessFileFunc& func, bool& stopCrawling)>;

		inline void Init(FileExistsFunc&& fileExistsFunc, TraverseFilesFunc&& traverseFilesFunc, bool useBloomFilter = true);
		inline bool AddRsp(const tchar* rsp, const tchar* workDir, CreateFileFunc&& createFileFunc, DevirtualizePathFunc&& devirtualizePathFunc, const tchar* app, DependencyCrawlerType type, u32 ruleIndex);
		inline bool AddCommandLine(const tchar* commandLine, const tchar* workDir, CreateFileFunc&& createFileFunc, DevirtualizePathFunc&& devirtualizePathFunc, const tchar* app, DependencyCrawlerType type, u32 ruleIndex);

		Logger& m_logger;
		WorkManager& m_workManager;

		struct HandledFile { Futex lock; bool handled = false; };
		struct HandledFiles
		{
			Futex lookupLock;
			UnorderedMap<StringKey, HandledFile> lookup;
		};
		HandledFiles m_handledFiles;

		Futex m_pchLookupLock;
		struct Pch { Futex lock; bool handled = false; UnorderedSet<StringKey> files; };
		UnorderedMap<StringKey, Pch> m_pchLookup;
		bool m_useBloomFilter = true;

		struct IncludeRoot
		{
			TString path;
			BloomFilter bloomFilter;
		};

		struct Instance
		{
			DependencyCrawlerType type;
			TString application;
			TString rsp;
			TString workDir;
			TString platform;
			TString compiledPlatform;
			TString overriddenPlatformName;
			TString builtinIncludesDir;
			TString frameworksDir;
			bool platformIsExtension = false;
			bool usePch = false;
			CreateFileFunc createFileFunc;
			DevirtualizePathFunc devirtualizePathFunc;
			Vector<IncludeRoot> includeRoots;
			Pch* pch = nullptr;

			Atomic<u32> refCount = 0;
		};

		struct InstanceRef
		{
			InstanceRef(Instance& i) : instance(i) { ++instance.refCount; }
			InstanceRef(const InstanceRef& i) : instance(i.instance) { ++instance.refCount; }
			~InstanceRef() { if (!--instance.refCount) delete &instance; }
			Instance& instance;
		};

		Futex m_builtIncludesHandledLock;
		UnorderedSet<u32> m_builtIncludesHandled;
		FileExistsFunc m_fileExistsFunc;
		TraverseFilesFunc m_traverseFilesFunc;

		Bottleneck m_bottleneck;

		struct CodeFile { TString path; bool hasPch = false; };
		using CodeFiles = List<CodeFile>;

		inline bool ParseRsp(TrackWorkScope& tracker, Instance& instance, const StringView& rsp, CodeFiles& outCodeFiles);
		template<typename CharType> inline bool ParseRsp2(TrackWorkScope& tracker, Instance& instance, const void* data, u64 dataSize, const tchar* rsp, CodeFiles& outCodeFiles);
		inline bool ParsePch(Pch& pch, Instance& instance, const void* data, u64 dataSize);
		inline bool ParseCodeFile(TrackWorkScope& tracker, Instance& instance, HandledFiles& handledFiles, const StringView& codeFile, bool parseDefines, const StringView& caller);
		inline bool ParseCodeFile2(TrackWorkScope& tracker, Instance& instance, HandledFiles& handledFiles, const void* data, u64 dataSize, const StringView& codeFile, bool parseDefines);
		inline bool HandleInclude(TrackWorkScope& tracker, Instance& instance, HandledFiles& handledFiles, const StringView& rootPath, const StringView& include, const StringView& codeFile, bool parseDefines);
		inline bool TraverseInclude(TrackWorkScope& tracker, Instance& instance, const StringView& rootPathWithSlash);
		inline bool TraverseCodeFiles(const WorkContext& context, Instance& instance, const CodeFiles& codeFiles, u32 ruleIndex);

		inline u32 GetBottleneckMax(WorkManager& workManager, u32 bottleneckMax)
		{
			// We never allow the crawler to use all workers since there could be jobs that do garbage collecting of cas entries 
			u32 max = workManager.GetWorkerCount();
			if (max <= 2)
				return 0;
			return Min(max - 2u, bottleneckMax);
		}
	};

	DependencyCrawler::DependencyCrawler(Logger& logger, WorkManager& workManager, u32 bottleneckMax)
	:	m_logger(logger)
	,	m_workManager(workManager)
	,	m_bottleneck(GetBottleneckMax(workManager, bottleneckMax))
	{
	}

	void DependencyCrawler::Init(FileExistsFunc&& fileExistsFunc, TraverseFilesFunc&& traverseFilesFunc, bool useBloomFilter)
	{
		UBA_ASSERT(m_bottleneck.GetMaxCount() > 0);
		m_fileExistsFunc = std::move(fileExistsFunc);
		m_traverseFilesFunc = std::move(traverseFilesFunc);
		m_useBloomFilter = useBloomFilter;
	}

	bool DependencyCrawler::AddRsp(const tchar* rsp, const tchar* workDir, CreateFileFunc&& createFileFunc, DevirtualizePathFunc&& devirtualizePathFunc, const tchar* app, DependencyCrawlerType type, u32 ruleIndex)
	{
		auto& instance = *new Instance();

		instance.type = type;
		instance.application = app;
		instance.rsp = rsp;
		instance.workDir = workDir;
		UBA_ASSERT(*workDir);
		if (CaseInsensitiveFs)
			ToLower(instance.workDir.data());
		if (instance.workDir.back() != PathSeparator)
			instance.workDir += PathSeparator;
		instance.createFileFunc = std::move(createFileFunc);
		instance.devirtualizePathFunc = std::move(devirtualizePathFunc);

		m_workManager.AddWork([this, ref = InstanceRef(instance), ruleIndex](const WorkContext& context)
			{
				auto& instance = ref.instance;
				CodeFiles codeFiles;
				if (ParseRsp(context.tracker, instance, instance.rsp, codeFiles))
					TraverseCodeFiles(context, instance, codeFiles, ruleIndex);
			}, 1, TC("CrawlRsp"), ColorWork, WorkPriority_Low, &m_bottleneck);
		return true;
	}

	bool DependencyCrawler::AddCommandLine(const tchar* commandLine, const tchar* workDir, CreateFileFunc&& createFileFunc, DevirtualizePathFunc&& devirtualizePathFunc, const tchar* app, DependencyCrawlerType type, u32 ruleIndex)
	{
		auto& instance = *new Instance();

		instance.type = type;
		instance.application = app;
		//instance.rsp = rsp;
		instance.platform = TC("Win64"); // Hard coded platform for now
		instance.workDir = workDir;
		UBA_ASSERT(*workDir);
		if (CaseInsensitiveFs)
			ToLower(instance.workDir.data());
		if (instance.workDir.back() != PathSeparator)
			instance.workDir += PathSeparator;
		instance.createFileFunc = std::move(createFileFunc);
		instance.devirtualizePathFunc = std::move(devirtualizePathFunc);

		m_workManager.AddWork([this, ref = InstanceRef(instance), ruleIndex, cmdLine = TString(commandLine)](const WorkContext& context)
			{
				auto& instance = ref.instance;
				CodeFiles codeFiles;
				if (ParseRsp2<tchar>(context.tracker, instance, cmdLine.data(), cmdLine.size(), TC(""), codeFiles))
					TraverseCodeFiles(context, instance, codeFiles, ruleIndex);
			}, 1, TC("CrawlCmd"), ColorWork, WorkPriority_Low, &m_bottleneck);
		return true;
	}

	bool DependencyCrawler::ParseRsp(TrackWorkScope& tracker, Instance& instance, const StringView& rsp, CodeFiles& outCodeFiles)
	{
		StringBuffer<> fixedPath;
		FixPath(rsp.data, instance.workDir.data(), instance.workDir.size(), fixedPath);
		if (!instance.devirtualizePathFunc(fixedPath))
			UBA_LOG_DEVIRTUALIZATION_ERROR("Failed to devirtualize path %s in %s", fixedPath.data, rsp.data);
		if (CaseInsensitiveFs)
			fixedPath.MakeLower();
		bool stopCrawling = false;
		if (!instance.createFileFunc(tracker, fixedPath, [&](const void* data, u64 dataSize) { return ParseRsp2<char>(tracker, instance, data, dataSize, rsp.data, outCodeFiles); }, stopCrawling))
			return stopCrawling ? false : m_logger.Warning(TC("Failed to parse rsp %s"), rsp.data);
		return true;
	}

	template<typename CharType>
	bool DependencyCrawler::ParseRsp2(TrackWorkScope& tracker, Instance& instance, const void* data, u64 dataSize, const tchar* rsp, CodeFiles& outCodeFiles)
	{
		auto AddCodeFile = [&](StringView file, bool pushFront, bool hasPch, bool fixPath = true)
			{
				UBA_CRAWLER_DEBUG_PRINT("CODEFILE", file.data);
				StringBuffer<> fixedPath2;
				if (fixPath)
				{
					FixPath(file.data, instance.workDir.data(), instance.workDir.size(), fixedPath2);
					if (!instance.devirtualizePathFunc(fixedPath2))
						UBA_LOG_DEVIRTUALIZATION_ERROR("Failed to devirtualize path %s in %s", fixedPath2.data, rsp);

					if (CaseInsensitiveFs)
						fixedPath2.MakeLower();
					file = fixedPath2;
				}
				if (pushFront)
					outCodeFiles.push_front({file.ToString(), hasPch});
				else
					outCodeFiles.push_back({file.ToString(), hasPch});
			};

		auto AddRoot = [&](const StringView& path)
			{
				UBA_CRAWLER_DEBUG_PRINT("ROOT", path.data);
				StringBuffer<> fixedPath2;
				FixPath(path.data, instance.workDir.data(), instance.workDir.size(), fixedPath2);
				if (!instance.devirtualizePathFunc(fixedPath2))
					UBA_LOG_DEVIRTUALIZATION_ERROR("Failed to devirtualize path %s in %s", fixedPath2.data, rsp);
				if (CaseInsensitiveFs)
					fixedPath2.MakeLower();
				fixedPath2.EnsureEndsWithSlash();

				BloomFilter bloomFilter;
				if (m_useBloomFilter)
				{
					m_traverseFilesFunc(fixedPath2, [&](const StringView& file, bool isDirectory)
						{
							bloomFilter.Add(ToStringKey(file));
						});
					if (bloomFilter.IsEmpty())
						return;
				}
				instance.includeRoots.push_back({fixedPath2.ToString(), bloomFilter});
			};

		auto IgnoreOption = [](const StringView& path) { UBA_CRAWLER_DEBUG_PRINT("IGNORED", path.data); };

		if (instance.type == DependencyCrawlerType_MsvcCompiler)
		{
			struct MsvcOptionWithArg { const StringView name; bool addRoot; };
			const MsvcOptionWithArg msvcOptionsWithArg[] = 
			{
				{ TCV("/I"), true },
				{ TCV("/external:I"), true },
				{ TCV("/imsvc"), true },
				{ TCV("/experimental:log"), false },
				{ TCV("/analyze:log"), false },
				{ TCV("/sourceDependencies"), false },
				{ TCV("/headerUnit:quote"), false },
			};

			constexpr const tchar* clangClOptionsWithArg[] = 
			{
				TC("-D"),
				TC("-x"),
				TC("-o"),
				TC("-include"),
				TC("-include-pch"),
				TC("-vctoolsdir"),
				TC("-Xclang"),
				TC("-target"),
				TC("-arch"),
			};

			StringBuffer<> prevArg;
			bool addRoot = false;
			bool handled = false;
			ParseArguments((const CharType*)data, dataSize, [&](const CharType* arg, u32 argLen)
				{
					StringBuffer<> sb;
					sb.Append(arg, argLen);

					if (handled)
					{
						handled = false;
						if (addRoot)
							AddRoot(sb);
						else
							IgnoreOption(sb);
						addRoot = false;
						return;
					}
					else if (sb[0] == '/')
					{
						for (auto& option : msvcOptionsWithArg)
						{
							if (!option.name.Equals(sb))
								continue;
							addRoot = option.addRoot;
							handled = true;
							return;
						}

						if (sb.StartsWith(TC("/FI")))
						{
							// Check if there is a precompiled header deps file
							StringBuffer<> fixedFile;
							FixPath(sb.data + 3, instance.workDir.data(), instance.workDir.size(), fixedFile);
							if (!instance.devirtualizePathFunc(fixedFile))
								UBA_LOG_DEVIRTUALIZATION_ERROR("Failed to devirtualize path %s in %s", fixedFile.data, rsp);

							if (CaseInsensitiveFs)
								fixedFile.MakeLower();

							if (fixedFile.Contains(TC("\\Definitions.")))
							{
								AddCodeFile(fixedFile, true, false, false);
							}
							else
							{
								StringKey key = ToStringKey(fixedFile);
								SCOPED_FUTEX(m_handledFiles.lookupLock, lock);
								auto insres = m_handledFiles.lookup.try_emplace(key);
								lock.Leave();

								HandledFile& handledFile = insres.first->second;
								SCOPED_FUTEX(handledFile.lock, fileLock);
								if (!handledFile.handled)
								{
									handledFile.handled = true;

									fixedFile.Append(TCV(".pch"));

									u32 attributes = 0;
									bool hasPch = m_fileExistsFunc(fixedFile, attributes);
									fixedFile.Resize(fixedFile.count - 4);
									AddCodeFile(fixedFile, true, hasPch, false);
								}
							}

							prevArg.Clear();
							return;
						}
						else if (sb.StartsWith(TC("/Yu")))
						{
							instance.usePch = true;
							prevArg.Clear();
							return;
						}
						else
						{
							IgnoreOption(sb);
						}
						prevArg = sb;
					}
					else if (sb[0] == '-')
					{
						for (auto option : clangClOptionsWithArg)
							handled |= sb.Equals(option);
						if (sb.StartsWith(TC("-resource-dir"), false))
						{
							instance.builtinIncludesDir = sb.data + 14;
							instance.builtinIncludesDir += TC("/include");
							UBA_CRAWLER_DEBUG_PRINT("RESOURCEDIR", sb.data + 14);
						}
						else if (sb.StartsWith(TCV("-I"), false)) // clang-cl.exe allows this
						{
							AddRoot(StringView(sb).Skip(2));
						}
						else
							IgnoreOption(sb);
					}
					else if (sb[0] == '@')
					{
						ParseRsp(tracker, instance, StringView(sb).Skip(1), outCodeFiles);
					}
					else
					{
						AddCodeFile(sb, false, false);
					}
				});
		}
		else if (instance.type == DependencyCrawlerType_ClangCompiler)
		{
			enum ArgType { ArgType_None, ArgType_Ignore, ArgType_RootPath, ArgType_ISysRootPath, ArgType_ResourceDir, ArgType_Code };
			struct OptionWithArg { const StringView name; ArgType type; };
			constexpr const OptionWithArg dashOptionsWithArg[] = 
			{
				{ TCV("-D"), ArgType_Ignore },
				{ TCV("-x"), ArgType_Ignore },
				{ TCV("-o"), ArgType_Ignore },
				{ TCV("-include"), ArgType_Code },
				{ TCV("-include-pch"), ArgType_Code },
				{ TCV("-vctoolsdir"), ArgType_Ignore },
				{ TCV("-Xclang"), ArgType_Ignore },
				{ TCV("-target"), ArgType_Ignore },
				{ TCV("-arch"), ArgType_Ignore },
				{ TCV("--sysroot"), ArgType_RootPath},
				{ TCV("-isystem"), ArgType_RootPath },
				{ TCV("-isysroot"), ArgType_ISysRootPath },
				{ TCV("-internal-isystem"), ArgType_RootPath },
				{ TCV("-I"), ArgType_RootPath },
				{ TCV("-F"), ArgType_RootPath },
				{ TCV("-resource-dir"), ArgType_ResourceDir },
				{ TCV("-dependency-file"), ArgType_Ignore },
				{ TCV("-internal-externc-isystem"), ArgType_Ignore },
				{ TCV("-MT"), ArgType_Ignore },
			};

			//StringBuffer<> prevArg;
			ArgType type = ArgType_None;
			ParseArguments((const char*)data, dataSize, [&](const char* arg, u32 argLen)
				{
					StringBuffer<> sb;
					sb.Append(arg, argLen);

					if (type == ArgType_None)
					{
						if (sb[0] == '-')
						{
							for (auto& option : dashOptionsWithArg)
							{
								if (!sb.Equals(option.name, false))
									continue;
								//prevArg = sb; // Arg comes in next parse call
								type = option.type;
								return;
							}

							if (auto equalsPos = TStrchr(sb.data, '='))
							{
								u32 equalsOffset = u32(equalsPos - sb.data);
								StringView option2(sb.data, equalsOffset);
								for (auto option : dashOptionsWithArg)
								{
									if (!option2.Equals(option.name, false))
										continue;
									++equalsOffset;
									sb.Clear().Append(arg + equalsOffset, argLen - equalsOffset);
									type = option.type;
									//prevArg = option;
									break;
								}
							}
							else if (sb.StartsWith(TC("-I"), false))
							{
								type = ArgType_RootPath;
								sb.Clear().Append(arg + 2, argLen - 2);
							}
							else if (sb.StartsWith(TC("-isystem"), false))
							{
								type = ArgType_RootPath;
								sb.Clear().Append(arg + 8, argLen - 8);
							}
							if (type == ArgType_None)
							{
								IgnoreOption(sb);
								return;
							}
						}
					}

					if (type != ArgType_None)
					{
						switch (type)
						{
						case ArgType_RootPath:
							AddRoot(sb);
							break;
						case ArgType_ISysRootPath:
							AddRoot(sb);
							#if PLATFORM_MAC
							instance.frameworksDir = sb.data;
							instance.frameworksDir += TC("/System/Library/Frameworks/");
							instance.builtinIncludesDir = sb.data;
							instance.builtinIncludesDir += TC("/usr/include");
							#endif
							break;

						case ArgType_Code:
							if (sb.EndsWith(TCV(".gch")) || sb.EndsWith(TCV(".pch")))
								sb.Resize(sb.count - 4);
							AddCodeFile(sb, true, false);
							break;
						case ArgType_ResourceDir:
							instance.builtinIncludesDir = sb.data;
							instance.builtinIncludesDir += TC("/include");
							break;
						default:
							IgnoreOption(sb);
						}
						type = ArgType_None;
						return;
					}
					else if (sb[0] == '@')
						ParseRsp(tracker, instance, StringView(sb).Skip(1), outCodeFiles);
					else if (sb.Contains('/'))
						AddCodeFile(sb, false, false);
					else
						IgnoreOption(sb);
				});
		}
		else if (instance.type == DependencyCrawlerType_MsvcLinker)
		{
			StringBuffer<> prevArg;
			ParseArguments((const char*)data, dataSize, [&](const char* arg, u32 argLen)
				{
					StringBuffer<> sb;
					sb.Append(arg, argLen);

					if (sb[0] == '/')
					{
						if (sb.StartsWith(TC("/LIBPATH")))
							AddRoot(StringView(sb).Skip(9));
						return;
					}
					else
					{
						if (sb.EndsWith(TCV(".obj")))
							AddCodeFile(sb, false, false);
						if (sb.Contains('/') && sb.EndsWith(TCV(".lib")))
							AddCodeFile(sb, false, false);
					}
				});
		}
				
		return true;
	}

	bool DependencyCrawler::ParsePch(Pch& pch, Instance& instance, const void* data, u64 dataSize)
	{
		auto AddHandled = [&](const StringView& str)
			{
				StringBuffer<> fullPath;
				FixPath(str.data, nullptr, 0, fullPath);
				//instance.devirtualizePathFunc(fullPath); // These paths are never virtual
				if (CaseInsensitiveFs)
					fullPath.MakeLower();
				StringKey key = ToStringKey(fullPath);
				pch.files.insert(key);
			};

		StringBuffer<> line;
		char lastChar = 0;

		const char* it = (const char*)data;
		const char* end = it + dataSize;
		for (; it != end; lastChar = *it, ++it)
		{
			if ((line.count == 0 && (*it == ' ' || *it == '\t')) || *it == '\r')
				continue;
			if (*it == '\\' && lastChar == ' ') // Remove the "dir \dir" extra space in clang deps files
			{
				line.Resize(line.count - 1);
				if (line[line.count-1] != ':')
					AddHandled(line);
				line.Clear();
				continue;
			}
			
			if (*it == '\n')
			{
				if (line.count > 3 && !line.Contains(TC("\":")))
				{
					AddHandled(line);
				}
				line.Clear();
				continue;
			}

			if (*it == ' ' && lastChar == '\\')
				--line.count;
			line.Append(*it);
		}
		return true;
	}

	bool DependencyCrawler::ParseCodeFile(TrackWorkScope& tracker, Instance& instance, HandledFiles& handledFiles, const StringView& codeFile, bool parseDefines, const StringView& caller)
	{
		bool stopCrawling = false;
		if (!instance.createFileFunc(tracker, codeFile, [&](const void* data, u64 dataSize) { return ParseCodeFile2(tracker, instance, handledFiles, data, dataSize, codeFile, parseDefines); }, stopCrawling))
			return stopCrawling ? false : m_logger.Warning(TC("Failed to parse code file %s found in %s"), codeFile.data, caller.data);
		return true;
	}

	bool DependencyCrawler::ParseCodeFile2(TrackWorkScope& tracker, Instance& instance, HandledFiles& handledFiles, const void* data, u64 dataSize, const StringView& codeFile, bool parseDefines)
	{
		UBA_CRAWLER_DEBUG_PRINT("PARSE", codeFile.data);

		const char* it = (const char*)data;
		const char* end = it + dataSize;
		bool hasNonSpace = false;

		while (it < end)
		{
			if (*it == '#' && !hasNonSpace)
			{
				++it;
				while (*it == ' ' || *it == '\t')
					++it;

				bool isInclude = strncmp(it, "include", 7) == 0;
				bool isImport = false;

				#if PLATFORM_MAC
				if (!isInclude)
					isImport = strncmp(it, "import", 6) == 0;
				#endif

				if (isInclude || isImport)
				{
					it += isInclude ? 7 : 6;

					while (*it == ' ' || *it == '\t')
						++it;
					StringBuffer<> include;
					bool isQuote = false;

					if (*it == '\"' || *it == '<')
					{
						isQuote = *it == '\"';
						char endChar = isQuote ? '\"' : '>';
						++it;
						const char* includeStart = it;
						const char* includeEnd = strchr(it, endChar);
						if (!includeEnd)
							break;
						u64 includeLen = includeEnd - includeStart;
						if (includeLen >= include.capacity)
							break;
						it = includeEnd + 1;
						include.Append(includeStart, u32(includeLen));
					}
					else
					{
						const char* defineBegin = it;
						const char* defineEnd = nullptr;
						const char* argBegin = nullptr;
						const char* argEnd = nullptr;

						while (it < end)
						{
							if (*it == '\r' || *it == '\n')
							{
								if (!defineEnd)
									defineEnd = it;
								break;
							}
							else if (*it == ' ' || *it == '\t')
							{
								if (!defineEnd)
									defineEnd = it;
							}
							else if (*it == '(')
							{
								defineEnd = it;
								argBegin = it + 1;
							}
							else if (*it == ')')
							{
								argEnd = it;
								break;
							}
							else if (!argBegin && defineEnd)
								break;
							++it;
						}

						u32 defineLen = u32(defineEnd - defineBegin);
						if (argBegin)
						{
							u64 argLen = argEnd - argBegin;
							if (argLen < include.capacity - 32)
							{
								if (strncmp(defineBegin, "UE_INLINE_GENERATED_CPP_BY_NAME", defineLen) == 0)
								{
									include.Append(argBegin, u32(argLen)).Append(TCV(".gen.cpp"));
								}
								else if (strncmp(defineBegin, "COMPILED_PLATFORM_HEADER", defineLen) == 0)
								{
									if (!instance.platform.empty())
									{
										if (!instance.platformIsExtension)
											include.Append(instance.platform).Append(PathSeparator);
										include.Append(instance.platform).Append(argBegin, u32(argLen));
									}
								}
							}
						}
						else if (defineBegin)
						{
							if (strncmp(defineBegin, "PER_MODULE_INLINE_FILE", defineLen) == 0)
							{
								include.Append(TCV("HAL/PerModuleInline.inl")); // TODO: This is not correct.. need revisit
							}
						}
					}

					if (!include.IsEmpty())
					{
						if (CaseInsensitiveFs)
							include.MakeLower();
						include.FixPathSeparators();

						auto work = 
						[this, ref = InstanceRef(instance), isQuote, isInclude, include2 = include.ToString(), codeFile2 = codeFile.ToString(), hf = &handledFiles, parseDefines](const WorkContext& context)
							{
								auto& instance = ref.instance;
								auto& handledFiles = *hf;
								if (isInclude)
								{
									if (isQuote)
									{
										StringBuffer<> localDir;
										if (const tchar* lastSeparator = TStrrchr(codeFile2.c_str(), PathSeparator))
											if (HandleInclude(context.tracker, instance, handledFiles, localDir.Append(codeFile2.c_str(), lastSeparator - codeFile2.c_str() + 1), include2, codeFile2, parseDefines))
												return;
									}

									StringView keyView;
									if (const tchar* firstSeparator = TStrchr(include2.c_str(), PathSeparator))
										keyView = StringView(include2.c_str(), u32(firstSeparator - include2.c_str()));
									else
										keyView = StringView(include2);
									StringKey fileKey =  ToStringKey(keyView);

									for (auto& root : instance.includeRoots)
									{
										if (m_useBloomFilter && root.bloomFilter.IsGuaranteedMiss(fileKey))
											continue;
										if (HandleInclude(context.tracker, instance, handledFiles, root.path, include2, codeFile2, parseDefines))
											return;
									}
								}

								#if PLATFORM_MAC
								if (!instance.frameworksDir.empty())
								{
									if (const tchar* firstSlash = TStrchr(include2.c_str(), '/'))
									{
										// TODO: Make a nicer solution for Frameworks in Frameworks. Also this is not the fastest doing HandleInclude on all these paths
										const StringView frameworkRoots[] = { TCV(""), TCV("CoreServices.framework/Frameworks/"), TCV("ApplicationServices.framework/Frameworks/"), TCV("Carbon.framework/Frameworks/")  };

										for (auto frameworkRoot : frameworkRoots)
										{
											StringBuffer<> tmp;
											tmp.Append(instance.frameworksDir).Append(frameworkRoot).Append(include2.c_str(), firstSlash - include2.c_str()).Append(TCV(".framework/"));
											u64 frameworkLen = tmp.count;
											tmp.Append(TCV("Headers")).Append(firstSlash);
											if (!HandleInclude(context.tracker, instance, handledFiles, {}, tmp, codeFile2, parseDefines))
												continue;
											tmp.Resize(frameworkLen).Append(TCV("Modules/module.modulemap"));
											instance.devirtualizePathFunc(tmp);
											if (CaseInsensitiveFs)
												tmp.MakeLower();
											StringKey moduleKey = ToStringKey(tmp);
											SCOPED_FUTEX(handledFiles.lookupLock, lock);
											if (!handledFiles.lookup.try_emplace(moduleKey).second)
												return;
											lock.Leave();
											bool stopCrawling = false;
											instance.createFileFunc(context.tracker, tmp, {}, stopCrawling);
											return;
										}
									}
								}
								#endif
							};
						if (parseDefines)
							work({tracker});
						else
							m_workManager.AddWork(std::move(work), 1, TC("CrawlIncludes"), ColorWork, WorkPriority_Low, &m_bottleneck);
					}
				}
				else if (parseDefines && strncmp(it, "define", 6) == 0)
				{
					it += 6;

					auto parseDefine = [&](const char* define, u32 defineLength, TString& out)
						{
							if (strncmp(it, define, defineLength) != 0)
								return false;
							it += defineLength;
							while (it < end && (*it == '\t' || *it == ' '))
								++it;
							const char* platformBegin = it;
							while (it < end && *it != '\t' && *it != ' ' && *it != '\r' && *it != '\n')
								++it;
							out.assign(platformBegin, it);
							return true;
						};

					while (*it == ' ' || *it == '\t')
						++it;
					TString platformIsExtension;
					if (parseDefine("UBT_COMPILED_PLATFORM", 21, instance.compiledPlatform))
					{}
					else if (parseDefine("OVERRIDE_PLATFORM_HEADER_NAME", 29, instance.overriddenPlatformName))
					{}
					else if (parseDefine("PLATFORM_IS_EXTENSION", 21, platformIsExtension))
					{
						instance.platformIsExtension = platformIsExtension != TC("0");
					}
				}
			}
			else if (*it == '\n')
			{
				hasNonSpace = false;
			}
			else if (*it != ' ' && *it != '\t')
			{
				hasNonSpace = true;
			}
			++it;
		}

		return true;
	}

	bool DependencyCrawler::HandleInclude(TrackWorkScope& tracker, Instance& instance, HandledFiles& handledFiles, const StringView& rootPath, const StringView& include, const StringView& codeFile, bool parseDefines)
	{
		if (include.EndsWith(TCV(".ush")))
			return true;
		StringBuffer<> fullPath;
		FixPath(include.data, rootPath.data, rootPath.count, fullPath);
		if (IsAbsolutePath(include.data))
			if (!instance.devirtualizePathFunc(fullPath))
				UBA_LOG_DEVIRTUALIZATION_ERROR("Failed to devirtualize include path %s in %s", fullPath.data, codeFile.data);

		if (CaseInsensitiveFs)
			fullPath.MakeLower();

		u32 attributes = 0;
		if (!m_fileExistsFunc(fullPath, attributes))
			return false;
		if (IsDirectory(attributes))
			return false;


		StringKey key = ToStringKey(fullPath);
		if (instance.pch)
		{
			SCOPED_FUTEX_READ(instance.pch->lock, lock);
			if (instance.pch->files.find(key) != instance.pch->files.end())
				return true;
		}
		{
			SCOPED_FUTEX(handledFiles.lookupLock, lock);
			if (!handledFiles.lookup.try_emplace(key).second)
				return true;
		}

		return ParseCodeFile(tracker, instance, handledFiles, fullPath, parseDefines, codeFile);
	}

	bool DependencyCrawler::TraverseInclude(TrackWorkScope& tracker, Instance& instance, const StringView& rootPathWithSlash)
	{
		StringBuffer<> codeFilePath(rootPathWithSlash);
		m_traverseFilesFunc(rootPathWithSlash, [&](const StringView& file, bool isDirectory)
			{
				codeFilePath.Resize(rootPathWithSlash.count).Append(file);
				if (isDirectory)
				{
					codeFilePath.Append(PathSeparator);
					m_workManager.AddWork([this, ref = InstanceRef(instance), codeFile = codeFilePath.ToString()](const WorkContext& context)
						{
							TraverseInclude(context.tracker, ref.instance, codeFile);
						}, 1, TC("CrawlBiDir"), ColorWork, WorkPriority_Low, &m_bottleneck);
				}
				else
				{
					#if PLATFORM_MAC
					if (!file.EndsWith(TCV(".h")) && !file.EndsWith(TCV(".modulemap")) && TStrchr(file.data, '.') != 0)
						return;
					#endif

					m_workManager.AddWork([this, ref = InstanceRef(instance), codeFile = codeFilePath.ToString()](const WorkContext& context)
						{
							bool stopCrawling = false;
							if (!ref.instance.createFileFunc(context.tracker, codeFile, {}, stopCrawling))
								if (!stopCrawling)
									m_logger.Warning(TC("Failed to open file %s from builtindir"), codeFile.c_str());
						}, 1, TC("CrawlBiFile"), ColorWork, WorkPriority_Low, &m_bottleneck);
				}
			});
		return true;
	}

	inline bool DependencyCrawler::TraverseCodeFiles(const WorkContext& context, Instance& instance, const CodeFiles& codeFiles, u32 ruleIndex)
	{
		for (auto& cf : codeFiles)
		{
			if (instance.usePch && cf.hasPch)
			{
				// Note, clang is always reading all the individual files that has been stored in the pch to validate content..
				// So this code path is not implemented for clang.. if -fno-validate-ast-input-files-content is added we can implement this path
				StringBuffer<> fixedFile;
				FixPath(cf.path.c_str(), instance.workDir.data(), instance.workDir.size(), fixedFile);
				if (CaseInsensitiveFs)
					fixedFile.MakeLower();
				StringKey pchKey = ToStringKey(fixedFile);

				SCOPED_FUTEX(m_pchLookupLock, lock);
				Pch& pch = m_pchLookup[pchKey];
				lock.Leave();

				SCOPED_FUTEX(pch.lock, lock2);
				if (!pch.handled)
				{
					fixedFile.Append(TCV(".dep.json"));
					bool stopCrawling = false;
					instance.createFileFunc(context.tracker, fixedFile, [&](const void* data, u64 dataSize)
						{
							return ParsePch(pch, instance, data, dataSize);
						}, stopCrawling);
					if (stopCrawling)
						return true;
					pch.handled = true;
				}
				instance.pch = &pch;
			}
			else if (instance.type == DependencyCrawlerType_MsvcLinker)
			{
				// If we end up here it means that we have compressed obj files enabled and want to decompress them in parallel
				m_workManager.AddWork([this, ref = InstanceRef(instance), codeFile = cf.path](const WorkContext& context)
					{
						bool stopCrawling = false;
						ref.instance.createFileFunc(context.tracker, codeFile, {}, stopCrawling);
					}, 1, TC("CrawlForDecomp"), ColorWork, WorkPriority_Low, &m_bottleneck);
			}
			else
			{
				bool parseDefines = instance.platform.empty();
				HandledFiles handledFiles;
				ParseCodeFile(context.tracker, instance, parseDefines ? handledFiles : m_handledFiles, cf.path, parseDefines, instance.rsp);
				if (instance.platform.empty())
				{
					if (!instance.overriddenPlatformName.empty())
						instance.platform = instance.overriddenPlatformName;
					else
						instance.platform = instance.compiledPlatform;
				}
			}
		}

		if (!instance.builtinIncludesDir.empty())
		{
			SCOPED_FUTEX(m_builtIncludesHandledLock, lock);
			bool shouldHandle = m_builtIncludesHandled.insert(ruleIndex).second;
			lock.Leave();
			if (shouldHandle)
			{
				StringBuffer<> fixedPath;
				FixPath(instance.builtinIncludesDir.c_str(), instance.workDir.data(), instance.workDir.size(), fixedPath);
				if (!instance.devirtualizePathFunc(fixedPath))
					UBA_LOG_DEVIRTUALIZATION_ERROR("Failed to devirtualize path %s found in builtin includes", fixedPath.data);
				if (CaseInsensitiveFs)
					fixedPath.MakeLower();
				fixedPath.Append(PathSeparator);

				TraverseInclude(context.tracker, instance, fixedPath);
			}
		}
		return true;
	}
}
