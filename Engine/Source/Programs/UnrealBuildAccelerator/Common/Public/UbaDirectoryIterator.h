// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaDefinitions.h"
#include "UbaFile.h"
#include "UbaLogger.h"
#include "UbaPathUtils.h"
#include "UbaProcessStats.h"
#include "UbaStringBuffer.h"

namespace uba
{
	struct DirectoryEntry
	{
		const tchar* name;
		u32 nameLen;
		u64 lastWritten;
		u32 attributes;
		u32 volumeSerial;
		u64 id;
		u64 size;
	};

	struct DirectoryInfo
	{
		u32 attributes;
		u32 volumeSerial;
		u64 id;
	};

	using IteratorFunc = Function<void(const DirectoryEntry&)>;
	using DirectoryInfoFunc = Function<void(const DirectoryInfo&)>;
}



#if PLATFORM_WINDOWS
#include "UbaWinTypes.h"

extern "C" NTSTATUS NtQueryDirectoryFile(HANDLE FileHandle, HANDLE Event, PIO_APC_ROUTINE ApcRoutine, PVOID ApcContext, PIO_STATUS_BLOCK IoStatusBlock, PVOID FileInformation, ULONG Length, FILE_INFORMATION_CLASS FileInformationClass, BOOLEAN ReturnSingleEntry, PUNICODE_STRING FileName, BOOLEAN RestartScan);

namespace uba
{
	inline bool TraverseDir(Logger& logger, StringView dirPath, const IteratorFunc& iteratorFunc, bool errorOnNotFound = false, const DirectoryInfoFunc& infoFunc = {})
	{
		StringBuffer<> str;
		str.Append(TCV("\\??\\"));

		if (IsUncPath(dirPath))
		{
			if (!dirPath.Skip(2).Contains('\\'))
				return false;
			str.Append(TCV("UNC")).Append(dirPath.Skip(1));
		}
		else
		{
			str.Append(dirPath);
			if (dirPath[1] == ':' && dirPath[2] == 0) // if we are all the way to the root of the drive we add a slash since NtCreateFile needs that
				str.Append('\\');
		}

		UNICODE_STRING uniName;
		RtlInitUnicodeString(&uniName, str.data);

		Timer timer(0, 1);
		auto mg = MakeGuard([&]() { KernelStats::GetCurrent().traverseDir += timer; });

		HANDLE handle;

		OBJECT_ATTRIBUTES ObjectAttributes;
		IO_STATUS_BLOCK IoStatusBlock;
		memset(&IoStatusBlock, 0, sizeof(IoStatusBlock));
		InitializeObjectAttributes(&ObjectAttributes, &uniName, OBJ_CASE_INSENSITIVE, NULL, NULL);
		ULONG ShareAccess = FILE_SHARE_READ | FILE_SHARE_WRITE | FILE_SHARE_DELETE;

		u64 startTime = GetTime();
		NTSTATUS res = NtCreateFile(&handle, FILE_READ_ATTRIBUTES | FILE_LIST_DIRECTORY | FILE_ATTRIBUTE_UNPINNED, &ObjectAttributes, &IoStatusBlock, 0, FILE_ATTRIBUTE_NORMAL, ShareAccess, FILE_OPEN, FILE_DIRECTORY_FILE | FILE_SYNCHRONOUS_IO_ALERT, NULL, 0);
		timer.time += GetTime() - startTime;

		if (res != STATUS_SUCCESS)
		{
			if (res == STATUS_OBJECT_NAME_NOT_FOUND || res == STATUS_OBJECT_PATH_NOT_FOUND)
				return !errorOnNotFound;
			if (res == STATUS_NOT_A_DIRECTORY || res == STATUS_ACCESS_DENIED || res == STATUS_NO_MEDIA_IN_DEVICE || res == STATUS_FVE_LOCKED_VOLUME)
				return false;
			#if UBA_DEBUG
			return logger.Error(L"NtCreateFile for TraverseDir on %s failed with code 0x%x", str.data, (DWORD)res);
			#else
			return false;
			#endif
		}
		auto hg = MakeGuard([&]() { NtClose(handle); });

		bool exists = IoStatusBlock.Information == 1;
		if (!exists)
			return !errorOnNotFound;

		BY_HANDLE_FILE_INFORMATION dirInfo;
		if (!GetFileInformationByHandle(handle, &dirInfo))
		{
			logger.Warning(L"GetFileInformationByHandle returned error %s", LastErrorToText().data);
			return false;
		}

		u32 volumeSerial = dirInfo.dwVolumeSerialNumber;

		if (infoFunc)
		{
			DirectoryInfo di;
			di.attributes = dirInfo.dwFileAttributes;
			di.volumeSerial = volumeSerial;
			di.id = ToLargeInteger(dirInfo.nFileIndexHigh, dirInfo.nFileIndexLow).QuadPart;
			UBA_ASSERT(di.id);
			infoFunc(di);
		}

		u8 buff[64*1024];
		while (true)
		{
			startTime = GetTime();
			res = NtQueryDirectoryFile(handle, 0, NULL, NULL, &IoStatusBlock, buff, sizeof(buff) - 2, FileIdBothDirectoryInformation, FALSE, NULL, FALSE);
			timer.time += GetTime() - startTime;

			if (res != STATUS_SUCCESS)
			{
				if (res != STATUS_NO_MORE_FILES)
					logger.Error(L"NtQueryDirectoryFile for %s returned error 0x%x", str.data, res);
				break;
			}

			u8* it = buff;
			while (true)
			{
				auto& FileInformation = *(FILE_ID_BOTH_DIR_INFORMATION*)it;

				auto fileName = FileInformation.FileName;
				u64 fileNameCharCount = FileInformation.FileNameLength/sizeof(tchar);
				wchar_t& fileNameEnd = fileName[fileNameCharCount];
				wchar_t old = fileNameEnd;
				fileNameEnd = 0;
				if (fileNameCharCount == 1 && fileName[0] == '.' || (fileNameCharCount == 2 && fileName[0] == '.' && fileName[1] == '.'))
				{
					if (!FileInformation.NextEntryOffset)
						break;
					fileNameEnd = old;
					it += FileInformation.NextEntryOffset;
					continue;
				}

				DirectoryEntry entry;
				entry.name = fileName;
				entry.nameLen = u32(fileNameCharCount);
				entry.lastWritten = FileInformation.LastWriteTime.QuadPart;
				entry.attributes = FileInformation.FileAttributes;
				entry.volumeSerial = volumeSerial;
				entry.id = FileInformation.FileId.QuadPart;
				entry.size = FileInformation.EndOfFile.QuadPart;
				iteratorFunc(entry);

				if (!FileInformation.NextEntryOffset)
					break;
				fileNameEnd = old;
				it += FileInformation.NextEntryOffset;
			}
		}
		return true;
	}
}

#else

#include <dirent.h>

namespace uba
{
	inline bool TraverseDir(Logger& logger, StringView dirPath, const IteratorFunc& iteratorFunc, bool errorOnNotFound = false, const DirectoryInfoFunc& infoFunc = {})
	{
		char dirPath2[1024];
		u32 dirPathLen = dirPath.count;

		memcpy(dirPath2, dirPath.data, dirPathLen);
		dirPath2[dirPathLen] = 0;

		DIR* dir = opendir(dirPath2);
		if (!dir)
		{
			if (errno == ENOENT || errno == ENOTDIR)
				return !errorOnNotFound;
			UBA_ASSERTF(false, "TraverseDir error handling not implemented %s (%s)", dirPath2, strerror(errno));
			return false;
		}

		auto dg = MakeGuard([&]() { closedir(dir); });

		if (infoFunc)
		{
			int fd = dirfd(dir);
			if (fd == -1)
			{
				UBA_ASSERTF(false, "TraverseDir:dirfd error handling not implemented");
				return false;
			}
			struct stat dirAttr;
			int res = fstat(fd, &dirAttr);
			if (res == -1)
			{
				UBA_ASSERTF(false, "TraverseDir:fstat error handling not implemented");
				return false;
			}
			DirectoryInfo di;
			di.attributes = dirAttr.st_mode;
			di.volumeSerial = dirAttr.st_dev;
			di.id = dirAttr.st_ino;

			infoFunc(di);
		}


		if (dirPath2[dirPathLen - 1] != '/')
			dirPath2[dirPathLen++] = '/';

		while (true)
		{
			errno = 0;
			struct dirent* pDirent = readdir(dir);
			if (!pDirent)
			{
				if (errno == 0)
					break;

				// This is actually an error.. should return false?
				break;
			}

			char* fileName = pDirent->d_name;
			if (fileName[0] == '.' && (fileName[1] == 0 || (fileName[1] == '.' && fileName[2] == 0))) 
				continue;

			strcpy(dirPath2 + dirPathLen, fileName);

			struct stat attr;
			bool skipEntry = false;

			while (true)
			{
				#if !UBA_SUPPORT_SYMLINKS
				int res = stat(dirPath2, &attr);
				if (res == 0)
					break;
				if (errno == ENOENT)
				#endif

				if (lstat(dirPath2, &attr) == 0)
					break;

				if (errno == EAGAIN)
					continue;

				skipEntry = true;
				// ENOENT is a benign race: readdir returned an entry that was
				// unlinked before we could stat it. Common for /proc/<pid> dirs
				// where the process exits mid-traversal.
				UBA_ASSERTF(errno == ENODEV || errno == EACCES || errno == EPERM || errno == ENOENT, "TraverseDir:lstat error handling not added for file %s (error %s)", dirPath2, strerror(errno));
				break;
			}

			if (skipEntry)
				continue;

			DirectoryEntry entry;
			entry.name = fileName;
			entry.nameLen = strlen(fileName);
			entry.lastWritten = FromTimeSpec(attr.st_mtimespec);
			entry.attributes = attr.st_mode;
			entry.volumeSerial = attr.st_dev;
			entry.id = attr.st_ino;
			entry.size = attr.st_size;
			iteratorFunc(entry);
		}

		return true;
	}
}
#endif
