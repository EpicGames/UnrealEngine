// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaDirectoryTable.h"
#include "UbaFile.h"
#include "UbaFileMappingHandle.h"
#include "UbaMap.h"

namespace uba
{
	class Logger;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct DirectoryTableOverlay
	{
		ReaderWriterLock lock;
		FileMappingHandle handle;
		u8* memory = nullptr;
		u32 committed = 0;
		u32 size = 0;
		struct Dir { u32 offset = 0; UnorderedMap<StringKey, u32> files; };
		UnorderedMap<StringKey, Dir> lookup;
		Vector<StringKey> roots;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct OverlayEditScope
	{
		OverlayEditScope(DirectoryTableOverlay& o);
		~OverlayEditScope();

		DirectoryTableOverlay& overlay;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class DirectoryTableHolder
	{
	public:
		DirectoryTableHolder(LogWriter& logWriter, const tchar* logPrefix = TC(""), bool runningRemote = false);
		virtual ~DirectoryTableHolder();
		void SetTreatTempDirAsEmpty(); // We never want to populate files in Temp

		bool RefreshDirectory(const tchar* dirPath, bool forceRegister = false); // Tell uba a directory on disk has been changed by some other system while session is running
		bool RegisterNewFile(const tchar* filePath); // Tell uba a new file on disk has been added by some other system while session is running
		void RegisterDeleteFile(const tchar* filePath); // Tell uba a file on disk has been deleted by some other system while session is running
		bool RegisterNewDirectory(const tchar* directoryPath, bool ensureAncestors = false); // Tell uba a directory on disk has been added by some other system while session is running
		bool EnsurePath(StringView path); // Ensure path's parent chain is at populatedOffset==1, so a subsequent register/write of path will see well-formed ancestors. The leaf is never inspected (no GetFileInformation), so it need not exist on disk yet. Repairs ancestors stuck at populatedOffset==2 from out-of-order RegisterNewDirectory calls.

		const u8* GetDirectoryTableMemory();
		DirTableSize GetDirectoryTableSize(DirectoryTableOverlay* overlay);

		void CreateOverlay(DirectoryTableOverlay& overlay);
		void ResetOverlay(DirectoryTableOverlay& overlay, void* data, u64 dataSize);
		bool EnsureOverlayMemory(DirectoryTableOverlay& overlay, u64 neededSize);
		void DestroyOverlay(DirectoryTableOverlay& overlay);

		void PrintOverlay(DirectoryTableOverlay& overlay);
		void PrintOverlay(DirectoryTableOverlay& overlay, DirectoryTableOverlay::Dir& dir, StringBufferBase& path);

		bool RegisterOverlayCreateOrWrite(OverlayEditScope& scope, StringKey filePathKey, StringView filePath, u64 fileSize, u64 lastWriteTime, u32 attributes, StringView symlinkContent = {});
		bool RegisterOverlayDelete(OverlayEditScope& scope, StringKey filePathKey, StringView filePath);
		bool RegisterNonRootedDir(OverlayEditScope& scope, StringView dirPath); // Will not register parent dirs recursively

		using GetOverlayFileInformation = Function<bool(FileInformation& out, StringKey, StringView)>;
		bool PopulateOverlayFile(DirectoryTableOverlay& overlay, StringView filePath, const GetOverlayFileInformation& getFileInformation = {});

		bool AddOverlayFile(DirTableOffset& outOffset, OverlayEditScope& scope, StringView filePath);
		bool WriteOverlayEntry(OverlayEditScope& scope, StringView filePath, const FileInformation& fileInfo, StringView symlinkContent = {}, DirTableOffset* outOffset = nullptr);
		bool WriteOverlayEntry(OverlayEditScope& scope, StringKey dirKey, StringView filePath, StringView fileName, const FileInformation& fileInfo, bool populateMainTableParentDir, StringView symlinkContent = {}, DirTableOffset* outOffset = nullptr);
		bool WriteOverlayEntry(OverlayEditScope& scope, StringKey dirKey, DirectoryTableOverlay::Dir& dir, StringView fileName, const FileInformation& fileInfo, u32& outFileOffset, StringView symlinkContent = {});
		bool WriteOverlayDirInLookup(OverlayEditScope& scope, StringView tempDir);
		bool WriteOverlayDirectoryEntriesRecursive(OverlayEditScope& scope, StringKey filePathKey, StringView filePath);

		bool RegisterCreateFileForWrite(StringKey filePathKey, StringView filePath, u64 fileSize, u64 lastWriteTime, u32 attributes, bool registerRealFile, bool invalidateStorage);
		bool RegisterDeleteFile(StringKey filePathKey, StringView filePath);

		u32 WriteDirectoryEntries(StringKey dirKey, StringView dirPath, DirTableOffset* outTableOffset = nullptr);
		void WriteDirectoryEntriesRecursive(DirTableOffset& outTableOffset, StringKey dirKey, StringView dirPath, StringKey tableOffsetIsFile);
		bool WriteDirectoryEntriesInternal(DirTableOffset& outTableOffset, DirectoryTable::Directory& dir, const StringKey& dirKey, StringView dirPath, bool isRefresh);
		
		void EnsureDirectoryTableMemory(u64 neededSize);

		virtual bool IsInTemp(StringView path);
		virtual void FileEntryAdded(StringKey filePathKey, u64 lastWritten, u64 size);
		virtual bool ShouldWriteToDisk(StringView filePath);
		virtual void InvalidateCachedFileInfo(const StringKey& filePathKey);

		bool WriteDirectoryTableToFile(StringView filePath);

		#if UBA_DEBUG_LOGGER
		void CreateDebugLogger(const tchar* outputFile);
		#endif

		MutableLogger m_logger;
		bool m_runningRemote;

		Atomic<u64> m_fileIndexCounter = 8000000000; // Try not to collide with ntfs
		MemoryBlock m_directoryTableMemory;
		FileMappingHandle m_directoryTableHandle;
		u8* m_directoryTableMem;
		u64 m_directoryTableMemCommitted = 0;
		DirectoryTable m_directoryTable;
		StringKey m_directoryForcedEmpty;
		VolumeCache m_volumeCache;

		#if UBA_DEBUG_LOGGER
		Logger* m_debugLogger = nullptr;
		#endif

	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool GetDirKey(StringKey& outDirKey, StringBufferBase& outDirPath, const tchar*& outLastSlash, StringView filePath);
	StringKey GetKeyAndFixedName(StringBuffer<>& outFixedFilePath, const tchar* filePath);

	struct DirTableSnapshot
	{
		FileMappingHandle handle;
		u8* mappedMemory = 0; // consumers must mmap at same VA — m_lookup buckets hold raw Node*
		u32 mappedSize = 0;
		Atomic<u32> refCount = 0;
	};
	bool CreateDirectoryTableSnapshot(DirTableSnapshot& out, u64 mappingSize, const u8* prevSnapshot, u64 prevSnapshotSize, const u8* mainMem, u32 mainSize, const u8* overlayMem, u32 overlaySize, const tchar* hint, Logger& logger);

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
