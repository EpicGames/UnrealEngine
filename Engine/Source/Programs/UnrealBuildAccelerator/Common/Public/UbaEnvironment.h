// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaLogger.h"

namespace uba
{
	enum MutexHandle : u64 {};
	inline constexpr MutexHandle InvalidMutexHandle = MutexHandle(0);
	MutexHandle CreateMutexW(bool bInitialOwner, const tchar* name);
	void ReleaseMutex(MutexHandle mutex);
	void CloseMutex(MutexHandle mutex);

	enum ProcHandle : u64 {};
	inline constexpr ProcHandle InvalidProcHandle = (ProcHandle)0ull;
	ProcHandle GetCurrentProcessHandle();

	u32 GetEnvironmentVariableW(const tchar* name, tchar* buffer, u32 nSize);
	bool SetEnvironmentVariableW(const tchar* name, const tchar* value);
	u32 ExpandEnvironmentStringsW(const tchar* lpSrc, tchar* lpDst, u32 nSize);
	u32 GetLogicalProcessorCount();
	u32 GetProcessorGroupCount();

	void ElevateCurrentThreadPriority();
	void PrefetchVirtualMemory(const void* mem, u64 size);

	void GetMachineId(StringBufferBase& out);
	bool GetMemoryInfo(Logger& logger, u64& outAvailable, u64& outTotal, u64* outMaxPageFileSize = nullptr);
	void GetSystemInfo(Logger& logger, StringBufferBase& out);
	bool GetCpuTime(u64& outTotalTime, u64& outIdleTime);

	void SetMemoryWorkingSet(Logger& logger, u64 minWorkingSet);
	void LockMemory(Logger& logger, void* dest, u64 size, const tchar* hint);

	#if !PLATFORM_WINDOWS
	void SetupOrphanReaper(); // Fork sidecar surviving host death; kills every descendant (cgroup.kill on Linux, ps-sid walk on macOS).
	void CancelProcessTree(int leaderPid); // Synchronous SIGKILL of leader + every descendant via kill(-pgid) + /proc walk by UBA_SESSION_PROCESS.
	#if PLATFORM_LINUX
	const char* GetOrphanReaperCgroup(); // Cgroup v2 path created by SetupOrphanReaper; empty when delegation unavailable.
	#endif
	#endif // !PLATFORM_WINDOWS

	#if PLATFORM_WINDOWS
	StringView GetUbaWineModulePath();
	void* GetUbaWineModule();
	#endif
}