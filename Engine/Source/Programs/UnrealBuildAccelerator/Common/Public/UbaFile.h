// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaFileHandle.h"
#include "UbaLogger.h"
#include "UbaMemory.h"
#include "UbaPlatform.h"
#include "UbaUnorderedMap.h"
#include <string> // TODO: Remove

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	#if !PLATFORM_WINDOWS
	inline constexpr u32 ERROR_FILE_NOT_FOUND = ENOENT;
	inline constexpr u32 ERROR_PATH_NOT_FOUND = ENOENT;
	inline constexpr u32 ERROR_ALREADY_EXISTS = EEXIST;
	inline constexpr u32 ERROR_FILE_EXISTS = EEXIST;
	inline constexpr u32 ERROR_ACCESS_DENIED = EACCES;
	inline constexpr u32 ERROR_DIRECTORY = ENOTDIR;
	inline constexpr u32 ERROR_DIR_NOT_EMPTY = ENOTEMPTY;
	inline constexpr u32 MOVEFILE_REPLACE_EXISTING = 0x00000001;
	inline constexpr u32 INVALID_FILE_ATTRIBUTES = (u32)-1;

	inline constexpr u32 GENERIC_WRITE = 0x40000000L;
	inline constexpr u32 DELETE = 0x00010000L;
	inline constexpr u32 GENERIC_READ = 0x80000000L;
	inline constexpr u32 FILE_WRITE_ATTRIBUTES = 0x100L;

	inline constexpr u32 FILE_SHARE_READ = 0x00000001;
	inline constexpr u32 FILE_SHARE_WRITE = 0x00000002;
	inline constexpr u32 FILE_SHARE_DELETE = 0x00000004;

	inline constexpr u32 FILE_FLAG_NO_BUFFERING = 0;
	inline constexpr u32 FILE_FLAG_OVERLAPPED = 0;
	inline constexpr u32 FILE_FLAG_BACKUP_SEMANTICS = 0x02000000;
	inline constexpr u32 FILE_FLAG_SEQUENTIAL_SCAN = 0;

	inline constexpr u32 CREATE_NEW = 1;
	inline constexpr u32 CREATE_ALWAYS = 2;
	inline constexpr u32 OPEN_EXISTING = 3;
	#endif
	inline constexpr u32 FILE_SHARE_ALL = 0x00000007;

	#if PLATFORM_WINDOWS
	inline constexpr u64 OverlappedIoFlag = 0x0000'0001'0000'0000; // Only used by windows
	#endif

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct FileBasicInformation
	{
		u32 attributes = 0;
		u64 lastWriteTime = 0;
		u64 size = 0;
	};
	bool GetFileBasicInformationByHandle(FileBasicInformation& out, Logger& logger, const tchar* fileName, FileHandle hFile, bool errorOnFail = true);
	bool GetFileBasicInformation(FileBasicInformation& out, Logger& logger, const tchar* fileName, bool errorOnFail = true);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct FileInformation
	{
		u32 attributes = 0;
		u32 volumeSerialNumber = 0;
		u64 lastWriteTime = 0;
		u64 size = 0;
		u64 index = 0;
	};
	bool GetFileInformationByHandle(FileInformation& out, Logger& logger, const tchar* fileName, FileHandle hFile);
	bool GetFileInformation(FileInformation& out, Logger& logger, StringView fileName);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool ReadFile(Logger& logger, const tchar* fileName, FileHandle fileHandle, void* b, u64 bufferLen);
	bool WriteFile(Logger& logger, const tchar* fileName, FileHandle fileHandle, const void* b, u64 bufferLen);
	bool OpenFileSequentialRead(Logger& logger, const tchar* fileName, FileHandle& outHandle, bool fileNotFoundIsError = true, bool overlapped = false, bool showErrors = true);
	bool FileExists(Logger& logger, const tchar* fileName, u64* outSize = nullptr, u32* outAttributes = nullptr, u64* lastWriteTime = nullptr);
	bool SetFilePointer(Logger& logger, const tchar* fileName, FileHandle handle, u64 position);
	bool SetEndOfFile(Logger& logger, const tchar* fileName, FileHandle handle, u64 size);
	bool MakeSparse(Logger& logger, const tchar* fileName, FileHandle h, u64 size);
	bool GetDirectoryOfCurrentModule(Logger& logger, StringBufferBase& out);
	bool DeleteAllFiles(Logger& logger, const tchar* dir, bool deleteDir = true, u32* count = nullptr);
	bool SearchPathForFile(Logger& logger, StringBufferBase& out, StringView file, StringView workingDir, StringView applicationDir);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	FileHandle CreateFileW(const tchar* fileName, u32 desiredAccess, u32 shareMode, u32 createDisp, u32 flagsAndAttributes);
	bool CloseFile(const tchar* fileName, FileHandle h);
	bool CreateDirectoryW(const tchar* pathName);
	bool RemoveDirectoryW(const tchar* pathName);
	bool DeleteFileW(const tchar* fileName);
	bool CopyFileW(const tchar* existingFileName, const tchar* newFileName, bool bFailIfExists, u8* scratch = nullptr, u64 scratchSize = 0);
	u32 GetLongPathNameW(const tchar* lpszShortPath, tchar* lpszLongPath, u32 cchBuffer);
	bool GetFileLastWriteTime(u64& outTime, FileHandle hFile);
	bool SetFileLastWriteTime(FileHandle fileHandle, u64 writeTime);
	bool MoveFileExW(const tchar* existingFileName, const tchar* newFileName, u32 dwFlags);
	bool GetFileSizeEx(u64& outFileSize, FileHandle hFile);

	u32 GetFileAttributesW(const tchar* fileName);
	bool IsReadOnly(u32 attributes);
	u32 DefaultAttributes(bool execute = false);
	u32 DefaultDirAttributes();
	bool CreateHardLinkW(const tchar* newFileName, const tchar* existingFileName);
	u32 GetFullPathNameW(const tchar* fileName, u32 nBufferLength, tchar* lpBuffer, tchar** lpFilePart);
	bool SearchPathW(const tchar* a, const tchar* b, const tchar* c, u32 d, tchar* e, tchar** f);
	u64 GetSystemTimeAsFileTime();
	u64 GetFileTimeAsSeconds(u64 fileTime);
	u64 GetFileTimeAsTime(u64 fileTime);
	u64 GetSecondsAsFileTime(u64 seconds);
	bool GetCurrentDirectoryW(StringBufferBase& out);
	bool GetAlternativeUbaPath(Logger& logger, StringBufferBase& out, StringView firstPath, bool isWindowsArm);

	void ReportSystemError(u32 error);
	void SetSystemErrorCallback(const Function<void(u32)>& callback);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class SearchPathCache
	{
	public:
		bool SearchPathForFile(Logger& logger, StringBufferBase& out, StringView file, StringView workingDir, StringView applicationDir);

	private:
		Futex m_lock;
		UnorderedMap<TString, TString> m_map;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class DirectoryCache
	{
	public:
		bool CreateDirectory(Logger& logger, const tchar* dir, bool* outAlreadyExists = nullptr);
		void Clear();

	private:
		Futex m_createdDirsLock;
		struct CreatedDir { Futex lock; Atomic<bool> handled = false; };
		UnorderedMap<TString, CreatedDir> m_createdDirs;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	template<typename LineFunc>
	bool ReadLines(Logger& logger, const tchar* file, const LineFunc& lineFunc)
	{
		FileHandle handle;
		if (!OpenFileSequentialRead(logger, file, handle))
			return logger.Error(TC("Failed to open file %s"), file);
		auto fg = MakeGuard([&]() { CloseFile(file, handle); });
		u64 fileSize = 0;
		if (!GetFileSizeEx(fileSize, handle))
			return logger.Error(TC("Failed to get size of file %s"), file);
		char buffer[512];
		u64 left = fileSize;
		std::string line;
		while (left)
		{
			u64 toRead = Min(left, u64(sizeof(buffer)));
			left -= toRead;
			if (!ReadFile(logger, file, handle, buffer, toRead))
				return false;

			u64 start = 0;
			for (u64 i=0;i!=toRead;++i)
			{
				if (buffer[i] != '\n')
					continue;
				u64 end = i;
				if (i > 0 && buffer[i-1] == '\r')
					--end;
				line.append(buffer + start, end - start);
				if (!line.empty())
					if (!lineFunc(TString(line.begin(), line.end())))
						return false;
				line.clear();
				start = i + 1;
			}
			if (toRead && buffer[toRead - 1] == '\r')
				--toRead;
			line.append(buffer + start, toRead - start);
		}
		if (!line.empty())
			if (!lineFunc(TString(line.begin(), line.end())))
				return false;
		return true;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct VolumeCache
	{
		~VolumeCache();
		bool Init(Logger& logger);
		u32 GetSerialIndex(u32 volumeSerial);
		void Write(BinaryWriter& writer);
		void Read(BinaryReader& reader);

		struct Volume
		{
			u32 serialNumber = 0;
			TString drives;
			FileHandle handle = InvalidFileHandle;
			u64 prevQueryTime = 0;
			u64 prevIdleTime = 0;
			u32 prevReadCount = 0;
			u32 prevWriteCount = 0;
			u64 prevReadBytes = 0;
			u64 prevWriteBytes = 0;

			bool UpdateStats(u8& outBusyPercent, u32& outReadCount, u64& outReadBytes, u32& outWriteCount, u64& outWriteBytes);
		};

		Vector<Volume> volumes;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
