// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaFile.h"
#include "UbaFileMapping.h"

namespace uba
{
	class FileAccessor
	{
	public:
		FileAccessor(Logger& logger, const tchar* fileName);
		~FileAccessor();

		// tempPath is only used by posix and will create a temp file and then move it to place when done. (Since DeleteOnClose does not exist)
		bool CreateWrite(bool allowRead = false, u32 flagsAndAttributes = DefaultAttributes(), u64 size = 0, const tchar* tempPath = nullptr);
		bool CreateMemoryWrite(bool allowRead = false, u32 flagsAndAttributes = DefaultAttributes(), u64 size = 0, const tchar* tempPath = nullptr);
		bool Close(u64* lastWriteTime = nullptr);

		bool Write(const void* data, u64 dataLen, u64 offset = 0, bool lastWrite = false);

		bool OpenRead();
		bool OpenMemoryRead(u64 offset = 0, bool errorOnFail = true);


		const tchar* GetFileName() { return m_fileName; }
		inline FileHandle GetHandle() { return m_fileHandle; }
		inline u8* GetData() { return m_data; }
		inline u64 GetSize() { return m_size; }

		// Can be called if file is opened
		bool GetFileBasicInformationByHandle(FileBasicInformation& out);

	private:
		bool InternalCreateWrite(bool allowRead, u32 flagsAndAttributes, u64 size, const tchar* tempPath, bool isMemoryMap);
		bool InternalWrite(const void* data, u64 dataLen, u64 offset, bool lastWrite);
		bool InternalClose(bool success, u64* lastWriteTime);
		Logger& m_logger;
		const tchar* m_fileName;
		FileHandle m_fileHandle = InvalidFileHandle;
		FileMappingHandle m_mappingHandle;
		u64 m_size = 0;
		u8* m_data = nullptr;
		u32 m_flagsAndAttributes = 0;
		bool m_isWrite = false;

		#if !PLATFORM_WINDOWS
		const tchar* m_tempPath = nullptr;
		u32 m_tempFileIndex = 0;
		#endif

		u32 m_writeThroughBufferSize = 0;
		u8* m_writeThroughBuffer = nullptr;
		u64 m_writeThroughBufferPos = 0;
	};
}