// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaHash.h"
#include "UbaSharedMemoryView.h"

namespace uba
{
	class Logger;
	class SharedMemoryAllocator;
	class Trace;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct MappedView
	{
		SharedMemoryHandle handle;
		u64 offset = 0;
		u64 size = 0;
		u8* memory = nullptr;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class FileMappingBuffer
	{
	public:
		FileMappingBuffer(Logger& logger, SharedMemoryAllocator& allocator);
		~FileMappingBuffer();

		bool Init(const tchar* name, u64 capacity, u64 commitStepSize);

		MappedView AllocAndMapView(u64 size, u64 alignment, const tchar* hint, bool isIndependent = false);
		MappedView MapView(SharedMemoryHandle handle, u64 offset, u64 size, const tchar* hint);
		MappedView MapView(StringView handleName, u64 size, const tchar* hint);
		void UnmapView(MappedView view, const tchar* hint, u64 newSize = InvalidValue);

		bool IsIndependent(SharedMemoryHandle handle);
		void CloseIndependent(SharedMemoryHandle handle, u64 size, const tchar* hint);

		SharedMemoryHandle GetMemoryHandle();
		u64 GetUsed();

		void TraceStatus(Trace& trace, u32 startRow, const tchar* hint, u32 traceLevel);

	private:
		MappedView InternalMapView(SharedMemoryHandle handle, u64 offset, u64 size, const tchar* hint);

		Logger& m_logger;
		SharedMemoryAllocator& m_allocator;

		TString m_name;
		u64 m_commitStepSize = 0;

		SharedMemoryHandle m_memoryHandle;
		Futex m_usedLock;
		u8* m_memory = nullptr;
		u64 m_capacity = 0;
		u64 m_committed = 0;
		u64 m_used = 0;

		Atomic<u64> m_independentMappingCount = 0;
		Atomic<u64> m_independentSize = 0;

		FileMappingBuffer(const FileMappingBuffer&) = delete;
		void operator=(const FileMappingBuffer&) = delete;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

}
