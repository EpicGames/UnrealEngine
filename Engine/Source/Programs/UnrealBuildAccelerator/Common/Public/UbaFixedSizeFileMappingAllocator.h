// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaFileMappingHandle.h"
#include "UbaMap.h"

namespace uba
{
	class Logger;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct FileMappingAllocation
	{
		FileMappingHandle handle;
		u64 offset = 0;
		u8* memory = nullptr;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class FixedSizeFileMappingAllocator
	{
	public:
		FixedSizeFileMappingAllocator(Logger& logger, const tchar* name);
		~FixedSizeFileMappingAllocator();

		bool Init(u64 blockSize, u64 capacity);

		FileMappingAllocation Alloc(const tchar* hint);
		void Free(FileMappingAllocation allocation);

		u64 GetSize();

	private:
		Logger& m_logger;
		const tchar* m_name;
		u64 m_capacity = 0;
		u64 m_blockSize = 0;

		Futex m_mappingLock;
		FileMappingHandle m_mappingHandle;
		u64 m_mappingCount = 0;

		Set<u64> m_availableBlocks;

		FixedSizeFileMappingAllocator(const FixedSizeFileMappingAllocator&) = delete;
		void operator=(const FixedSizeFileMappingAllocator&) = delete;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////
}