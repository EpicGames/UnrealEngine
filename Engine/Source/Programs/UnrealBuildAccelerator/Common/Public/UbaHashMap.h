// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaMemory.h"
#include "UbaPlatform.h"

namespace uba
{
	// Hash map that contains three parts
	//  1. Lookup table indexed with hashed key. Points to index in entry table
	//  2. Entry table. Contains actual key and an index to next entry. Entry table order is stable during rehash
	//  3. Value table. Matches entry table ordering.

	template<typename Key, typename Value, bool AllowGrow = false>
	struct HashMap
	{
		struct Entry
		{
			Key key;
			u32 next; // If this is 0 it means that it is unused, ~0u means it is the end of a chain, otherwise offset into entry array
		};

		~HashMap()
		{
			if (m_ownsMemory)
				delete m_memory;
		}

		void Init(MemoryBlock& mem, u64 reserveCount, const tchar* hint = TC(""))
		{
			reserveCount = Max(reserveCount + 1, 4ull);
			m_memory = &mem;
			UBA_ASSERT(!AllowGrow);
			InternalInit(reserveCount, RoundUpPow2(reserveCount), hint);
		}

		void Init(u64 reserveCount = 4, const tchar* hint = TC(""))
		{
			reserveCount = Max(reserveCount + 1, 4ull);
			u64 lookupCount = RoundUpPow2(reserveCount);

			u64 lookupSize = lookupCount*sizeof(u32);
			u64 entriesSize = sizeof(Entry)*(reserveCount+1);
			u64 valuesSize = sizeof(Value)*(reserveCount+1);
			m_memory = new MemoryBlock(hint);
			m_memory->Init(lookupSize + entriesSize + valuesSize);
			m_ownsMemory = true;

			InternalInit(reserveCount, lookupCount, hint);
		}

		bool IsInitialized()
		{
			return m_memory != nullptr;
		}

		Value& Insert(const Key& key)
		{
			bool added = false;
			return Insert(key, added);
		}

		Value& Insert(const Key& key, bool& outAdded)
		{
			u32 index = InsertIndex(key, outAdded);
			return m_values[index];
		}

		u32 InsertIndex(const Key& key, bool& outAdded)
		{
			UBA_ASSERT(m_nextAvailableEntry < m_reserveCount || AllowGrow);
			if constexpr (AllowGrow)
				if (m_nextAvailableEntry == m_reserveCount)
				{
					// Create a new map, populate and swap. Note that we need the entries and values to be at the same index as before
					HashMap newMap;
					newMap.Init(GetNextReserveCount(m_reserveCount));
					for (u32 i=1, e=m_nextAvailableEntry; i!=e; ++i)
						newMap.Insert(m_entries[i].key) = m_values[i];
					Swap(newMap);
				}

			u32 index = std::hash<Key>()(key) & m_mask;
			u32 entryIndex = m_lookup[index];

			if (entryIndex == 0)
			{
				entryIndex = m_nextAvailableEntry++;
				m_lookup[index] = entryIndex;
				Entry& entry = m_entries[entryIndex];
				entry.key = key;
				entry.next = ~0u;
				outAdded = true;
				return entryIndex;
			}

			while (true)
			{
				Entry& entry = m_entries[entryIndex];
				if (entry.key == key)
					return entryIndex;

				if (entry.next != ~0u)
				{
					entryIndex = entry.next;
					//static u32 counter = 0;
					//printf("MULTIPLE: %u\n", counter++);
					continue;
				}

				u32 newEntryIndex = entry.next = m_nextAvailableEntry++;
				Entry& newEntry = m_entries[newEntryIndex];
				newEntry.key = key;
				newEntry.next = ~0u;
				outAdded = true;
				return newEntryIndex;
			}
		}

		Value* Find(const Key& key) const
		{
			u32 index = std::hash<Key>()(key) & m_mask;
			u32 entryIndex = m_lookup[index];
			if (entryIndex == 0)
				return nullptr;
			Entry* entry = m_entries + entryIndex;
			while (true)
			{
				if (entry->key == key)
					return m_values + entryIndex;
				entryIndex = entry->next;
				if (entryIndex == ~0u)
					return nullptr;
				entry = m_entries + entryIndex;
			}
		}

		const Key* GetKey(Value* value)
		{
			u32 pos = u32(value - m_values);
			auto& entry = m_entries[pos];
			if (entry.next == 0)
				return nullptr;
			return &entry.key;
		}

		Value& GetValueFromIndex(u32 index)
		{
			return m_values[index];
		}

		u32 Size()
		{
			return m_nextAvailableEntry - 1;
		}

		void Erase(const Key& key)
		{
			UBA_ASSERT(!AllowGrow);
			u32 index = std::hash<Key>()(key) & m_mask;
			u32 entryIndex = m_lookup[index];
			if (entryIndex == 0)
				return;
			Entry* entry = m_entries + entryIndex;
			u32* prevNext = nullptr;
			while (true)
			{
				if (entry->key == key)
				{
					if (!prevNext)
					{
						if (entry->next == ~0u)
							m_lookup[index] = 0;
						else
							m_lookup[index] = entry->next;
					}
					else
						*prevNext = entry->next;
					entry->next = 0;

					// TODO: We could swap with last and reduce size

					return;
				}
				entryIndex = entry->next;
				if (entryIndex == ~0u)
					return;
				prevNext = &entry->next;
				entry = m_entries + entryIndex;
			}
		}

		Value* ValuesBegin() const { return m_values + 1; }
		Value* ValuesEnd() const { return m_values + m_nextAvailableEntry; }

		void Swap(HashMap& o)
		{
			u8 temp[sizeof(HashMap)];
			memcpy(temp, this, sizeof(HashMap));
			memcpy(this, &o, sizeof(HashMap));
			memcpy(&o, temp, sizeof(HashMap));
		}

		u64 GetMemoryNeeded(u64 reserveCount)
		{
			reserveCount = Max(reserveCount + 1, 4ull);
			u64 lookupCount = RoundUpPow2(reserveCount);
			return lookupCount*sizeof(u32) + reserveCount*sizeof(Entry) + reserveCount*sizeof(Value) + 256; // Add some padding for alignment between tables
		}

private:
		void InternalInit(u64 reserveCount, u64 lookupCount, const tchar* hint)
		{
			m_mask = u32(lookupCount - 1);
			m_nextAvailableEntry = 1;
			m_reserveCount = u32(reserveCount);
			// This code relies on that allocated memory is zeroed out
			m_lookup = (u32*)m_memory->AllocateNoLock(lookupCount*sizeof(u32), 1, hint);
			m_entries = (Entry*)m_memory->AllocateNoLock(m_reserveCount*sizeof(Entry), alignof(Entry), hint);
			m_values = (Value*)m_memory->AllocateNoLock(m_reserveCount*sizeof(Value), alignof(Value), hint);
		}

		u64 GetNextReserveCount(u64 reserveCount)
		{
			if (reserveCount < 50'000)
				return reserveCount*2;
			return reserveCount + 50'000;
		}

		MemoryBlock* m_memory = nullptr;

		Entry* m_entries = nullptr;
		Value* m_values = nullptr;
		u32* m_lookup = nullptr;

		u32 m_mask = 0;
		u32 m_nextAvailableEntry = 0;
		u32 m_reserveCount = 0;
		bool m_ownsMemory = false;
	};

	// Hash map that contains two parts
	//  1. Lookup table indexed with hashed key. Points to index in entry table
	//  2. Entry table. Contains actual key and value and an index to next entry. Entry table order is stable during rehash

	template<typename Key, typename Value>
	struct HashMap2
	{
		struct Entry
		{
			Key key;
			Value value;
			u32 next; // If this is 0 it means that it is unused, ~0u means it is the end of a chain, otherwise offset into entry array
		};

		void Init(MemoryBlock& memory, u64 maxSize, const tchar* hint = TC(""))
		{
			u32 v = RoundUpPow2(maxSize);
			u32 lookupSize = v;// << 1u;
			mask = lookupSize - 1;
			nextAvailableEntry = 1;

			// This code relies on that AllocateNoLockd memory is zeroed out
			lookup = (u32*)memory.AllocateNoLock(lookupSize*sizeof(u32), 1, hint);
			entries = (Entry*)memory.AllocateNoLock((maxSize+1)*sizeof(Entry), alignof(Entry), hint);
		}

		Value& Insert(const Key& key)
		{
			u32 index = std::hash<Key>()(key) & mask;
			u32 entryIndex = lookup[index];

			if (entryIndex == 0)
			{
				entryIndex = nextAvailableEntry++;
				lookup[index] = entryIndex;
				Entry& entry = entries[entryIndex];
				entry.key = key;
				entry.next = ~0u;
				return entry.value;
			}

			while (true)
			{
				Entry& entry = entries[entryIndex];
				if (entry.key == key)
					return entry.value;

				if (entry.next != ~0u)
				{
					entryIndex = entry.next;
					//static u32 counter = 0;
					//printf("MULTIPLE: %u\n", counter++);
					continue;
				}

				u32 newEntryIndex = entry.next = nextAvailableEntry++;
				Entry& newEntry = entries[newEntryIndex];
				newEntry.key = key;
				newEntry.next = ~0u;
				return newEntry.value;
			}
		}

		Value* Find(const Key& key) const
		{
			u32 index = std::hash<Key>()(key) & mask;
			u32 entryIndex = lookup[index];
			if (entryIndex == 0)
				return nullptr;
			Entry* entry = entries + entryIndex;
			while (true)
			{
				if (entry->key == key)
					return &entry->value;
				entryIndex = entry->next;
				if (entryIndex == ~0u)
					return nullptr;
				entry = entries + entryIndex;
			}
		}

		u32 Size()
		{
			return nextAvailableEntry - 1;
		}

		Entry* entries;
		u32* lookup;
		u32 mask;
		u32 nextAvailableEntry;
	};
}