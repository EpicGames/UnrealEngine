// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <netinet/tcp.h>
#include <netinet/in.h>
#include <errno.h>
#include <stdbool.h>

bool CatFile(char* outBuffer, int bufferCapacity, const char* path)
{
	FILE *f = fopen(path, "r");
	if (!f)
		return false;
	char buffer[256];
	if (!fgets(outBuffer, bufferCapacity, f))
		return false;
	fclose(f);
	return true;
}

bool UnixGetTcpAutoTuning(int* outReadMin, int* outReadDefault, int* outReadMax, int* outWriteMin, int* outWriteDefault, int* outWriteMax)
{
	char buffer[256];
	if (!CatFile(buffer, sizeof(buffer), "/proc/sys/net/core/rmem_max"))
		return false;
	int readMax = atoi(buffer);

	if (!CatFile(buffer, sizeof(buffer), "/proc/sys/net/core/wmem_max"))
		return false;
	int writeMax = atoi(buffer);

	if (!CatFile(buffer, sizeof(buffer), "/proc/sys/net/ipv4/tcp_rmem"))
		return false;
	sscanf(buffer, "%i %i %i", outReadMin, outReadDefault, outReadMax);
	if (readMax < *outReadMax)
		*outReadMax = readMax;

	if (!CatFile(buffer, sizeof(buffer), "/proc/sys/net/ipv4/tcp_wmem"))
		return false;
	sscanf(buffer, "%i %i %i", outWriteMin, outWriteDefault, outWriteMax);
	if (writeMax < *outWriteMax)
		*outWriteMax = writeMax;

	return true;
}
