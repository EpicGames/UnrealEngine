// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBase.h"

namespace uba
{
	enum LogEntryType : u8
	{
		LogEntryType_Error = 0,
		LogEntryType_Warning = 1,
		LogEntryType_Info = 2,
		LogEntryType_Detail = 3,
		LogEntryType_Debug = 4,
	};

	class LogWriter
	{
	public:
		virtual ~LogWriter() = default;
		virtual void BeginScope() = 0;
		virtual void EndScope() = 0;
		virtual void Log(LogEntryType type, const tchar* str, u32 strLen, const tchar* prefix = nullptr, u32 prefixLen = 0) = 0;
	};

	UBA_API extern LogWriter& g_consoleLogWriter;
	UBA_API extern LogWriter& g_nullLogWriter;
}
