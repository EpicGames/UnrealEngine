// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaFunctional.h"
#include "UbaLogWriter.h"
#include "UbaString.h"
#include "UbaVector.h"
#include <stdarg.h>

#define UBA_DEBUG_LOGGER 0
#define UBA_LOG_STALLS 0

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	class StringBufferBase;
	struct BinaryReader;
	struct StringView;
	using TraverseThreadErrorFunc = Function<void(const StringView& error)>;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct LogEntry
	{
		LogEntryType type;
		const tchar* string;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class Logger
	{
	public:
		Logger() {}
		bool Error(const tchar* format, ...);
		bool Warning(const tchar* format, ...);
		Logger& Info(const tchar* format, ...);
		Logger& Detail(const tchar* format, ...);
		Logger& Debug(const tchar* format, ...);
		Logger& Logf(LogEntryType type, const tchar* format, ...);
		void LogArg(LogEntryType type, const tchar* format, va_list& args);
		void Log(LogEntryType type, const StringView& str);

		virtual void BeginScope() = 0;
		virtual void EndScope() = 0;
		virtual void Log(LogEntryType type, const tchar* str, u32 strLen) = 0;
		virtual ~Logger() {}

		bool ToFalse() { return false; }
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class LoggerWithWriter : public Logger
	{
	public:
		LoggerWithWriter(LogWriter& writer, const tchar* prefix = nullptr);
		virtual void BeginScope() { m_writer.BeginScope(); }
		virtual void EndScope() { m_writer.EndScope(); }
		virtual void Log(LogEntryType type, const tchar* str, u32 strLen) { m_writer.Log(type, str, strLen, m_prefix, m_prefixLen); }

		using Logger::Log;
		LogWriter& m_writer;
		const tchar* m_prefix;
		u32 m_prefixLen;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct MutableLogger : public LoggerWithWriter
	{
		MutableLogger(LogWriter& writer, const tchar* prefix) : LoggerWithWriter(writer, prefix) {}
		virtual void Log(LogEntryType type, const tchar* str, u32 strLen) override { if (!isMuted) LoggerWithWriter::Log(type, str, strLen); }
		using LoggerWithWriter::Log;
		Atomic<u32> isMuted = false;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class FilteredLogWriter : public LogWriter
	{
	public:
		FilteredLogWriter(LogWriter& writer, LogEntryType level = LogEntryType_Detail) : m_writer(writer), m_level(level) {}
		virtual void BeginScope() override { m_writer.BeginScope(); }
		virtual void EndScope() override { m_writer.EndScope(); }
		virtual void Log(LogEntryType type, const tchar* str, u32 strLen, const tchar* prefix = nullptr, u32 prefixLen = 0) override;
	private:
		LogWriter& m_writer;
		LogEntryType m_level;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct BytesToText
	{
		BytesToText(u64 bytes, bool base2 = false);
		operator const tchar* () const { return str; };
		operator StringView () const;
		tchar str[32];
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct CountToText
	{
		CountToText(u64 count);
		operator const tchar* () const { return str; };
		operator StringView () const;
		tchar str[32];
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct TimeToText
	{
		TimeToText(float seconds, bool allowMinutes = false);
		TimeToText(u64 time, bool allowMinutes = false);
		TimeToText(u64 time, bool allowMinutes, u64 frequency);
		operator const tchar*() const { return str; };
		operator StringView () const;
		tchar str[32];
	};

	struct DateToText
	{
		DateToText(float seconds);
		DateToText(u64 fileTime);
		tchar str[128];
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	#if UBA_DEBUG_LOGGER || UBA_DEBUG
	Logger* StartDebugLogger(Logger& outerLogger, const tchar* fileName);
	Logger* StopDebugLogger(Logger* logger);
	#endif

	void ParseCallstackInfo(StringBufferBase& out, BinaryReader& reader, const tchar* executable, const StringView* searchPaths, bool writeHeader = true);
	void PrintContentionSummary(class Logger& logger);

	struct CallstackInfo { Vector<u8> data; Vector<u32> threadIds; TString desc; };
	void TraverseAllCallstacks(const Function<void(const CallstackInfo&)>& func, const TraverseThreadErrorFunc& errorFunc);
	void PrintAllCallstacks(Logger& logger);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct LogStallScope
	{
		LogStallScope(Logger& logger, LogEntryType type, u64 timeSeconds, const tchar* messageFormat);
		~LogStallScope();
		void Leave();
		Logger& logger;
		LogEntryType type;
		u64 timeSeconds;
		u64 timeStart;
		const tchar* messageFormat;
	};
	#if UBA_LOG_STALLS
	#define LOG_STALL_SCOPE(logger, timeSeconds, messageFormat) LogStallScope lss(logger, LogEntryType_Error, timeSeconds, messageFormat);
	#else
	#define LOG_STALL_SCOPE(logger, timeSeconds, messageFormat)
	#endif

	void LogToConsoleNoLineFeed(LogEntryType type, const tchar* str, u32 strLen);
	u64 GetConsoleLogLineCount(); // monotonic count of complete (newline-terminated) lines written to g_consoleLogWriter

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
