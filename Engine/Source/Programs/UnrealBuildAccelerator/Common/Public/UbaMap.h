// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaMemory.h"
#include <map>
#include <set>

namespace uba
{
	#if UBA_USE_MIMALLOC
	template<typename Type>
	class StdAllocator {

	public:
		using value_type = Type;

		StdAllocator() {}
		StdAllocator(const StdAllocator& o) {}
		StdAllocator(StdAllocator&& o) noexcept {}
		constexpr bool operator==(const StdAllocator&) const noexcept { return true; }
		template <class _Other>
		constexpr StdAllocator(const StdAllocator<_Other>&) noexcept {}

		value_type* allocate(u64 n) { return (value_type*)mi_malloc_aligned(sizeof(value_type)*n, alignof(value_type)); }
		void deallocate(value_type* p, u64 n) { mi_free_size_aligned(p, sizeof(value_type)*n, alignof(value_type)); }
		bool try_grow_in_place(value_type* p, u64 /*oldCount*/, u64 newCount) { return mi_expand(p, sizeof(value_type) * newCount) != nullptr; }
		u64 max_size() const { return static_cast<size_t>(-1) / sizeof(value_type); }
	};
	#else
	template<typename Type>
	using StdAllocator = std::allocator<Type>;
	#endif

	template<typename Key, typename Value, typename Less = std::less<>>
	using Map = std::map<Key, Value, Less, StdAllocator<std::pair<const Key, Value>>>;

	template<typename Key, typename Less = std::less<>>
	using Set = std::set<Key, Less, StdAllocator<Key>>;
}
