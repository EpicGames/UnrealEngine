// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaDefaultConstants.h"
#include "UbaStringBuffer.h"

namespace uba
{
	static constexpr u32 SendMaxSize = 256*1024;

	static constexpr u8 SystemServiceId = 0;
	static constexpr u8 StorageServiceId = 1;
	static constexpr u8 SessionServiceId = 2;
	static constexpr u8 CacheServiceId = 3;
	
	static constexpr u32 SystemNetworkVersion = 1339;
	static constexpr u32 SystemNetworkVersionMin = 1339;
	static constexpr u32 SystemNetworkVersionRoute = 1340;
	static constexpr u32 StorageNetworkVersion = 4;
	static constexpr u32 SessionNetworkVersion = 55;
	static constexpr u32 CacheNetworkVersion = 5;
	static constexpr u32 CacheBucketVersion = 3;

	#define UBA_TRACK_NETWORK_TIMES 1 // Enable to track times on messages, connections etc

	// Messages used over network between client/server (system, storage and session)

	#define UBA_SYSTEM_MESSAGES \
		UBA_SYSTEM_MESSAGE(SetConnectionCount) \
		UBA_SYSTEM_MESSAGE(KeepAlive) \
		UBA_SYSTEM_MESSAGE(FetchConfig) \
		UBA_SYSTEM_MESSAGE(GetTraceInformation) \

	#define UBA_STORAGE_MESSAGES \
		UBA_STORAGE_MESSAGE(FetchBegin) \
		UBA_STORAGE_MESSAGE(FetchSegment) \
		UBA_STORAGE_MESSAGE(FetchEnd) \
		UBA_STORAGE_MESSAGE(ExistsOnServer) \
		UBA_STORAGE_MESSAGE(StoreBegin) \
		UBA_STORAGE_MESSAGE(StoreSegment) \
		UBA_STORAGE_MESSAGE(StoreEnd) \
		UBA_STORAGE_MESSAGE(Connect) \
		UBA_STORAGE_MESSAGE(ProxyFetchBegin) \
		UBA_STORAGE_MESSAGE(ProxyFetchEnd) \
		UBA_STORAGE_MESSAGE(ReportBadProxy) \
		UBA_STORAGE_MESSAGE(ReportBadCas) \

	#define UBA_SESSION_MESSAGES \
		UBA_SESSION_MESSAGE(Connect) \
		UBA_SESSION_MESSAGE(EnsureBinaryFile) \
		UBA_SESSION_MESSAGE(GetApplication) \
		UBA_SESSION_MESSAGE(GetFileFromServer) \
		UBA_SESSION_MESSAGE(GetLongPathName) \
		UBA_SESSION_MESSAGE(SendFileToServer) \
		UBA_SESSION_MESSAGE(DeleteFile) \
		UBA_SESSION_MESSAGE(CopyFile) \
		UBA_SESSION_MESSAGE(CreateDirectory) \
		UBA_SESSION_MESSAGE(RemoveDirectory) \
		UBA_SESSION_MESSAGE(MoveFile) \
		UBA_SESSION_MESSAGE(ListDirectory) \
		UBA_SESSION_MESSAGE(GetDirectoriesFromServer) \
		UBA_SESSION_MESSAGE(GetNameToHashFromServer) \
		UBA_SESSION_MESSAGE(ProcessAvailable) \
		UBA_SESSION_MESSAGE(ProcessInputs) \
		UBA_SESSION_MESSAGE(ProcessFinished) \
		UBA_SESSION_MESSAGE(ProcessReturned) \
		UBA_SESSION_MESSAGE(GetRoots) \
		UBA_SESSION_MESSAGE(VirtualAllocFailed) \
		UBA_SESSION_MESSAGE(GetTraceInformation) \
		UBA_SESSION_MESSAGE(Ping) \
		UBA_SESSION_MESSAGE(Notification) \
		UBA_SESSION_MESSAGE(GetNextProcess) \
		UBA_SESSION_MESSAGE(FlushProcess) \
		UBA_SESSION_MESSAGE(Custom) \
		UBA_SESSION_MESSAGE(UpdateEnvironment) \
		UBA_SESSION_MESSAGE(Summary) \
		UBA_SESSION_MESSAGE(Command) \
		UBA_SESSION_MESSAGE(SHGetKnownFolderPath) \
		UBA_SESSION_MESSAGE(DebugFileNotFoundError) \
		UBA_SESSION_MESSAGE(HostRun) \
		UBA_SESSION_MESSAGE(GetSymbols) \
		UBA_SESSION_MESSAGE(SpeedTest) \
		UBA_SESSION_MESSAGE(LogEntry) \
		UBA_SESSION_MESSAGE(Symlink) \

	#define UBA_CACHE_MESSAGES \
		UBA_CACHE_MESSAGE(Connect) \
		UBA_CACHE_MESSAGE(StorePathTable) \
		UBA_CACHE_MESSAGE(StoreCasTable) \
		UBA_CACHE_MESSAGE(StoreEntry) \
		UBA_CACHE_MESSAGE(StoreEntryDone) \
		UBA_CACHE_MESSAGE(FetchPathTable) \
		UBA_CACHE_MESSAGE(FetchCasTable) \
		UBA_CACHE_MESSAGE(FetchEntries) \
		UBA_CACHE_MESSAGE(ExecuteCommand) \
		UBA_CACHE_MESSAGE(RequestShutdown) \
		UBA_CACHE_MESSAGE(ReportUsedEntry) \
		UBA_CACHE_MESSAGE(FetchPathTable2) \
		UBA_CACHE_MESSAGE(FetchCasTable2) \

	enum SystemMessageType : u8
	{
		#define UBA_SYSTEM_MESSAGE(x) SystemMessageType_##x,
		UBA_SYSTEM_MESSAGES
		#undef UBA_SYSTEM_MESSAGE
	};

	enum StorageMessageType : u8
	{
		#define UBA_STORAGE_MESSAGE(x) StorageMessageType_##x,
		UBA_STORAGE_MESSAGES
		#undef UBA_STORAGE_MESSAGE
	};

	enum SessionMessageType : u8 
	{
		#define UBA_SESSION_MESSAGE(x) SessionMessageType_##x,
		UBA_SESSION_MESSAGES
		#undef UBA_SESSION_MESSAGE
	};

	enum CacheMessageType : u8
	{
		#define UBA_CACHE_MESSAGE(x) CacheMessageType_##x,
		UBA_CACHE_MESSAGES
		#undef UBA_CACHE_MESSAGE
	};

	StringView ToString(SystemMessageType type);
	StringView ToString(StorageMessageType type);
	StringView ToString(SessionMessageType type);
	StringView ToString(CacheMessageType type);
	StringView MessageToString(u8 serviceId, u8 messageType);

	constexpr u32 MessageErrorSize = 0xffffff;
	constexpr u32 MessageKeepAliveSize = 0xffffff - 1;

	constexpr u16 FetchCasIdDone = u16(~0);
	constexpr u16 FetchCasIdDisallowed = u16(~0) - 1;

	constexpr bool DownloadDebugSymbols = false;

	static constexpr u32 KeepAliveIdleSeconds = 60;
	static constexpr u32 KeepAliveIntervalSeconds = 1;
	static constexpr u32 KeepAliveProbeCount = 10;
	static constexpr u32 UserTimeoutSeconds = 60;

	// Response types for SessionMessageType_ProcessAvailable
	enum SessionProcessAvailableResponse : u32
	{
		SessionProcessAvailableResponse_None = 0,
		SessionProcessAvailableResponse_Disconnect = ~u32(0),
		SessionProcessAvailableResponse_RemoteExecutionDisabled = ~u32(0) - 1,
	};

	inline constexpr const char EncryptionHandshakeString[] = "This is a test string used to check so encryption keys matches between client and server. This string is 128 characters long...";
}
