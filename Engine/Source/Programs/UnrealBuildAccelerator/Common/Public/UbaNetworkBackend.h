// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaFunctional.h"
#include "UbaNetwork.h"
#include "UbaVector.h"

#if PLATFORM_WINDOWS
typedef struct sockaddr SOCKADDR;
#else
struct sockaddr;
#endif

namespace uba
{
	class Logger;
	class StringBufferBase;

	void TraverseNetworkAddresses(Logger& logger, const Function<bool(const StringBufferBase& addr)>& func);
	bool TraverseRemoteAddresses(Logger& logger, const tchar* addr, u16 port, const Function<bool(const sockaddr& remoteSockaddr)>& func);

	class NetworkBackend
	{
	public:
		virtual ~NetworkBackend() {}

		// Shutdown the connection (stops/finishes traffic)
		// External code should wait for disconnect callback before assuming that no more recv callbacks can happen
		virtual void Shutdown(void* connection) = 0;

		enum : u8
		{
			SendFlags_Async = 1 << 0, 	// Data is sent async and call is returned before data has been transmitted. (can be used inside recv callbacks). Will make copy of data
			SendFlags_ExternalWait = 1 << 1,	// Same as async but will not do any copies of context or data. Assumes something outside is handling the waiting
		};
		struct SendContext;

		// Send data to connection. If sendContext is set it means that backend expects sendContext to stay around in memory until send is done (or rather, response is received)
		virtual bool Send(Logger& logger, void* connection, const void* data, u32 dataSize, SendContext& sendContext, const tchar* sendHint) = 0;

		// Recv data from connection. This is callback based so header callback is called first after headerSize bytes have been read
		// If header callback sets a value in outBodySize, body callback will be called once body size bytes are received.
		using RecvHeaderCallback = bool(void* context, const Guid& connectionUid, u8* headerData, void*& outBodyContext, u8*& outBodyData, u32& outBodySize);
		using RecvBodyCallback = bool(void* context, bool recvError, u8* headerData, void* bodyContext, u8* bodyData, u32 bodySize);
		virtual void SetRecvCallbacks(void* connection, void* context, u32 headerSize, RecvHeaderCallback* h, RecvBodyCallback* b, const tchar* recvHint) = 0;

		using RecvTimeoutCallback = bool(void* context, u32 timeoutMs, const tchar* recvHint, const tchar* hint);
		virtual void SetRecvTimeout(void* connection, u32 timeoutMs, void* context, RecvTimeoutCallback* callback) = 0;

		// Callbacks to be able to track stats
		using DataSentCallback = void(void* context, u32 bytes);
		using DataReceivedCallback = void(void* context, u32 bytes);
		virtual void SetDataSentCallback(void* connection, void* context, DataSentCallback* callback) {}
		virtual void SetDataReceivedCallback(void* connection, void* context, DataReceivedCallback* callback) {}
		virtual void TraverseConnections(const Function<void(void*)>& func) {}

		// Change so send is handled in a separate thread instead of directly on calling thread
		virtual void SetThreadedSend(void* connection) = 0;

		// Disconnect callback. This is called as soon as connection is interrupted from send, recv or shutdown.
		using DisconnectCallback = void(void* context, const Guid& connectionUid, void* connection);
		virtual void SetDisconnectCallback(void* connection, void* context, DisconnectCallback* callback) = 0;
		virtual void SetAllowLessThanBodySize(void* connection, bool allow) = 0;

		// Set priority of connection
		virtual void SetPriority(void* connection) = 0;

		// Start listen on port/ip.
		using ListenConnectedFunc = Function<bool(void* connection, const sockaddr& remoteSocketAddr)>;
		virtual bool StartListen(Logger& logger, u16 port, const tchar* ip, const ListenConnectedFunc& connectedFunc, bool linger = false) = 0;
		virtual void StopListen() = 0;

		// Connect to remote ip/port
		using ConnectedFunc = Function<bool(void* connection, const sockaddr& remoteSocketAddr, bool* timedOut)>;
		virtual bool Connect(Logger& logger, const tchar* ip, const ConnectedFunc& connectedFunc, u16 port = DefaultPort, bool* timedOut = nullptr) = 0;
		virtual bool Connect(Logger& logger, const sockaddr& remoteSocketAddr, const ConnectedFunc& connectedFunc, bool* timedOut = nullptr, const tchar* nameHint = nullptr) = 0;

		// Remove connection (user guarantee connection is not referenced anywhere)
		virtual void DeleteConnection(void* connection) = 0;

		// Set internal buffer sizes for send and recv
		virtual void SetSendBufferSize(void* connection, u32 bytes) = 0;
		virtual void SetRecvBufferSize(void* connection, u32 bytes) = 0;

		virtual void GetTotalSendAndRecv(u64& outSend, u64& outRecv) = 0;
		virtual void Validate(Logger& logger, const Vector<void*>& connections, bool full) = 0;
	};

	struct NetworkBackend::SendContext
	{
		u64 data[2];
		u32 size = 0;
		u8 flags;
		bool isUsed = false;
		bool isFinished = false;

		SendContext(u8 sendFlags = 0) : flags(sendFlags) { *data = 0; }
		~SendContext() { }//UBA_ASSERT(isFinished); }
	};

	class HttpConnection
	{
	public:
		HttpConnection();
		~HttpConnection();
		bool Query(Logger& logger, const char* type, StringBufferBase& outResponse, u32& outStatusCode, const char* host, const char* path, const char* header = "", u32 timeoutMs = 3000);

		void SetConnectTimeout(u32 timeOutMs);

	private:
		bool Connect(Logger& logger, const char* host);

		char m_host[256];
		u64 m_socket;
		bool m_wsaInitDone = false;
		u32 m_connectTimeOutMs = 0;
	};
}
