// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaLogger.h"
#include "UbaNetworkBackend.h"
#include "UbaThread.h"

namespace uba
{
	class NetworkBackendMemory final : public NetworkBackend
	{
	public:
		NetworkBackendMemory(LogWriter& writer, const tchar* prefix = TC("NetworkBackendMemory"));
		virtual ~NetworkBackendMemory();
		virtual void Shutdown(void* connection) override;
		virtual bool Send(Logger& logger, void* connection, const void* data, u32 dataSize, SendContext& sendContext, const tchar* sendHint) override;
		virtual void SetDataSentCallback(void* connection, void* context, DataSentCallback* callback) override;
		virtual void SetDataReceivedCallback(void* connection, void* context, DataReceivedCallback* callback) override;
		virtual void SetRecvCallbacks(void* connection, void* context, u32 headerSize, RecvHeaderCallback* h, RecvBodyCallback* b, const tchar* recvHint) override;
		virtual void SetRecvTimeout(void* connection, u32 timeoutMs, void* context, RecvTimeoutCallback* callback) override;
		virtual void SetThreadedSend(void* connection) override;
		virtual void SetDisconnectCallback(void* connection, void* context, DisconnectCallback* callback) override;
		virtual void SetAllowLessThanBodySize(void* connection, bool allow) override;
		virtual void SetPriority(void* connection) override;

		virtual bool StartListen(Logger& logger, u16 port, const tchar* ip, const ListenConnectedFunc& connectedFunc, bool linger = false) override;
		virtual void StopListen() override;
		virtual bool Connect(Logger& logger, const tchar* ip, const ConnectedFunc& connectedFunc, u16 port = DefaultPort, bool* timedOut = nullptr) override;
		virtual bool Connect(Logger& logger, const sockaddr& remoteSocketAddr, const ConnectedFunc& connectedFunc, bool* timedOut = nullptr, const tchar* nameHint = nullptr) override;
		virtual void DeleteConnection(void* connection) override;

		virtual void SetSendBufferSize(void* connection, u32 bytes) override;
		virtual void SetRecvBufferSize(void* connection, u32 bytes) override;

		virtual void GetTotalSendAndRecv(u64& outSend, u64& outRecv) override;
		virtual void Validate(Logger& logger, const Vector<void*>& connections, bool full) override;

	private:
		struct Connection;
		Connection* m_connection = nullptr;

		ReaderWriterLock m_connectedFuncLock;
		ListenConnectedFunc m_connectedFunc;
	};
};
