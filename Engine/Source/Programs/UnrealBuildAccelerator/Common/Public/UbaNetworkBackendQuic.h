// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaEvent.h"
#include "UbaNetworkBackend.h"
#include "UbaLogger.h"

#ifndef UBA_USE_QUIC
#define UBA_USE_QUIC 0
#endif

#if UBA_USE_QUIC

typedef struct QUIC_API_TABLE QUIC_API_TABLE;
typedef struct QUIC_HANDLE QUIC_HANDLE;
typedef struct QUIC_HANDLE* HQUIC;
typedef struct QUIC_CONNECTION_EVENT QUIC_CONNECTION_EVENT;
typedef struct QUIC_STREAM_EVENT QUIC_STREAM_EVENT;
typedef struct QUIC_LISTENER_EVENT QUIC_LISTENER_EVENT;
typedef struct QUIC_SETTINGS QUIC_SETTINGS;
#define QUIC_STATUS long

namespace uba
{
	struct NetworkBackendQuic : public NetworkBackend
	{
	public:
		NetworkBackendQuic(LogWriter& writer, const wchar_t* prefix = L"NetworkBackendQuic");
		bool Init(const char* app);
		virtual ~NetworkBackendQuic();
		virtual void Shutdown(void* connection) override;
		virtual void Close(void* connection) override;
		virtual bool Send(Logger& logger, void* connection, const void* data, u32 dataSize, SendContext& sendContext, const tchar* sendHint) override;
		virtual void SetDataSentCallback(void* connection, void* context, DataSentCallback* callback) override;
		virtual void SetRecvCallbacks(void* connection, void* context, u32 headerSize, RecvHeaderCallback* h, RecvBodyCallback* b, const wchar_t* recvHint) override;
		virtual void SetRecvTimeout(void* connection, u32 timeoutMs) override;
		virtual void SetThreadedSend(void* connection) override;
		virtual void SetDisconnectCallback(void* connection, void* context, DisconnectCallback* callback) override;

		virtual bool StartListen(Logger& logger, u16 port, const wchar_t* ip, const ListenConnectedFunc& connectedFunc, bool linger = false) override;
		virtual void StopListen() override;

		virtual bool Connect(Logger& logger, const wchar_t* ip, const ConnectedFunc& connectedFunc, u16 port = DefaultPort, bool* timedOut = nullptr) override;
		virtual bool Connect(Logger& logger, const sockaddr& remoteSocketAddr, const ConnectedFunc& connectedFunc, bool* timedOut = nullptr, const wchar_t* nameHint = nullptr) override;

	private:
		struct Connection;
		QUIC_STATUS StreamCallback(Connection& connection, HQUIC streamHandle, QUIC_STREAM_EVENT* Event);

		LoggerWithWriter m_logger;
		const QUIC_API_TABLE* m_msQuic = nullptr;
		HQUIC m_registration = NULL;
		HQUIC m_configuration = NULL;

		struct Stream
		{
			HQUIC handle;

			u8 headerData[16];
			u8 headerRead = 0;

			void* bodyContext = nullptr;
			u8* bodyData = nullptr;
			u32 bodySize = 0;
			u32 bodyRead = 0;
		};

		struct Connection
		{
			Connection(NetworkBackendQuic& b, HQUIC ch);
			void AddRef();
			void Release();

			NetworkBackendQuic& backend;
			HQUIC connectionHandle;
			AtomicU64 refCount;
			UnorderedMap<HQUIC, Stream> recvStreams;
			Event ready;
			AtomicU64 streamHandleIt;
			bool connected = false;

			void* disconnectContext = nullptr;
			DisconnectCallback* disconnectCallback = nullptr;

			DataSentCallback* dataSentCallback = nullptr;

			RecvHeaderCallback* headerCallback = nullptr;
			RecvBodyCallback* bodyCallback = nullptr;
			void* context = nullptr;
			u32 headerSize = 0;

		};
		ReaderWriterLock m_connectionsLock;
		UnorderedMap<HQUIC, Connection*> m_connections;

		QUIC_STATUS ConnectionCallback(HQUIC Connection, QUIC_CONNECTION_EVENT* Event);
		QUIC_STATUS ListenerCallback(HQUIC Listener, QUIC_LISTENER_EVENT* Event);
		QUIC_STATUS ListenerConnectionCallback(HQUIC Connection, QUIC_CONNECTION_EVENT* Event);
		void StreamCreated(HQUIC conn, HQUIC stream);
		void StreamShutdown(Connection& conn, HQUIC streamHandle);
		void CloseConnection(HQUIC connection);
		void CloseConnection(Connection& connection);
		void InitNetworkSettings(QUIC_SETTINGS& Settings, bool isClient);
		bool LoadCredentials(bool isClient);

		HQUIC m_listener = NULL;
		ListenConnectedFunc m_listenConnectedFunc;
	};
}

#endif // UBA_USE_QUIC
