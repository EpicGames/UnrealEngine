// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaList.h"
#include "UbaLogger.h"
#include "UbaNetworkBackend.h"
#include "UbaThread.h"

namespace uba
{
	class Config;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct NetworkBackendTcpCreateInfo
	{
		NetworkBackendTcpCreateInfo(LogWriter& w = g_consoleLogWriter) : logWriter(w) {}
		LogWriter& logWriter;
		u32 statusUpdateSeconds = 0; // Zero means never
		bool disableNagle = true;
		bool useOverlappedSend = true;
		u32 iocpWorkerCount = 0; // Iocp is turned off by default

		u32 recvBufferSize = 0; // 0 means default
		u32 sendBufferSize = 0; // 0 means default

		void Apply(const Config& config, const tchar* tableName = TC("NetworkBackendTcp"));
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class NetworkBackendTcp final : public NetworkBackend
	{
	public:
		NetworkBackendTcp(const NetworkBackendTcpCreateInfo& info, const tchar* prefix = TC("NetworkBackendTcp"));
		virtual ~NetworkBackendTcp();
		virtual void Shutdown(void* connection) override;
		virtual bool Send(Logger& logger, void* connection, const void* data, u32 dataSize, SendContext& sendContext, const tchar* sendHint) override;
		virtual void SetRecvCallbacks(void* connection, void* context, u32 headerSize, RecvHeaderCallback* h, RecvBodyCallback* b, const tchar* recvHint) override;
		virtual void SetRecvTimeout(void* connection, u32 timeoutMs, void* context, RecvTimeoutCallback* callback) override;
		virtual void SetThreadedSend(void* connection) override;
		virtual void SetDisconnectCallback(void* connection, void* context, DisconnectCallback* callback) override;
		virtual void SetAllowLessThanBodySize(void* connection, bool allow) override;
		virtual void SetPriority(void* connection) override;

		virtual bool StartListen(Logger& logger, u16 port, const tchar* ip, const ListenConnectedFunc& connectedFunc, bool linger = false) override;
		virtual void StopListen() override;
		virtual bool Connect(Logger& logger, const tchar* ip, const ConnectedFunc& connectedFunc, u16 port = DefaultPort, bool* timedOut = nullptr) override;
		virtual bool Connect(Logger& logger, const sockaddr& remoteSocketAddr, const ConnectedFunc& connectedFunc, bool* timedOut = nullptr, const tchar* nameHint = nullptr) override;
		virtual void DeleteConnection(void* connection) override;

		virtual void SetDataSentCallback(void* connection, void* context, DataSentCallback* callback) override;
		virtual void SetDataReceivedCallback(void* connection, void* context, DataReceivedCallback* callback) override;
		virtual void TraverseConnections(const Function<void(void*)>& func) override;

		virtual void SetSendBufferSize(void* connection, u32 bytes) override;
		virtual void SetRecvBufferSize(void* connection, u32 bytes) override;

		virtual void GetTotalSendAndRecv(u64& outSend, u64& outRecv) override;
		virtual void Validate(Logger& logger, const Vector<void*>& connections, bool full) override;

		static bool CheckEnvironment(Logger& logger, bool printTips);
		static bool KillTcpHogs(Logger& logger, u16 port);

	private:
		bool EnsureInitialized(Logger& logger);

		struct Connection;
		struct ListenEntry;
		struct RecvCache;
		bool ThreadListen(Logger& logger, ListenEntry& entry);
		void ThreadRecv(Connection& connection);
		void ThreadSend(Connection& connection);
		bool HandleSendError(Connection& connection, Logger& logger, const tchar* hint);
		bool SendSocket(Connection& connection, Logger& logger, const void* b, u64 bufferLen, const tchar* hint);
		bool RecvSocket(Connection& connection, RecvCache& recvCache, void* b, u32& bufferLen, const tchar* hint, bool isFirstCall, bool allowLess);

		void ThreadIocp();
		bool PostIocpRead(Connection& connection, u8* data, u32 dataSize);

		void ThreadStatus(u32 statusUpdateSeconds);
		void PrintTcpStatistics(Logger& logger, u32 statusUpdateSeconds, void* prevStats);

		LoggerWithWriter m_logger;
		Futex m_listenEntriesLock;
		List<ListenEntry> m_listenEntries;

		Futex m_connectionsLock;
		List<Connection> m_connections;

		Atomic<u64> m_totalSend = 0;
		Atomic<u64> m_totalRecv = 0;

		bool m_disableNagle;

		#if PLATFORM_WINDOWS
		Futex m_initLock;
		bool m_initDone = false;
		bool m_initSuccess = false;
		bool m_useOverlappedSend;
		u16 m_iocpWorkerCount;
		#endif

		u32 m_recvBufferSize;
		u32 m_sendBufferSize;

		List<Thread> m_iocpThreads;
		void* m_iocpHandle = nullptr;

		Event m_tcpStatusLoop;
		Thread m_tcpStatusThread;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////
};
