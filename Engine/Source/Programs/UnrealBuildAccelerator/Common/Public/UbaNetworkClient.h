// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaCrypto.h"
#include "UbaLogger.h"
#include "UbaNetwork.h"
#include "UbaWorkManager.h"
#include "UbaTimer.h"
#include "UbaEvent.h"

namespace uba
{
	class Config;
	class NetworkBackend;
	class NetworkBackendTcp;
	struct BinaryReader;
	struct BinaryWriter;
	struct CasKey;
	struct NetworkMessage;

	struct NetworkClientCreateInfo
	{
		NetworkClientCreateInfo(LogWriter& w = g_consoleLogWriter) : logWriter(w) {}
		LogWriter& logWriter;
		u32 sendSize = SendDefaultSize;
		u32 receiveTimeoutSeconds = DefaultNetworkReceiveTimeoutSeconds;
		u32 workerCount = ~0u; // ~0u means logical processor count will be used
		bool useMessagePriority = false;
		bool useThreadedSend = false;
		const u8* cryptoKey128 = nullptr;

		u32 recvBufferSize = 0;
		u32 sendBufferSize = 0;

		void Apply(Config& config, const tchar* tableName = TC("NetworkClient"));
	};

	class NetworkClient : public WorkManagerImpl
	{
	public:
		NetworkClient(bool& outCtorSuccess, const NetworkClientCreateInfo& info = {}, const tchar* name = TC("UbaClient"));
		~NetworkClient();

		bool Connect(NetworkBackend& backend, const tchar* ip, u16 port = DefaultPort, bool* timedOut = nullptr);
		void Disconnect(bool flushWork = true);

		bool StartListen(NetworkBackend& backend, u16 port = DefaultPort, const tchar* ip = TC("0.0.0.0"));
		bool SetConnectionCount(u32 count);
		bool SendKeepAlive();

		bool FetchConfig(Config& config);

		bool IsConnected(u32 waitTimeoutMs = 0);
		bool IsOrWasConnected(u32 waitTimeoutMs = 0);

		void ValidateNetwork(Logger& logger, bool full);
		void PrintSummary(Logger& logger);

		using OnConnectedFunction = Function<void()>;
		void RegisterOnConnected(const OnConnectedFunction& function);

		using OnDisconnectedFunction = Function<void()>;
		void RegisterOnDisconnected(const OnDisconnectedFunction& function);

		using OnVersionMismatchFunction = Function<void(const CasKey& exeKey, const CasKey& dllKey)>;
		void RegisterOnVersionMismatch(OnVersionMismatchFunction&& function);
		void InvokeVersionMismatch(const CasKey& exeKey, const CasKey& dllKey);

		u64 GetMessageHeaderSize();
		u64 GetMessageMaxSize();
		u64 GetMessageReceiveHeaderSize();
		const Guid& GetUid() { return m_uid; }
		LogWriter& GetLogWriter() { return m_logWriter; }
		u32 GetConnectionCount() { return m_connectionCount; }
		u64 GetTotalSentBytes() { return m_sendBytes; }
		u64 GetTotalRecvBytes() { return m_recvBytes; }

		NetworkBackend* GetFirstConnectionBackend();
		void TraverseConnections(const Function<void(NetworkBackend&, void*)>& func);

	private:
		struct Connection;

		bool AddConnection(NetworkBackend& backend, void* backendConnection, bool* timedOut);
		bool ConnectedCallback(NetworkBackend& backend, void* backendConnection, const Guid& cryptoIV);
		static void DisconnectCallback(void* context, const Guid& connectionUid, void* connection);
		static bool ReceiveResponseHeader(void* context, const Guid& connectionUid, u8* headerData, void*& outBodyContext, u8*& outBodyData, u32& outBodySize);
		static bool ReceiveResponseBody(void* context, bool recvError, u8* headerData, void* bodyContext, u8* bodyData, u32 bodySize);
		void OnDisconnected(Connection& connection, u32 reason);
		bool Send(NetworkMessage& message, void* response, u32 responseCapacity, bool async);
		const tchar* SetGetPrefix(const tchar* originalPrefix);

		LogWriter& m_logWriter;
		Guid m_uid;
		TString m_prefix;
		LoggerWithWriter m_logger;
		u32 m_sendSize;
		u32 m_recvBufferSize;
		u32 m_sendBufferSize;
		u32 m_receiveTimeoutSeconds;
		Atomic<u64> m_sendBytes = 0;
		Atomic<u64> m_recvBytes = 0;
		Atomic<u32> m_recvCount = 0;
		Atomic<bool> m_isDisconnecting = false;
		Timer m_sendTimer;

		Futex m_serverUidLock;
		Guid m_serverUid;

		Event m_isConnected;
		Event m_isOrWasConnected;
		Atomic<u32> m_connectionCount = 0;
		bool m_useMessagePriority;
		bool m_useThreadedSend;

		Futex m_onConnectedFunctionsLock;
		Vector<OnConnectedFunction> m_onConnectedFunctions;
		ReaderWriterLock m_onDisconnectedFunctionsLock;
		Vector<OnDisconnectedFunction> m_onDisconnectedFunctions;
		OnVersionMismatchFunction m_versionMismatchFunction;

		ReaderWriterLock m_connectionsLock;
		List<Connection> m_connections;
		Futex m_connectionsItLock;
		List<Connection>::iterator m_connectionsIt;


		ReaderWriterLock m_activeMessagesLock;
		u16 m_activeMessageIdMax = 1;
		Vector<u16> m_availableMessageIds;
		Vector<NetworkMessage*> m_activeMessages;

		CryptoKey m_cryptoKey = InvalidCryptoKey;
		Timer m_encryptTimer;
		Timer m_decryptTimer;

		#if UBA_TRACK_NETWORK_TIMES
		u64 m_startTime = 0;
		#endif

		friend NetworkMessage;

		NetworkClient(const NetworkClient&) = delete;
		NetworkClient& operator=(const NetworkClient&) = delete;
	};
}
