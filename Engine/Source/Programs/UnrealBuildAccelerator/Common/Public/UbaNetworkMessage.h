// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaNetworkBackend.h"
#include "UbaNetworkClient.h"

namespace uba
{
	enum MessagePriority : u8 { NormalPriority, HasPriority };

	struct NetworkMessage
	{
		NetworkMessage(NetworkClient& client, u8 serviceId, u8 messageType, BinaryWriter& sendWriter, MessagePriority priority = NormalPriority);
		NetworkMessage() {}
		~NetworkMessage();

		void Init(NetworkClient& client, u8 serviceId, u8 messageType, BinaryWriter& sendWriter, MessagePriority priority = NormalPriority);

		bool Send(BinaryReader& response);
		bool Send(BinaryReader& response, Timer& outTimer);
		bool Send();
		bool Send(Timer& outTimer);

		using DoneFunc = void(bool error, void* userData);
		bool SendAsync(BinaryReader& response, DoneFunc* func, void* userData); // Try to minimize work in GetResponseFunc since it is running on receiving thread
		bool ProcessAsyncResults(BinaryReader& response); // Note, this must be called after GetResponseFunc has been called

		u32 GetError() const { return m_error; }
		u8 GetServiceId();
		u8 GetMessageType();

	private:
		void Done(bool shouldLock = true);

		NetworkClient* m_client = nullptr;
		BinaryWriter* m_sendWriter = nullptr;
		DoneFunc* m_doneFunc = nullptr;
		void* m_doneUserData = nullptr;
		void* m_response = nullptr;
		NetworkBackend::SendContext m_sendContext;
		u32 m_responseSize = 0;
		u32 m_responseCapacity = 0;
		u16 m_id = 0;
		MessagePriority m_priority;
		Atomic<u32> m_error = 0;
		NetworkClient::Connection* m_connection = nullptr;
		Guid m_cryptoIV;
		friend NetworkClient;
		NetworkMessage(const NetworkMessage&) = delete;
		NetworkMessage(NetworkMessage&&) = delete;

		#if UBA_TRACK_NETWORK_TIMES
		u64 m_sendTime = 0;
		#endif
	};

	enum NetworkMessageError
	{
		NetworkMessageError_Disconnecting = 1,
	};
}
