// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaCrypto.h"
#include "UbaEvent.h"
#include "UbaTimer.h"
#include "UbaNetwork.h"
#include "UbaNetworkBackend.h"
#include "UbaLogger.h"
#include "UbaThread.h"
#include "UbaUnorderedMap.h"
#include "UbaWorkManager.h"

namespace uba
{
	class Config;
	class Trace;
	struct BinaryReader;
	struct BinaryWriter;
	struct Bottleneck;

	struct ConnectionInfo
	{
		const Guid& GetUid() const;
		u32 GetId() const;
		bool GetName(StringBufferBase& out) const;
		u64 GetCryptoUserData() const;
		bool ShouldDisconnect() const;

		void SetStorageVersion(u32 version) const;
		u32 GetStorageVersion() const;

		void* internalData = nullptr;
	};

	struct NetworkServerCreateInfo
	{
		NetworkServerCreateInfo(LogWriter& w = g_consoleLogWriter) : logWriter(w) {}

		void Apply(Config& config, const tchar* tableName = TC("NetworkServer"));

		LogWriter& logWriter;
		u32 workerCount = 0; // Zero means it will use the number of logical cores as worker count
		u32 sendSize = SendDefaultSize;
		u32 receiveTimeoutSeconds = DefaultNetworkReceiveTimeoutSeconds;
		bool logConnections = true;
		bool useKeepAlive = false;
		bool useMessagePriority = false;
		bool useThreadedSend = false;

		u32 recvBufferSize = 0;
		u32 sendBufferSize = 0;
	};

	struct MessageInfo
	{
		u32 connectionId;
		u16 messageId;
		u8 type;
	};

	class NetworkServer : public WorkManager
	{
	public:
		NetworkServer(bool& outCtorSuccess, const NetworkServerCreateInfo& info = {}, const tchar* name = TC("UbaServer"));
		virtual ~NetworkServer();

		bool StartListen(NetworkBackend& backend, u16 port = DefaultPort, const tchar* ip = TC("0.0.0.0"), bool requiresCrypto = false); // Start listen for new connections/clients
		void DisallowNewClients();	// Disallow new clients to connect but old clients can still create more connections
		void DisconnectClients();	// Disconnect all active connections
		void ShutdownActiveConnections(); // Signal each currently-active connection to disconnect (backend.Shutdown — sets shutdownCalled + wakes recv/send polls). Connections tear down via the normal recv-exit path; m_workersEnabled / m_allClientsDisconnected are NOT touched, so the server stays usable. Used by CacheServer maintenance to drain stuck clients without DisconnectClients' shutdown semantics.

		bool RegisterCryptoKey(const u8* cryptoKey128, u64 expirationTime = ~u64(0), u64 cryptoUserData = 0);
		void SetClientsConfig(const Config& config);
		void SetTrace(Trace* trace);

		bool AddClient(NetworkBackend& backend, const tchar* ip, u16 port = DefaultPort, const u8* cryptoKey128 = nullptr); // Adds a client that server will create one or more connections to (note this will return before we know if it was a success or not)
		bool HasConnectInProgress();

		void PrintSummary(Logger& logger);

		using TypeToNameFunction = StringView(u8 type);
		using WorkerFunction = Function<bool(const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer)>;
		void RegisterService(u8 serviceId, const WorkerFunction& function, TypeToNameFunction* typeToNameFunc = nullptr);
		void UnregisterService(u8 serviceId);

		using OnConnectionFunction = Function<bool(const Guid& clientUid, u32 clientId)>;
		void RegisterOnClientConnected(u8 id, const OnConnectionFunction& func);
		void UnregisterOnClientConnected(u8 id);

		using OnDisconnectFunction = Function<void(const Guid& clientUid, u32 clientId)>;
		void RegisterOnClientDisconnected(u8 id, const OnDisconnectFunction& func);
		void UnregisterOnClientDisconnected(u8 id);

		//using OnConnectionFunction = Function<void(const Guid& clientUid)>;
		//void RegisterOnConnection(u8 id, const OnConnectionFunction& func);
		//void UnregisterOnConnection(u8 id);

		virtual void AddWork(WorkFunction&& work, u32 count, const tchar* desc, const Color& color = ColorWork, WorkPriority priority = WorkPriority_Normal, Bottleneck* bottleneck = nullptr) override final;
		virtual void DoWork(u32 count = 1) override final;
		virtual u32 GetWorkerCount() override final;
		virtual bool FlushWork(u32 timeoutMs = 0) override final;

		struct ClientStats
		{
			u64 send = 0;
			u64 recv = 0;
			u32 connectionCount = 0;
		};

		MutableLogger& GetLogger();
		u64 GetTotalSentBytes();
		u64 GetTotalRecvBytes();
		Timer& GetTotalSentTimer();
		u32 GetClientCount();
		u32 GetConnectionCount();
		void GetClientStats(ClientStats& out, u32 clientId);
		bool IsConnectedSlow(u32 clientId);
		void ResetTotalStats();

		bool DoAdditionalWork();

		bool SendResponse(const MessageInfo& info, const u8* body, u32 bodySize);
		bool SendKeepAlive();

		void RemoveDisconnectedConnections();

		class Worker;
	private:
		struct WorkerContext;
		class Connection;
		friend ConnectionInfo;

		Worker* PopWorker();
		Worker* PopWorkerNoLock();
		void PushWorker(Worker* worker);
		void PushWorkerNoLock(Worker* worker);
		void FlushWorkers();

		bool HandleSystemMessage(const ConnectionInfo& connectionInfo, u8 messageType, BinaryReader& reader, BinaryWriter& writer);
		bool AddConnection(NetworkBackend& backend, void* backendConnection, const sockaddr& remoteSocketAddr, bool requiresCrypto, CryptoKey cryptoKey);

		MutableLogger m_logger;

		struct CryptoEntry
		{
			CryptoKey key;
			u64 expirationTime;
			u64 userData;
		};
		Futex m_cryptoKeysLock;
		List<CryptoEntry> m_cryptoKeys;

		Guid m_uid;
		bool m_allowNewClients = true;

		struct WorkerRec { WorkerFunction func; TypeToNameFunction* toString = nullptr; };
		WorkerRec m_workerFunctions[4];
		OnConnectionFunction m_onConnectionFunction;
		
		struct OnDisconnectEntry { u8 id; OnDisconnectFunction function; };
		ReaderWriterLock m_onDisconnectFunctionsLock;
		List<OnDisconnectEntry> m_onDisconnectFunctions;

		u32 m_maxWorkerCount = 0;

		WorkQueue m_additionalWorkQueue;

		Futex m_availableWorkersLock;
		Worker* m_firstAvailableWorker = nullptr;
		Worker* m_firstActiveWorker = nullptr;

		struct PopWorkerRequest
		{
			EventSlim ev;
			PopWorkerRequest* next = nullptr;
			Worker* worker = nullptr;
		};
		PopWorkerRequest* m_firstRequest = nullptr;
		PopWorkerRequest* m_lastRequest = nullptr;

		u32 m_sendSize = 0;
		u32 m_receiveTimeoutMs = 0;
		u32 m_createdWorkerCount = 0;
		u32 m_maxCreatedWorkerCount = 0;
		u32 m_recvBufferSize;
		u32 m_sendBufferSize;
		bool m_workersEnabled = true;
		bool m_logConnections = true;
		bool m_useKeepAlive = false;
		bool m_useMessagePriority = false;
		bool m_useThreadedSend = false;
		bool m_allClientsDisconnected = false;

		Futex m_addConnectionsLock;
		List<Thread> m_addConnections;

		ReaderWriterLock m_connectionsLock;
		List<Connection> m_connections;
		u32 m_connectionIdCounter = 1;
		u32 m_maxActiveConnections = 0;

		Vector<char> m_clientsConfig;

		struct Client;
		ReaderWriterLock m_clientsLock;
		UnorderedMap<u32, Client> m_clients;
		u32 m_clientCounter = 0;

		Timer m_sendTimer;
		Timer m_encryptTimer;
		Timer m_decryptTimer;
		Atomic<u64> m_sendBytes = 0;
		Atomic<u64> m_recvBytes = 0;
		Atomic<u32> m_recvCount = 0;

		Trace* m_trace = nullptr;

		NetworkServer(const NetworkServer&) = delete;
		void operator=(const NetworkServer&) = delete;
	};
}
