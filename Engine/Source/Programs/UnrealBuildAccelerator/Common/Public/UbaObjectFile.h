// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaHash.h"
#include "UbaLogger.h"
#include "UbaString.h"
#include "UbaStringBuffer.h"
#include "UbaUnorderedMap.h"
#include <string>

namespace uba
{
	class FileAccessor;
	
	extern u8 SymbolFileVersion; // Current/default .exi format written. Reader accepts this and older (down to 2).

	using UnorderedSymbols = UnorderedSet<std::string>;
	using AllExternalImports = UnorderedSymbols;
	using AllInternalImports = UnorderedSymbols;
	using ExtraExports = Vector<std::string>;

	struct ExportInfo { std::string symbol; u8 isData = 0; u32 index = 0; u64 size = 0; u8 isHidden = 0; u32 alignment = 0; };
	using UnorderedExports = UnorderedMap<StringKey, ExportInfo>;

	using AllExports = UnorderedExports;

	enum ObjectFileType : u8
	{
		ObjectFileType_Unknown,
		ObjectFileType_Coff,
		ObjectFileType_Elf,
		ObjectFileType_LLVMIR,
		ObjectFileType_MachO,
	};

	enum ObjectFileParseMode : u8
	{
		ObjectFileParseMode_Exports,
		ObjectFileParseMode_All,
	};

	class ObjectFile
	{
	public:
		static ObjectFile* OpenAndParse(Logger& logger, ObjectFileParseMode parseMode, const tchar* fileName);
		static ObjectFile* Parse(Logger& logger, ObjectFileParseMode parseMode, u8* data, u64 dataSize, const tchar* hint);

		virtual bool CopyMemoryAndClose();
		bool WriteImportsAndExports(Logger& logger, MemoryBlock& memoryBlock, bool verbose, u8 version = SymbolFileVersion);
		bool WriteImportsAndExports(Logger& logger, const tchar* exportsFilename, bool verbose, u8 version = SymbolFileVersion);
		template<typename WriteFunc> bool WriteImportsAndExports(Logger& logger, const WriteFunc& write, bool verbose, u8 version = SymbolFileVersion);

		virtual const char* GetLibName();

		const tchar* GetFileName() const;
		const UnorderedSymbols& GetImports() const;
		const UnorderedExports& GetExports() const;
		const UnorderedExports& GetExplicitExports() const; // Symbols this file explicitly exports (COFF /EXPORT: directives). Verbatim names.
		const UnorderedSymbols& GetPotentialDuplicates() const;

		static bool CreateExtraFile(Logger& logger, const StringView& extraObjFilename, const StringView& moduleName, const StringView& platform, const AllExternalImports& allExternalImports, const AllInternalImports& allInternalImports, const AllExports& allExports, const AllExports& allExplicitExports, const ExtraExports& extraExports, bool includeExportsInFile);

		virtual ~ObjectFile();

		void RemoveExportedSymbol(const char* symbol);
		u8* GetData() { return m_data; }
		u64 GetDataSize() { return m_dataSize; }

	protected:
		virtual bool Parse(Logger& logger, ObjectFileParseMode parseMode, const tchar* hint) = 0;

		static bool CreateVersionScript(Logger& logger, MemoryBlock& memoryBlock, const AllExternalImports& allExternalImports, const AllInternalImports& allInternalImports, const AllExports& allExports, const ExtraExports& extraExports, bool includeExportsInFile, bool isDynList);
		static bool CreateEmdFile(Logger& logger, MemoryBlock& memoryBlock, const StringView& moduleName, const AllExternalImports& allExternalImports, const AllInternalImports& allInternalImports, const AllExports& allExports, bool includeExportsInFile);
		static bool CreateExportedSymbolsList(Logger& logger, MemoryBlock& memoryBlock, const AllExternalImports& allExternalImports, const AllInternalImports& allInternalImports, const AllExports& allExports, const ExtraExports& extraExports);

		FileAccessor* m_file = nullptr;
		u8* m_data = nullptr;
		u64 m_dataSize = 0;
		bool m_ownsData = false;

		ObjectFileType m_type;
		UnorderedSymbols m_imports;
		UnorderedExports m_exports;
		UnorderedExports m_explicitExports;
		UnorderedSymbols m_potentialDuplicates;

	public:
		struct AnsiStringView
		{
			const char* strBegin;
			const char* strEnd;

			char operator[](u64 pos) const
			{
				return strBegin[pos];
			}

			u32 Length() const
			{
				return u32(strEnd - strBegin);
			}

			bool StartsWith(const char* str, u32 strLen) const
			{
				if (strLen > Length())
					return false;
				return memcmp(strBegin, str, strLen) == 0;
			}

			bool Contains(const char* str, u32 strLen) const
			{
				const char* it = strBegin;
				const char* itEnd = strEnd - strLen + 1;
				while (it < itEnd)
				{
					if (memcmp(it, str, strLen) == 0)
						return true;
					++it;
				}
				return false;
			}

			bool Equals(const char* str, u32 strLen) const
			{
				if (strLen != Length())
					return false;
				return memcmp(strBegin, str, strLen) == 0;
			}

			std::string ToString() const
			{
				return std::string(strBegin, strEnd);
			}

			std::string& ToString(std::string& out) const
			{
				out.assign(strBegin, strEnd);
				return out;
			}

			AnsiStringView Skip(u64 count) const { return { strBegin + count, strEnd }; }

			AnsiStringView(const char* b, const char* e) : strBegin(b), strEnd(e) {}
			AnsiStringView(const std::string& s) : strBegin(s.data()), strEnd(strBegin + s.size()) {}
		};
	};

	struct SymbolFile
	{
		UnorderedSymbols imports;
		UnorderedExports exports;
		UnorderedExports explicitExports; // Verbatim names of symbols the source file explicitly exported (COFF /EXPORT:).
		ObjectFileType type;

		bool ParseFile(Logger& logger, const tchar* filename);
	};
}
