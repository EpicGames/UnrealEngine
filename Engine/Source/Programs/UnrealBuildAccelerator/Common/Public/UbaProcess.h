// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaProcessStartInfoHolder.h"
#include "UbaProtocol.h"
#include "UbaSession.h"
#include "UbaStats.h"

#define UBA_EXIT_CODE(x) (9000 + x)

namespace uba
{
	class Process
	{
	public:
		virtual const ProcessStartInfo& GetStartInfo() const = 0;
		virtual u32 GetId() { UBA_ASSERT(false); return 0; }
		virtual u32 GetExitCode() { UBA_ASSERT(false); return ~0u; }
		virtual bool HasExited()  { UBA_ASSERT(false); return false; }
		virtual bool WaitForExit(u32 millisecondsTimeout) { UBA_ASSERT(false); return false; }
		virtual u64 GetTotalProcessorTime() const { UBA_ASSERT(false); return 0; }
		virtual u64 GetTotalWallTime() const { UBA_ASSERT(false); return 0; }
		virtual u64 GetPeakMemory() const { UBA_ASSERT(false); return 0; }
		virtual const Vector<ProcessLogLine>& GetLogLines() const = 0;
		virtual const Vector<u8>& GetTrackedInputs() const = 0;
		virtual const Vector<u8>& GetTrackedOutputs() const = 0;
		virtual bool Cancel() { UBA_ASSERT(false); return false; }
		virtual bool IsCalledFromThis() const { return false; }
		virtual const tchar* GetExecutingHost() const { UBA_ASSERT(false); return nullptr; }
		virtual bool IsRemote() const { UBA_ASSERT(false); return false; }
		virtual ProcessExecutionType GetExecutionType() const { UBA_ASSERT(false); return ProcessExecutionType_Native; }
		virtual bool IsChild() { UBA_ASSERT(false); return false; }
		virtual bool IsArm() { UBA_ASSERT(false); return false; }
		virtual void TraverseOutputFiles(const Function<void(StringView file)>& func) {}

	protected:
		void AddRef();
		void Release();
		virtual ~Process() {}
		Atomic<u32> m_refCount = 0;
		friend ProcessHandle;
	};

	class ProcessImpl final : public Process
	{
	public:

		virtual const ProcessStartInfo& GetStartInfo() const override { return m_startInfo; }
		virtual u32 GetId() override { return m_id; }
		virtual u32 GetExitCode() override { return m_exitCode; }
		virtual bool HasExited() override  { return m_hasExited; }
		virtual bool WaitForExit(u32 millisecondsTimeout) override;
		virtual u64 GetTotalProcessorTime() const override;
		virtual u64 GetTotalWallTime() const override;
		virtual u64 GetPeakMemory() const override;
		virtual const Vector<ProcessLogLine>& GetLogLines() const override { return m_logLines; }
		virtual const Vector<u8>& GetTrackedInputs() const override { return m_trackedInputs; }
		virtual const Vector<u8>& GetTrackedOutputs() const override { return m_trackedOutputs; }
		virtual bool Cancel() override;
		virtual bool IsCalledFromThis() const override;
		virtual const tchar* GetExecutingHost() const override { return TC(""); }
		virtual bool IsRemote() const override { return false; }
		virtual ProcessExecutionType GetExecutionType() const override { return m_detourEnabled ? ProcessExecutionType_Local : ProcessExecutionType_Native; }
		virtual bool IsChild() override { return m_parentProcess != nullptr; }
		virtual bool IsArm() override { return m_isArmBinary; }
		virtual void TraverseOutputFiles(const Function<void(StringView file)>& func) override;

		ProcessImpl(Session& session, u32 id, ProcessImpl* parent, bool detourEnabled);
		~ProcessImpl();

		struct PipeReader;

		bool Start(const ProcessStartInfo& startInfo, bool runningRemote, Vector<tchar>& environment, bool async, void* userDataOverride);

		bool IsActive();
		bool IsCancelled();
		bool HasFailedMessage();

		bool WaitForRead();
		void SetWritten();

		void ThreadRun(Vector<tchar>& environment);
		void ThreadExit();

		bool HandleMessage(BinaryReader& reader, BinaryWriter& writer);
		bool HandleSpecialApplication();
		bool HandleGetWrittenFiles(bool isInit, BinaryWriter& writer);
		void CallSession(MessageType type, bool success);

		#define UBA_PROCESS_MESSAGE(M) bool Handle##M(BinaryReader& reader, BinaryWriter& writer);
		UBA_PROCESS_MESSAGES
		#undef UBA_PROCESS_MESSAGE
		bool WriteCreateProcessResponse(ProcessHandle h, BinaryWriter& writer, bool isChild);

		void LogLine(bool printInSession, TString&& line, LogEntryType logType);
		bool CreateTempFile(BinaryReader& reader, ProcHandle nativeProcessHandle, const tchar* application);
		bool OpenTempFile(BinaryReader& reader, BinaryWriter& writer, const tchar* application);
		bool WriteFilesToDisk();
		void CleanupWrittenFiles();

		const tchar* InternalGetChildLogFile(StringBufferBase& temp);
		
		#if !PLATFORM_WINDOWS
		bool PollStdPipes(PipeReader& outReader, PipeReader& errReader, int timeoutMs = -1);
		#endif

		u32 InternalCreateProcess(Vector<tchar>& environment, FileMappingHandle communicationHandle, u64 communicationOffset);
		u32	InternalExitProcess(bool cancel);
		void InternalLogLine(bool printInSession, TString&& line, LogEntryType logType);
		void InternalDestroyEvents();
		void WaitForParent();
		void WaitForChildrenExit();
		bool CancelWithError(bool dummy = true);
		void CallExitedFunc();

		ProcessStartInfoHolder m_startInfo;
		Session& m_session;
		ProcessImpl* m_parentProcess;
		Futex m_initLock;
		u32 m_id;
		FileMappingAllocation m_comMemory;
		void* m_userDataOverride = nullptr;

	#if PLATFORM_WINDOWS
		SharedEvent m_cancelEvent;
		SharedEvent m_writeEvent;
		SharedEvent m_readEvent;
	#else
		Futex m_comMemoryLock;
		SharedEvent& m_cancelEvent;
		SharedEvent& m_writeEvent;
		SharedEvent& m_readEvent;

		int m_stdOutPipe = -1;
		int m_stdErrPipe = -1;
		bool m_doOneExtraCheckForExitMessage = true;
	#endif

		ProcHandle m_nativeProcessHandle = InvalidProcHandle;

		#if PLATFORM_WINDOWS
		HANDLE m_nativeThreadHandle = 0;
		HANDLE m_accountingJobObject = NULL;
		#endif

		struct OverflowedWrittenFilesMessage;

		struct MoveSource { TString name; TString backedName; };

		struct Shared
		{
			Futex activeFilesLock;
			UnorderedMap<u32, ActiveFile> activeFiles;

			Futex writtenFilesLock;
			UnorderedMap<StringKey, WrittenFile> writtenFiles;
			UnorderedSet<SharedMemoryHandle> sharedMemoryHandles; // Track created memory handles. entries can be transferred to WrittenFiles
			DirectoryTableOverlay overlay;
			TString m_tempPath;
			u32 m_tempPathId = ~0u;
		
			Set<TString> createdDirectories;
			Set<TString> removedDirectories;
			Set<TString> deletedFiles;
			Map<TString, MoveSource> movedFiles; // To, From (note it is backwards). From can have backedName so we can turn it to a write if source gets overwritten

			#if !PLATFORM_WINDOWS
			Map<TString, TString> symlinks;
			#endif
		};

		OverflowedWrittenFilesMessage* m_firstOverflowMessage = nullptr;
		DirTableSnapshot* m_dirTableSnapshot = nullptr;

		u32 m_nativeProcessId = 0;
		u32 m_nativeProcessExitCode = ~0u;
		u64 m_nativeProcessCreationTime = 0;
		u32 m_exitCode = ~0u;
		u32 m_messageCount = 0;
		Atomic<bool> m_cancelAllowed = true;
		Atomic<bool> m_cancelled = false;
		Atomic<bool> m_hasExited = false;
		Atomic<bool> m_parentReportedExit = false;
		MessageType m_firstMessageError = MessageType_None;
		bool m_runningRemote = false;
		bool m_gotExitMessage = false;
		bool m_detourEnabled = true;
		bool m_isArmBinary = false;
		TString m_realApplication;
		const tchar* m_realWorkingDir = nullptr;
		u64 m_startTime = 0;
		u64 m_exitTime = 0;
		Event m_waitForParent;
		Futex m_logLinesLock;
		Vector<ProcessLogLine> m_logLines;
		Vector<u8> m_trackedInputs;
		Vector<u8> m_trackedOutputs;
		Vector<ProcessHandle> m_childProcesses;
		Shared& m_shared;
		SessionStats m_sessionStats;
		StorageStats m_storageStats;
		ProcessStats m_processStats;
		KernelStats m_kernelStats;

		struct UsedFileMapping
		{
			bool closeAfterWritten;
		};
		Futex m_usedFileMappingsLock;
		UnorderedMap<StringKey, UsedFileMapping> m_usedFileMappings;

		Event m_messageThreadDone;
		Thread* m_messageThread = nullptr;

		u8 m_extractExportsVersion = 0; // .exi format version requested by the /extractexports<N> token (2 or 3); 0 = no extraction

		ProcessImpl(const ProcessImpl&) = delete;
		void operator=(const ProcessImpl&) = delete;
	};
}
