// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaFunctional.h"
#include "UbaLogWriter.h"
#include "UbaString.h"
#include "UbaVector.h"

namespace uba
{
	class Process;
	struct ProcessLogLine;
	struct ProcessStartInfo;

	// Exit code returned by ProcessHandle.GetExitCode() if build was cancelled
	constexpr u32 ProcessCancelExitCode = 99999;


	struct ProcessLogLine
	{
		TString text;
		LogEntryType type = LogEntryType_Info;
	};

	enum ProcessExecutionType : u8
	{
		ProcessExecutionType_Native,    // Local with no detour
		ProcessExecutionType_Local,     // Local with detour
		ProcessExecutionType_Remote,    // Remotely with detour
		ProcessExecutionType_FromCache, // Downloaded from cache (scheduler only)
		ProcessExecutionType_Skipped,   // Skipped (scheduler only)
	};

	class ProcessHandle
	{
	public:
		bool IsValid() const;								// Returns true if handle points to a process
		const ProcessStartInfo& GetStartInfo() const;		// Get the info used to start process
		u32 GetId() const;									// Unique id of process. Is an increasing index internally
		u32 GetExitCode() const;							// Exit code of process. Will return ~0u if not finished or default error
		bool HasExited() const;								// Returns true if process has exited
		bool WaitForExit(u32 millisecondsTimeout) const;	// Wait for process until it exists or time reach timeout
		const Vector<ProcessLogLine>& GetLogLines() const;	// Log lines produced by the process
		const Vector<u8>& GetTrackedInputs() const;			// If ProcessStartInfo.trackInputs was true, this will return a buffer to wchar/char strings 
		const Vector<u8>& GetTrackedOutputs() const;		// If ProcessStartInfo.trackInputs was true, this will return a buffer to wchar/char strings 
		u64 GetTotalProcessorTime() const;					// Total used cpu time (in time units. Use TimeToMs etc)
		u64 GetTotalWallTime() const;						// Total wall time (in time units. Use TimeToMs etc)
		u64 GetPeakMemory() const;							// Peak memory usage (in bytes)
		bool Cancel() const;								// Request to cancel process. Returns false if failed to cancel (might be at the end of exiting process)
		const tchar* GetExecutingHost() const;				// Host that is executing process.
		bool IsRemote() const;								// Returns true if process is a remote process
		ProcessExecutionType GetExecutionType() const;		// Returns execution type

		void TraverseOutputFiles(const Function<void(StringView file)>& func) const; // Traverse output files.

		ProcessHandle();
		ProcessHandle(const ProcessHandle& o);
		ProcessHandle(ProcessHandle&& o) noexcept;
		ProcessHandle& operator=(const ProcessHandle& o);
		ProcessHandle& operator=(ProcessHandle&& o) noexcept;
		~ProcessHandle();
		inline bool operator==(const ProcessHandle& o) const { return m_process == o.m_process; }
		inline u64 GetHash() const { return u64(m_process); }

		ProcessHandle(Process* process);
	private:
		Process* m_process;
		friend class ProcessImpl;
		friend class Scheduler;
		friend class Session;
		friend class SessionClient;
		friend class SessionServer;
	};
}