// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaDefaultConstants.h"
#include "UbaLogWriter.h"

namespace uba
{
	class ApplicationRules;
	class ProcessHandle;
	enum ProcessExitedResponse : u8;
	using RootsHandle = u64;

	struct ProcessStartInfo
	{
		UBA_API ProcessStartInfo();
		UBA_API ~ProcessStartInfo();
		UBA_API ProcessStartInfo(const ProcessStartInfo&);

		UBA_API const tchar* GetDescription() const;	// Use this instead since it will use application name if description is not set

		const tchar* application = TC("");		// Application name, cl.exe etc. Use full path
		const tchar* arguments = TC("");		// Arguments. Should not include application name
		const tchar* workingDir = TC("");		// Working directory. Use full path
		const tchar* description = TC("");		// Description. Used for on-screen logging and log file names if session.logToFile is set but logFile is ""
		const tchar* logFile = TC("");			// Log file. If set, will always log. If not full path the session log dir will be prepended.
		u32 priorityClass = ProcessPriority_Normal; // Priority of process.
		bool trackInputs = false;				// Track all files read. Can read result in ProcessHandle.GetTrackedInputs()
		bool useCustomAllocator = true;			// Disable detouring of allocator inside processes. If Session.disableCustomAllocator is false this will be overridden
		bool writeOutputFilesOnFail = false;	// If set to true, output files will be written to disk regardless if process succeeds or not
		const tchar* breadcrumbs = TC("");		// If not empty, write additional information to the UBA trace file
		bool startSuspended = false;			// Start process suspended.. a bit internal atm and only supported on windows
		bool reportAllExceptions = false;		// Report all SEH exceptions happening in the process regardless if they are handled or not
		bool expanded = false;					// Internal: set to true by ProcessStartInfoHolder::Expand so subsequent Expand calls are no-ops. Lives on the base struct so it survives slicing via ProcessStartInfoHolder's conversion constructor (which copies the base via *(ProcessStartInfo*)this = si).

		RootsHandle rootsHandle = 0;			// Handle if using vfs. Handle is generated through Session::RegisterRoots

		using LogLineCallback = void(void* userData, const tchar* line, u32 length, LogEntryType type);
		LogLineCallback* logLineFunc = nullptr;	// Callback for when log entries happens
		void* logLineUserData = nullptr;		// User data provided to logLine callback

		using ExitedCallback = void(void* userData, const ProcessHandle&, ProcessExitedResponse&);
		ExitedCallback* exitedFunc = nullptr;	// Callback for when process is done (it has already exited)
		void* userData = nullptr;				// User data provided to exit and predicted memory callbacks

		const tchar* overlayFiles = TC("");   // Semi colon separated string containing paths to overlay files (files that are only visible to this process)

		const ApplicationRules* rules = nullptr;// Internal use (for now)
		int uiLanguage = 1033;					// Internal use
	};

	enum ProcessExitedResponse : u8
	{
		ProcessExitedResponse_None,
		ProcessExitedResponse_RerunLocal,
		ProcessExitedResponse_RerunNative,
	};
}
