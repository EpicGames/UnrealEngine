// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBinaryReaderWriter.h"
#include "UbaFile.h"
#include "UbaPathUtils.h"
#include "UbaProcessStartInfo.h"

namespace uba
{
	class Config;

	inline void FixFileName(StringBufferBase& out, const tchar* fileName, const tchar* workingDir)
	{
		tchar buffer[1024];
		u32 charLen;
		u64 workingDirLen = 0;
		if (workingDir)
			workingDirLen = TStrlen(workingDir);
		FixPath2(fileName, ~0u, workingDir, workingDirLen, buffer, sizeof_array(buffer), &charLen);
		out.Append(buffer);
	}

	struct ProcessStartInfoHolder : public ProcessStartInfo
	{
		ProcessStartInfoHolder() = default;

		ProcessStartInfoHolder(const ProcessStartInfo& si)
		{
			*(ProcessStartInfo*)this = si;

			StringBuffer<512> temp;
			FixFileName(temp, si.workingDir, nullptr);
			temp.EnsureEndsWithSlash();
			workingDirStr = temp.data;
			workingDir = workingDirStr.c_str();

			applicationStr = si.application;
			application = applicationStr.c_str();

			argumentsStr = si.arguments;
			arguments = argumentsStr.c_str();

			if (si.description)
				descriptionStr = si.description;
			description = descriptionStr.c_str();

			logFileStr = si.logFile;
			logFile = logFileStr.c_str();

			breadcrumbsStr = si.breadcrumbs;
			breadcrumbs = breadcrumbsStr.c_str();

			overlayFilesStr = si.overlayFiles;
			overlayFiles = overlayFilesStr.c_str();
		}

		void Write(BinaryWriter& writer, const StringView& applicationOverride)
		{
			writer.WriteString(descriptionStr);
			if (applicationOverride.count)
				writer.WriteString(applicationOverride);
			else
				writer.WriteString(applicationStr);
			writer.WriteLongString(argumentsStr, 1000);
			writer.WriteString(workingDirStr);
			writer.WriteString(logFileStr);
			// ignore breadcrumbs here
			writer.WriteU32(*(u32*)&weight);
			writer.WriteBool(trackInputs);
			writer.WriteBool(writeOutputFilesOnFail);
			writer.WriteU64(rootsHandle);
		}

		void Read(BinaryReader& reader)
		{
			descriptionStr = reader.ReadString();
			applicationStr = reader.ReadString();
			argumentsStr = reader.ReadLongString();
			workingDirStr = reader.ReadString();
			logFileStr = reader.ReadString();
			// ignore breadcrumbs here

			Replace(applicationStr.data(), '/', PathSeparator); // TODO: Is this needed?

			u32 weight32 = reader.ReadU32();
			weight = *(float*)&weight32;
			
			trackInputs = reader.ReadBool();
			writeOutputFilesOnFail = reader.ReadBool();
			rootsHandle = reader.ReadU64();

			description = descriptionStr.c_str();
			application = applicationStr.c_str();
			arguments = argumentsStr.c_str();
			workingDir = workingDirStr.c_str();
			logFile = logFileStr.c_str();

			// The wire form was written by Write() from a Holder that had
			// already been Expanded on the host — do not re-wrap it here.
			expanded = true;
		}

		ProcessStartInfoHolder(const ProcessStartInfoHolder& o)
		{
			*this = o;
		}

		void operator=(const ProcessStartInfoHolder& o)
		{
			*(ProcessStartInfo*)this = o;

			workingDirStr = o.workingDirStr;
			workingDir = workingDirStr.c_str();
			applicationStr = o.applicationStr;
			application = applicationStr.c_str();
			argumentsStr = o.argumentsStr;
			arguments = argumentsStr.c_str();
			descriptionStr = o.descriptionStr;
			description = descriptionStr.c_str();
			logFileStr = o.logFileStr;
			logFile = logFileStr.c_str();
			breadcrumbsStr = o.breadcrumbsStr;
			breadcrumbs = breadcrumbsStr.c_str();
			overlayFilesStr = o.overlayFiles;
			overlayFiles = overlayFilesStr.c_str();

			weight = o.weight;
			// Propagate the "already-expanded" flag so callers that copy an
			// expanded Holder into a new one (ProcessImpl::Start does this via
			// m_startInfo = startInfo) don't trigger a second wrap on the same
			// command — the second wrap produces `-c "sh -c \"\\"...\""` which
			// ParseArguments can't re-tokenize correctly.
			expanded = o.expanded;
		}

		bool Expand();

		void Apply(const Config& config, const uba::tchar* configTable = TC(""));

		TString descriptionStr;
		TString applicationStr;
		TString argumentsStr;
		TString workingDirStr;
		TString logFileStr;
		TString breadcrumbsStr;
		TString overlayFilesStr;
		float weight = 1.0f;

		bool InternalExpand(const tchar* firstArgBegin, const tchar* firstArgEnd);
	};
}