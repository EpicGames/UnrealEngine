// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaHash.h"
#include "UbaLogger.h"

namespace uba
{
	static constexpr u32 PathsPerRoot = IsWindows ? 4u : 1;

	class RootPaths
	{
	public:

		bool RegisterRoot(Logger& logger, const tchar* rootPath, bool includeInKey = true, u8 id = 0);
		bool RegisterSystemRoots(Logger& logger, u8 startId = 0);
		bool RegisterIgnoredRoot(Logger& logger, const tchar* rootPath);

		bool IsEmpty() const;

		struct Root
		{
			TString path;
			StringKey shortestPathKey;
			u8 index;
			bool includeInKey;
		};

		const Root* FindRoot(const StringView& path) const;
		const TString& GetRoot(u32 index) const;

		TString GetAllRoots() const;

		template<typename CharType, typename Func>
		bool NormalizeString(Logger& logger, const CharType* str, u64 strLen, const Func& func, bool allowPathsWithoutRoot, const tchar* hint, const tchar* hint2 = TC("")) const;

		CasKey NormalizeAndHashFile(Logger& logger, const tchar* filename, bool warnOnFileNotFound = false) const;

		static constexpr u8 RootStartByte = ' ';

	private:
		struct Roots : Vector<Root>
		{
			u32 shortestRoot = 0;
			u32 longestRoot = 0;
		};

		bool InternalRegisterRoot(Logger& logger, Roots& roots, const tchar* rootPath, bool includeInKey, u8 id);
		bool InternalRegisterRoot2(Logger& logger, Roots& roots, const tchar* rootPath, bool includeInKey, u8 index);
		const Root* InternalFindRoot(const Roots& roots, const StringView& path) const;

		Roots m_roots;
		Roots m_ignoredRoots;
	};



	template<typename CharType, typename Func>
	bool RootPaths::NormalizeString(Logger& logger, const CharType* str, u64 strLen, const Func& func, bool allowPathsWithoutRoot, const tchar* hint, const tchar* hint2) const
	{
		auto strEnd = str + strLen;
		auto searchPos = str;

		u32 destPos = 0;

#if !PLATFORM_WINDOWS
		allowPathsWithoutRoot = true; // Posix uses forward slash and no drive letter so we can't see if a path is in the middle of a path or beginning
#endif

		while (true)
		{
			auto absPathChars = searchPos;
			CharType lastChar = 0;
#if PLATFORM_WINDOWS
			while (absPathChars < strEnd && !(lastChar == ':' && (*absPathChars == '\\' || *absPathChars == '/')))
			{
				lastChar = *absPathChars;
				++absPathChars;
			}
#else
			while (absPathChars < strEnd && *absPathChars != '/')
			{
				lastChar = *absPathChars;
				++absPathChars;
			}
#endif
		
			if (absPathChars == strEnd)
			{
				func(searchPos, strEnd - searchPos, ~0u); // Yes, this contains the terminating character too which is a bit weird
				return true;
			}

			auto pathStart = absPathChars - (IsWindows ? 2 : 0);

			auto pathEndOrMore = pathStart;
			while (pathEndOrMore < strEnd && *pathEndOrMore != '\n')
				++pathEndOrMore;

			u32 lenOrMore = u32(pathEndOrMore - pathStart);
			u32 toCopy = Min(lenOrMore, m_roots.longestRoot);
			StringBuffer<512> path;
			path.Append(pathStart, toCopy);

			auto root = FindRoot(path);
			if (!root)
			{
				bool skip = allowPathsWithoutRoot;
#if PLATFORM_WINDOWS
				if (str + 2 <= absPathChars)
				{
					CharType driveLetter = absPathChars[-2];
					if (!(driveLetter >= 'a' && driveLetter <= 'z' || driveLetter >= 'A' && driveLetter <= 'Z')) // Not a drive
						skip = true;
					else if (str + 3 <= absPathChars && absPathChars[0] == '/' && absPathChars[1] == '/')
					{
						CharType preDrive = absPathChars[-3];
						if (preDrive >= 'a' && preDrive <= 'z' || preDrive >= 'A' && preDrive <= 'Z')  // http://, https:// or file://
							skip = true;
					}
				}
#endif

				if (skip || InternalFindRoot(m_ignoredRoots, path))
				{
					u32 len = u32(absPathChars - searchPos) + 1;
					destPos += len;
					func(searchPos, len, ~0u);
					searchPos += len;
					continue;
				}

				if (auto lastQuote = path.Last('\"'))
					path.Resize(lastQuote - path.data);
				if (auto lineEnd = path.Last('\r'))
					path.Resize(lineEnd - path.data);
#if PLATFORM_WINDOWS
				logger.Info(TC("PATH WITHOUT ROOT: %s (inside %s at offset %u%s)"), path.data, hint, destPos, hint2);
#endif
				return false;
			}

			if (u32 len = u32(pathStart - searchPos))
			{
				destPos += len;
				func(searchPos, len, ~0u);
			}
			CharType temp = RootStartByte + CharType(root->index);
			func(&temp, 1, destPos);
			destPos += 1;

			searchPos = pathStart + root->path.size();
		}
	}
}
