// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaList.h"
#include "UbaMap.h"
#include "UbaMemory.h"
#include "UbaProcessStartInfo.h"
#include "UbaThread.h"
#include "UbaUnorderedMap.h"

namespace uba
{
	class CacheClient;
	class Config;
	class ConfigTable;
	class Process;
	class ProcessImpl;
	class RootPaths;
	class SessionServer;
	class WorkManagerImpl;
	struct NextProcessInfo;
	struct ProcessStartInfoHolder;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct SchedulerCreateInfo
	{
		SchedulerCreateInfo(SessionServer& s) : session(s) {}

		void Apply(const Config& config);

		SessionServer& session;
		CacheClient** cacheClients = nullptr; // Set cache clients for scheduler to use when building
		u32 cacheClientCount = 0;
		u32 maxLocalProcessors = ~0u; // Max local processors to use. ~0u means it will use all processors
		u32 maxParallelCacheQueries = 16u; // Max number of parallel cache queries allowed
		bool enableProcessReuse = false; // If this is true, the system will allow processes to be reused when they're asking for it.
		bool forceRemote = false; // Force all processes that can run remotely to run remotely.
		bool forceNative = false; // Force all processes to run native (not detoured)
		bool writeToCache = false; // Set to true in combination with setting cacheClient to populate cache
		bool useThreadToExitRemoteProcess = false; // Set this to true to move exit process call from job to thread. 
		bool memWatchdog = false; // Will watch memory and throttle/kill processes if needed
		bool writeTrackedInputs = false; // Set to true to write out log files with tracked inputs
		bool deferRemoteCapableProcesses = false; // Set to true to have local session first pick local-only processes
		bool enableSpeculativeCacheTest = false; // Set to true to allow cache testing of a process before its dependencies have finished downloading. Requires CacheClient support for speculative CAS keys to be correct.
		bool traceDetails = false; // Show details of scheduler states in visualizer 
		bool leakMemoryAtShutdown = false;
		u32 maxRacingPercent = 0u; // If processor weights go below this percent, then we will start filling up with racing processes
		u8 memTracingLevel = 0;
		u8 memStartWaitPercent = 85; // When memory usage goes above this percent, no new processes will be spawned until back below
		u8 memStartKillPercent = 95; // When memory usage goes above this percent, newest processes will be killed to bring it back below
		u32 memPagefileActivityThreshold = 0; // When page file activity goes over this number between each watchdog update we start killing processes. 0=disabled
		ConfigTable* processConfigs = nullptr;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct EnqueueProcessInfo
	{
		EnqueueProcessInfo(const ProcessStartInfo& i) : info(i) {}

		const ProcessStartInfo& info;
		
		float weight = 1.0f; // Weight of process. This is used towards max local processors. If a process is multithreaded it is likely it's weight should be more than 1.0
		bool canDetour = true; // If true, uba will detour the process. If false it will just create pipes for std out and then run the process as-is.
		bool canExecuteRemotely = true; // If true, this process can run on other machines, if false it will always be executed locally
		bool canCrossArchitecture = false; // If true, this process can run on other architecture
		bool canCrossPlatform = false; // If true, this process can run on other platform.

		const void* knownInputs = nullptr; // knownInputs is a memory block with null terminated tchar strings followed by an empty null terminated string to end.
		u32 knownInputsBytes = 0; // knownInputsBytes is the total size in bytes of knownInputs
		u32 knownInputsCount = 0; // knownInputsCount is the number of strings in the memory block

		const void* knownOutputs = nullptr; // knownOutputs is a memory block with null terminated tchar strings followed by an empty null terminated string to end.
		u32 knownOutputsBytes = 0; // knownOutputsBytes is the total size in bytes of knownOutputs
		u32 knownOutputsCount = 0; // knownOutputsCount is the number of strings in the memory block

		const u32* dependencies = nullptr; // An array of u32 holding indicies to processes this process depends on. Index is a rolling number returned by EnqueueProcess
		u32 dependencyCount = 0; // Number of elements in dependencies

		u32 cacheBucketId = 0; // Bucket that cache should be fetched from. Zero means that it will not fetch anything
		u64 memoryGroupId = 0; // Group that calculates average of memory usage.

		u64 predictedMemoryUsage = 0; // If zero, this will not be used for memory predictions

		bool skipAsSuccessWhenScheduled = false; // If set to true, then this process will be seen as successful straight away when scheduled
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	#define UBA_PROCESS_CATEGORIES \
		UBA_PROCESS_CATEGORY(Local) \
		UBA_PROCESS_CATEGORY(Remote) \
		UBA_PROCESS_CATEGORY(RemoteCrossArch) \
		UBA_PROCESS_CATEGORY(RemoteCrossPlatform) \
		UBA_PROCESS_CATEGORY(RemoteCrossArchAndPlatform) \

	// SchedulerStats contains following variables. Count is number of processes, weight is the accumulated weight of same processes
	//
	//   active   - Processes currently executing. Note: categorised by capability, not where they are actually running.
	//   ready    - Cache miss confirmed (or no cache); available to run immediately if a slot is free.
	//   pending  - Not yet confirmed as needing a worker: either blocked on dependencies or somewhere in the cache pipeline.
	//   running  - Where processes are actually executing (by helper location, not capability).

	struct SchedulerStats
	{
		#define UBA_PROCESS_CATEGORY(cat) \
			u32 active##cat##Count = 0; \
			float active##cat##Weight = 0; \
			u32 ready##cat##Count = 0; \
			float ready##cat##Weight = 0; \
			u32 pending##cat##Count = 0; \
			float pending##cat##Weight = 0; \
			u32 running##cat##Count = 0; \
			float running##cat##Weight = 0;
		UBA_PROCESS_CATEGORIES
		#undef UBA_PROCESS_CATEGORY
		u32 activeCacheQueries = 0;
		u32 activeCacheDownloads = 0;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class Scheduler
	{
	public:
		Scheduler(const SchedulerCreateInfo& info);
		~Scheduler();

		void Start(); // Start scheduler thread. Should be called before server starts listen to connections if using remote help
		void Stop(); // Will wait on all active processes and then exit.
		void Cancel(); // Cancel and wait
		void SetMaxLocalProcessors(u32 maxLocalProcessors); // Set max local processes
		void SetAllowDisableRemoteExecution(bool allow); // Allow scheduler to tell clients to disconnect early if running out of processes

		u32 EnqueueProcess(const EnqueueProcessInfo& info); // Returns index of process. Index is a rolling number

		u32 EnqueueSuspendedProcess(); // Can be used to put entries in the queue early in order to calculate process info afterwards
		void ResumeQueuedProcess(u32 index, const EnqueueProcessInfo& info); // Resume process queued with EnqueueSuspended

		u32 GetTotalProcesses();
		void GetStats(u32& outQueued, u32& outActiveLocal, u32& outActiveRemote, u32& outFinished);
		void GetStats(SchedulerStats& outStats);
		void GetResults(u32& outSucceeded, u32& outFailed, u32& outSkipped);

		bool IsEmpty(); // Returns true if scheduler is entirely empty.. as in no processes are left in the system

		bool EnqueueFromFile(const tchar* yamlFilename, const Function<void(EnqueueProcessInfo&)>& enqueued = {}); // Enqueue actions from file. Example of format of file is at the end of this file
		bool EnqueueFromSpecialJson(const tchar* jsonFilename, const tchar* workingDir, const tchar* description, RootsHandle rootsHandle, void* userData = nullptr);

		void SetProcessFinishedCallback(const Function<void(const ProcessHandle&)>& processFinished); // Set callback
		void SetPredictedMemoryCallback(const Function<u64(void* userData)>& predictedMemory); // Set callback queried per process (via its userData) to obtain an up-to-date memory estimate in bytes (0 means unknown). Called from the scheduler thread while it holds internal locks, so it must return quickly and must not call back into the scheduler.

		SessionServer& GetSession() { return m_session; }
		u32 GetCacheClientCount() { return u32(m_cacheClients.size()); }
		CacheClient& GetCacheClient(u32 index) { return *m_cacheClients[index]; }
		
		u32 GetProcessCountThatCanRunRemotelyNow();
		void GetProcessWeightThatCanRunRemotelyNow(float& outTotal, float& outCrossArchitecture, float& outCrossPlatform);

	private:
		struct CacheFetchInfo;
		struct ExitProcessInfo;
		struct ProcessStartInfo2;

		enum RunStatus : u8
		{
			RunStatus_Suspended,
			RunStatus_QueuedForRun,
			RunStatus_Running,
			RunStatus_Success,
			RunStatus_Failed,
			RunStatus_Skipped,
		};

		enum StatsState : u8 { StatsState_None, StatsState_Pending, StatsState_Ready, StatsState_Active };

		enum CacheStatus : u8
		{
			CacheStatus_Suspended,
			CacheStatus_QueuedForTest,             // not yet in any queue; blocked on deps or awaiting initial scheduling
			CacheStatus_QueuedForTestSpeculative,  // in m_readyToTestCache; deps still pending (speculative start)
			CacheStatus_Testing,                   // test worker in flight
			CacheStatus_QueuedForDownload,         // hit confirmed; in m_readyToDownloadCache
			CacheStatus_Downloading,               // download worker in flight
			CacheStatus_PendingCompletion,         // download done; deps still pending (will fire when last dep completes)
			CacheStatus_Success,
			CacheStatus_Failed,
		};

		enum Category : u8
		{
			#define UBA_PROCESS_CATEGORY(cat) Category_##cat,
			UBA_PROCESS_CATEGORIES
			#undef UBA_PROCESS_CATEGORY
		};

		UBA_NOINLINE void ThreadLoop();

		struct ProcessEntry;
		struct ProcessLink;
		struct ProcessQueue;

		ProcessEntry CreateProcessEntry(const EnqueueProcessInfo& info);
		ProcessQueue& GetReadyToRunQueue(ProcessEntry& entry);
		void AddToQueue(ProcessQueue& queue, u32 entryIndex);
		u32 PopFromQueue(ProcessQueue& queue);
		ProcessEntry* PeekFromQueue(ProcessQueue& queue);
		void EraseFromQueue(ProcessQueue& queue, u32 entryIndex);
		u32 GetQueueCount(ProcessQueue& queue);
		void ClearQueue(ProcessQueue& queue);
		void SetupDependents(ProcessEntry& entry, u32 entryIndex, const EnqueueProcessInfo& info);
		void UpdateDependents(RunStatus newRunStatus, ProcessLink* firstDependent, bool hadCacheHit);
		void UpdateSpeculativeDependentsOnCacheHit(ProcessLink* firstDependent);
		bool DrainCompleteFromCache();
		void SkipAllQueued();
		bool ProcessReturned(Process& process, bool isLocal);
		void HandleCacheMissed(u32 processIndex);
		void RemoteSlotAvailable(bool isCrossArchitecture, bool isCrossPlatform);
		void ProcessExited(ExitProcessInfo* info, const ProcessHandle& handle);
		bool CanRun();
		u32 PopProcessNoLock(bool isLocal, bool isCrossArchitecture, bool isCrossPlatform, RunStatus& outPrevStatus, bool forReuse, bool& outShouldSleep, ProcessLink*& outSkipFirstDependent);
		bool RunCacheQuery();
		bool RunQueuedProcess(bool isLocal, bool isCrossArchitecture, bool isCrossPlatform);
		bool RaceRemoteProcess();
		bool HandleGetNextProcess(Process& process, NextProcessInfo& outNextProcess, u32 prevExitCode, BinaryReader& prevStats, u32 timeoutMs);
		void ExitProcess(ExitProcessInfo& info, Process& process, u32 exitCode, bool fromCache, bool forReuse);
		void SkipProcess(ProcessStartInfo2& info);
		void UpdateStatusHitMissCount();
		void UpdateQueueCounter();
		void StatsActive(SchedulerStats& s, Category cat, int d, float w);
		void StatsReady(SchedulerStats& s, Category cat, int d, float w);
		void StatsPending(SchedulerStats& s, Category cat, int d, float w);
		void StatsRunning(SchedulerStats& s, Category cat, int d, float w);
		void TransitionStats(ProcessEntry& entry, StatsState newState, float weight);
		void FinishProcess(const ProcessHandle& handle);
		void CreateCacheInputsAndOutputs(Vector<u8>& outInputs, Vector<u8>& outOutputs, ProcessStartInfo2& info, Process& process);
		void WriteTrackedInputs(const Vector<u8>& inputs, const Vector<u8>& outputs, Process& process);
		void KillNewestLocalProcess(StringView reason);
		void NativeProcessCreated(ProcessImpl& process);
		void LocalProcessStart(ProcessStartInfo2& info);
		void UpdatePredictedMemoryUsage(ProcessStartInfo2& info);
		bool LocalProcessMemoryOkToSpawn(u32 runningLocalCount, u64 predictedMemoryUsage);
		void SetMemDelaySpawning(bool delaySpawning);
		bool LocalProcessOkToSpawn(u32 runningLocalCount, u64 predictedMemoryUsage = 0);
		void LocalProcessExit(ProcessStartInfo2& info, const ProcessHandle& handle);
		void LoadMemTrackTable();
		void SaveMemTrackTable();
		UBA_NOINLINE void ThreadMemoryCheckLoop();

		struct ProcessEntry
		{
			ProcessStartInfo2* info;
			ProcessLink* firstDependent;
			u32 dependencyCount;
			u32 speculativeDependencyCount;
			CacheStatus cacheStatus;
			RunStatus runStatus;
			StatsState statsState;
			Category category;
			bool shouldSkip;
			bool canDetour;
		};

		struct ProcessQueue : Set<u32> { Futex lock; };

		SessionServer& m_session;
		u32 m_maxLocalProcessors;
		float m_maxRacingWeight;
		u32 m_maxParallelCacheQueries;

		Futex m_processEntriesLock;
		Vector<ProcessEntry> m_processEntries;

		ProcessQueue m_readyToTestCache;
		ProcessQueue m_readyToDownloadCache;
		ProcessQueue m_readyToRunLocal; // Can only run locally
		ProcessQueue m_readyToRunRemote; // Can run remotely on same architecture
		ProcessQueue m_readyToRunCrossArchitecture;    // Can run remotely on a different architecture
		ProcessQueue m_readyToRunCrossPlatform;        // Can run remotely on a different platform
		ProcessQueue m_readyToRunCrossArchAndPlatform; // Can run remotely on a different architecture AND platform
		ProcessQueue m_readyToSkip;
		ProcessQueue m_readyToCompleteFromCache; // Cache download done but deps were still pending; fired when last dep completes

		Function<void(const ProcessHandle&)> m_processFinished;
		Function<u64(void* userData)> m_predictedMemoryFunc;

		Event m_updateThreadWakeup;
		Thread m_updateThread;

		Atomic<bool> m_loop = false;
		bool m_enableProcessReuse;
		bool m_forceRemote;
		bool m_forceNative;
		bool m_useThreadToExitRemoteProcess;
		bool m_allowDisableRemoteExecution = false;
		bool m_writeTrackedInputs = false;
		bool m_deferRemoteCapableProcesses = false;
		bool m_enableSpeculativeCacheTest = false;
		bool m_leakMemoryAtShutdown = false;
		bool m_traceDetails = false;
		bool m_cancelled = false;
		bool m_hadMatchingSystemRequest = false;
		bool m_hadCrossSystemRequest = false;
		ConfigTable* m_processConfigs = nullptr;

		Atomic<float> m_activeLocalProcessWeight = 0.0f;
		Atomic<float> m_activeRacingProcessWeight = 0.0f;
		Atomic<u32> m_getNextQueueCount = 0;

		SchedulerStats m_stats;

		Atomic<u32> m_totalProcesses = 0; // Process count registered in to scheduler
		Atomic<u32> m_finishedProcesses = 0; // Processes finished (includes skipped)
		Atomic<u32> m_errorCount = 0;
		Atomic<u32> m_cacheHitCount = 0;
		Atomic<u32> m_cacheMissCount = 0;

		Vector<CacheClient*> m_cacheClients;
		Vector<RootPaths*> m_rootPaths;
		bool m_writeToCache;

		WorkManagerImpl* m_cacheWorkManager = nullptr;
		List<Thread> m_finishedRemoteProcessThreads;

		CriticalSection m_wasReturnedLock;

		bool m_memWatchdog = false;
		Event m_memThreadEvent;
		Thread m_memThread;
		bool m_memDelaySpawning = false;
		Atomic<u64> m_memEstimatedUsage = 0;
		Atomic<u64> m_memWaitThreshold = 0;
		Atomic<u64> m_lastKillTime = 0;
		u8 m_memStartWaitPercent = 0;
		u8 m_memStartKillPercent = 0;
		u8 m_memTracingLevel = 0;
		u32 m_memPagefileActivityThreshold = 0;

		struct MemGroupStats;
		struct MemGroupEntry;
		Futex m_memGroupLookupLock;
		UnorderedMap<u64, MemGroupStats> m_memGroupLookup;
		UnorderedMap<u64, MemGroupEntry> m_memGroupDb;

		Futex m_processesLock;
		struct ProcessRecord { ProcessImpl& process; ExitProcessInfo* ep; };
		UnorderedMap<StringKey, ProcessRecord> m_processes;

		Futex m_racingProcessesLock;
		List<ProcessHandle> m_racingProcesses;

		Scheduler(const Scheduler&) = delete;
		void operator=(const Scheduler&) = delete;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////
}

#if 0
// Example of yaml file with processes that can be queued up in scheduler
// id - Not used right now. Number does not matter because id will be a rolling number
// app - Application to execute
// arg - Arguments to application
// dir - Working directory
// desc - Description of process
// weight - How much cpu this process is using. Optional. Defaults to 1.0
// remote - Decides if this process can execute remotely. Optional. Defaults to true
// detour - Decides if this process can be detoured. Optional. Defaults to true.
// dep - Dependencies. An array of indices to other processes

processes:
  - id: 0
    app: C:\Program Files\Microsoft Visual Studio\2022\Professional\VC\Tools\MSVC\14.38.33130\bin\Hostx64\x64\cl.exe
    arg: @"..\Intermediate\Build\Win64\x64\UnrealPak\Development\Core\SharedPCH.Core.Cpp20.h.obj.rsp"
    dir: E:\dev\fn\Engine\Source
    desc: SharedPCH.Core.Cpp20.cpp
    weight: 1.25
    remote: false

  - id: 44
    app: C:\Program Files\Microsoft Visual Studio\2022\Professional\VC\Tools\MSVC\14.38.33130\bin\Hostx64\x64\cl.exe
    arg: @"..\Intermediate\Build\Win64\x64\UnrealPak\Development\Json\Module.Json.cpp.obj.rsp"
    dir: E:\dev\fn\Engine\Source
    desc: Module.Json.cpp
    weight: 1.5
    dep: [0]

  - id: 337
    app: E:\dev\fn\Engine\Binaries\ThirdParty\DotNet\6.0.302\windows\dotnet.exe
    arg: "E:\dev\fn\Engine\Binaries\DotNET\UnrealBuildTool\UnrealBuildTool.dll" -Mode=WriteMetadata -Input="E:\dev\fn\Engine\Intermediate\Build\Win64\x64\UnrealPak\Development\TargetMetadata.dat" -Version=2
    dir: E:\dev\fn\Engine\Source
    desc: UnrealPak.target
    detour: false
    dep: [336, 0]

#endif