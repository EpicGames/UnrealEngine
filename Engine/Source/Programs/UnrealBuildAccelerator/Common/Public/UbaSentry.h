// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(UBA_USE_SENTRY)
#define SENTRY_BUILD_STATIC 1
#include <sentry.h>
#define UBA_SENTRY_INFO(...) do { if (x) break; StringBuffer<> _buf; _buf.Appendf(__VA_ARGS__); sentry_capture_event(sentry_value_new_message_event(SENTRY_LEVEL_INFO, "custom", _buf.data); } while(false)
#else
#define UBA_SENTRY_INFO(...)
#endif
