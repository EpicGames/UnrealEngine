// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBottleneck.h"
#include "UbaDependencyCrawler.h"
#include "UbaFixedSizeFileMappingAllocator.h"
#include "UbaDefinitions.h"
#include "UbaDirectoryTableHolder.h"
#include "UbaFileMappingBuffer.h"
#include "UbaProcessHandle.h"
#include "UbaRootPaths.h"
#include "UbaSessionCreateInfo.h"
#include "UbaSharedMemoryView.h"
#include "UbaStats.h"
#include "UbaTrace.h"
#include "UbaThread.h"
#include <string>

#define UBA_DEBUG_TRACK_MAPPING 0//UBA_DEBUG_LOGGER

namespace uba
{
	class FileAccessor;
	class Process;
	class ProcessHandle;
	class ProcessImpl;
	class SharedMemoryAllocator;
	class Storage;
	class WorkManager;
	struct ProcessStartInfo;
	struct ProcessStartInfoHolder;
	struct ProcessStats;
	struct InitMessage;
	struct InitResponse;
	struct CreateFileMessage;
	struct CreateFileResponse;
	struct CloseFileMessage;
	struct CloseFileResponse;
	struct DeleteFileMessage;
	struct DeleteFileResponse;
	struct CopyFileMessage;
	struct CopyFileResponse;
	struct MoveFileMessage;
	struct MoveFileResponse;
	struct ChmodResponse;
	struct ChmodMessage;
	struct SymlinkMessage;
	struct SymlinkResponse;
	struct GetFullFileNameMessage;
	struct GetFullFileNameResponse;
	struct GetLongPathNameMessage;
	struct GetLongPathNameResponse;
	struct CreateDirectoryMessage;
	struct CreateDirectoryResponse;
	struct RemoveDirectoryMessage;
	struct RemoveDirectoryResponse;
	struct ListDirectoryMessage;
	struct ListDirectoryResponse;
	struct WrittenFile;
	struct DirectoryTableOverlay;
	struct NextProcessInfo;
	using RootsHandle = u64;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class Session : public DirectoryTableHolder
	{
	public:
		ProcessHandle RunProcess(const ProcessStartInfo& startInfo, bool async = true, bool enableDetour = true); // Run process. if async is false it will not return until process is done
		void CancelAllProcessesAndWait(bool terminate = true); // Cancel all processes and wait for them to go away
		void CancelAllProcesses(); // Cancel all processes

		void PrintSummary(Logger& logger); // Print summary stats of session
		bool RegisterVirtualFile(const tchar* filePath, const tchar* sourceFile, u64 sourceOffset, u64 sourceSize);
		bool CreateVirtualFile(const tchar* filePath, const void* memory, u64 memorySize, bool transient, bool registerToDir = true);
		bool DeleteVirtualFile(const tchar* filePath, bool registerToDir);

		RootsHandle RegisterRoots(const void* rootsData, uba::u64 rootsDataSize);

		using CustomServiceFunction = Function<u32(Process& handle, const void* recv, u32 recvSize, void* send, u32 sendCapacity)>;
		void RegisterCustomService(CustomServiceFunction&& function); // Register a custom service (that can be communicated with from the remote agents)

		using GetNextProcessFunction = Function<bool(Process& handle, NextProcessInfo& outNextProcess, u32 prevExitCode, BinaryReader& prevStats, u32 timeoutMs, bool& outShouldExit)>;
		void RegisterGetNextProcess(GetNextProcessFunction&& function); // Register a custom service (that can be communicated with from the remote agents)

		bool AddEnvironmentVariable(const tchar* name, const tchar* value);

		bool GetOutputFileSize(u64& outSize, const tchar* filePath); // Available if ShouldWriteToDisk is false
		bool GetOutputFileData(void* outData, const tchar* filePath, bool deleteInternalMapping); // Available if ShouldWriteToDisk is false
		bool WriteOutputFile(const tchar* filePath, bool deleteInternalMapping); // Available if ShouldWriteToDisk is false

		const tchar* GetId(); // Id for session. Will be "yymmdd_hhmmss" unless SessionCreateInfo.useUniqueId is set to false
		u32 GetActiveProcessCount(); // Current active processes running inside session
		Storage& GetStorage(); // Storage (only used when remote machines are connected)
		MutableLogger& GetLogger(); // Logger used for logging 
		LogWriter& GetLogWriter(); // LogWriter used by logger
		WorkManager& GetWorkManager();
		Trace& GetTrace(); // Trace written to be session
		SearchPathCache& GetSearchPathCache();
		const ApplicationRules* GetRules(const ProcessStartInfo& si); // Get application rules used for process
		const tchar* GetTempPath(); // Path for temp files used for current session
		const tchar* GetRootDir(); // Root dir for session files
		const tchar* GetSessionDir(); // Session dir
		bool ShouldStoreIntermediateFilesCompressed() const { return m_storeIntermediateFilesCompressed; }
		bool WritePlaceholders() const { return m_writePlaceholders; }
		bool HasDetailedtrace() const { return m_detailedTrace; }
		bool AllowKeepFilesInMemory() const { return m_allowKeepFilesInMemory; }

		// Will write current state of trace directly to disk
		bool SaveSnapshotOfTrace();

		virtual ~Session();

		u32 CreateProcessId();

		bool VirtualizePath(StringBufferBase& inOut, RootsHandle rootsHandle);
		bool DevirtualizePath(StringBufferBase& inOut, RootsHandle rootsHandle, bool reportError = true);
		bool DevirtualizeString(TString& inOut, RootsHandle rootsHandle, bool allowPathsWithoutRoot, const tchar* hint = TC(""));
		bool PopulateLocalToIndexRoots(RootPaths& out, RootsHandle rootsHandle);

		bool PopulateOverlayFiles(DirectoryTableOverlay& overlay, StringView overlayFiles);

		void FlushDeadProcesses(bool reapZombies);

		bool UpdateDirTableSnapshot(bool force);

		const tchar* FixLogFile(const tchar* logFile, StringBufferBase& temp, const tchar* processArguments, u32 processId);
		void SetProcessAsIdle(ProcessImpl& process, BinaryReader& prevStats, u32 prevExitCode);

	protected:
		Session(const SessionCreateInfo& info, const tchar* logPrefix, bool runningRemote, WorkManager& workManager);
		bool Create(const SessionCreateInfo& info);

		void ValidateStartInfo(const ProcessStartInfo& startInfo);
		ProcessHandle InternalRunProcess(const ProcessStartInfo& startInfo, bool async, ProcessImpl* parent, bool enableDetour, void* userData);
		void ProcessAdded(Process& process, u32 sessionId);
		void ProcessExited(ProcessImpl& process, u64 executionTime);
		void PrintProcessStats(ProcessStats& stats, const tchar* logName);
		u32 GetFileMappingSize();
		u32 GetMemoryMapAlignment(const StringView& fileName) const;
		u32 GetMemoryMapAlignment(const StringView& fileName, bool runningRemote) const;

		SessionStats& Stats();

		struct BinaryModule { TString name; TString path; u32 fileAttributes = 0; bool isSystem = false; u32 minOsVersion = 0; };
		struct BinaryAndDeps { Vector<BinaryModule> modules; bool isShebang = false; };
		bool GetBinaryModules(BinaryAndDeps& out, const tchar* application, bool isLinuxBinary);
		void Free(BinaryAndDeps& v);
		bool IsRarelyRead(ProcessImpl& process, const StringView& fileName) const;
		bool IsMaybeReadAfterWritten(ProcessImpl& process, const StringView& fileName) const;
		bool IsKnownSystemFile(const tchar* applicationName);
		virtual bool IsInTemp(StringView path) override;
		virtual bool ShouldWriteToDisk(StringView fileName) override;
		virtual void InvalidateCachedFileInfo(const StringKey& fileNameKey);

		u32 AddFileMapping(StringKey fileNameKey, const tchar* fileName, const tchar* newFileName, u64 fileSize = InvalidValue);
		
		struct MemoryMap { StringBuffer<128> name; u64 size = 0; };
		struct FileMappingEntry;
		bool GetOrCreateMemoryMapFromFile(MemoryMap& out, StringKey fileNameKey, const tchar* fileName, bool isCompressedCas, u64 alignment, const tchar* hint, ProcessImpl* requestingProcess, bool canBeFreed);
		bool GetOrCreateMemoryMapFromStorage(MemoryMap& out, StringKey fileNameKey, const tchar* fileName, const CasKey& casKey, u64 alignment);
		void ThreadGcFileMappings();

		virtual bool PrepareProcess(ProcessImpl& process, bool isChild, StringBufferBase& outRealApplication, const tchar*& outRealWorkingDir);
		virtual Vector<tchar>& GetProcessEnvironmentVariables();
		virtual void PrintSessionStats(Logger& logger);

		bool RegisterVirtualFileInternal(const StringKey& fileNameKey, const StringView& filePath, const tchar* sourceFile, u64 sourceOffset, u64 sourceSize);
		bool CreateVirtualFileInternal(const StringKey& fileNameKey, const StringView& filePath, const void* memory, u64 memorySize, bool transient);
		bool RegisterVirtualFileInternal(const StringKey& fileNameKey, const StringView& filePath, SharedMemoryHandle memoryHandle, u64 mappingSize, u64 mappingOffset, bool transient);

		virtual bool GetOutputFileSizeInternal(u64& outSize, const StringKey& fileNameKey, StringView filePath);
		virtual bool GetOutputFileDataInternal(void* outData, const StringKey& fileNameKey, StringView filePath, bool deleteInternalMapping);
		virtual bool WriteOutputFileInternal(const StringKey& fileNameKey, StringView filePath, bool deleteInternalMapping);

		virtual bool ProcessThreadStart(ProcessImpl& process);
		virtual bool ProcessNativeCreated(ProcessImpl& process);
		virtual bool ProcessCancelled(ProcessImpl& process);
		virtual bool GetInitResponse(InitResponse& out, const InitMessage& msg);
		virtual bool CreateFile(CreateFileResponse& out, const CreateFileMessage& msg);
		virtual bool CreateFileForRead(CreateFileResponse& out, TrackWorkScope& tws, StringView fileName, StringKey fileNameKey, ProcessImpl& process, const ApplicationRules& rules);
		virtual bool CloseFile(CloseFileResponse& out, const CloseFileMessage& msg);
		virtual bool DeleteFile(DeleteFileResponse& out, const DeleteFileMessage& msg);
		virtual bool CopyFile(CopyFileResponse& out, const CopyFileMessage& msg);
		virtual bool MoveFile(MoveFileResponse& out, const MoveFileMessage& msg);
		virtual bool Chmod(ChmodResponse& out, const ChmodMessage& msg);
		virtual bool Symlink(SymlinkResponse& out, const SymlinkMessage& msg);
		virtual bool CreateDirectory(CreateDirectoryResponse& out, const CreateDirectoryMessage& msg);
		virtual bool RemoveDirectory(RemoveDirectoryResponse& out, const RemoveDirectoryMessage& msg);
		virtual bool GetFullFileName(GetFullFileNameResponse& out, const GetFullFileNameMessage& msg);
		virtual bool GetLongPathName(GetLongPathNameResponse& out, const GetLongPathNameMessage& msg);
		virtual bool GetListDirectoryInfo(ListDirectoryResponse& out, const StringView& dirName, const StringKey& dirKey, DirectoryTableOverlay* overlay);
		virtual bool WriteFilesToDisk(ProcessImpl& process, WrittenFile** files, u32 fileCount);
		virtual bool AllocFailed(Process& process, const tchar* allocType, u32 error);
		virtual bool GetNextProcess(Process& process, bool& outNewProcess, bool& outShouldExit, NextProcessInfo& outNextProcess, u32 prevExitCode, BinaryReader& prevStats, u32 timeoutMs);
		virtual bool CustomMessage(Process& process, BinaryReader& reader, BinaryWriter& writer);
		virtual bool SHGetKnownFolderPath(Process& process, BinaryReader& reader, BinaryWriter& writer);
		virtual bool HostRun(BinaryReader& reader, BinaryWriter& writer);
		virtual bool GetSymbols(const tchar* application, bool isLinux, bool isArm, BinaryReader& reader, BinaryWriter& writer);
		virtual bool CheckRemapping(ProcessImpl& process, BinaryReader& reader, BinaryWriter& writer);
		virtual bool RunSpecialProgram(ProcessImpl& process, BinaryReader& reader, BinaryWriter& writer);
		virtual bool FlushWrittenFiles(ProcessImpl& process);
		virtual bool UpdateEnvironment(ProcessImpl& process, const StringView& reason, bool resetStats);
		virtual bool LogLine(ProcessImpl& process, const tchar* line, LogEntryType logType);
		virtual void EmergencyShutdown();

		static constexpr CasKey CasKeyIsDirectory = { ~u64(0), ~u64(0), ~u32(0) };

		void RemoveWrittenFile(ProcessImpl& process, const StringKey& fileKey);
		bool WriteFileToDisk(ProcessImpl& process, WrittenFile& file);
		bool WriteMemoryToDisk(FileAccessor& destinationFile, const void* fileMem, u64 fileSize, StringBufferBase* outWriteMode);
		bool GetFileMemory(const Function<bool(const void*, u64)>& func, const StringKey& fileNameKey, StringView filePath, bool deleteInternalMapping);
		void AddEnvironmentVariableNoLock(StringView key, StringView value);
		bool CopyImports(BinaryAndDeps& out, const tchar* library, tchar* applicationDir, tchar* applicationDirEnd, UnorderedSet<TString>& handledImports, const tchar* const* loaderPaths, bool isLinuxBinary, const tchar* hint);
		bool CreateProcessJobObject();
		void GetSessionInfo(StringBufferBase& out);

		bool HasVfs(RootsHandle handle) const;
		RootsHandle WithVfs(RootsHandle key, bool vfs) const;
		struct RootsEntry { Vector<u8> memory; RootPaths roots; Vector<TString> locals; Vector<TString> vfs; Futex lock; bool handled = false; };
		const RootsEntry* GetRootsEntry(RootsHandle rootsHandle);
		void PopulateRootsEntry(RootsEntry& entry, const void* rootsData, uba::u64 rootsDataSize);

		u32 PopTempDirectory(TString& out);
		void PushTempDirectory(u32 id);
		void ClearTempDirectory(u32 id);

		bool CreateSnapshot(u32 currSize);
		DirTableSnapshot* ReferenceDirTableSnapshot();
		void ReleaseDirTableSnapshot(DirTableSnapshot* snapshot);

		bool ExtractSymbolsFromObjectFile(ProcessImpl& process, const tchar* fileName, const tchar* backedName, u64 fileSize, SharedMemoryHandle memoryHandle, u32 attributes);
		bool ExtractSymbolsForProcessTree(ProcessImpl& rootProcess);
		bool DevirtualizeDepsFile(RootsHandle rootsHandle, MemoryBlock& destData, const void* sourceData, u64 sourceSize, bool escapeSpaces, const tchar* hint);

		#if PLATFORM_LINUX
		bool TryPatchBinary(StringBufferBase& out, StringView in);
		#endif

		void TraceWrittenFile(u32 processId, const StringView& file, u64 size = 0);

		void RunDependencyCrawler(ProcessImpl& process);
		bool OpenFileForDependencyCrawler(TrackWorkScope& tracker, StringView fileName, const DependencyCrawler::AccessFileFunc& func, bool& stopCrawling, ProcessHandle& ph, bool isLinkCrawler, const ApplicationRules& rules);

		Storage& m_storage;
		SharedMemoryAllocator& m_sharedMemory;
		WorkManager& m_workManager;

		StringBuffer<32> m_id;
		StringBuffer<MaxPath> m_rootDir;
		StringBuffer<MaxPath> m_sessionDir;
		StringBuffer<MaxPath> m_sessionBinDir;
		StringBuffer<MaxPath> m_sessionOutputDir;
		StringBuffer<MaxPath> m_sessionLogDir;
		StringBuffer<MaxPath> m_systemPath;
		StringBuffer<MaxPath> m_tempPath;


		bool m_allowCustomAllocator;
		bool m_allowLazyCopyOnWrite;
		bool m_allowMemoryMaps;
		bool m_allowKeepFilesInMemory;
		bool m_allowOutputFiles;
		bool m_allowSpecialApplications;
		bool m_suppressLogging;
		bool m_shouldWriteToDisk;
		bool m_writePlaceholders;
		bool m_detailedTrace;
		bool m_traceChildProcesses;
		bool m_traceWrittenFiles;
		bool m_logToFile;
		bool m_storeIntermediateFilesCompressed;
		bool m_readIntermediateFilesCompressed;
		bool m_allowLocalDetour;
		bool m_extractObjFilesSymbols;
		bool m_allowLinkDependencyCrawler;
		bool m_allowDiscardVirtualMemory;
		bool m_dirTableSnapshotEnabled;
		bool m_ownsTrace;

		u64 m_keepOutputFileMemoryMapsThreshold;

		Atomic<u32> m_processIdCounter = 0;
		FileMappingHandle m_fileMappingTableHandle;
		FileMappingBuffer m_fileMappingBuffer;

		Atomic<u64> m_independentMappingCreated = 0;
		Atomic<u64> m_independentMappingActive = 0;
		Atomic<u64> m_reuseCount = 0;

		ReaderWriterLock m_fileMappingTableMemLock;
		u8* m_fileMappingTableMem = nullptr;
		u32 m_fileMappingTableSize = 0;
		Futex m_fileMappingTableLookupLock;
		struct FileMappingEntry
		{
			Futex lock;

			SharedMemoryHandle memoryHandle;
			u64 memoryOffset = 0;

			u64 contentSize = 0; // Size of uncompressed file or mapping
			u64 lastWriteTime = 0;
			u64 storedSize = InvalidValue; // Size stored on disk. Can be compressed or uncompressed
			u16 refCount = 0;
			u8 usedCount = 0;
			u8 usedCountBeforeFree = 0;

			u8 isDir:1 = false;
			u8 handled:1 = false;
			u8 success:1 = false;
			u8 canBeFreed:1 = false;
			u8 createIndependentMapping:1 = false;
			u8 isInvisible:1 = false;
			u8 dropCasAfterUse:1 = false;

			#if UBA_DEBUG_TRACK_MAPPING
			TString name;
			#endif
		};
		UnorderedMap<StringKey, FileMappingEntry> m_fileMappingTableLookup;

		Futex m_fileMappingsThatCanBeFreedLock;
		UnorderedMap<StringKey, u64> m_fileMappingsThatCanBeFreed;
		Thread m_fileMappingGcThread;
		Event m_fileMappingGcDone;

		static constexpr u64 NameToHashMemSize = 48*1024*1024;
		MemoryBlock m_nameToHashTableMem;

		FixedSizeFileMappingAllocator m_processCommunicationAllocator;
		std::string m_detoursLibrary[2][2]; // [nonlinux/linux][x64/arm64]

		#if PLATFORM_LINUX
		TString m_stubBinary[2]; // [x64/arm64]
		#endif

		Futex m_processStatsLock;
		ProcessStats m_processStats;

		Futex m_processesLock;
		UnorderedMap<u32, ProcessHandle> m_processes;
		Vector<ProcessHandle> m_deadProcesses;
		WorkManagerImpl& m_processMessageThreads;

		UnorderedMap<TString, Timer> m_applicationStats;

		Futex m_virtualSourceFilesLock;
		struct VirtualSourceFile { FileMappingHandle mappingHandle; SharedMemoryHandle memoryHandle; u64 size; };
		UnorderedMap<StringKey, VirtualSourceFile> m_virtualSourceFiles;

		Futex m_rootsLookupLock;
		UnorderedMap<RootsHandle, RootsEntry> m_rootsLookup;

		Atomic<u32> m_wantsOnCloseIdCounter = 1;

		SessionStats m_stats;
		Trace& m_trace;
		TString m_traceOutputFile;
		TString m_extraInfo;

		#if PLATFORM_WINDOWS
		Futex m_processJobObjectLock;
		HANDLE m_processJobObject = NULL;
		#endif

		Vector<u8> m_environmentMemory;

		Futex m_environmentVariablesLock;
		Vector<tchar> m_environmentVariables;
		UnorderedSet<const tchar*, HashStringNoCase, EqualStringNoCase> m_localEnvironmentVariables;
		Vector<std::pair<TString, TString>> m_additionalEnvironmentVariables;

		GetNextProcessFunction m_getNextProcessFunction;
		CustomServiceFunction m_customServiceFunction;

		Bottleneck m_writeFilesBottleneck;
		u64 m_writeFilesFileMapMax;
		u64 m_writeFilesNoBufferingMin;

		#if PLATFORM_WINDOWS
		Futex m_isX64ApplicationLock;
		UnorderedMap<TString, bool> m_isX64Application;
		#endif

		#if UBA_USE_NATIVE_MAC_SEMAPHORES
		u32 GetSessionId();
		bool StartSemaphoreService();
		void StopSemaphoreService();
		void ThreadSemaphoreService();
		u32 m_sessionId;
		struct SemaphoreServiceInfo;
		Thread m_semaphoreServiceThread;
		SemaphoreServiceInfo* m_semaphoreServiceInfo;
		#endif

		Futex m_availableTempDirectoriesLock;
		List<u32> m_availableTempDirectories;
		u32 m_availableTempDirectoriesMax = 0;

		DependencyCrawler m_dependencyCrawler;

		SearchPathCache m_searchPathCache;

		ReaderWriterLock m_dirTableSnapshotLock;
		DirTableSnapshot* m_dirTableSnapshot = nullptr;
		Atomic<u32> m_dirTableSnapshotInProgress = 0;
		u32 m_dirTableSizeForSnapshot = 0;

		#if PLATFORM_LINUX
		Futex m_patchedBinariesLock;
		struct PatchedBinary { CasKey key; Futex lock; bool handled = false; };
		UnorderedMap<TString, PatchedBinary> m_patchedBinaries;
		Futex m_handledPatchedCasKeysLock;
		UnorderedMap<CasKey, Futex> m_handledPatchedCasKeys;
		#endif

		friend class ProcessImpl;
		friend struct WrittenFile;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void GenerateNameForProcess(StringBufferBase& out, const tchar* arguments, u32 counterSuffix);
	bool GetZone(StringBufferBase& outZone);


	using FileAccess = u8;

	enum : u8
	{
		FileAccess_Read = 1,
		FileAccess_Write = 2,
		FileAccess_ReadWrite = 3
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct InitMessage
	{
		ProcessImpl& process;
	};

	struct InitResponse
	{
		SharedMemoryAllocatorHandle sharedMemoryAllocatorHandle;
		SharedMemoryHandle permanentFilesHandle;
		u64 permanentFilesOffset = 0;
		FileMappingHandle directoryTableHandle = 0;
		DirTableSize directoryTableSize;
		u32 directoryTableCount = 0;
		FileMappingHandle mappedFileTableHandle = 0;
		u32 mappedFileTableSize = 0;
		u32 mappedFileTableCount = 0;
	};

	struct CreateFileMessage
	{
		ProcessImpl& process;
		StringBuffer<> fileName;
		StringKey fileNameKey;
		FileAccess access;
	};

	struct CreateFileResponse
	{
		StringBuffer<> fileName;
		StringBuffer<> virtualFileName;
		u64 size = InvalidValue;
		u32 closeId = 0;
		u32 mappedFileTableSize = 0;
		DirTableSize directoryTableSize;
		DirTableOffset dirTableOffset;
	};

	struct CloseFileMessage
	{
		ProcessImpl& process;
		StringBuffer<> fileName;
		StringKey newNameKey;
		StringBuffer<> newName;
		u32 closeId = 0;
		u32 attributes = 0;
		bool deleteOnClose = false;
		bool success = true;
		SharedMemoryHandle memoryHandle;
		u64 fileSize = 0;
		u64 lastWriteTime = 0;
	};

	struct CloseFileResponse
	{
		DirTableSize directoryTableSize;
	};

	struct DeleteFileMessage
	{
		ProcessImpl& process;
		StringBuffer<> fileName;
		StringKey fileNameKey;
		u32 closeId = 0;
	};

	struct DeleteFileResponse
	{
		bool result = false;
		u32 errorCode = ~0u;
		DirTableSize directoryTableSize;
	};

	struct CopyFileMessage
	{
		ProcessImpl& process;
		StringKey fromKey;
		StringBuffer<> fromName;
		u64 fileSize = 0;
		u64 lastWriteTime = 0;
		u32 fileAttributes = 0;

		StringKey toKey;
		StringBuffer<> toName;
	};

	struct CopyFileResponse
	{
		StringBuffer<> fromName;
		StringBuffer<> toName;
		u32 closeId = 0;
		u32 errorCode = ~0u;
		DirTableSize directoryTableSize;
	};

	struct MoveFileMessage
	{
		ProcessImpl& process;
		StringKey fromKey;
		StringBuffer<> fromName;
		u64 fileSize = 0;
		u64 lastWriteTime = 0;
		u32 fileAttributes = 0;

		StringKey toKey;
		StringBuffer<> toName;

		u32 flags = 0;

		// Preserved across the rename when fromName is a symlink — RegisterOverlayCreateOrWrite
		// needs the target string to keep S_IFLNK on the destination entry.
		StringBuffer<MaxPath> symlinkContent;
	};

	struct MoveFileResponse
	{
		bool result = false;
		u32 errorCode = ~0u;
		DirTableSize directoryTableSize;
	};

	struct ChmodMessage
	{
		ProcessImpl& process;
		StringKey fileNameKey;
		StringBuffer<> fileName;
		u32 fileMode = 0;
	};

	struct ChmodResponse
	{
		u32 errorCode = ~0u;
	};

	struct SymlinkMessage
	{
		ProcessImpl& process;
		StringKey filePathKey;
		StringBuffer<> symlinkFilePath;
		StringKey contentPathKey;
		StringBuffer<MaxPath> symlinkContentPath;
		StringBuffer<MaxPath> symlinkContent;
		u64 fileSize = 0;
		u64 lastWriteTime = 0;
		u32 fileAttributes = 0;
	};

	struct SymlinkResponse
	{
		bool success = false;
		u32 errorCode = ~0u;
		DirTableSize directoryTableSize;
	};

	struct GetFullFileNameMessage
	{
		ProcessImpl& process;
		StringBuffer<> fileName;
		StringKey fileNameKey;
		const u8* loaderPaths = nullptr;
		u32 loaderPathsSize = 0;
	};

	struct GetFullFileNameResponse
	{
		StringBuffer<> fileName;
		StringBuffer<> virtualFileName;
		u32 mappedFileTableSize = 0;
	};

	struct GetLongPathNameMessage
	{
		ProcessImpl& process;
		StringBuffer<> fileName;
	};

	struct GetLongPathNameResponse
	{
		StringBuffer<> fileName;
		u32 errorCode = ~0u;
	};

	struct CreateDirectoryMessage
	{
		ProcessImpl& process;
		StringKey nameKey;
		StringBuffer<> name;
	};

	struct CreateDirectoryResponse
	{
		DirTableSize directoryTableSize;
	};

	struct RemoveDirectoryMessage
	{
		ProcessImpl& process;
		StringKey nameKey;
		StringBuffer<> name;
	};

	struct RemoveDirectoryResponse
	{
		DirTableSize directoryTableSize;
	};

	struct ListDirectoryMessage
	{
		StringBuffer<> directoryName;
		StringKey directoryNameKey;
	};

	struct ListDirectoryResponse
	{
		DirTableOffset tableOffset;
		DirTableSize directoryTableSize;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct ActiveFile
	{
		TString name;
		StringKey nameKey;
		u32 refCount = 0;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct WrittenFile
	{
		StringKey key;
		TString name;
		TString backedName;

		SharedMemoryHandle memoryHandle;
		SharedMemoryHandle originalMemoryHandle;

		u64 memoryWritten = 0;
		u64 lastWriteTime = 0;
		u32 attributes = 0;

		u32 ownerId = 0; // Last process who write it

		// TODO: Ugly placement of this pointer.
		// Here to make sure FileMappingEntry is not garbage collecting before process exit callback is called
		Session::FileMappingEntry* entryToReduceRefCount = nullptr;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct NextProcessInfo
	{
		TString arguments;
		TString workingDir;
		TString description;
		TString logFile;
		TString breadcrumbs;
		Vector<u8> overlayFiles;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////
}

template<> struct std::hash<uba::ProcessHandle> { size_t operator()(const uba::ProcessHandle& g) const { return g.GetHash(); } };
