// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaNetwork.h"
#include "UbaSession.h"

namespace uba
{
	class NetworkClient;

	struct SessionClientCreateInfo : SessionCreateInfo
	{
		SessionClientCreateInfo(Storage& s, NetworkClient& c, LogWriter& writer = g_consoleLogWriter);
		void Apply(const Config& config);

		NetworkClient& client;
		StringBuffer<128> name;
		u32 maxProcessCount = 1;
		u32 defaultPriorityClass = ProcessPriority_BelowNormal;
		u32 maxIdleSeconds = ~0u;
		u32 pingTimeoutSecondsPrintCallstacks = 0;
		u32 osVersion = 0;
		u8 memWaitLoadPercent = 80; // When memory usage goes above this percent, no new processes will be spawned until back below
		u8 memKillLoadPercent = 90; // When memory usage goes above this percent, newest processes will be killed to bring it back below
		bool dedicated = false;  // If true, server will not disconnect client when starting to run out of work.
		bool disableCustomAllocator = false;
		bool useBinariesAsVersion = false;
		bool killRandom = false;
		bool useStorage = true;
		bool downloadDetoursLib = true;
		bool useDependencyCrawler = false;	// Dependency crawler will go wide and try to prefetch all dependencies needed by the process
		bool useKnownLocalBinaries = true; // Will use local binaries like sh, ln, cp etc if available
		Function<void(const ProcessHandle&)> processFinished;
	};

	class SessionClient final : public Session
	{
	public:
		SessionClient(const SessionClientCreateInfo& info);
		~SessionClient();

		bool Start();
		void Stop(bool wait = true);
		bool Wait(u32 milliseconds = 0xFFFFFFFF, Event* wakeupEvent = nullptr);
		void SendSummary(const Function<void(Logger&)>& extraInfo);
		void SetIsTerminating(const tchar* reason = TC("Terminating"), u64 delayMs = 0); // Session stores pointer directly. Can't be temporary
		void SetMaxProcessCount(u32 count);
		void SetMaxIdleTime(u32 seconds);
		void SetAllowSpawn(bool allow);

		u64 GetBestPing();
		u32 GetMaxProcessCount();
		u32 GetDesiredConnectionCount();

		bool Exists(const StringView& path, u32& outAttributes);

		void SendNotification(StringView text);

	private:
		bool RetrieveCasFile(CasKey& outNewKey, u64& outSize, const CasKey& casKey, StringKey keyHint, const tchar* hint, bool storeUncompressed, bool allowProxy = true);

		virtual bool PrepareProcess(ProcessImpl& process, bool isChild, StringBufferBase& outRealApplication, const tchar*& outRealWorkingDir) override;
		virtual Vector<tchar>& GetProcessEnvironmentVariables() override;
		virtual bool ProcessThreadStart(ProcessImpl& process) override;
		virtual bool CreateFileForRead(CreateFileResponse& out, TrackWorkScope& tws, StringView fileName, StringKey fileNameKey, ProcessImpl& process, const ApplicationRules& rules) override;
		virtual bool DeleteFile(DeleteFileResponse& out, const DeleteFileMessage& msg) override;
		virtual bool MoveFile(MoveFileResponse& out, const MoveFileMessage& msg) override;
		virtual bool Chmod(ChmodResponse& out, const ChmodMessage& msg) override;
		virtual bool GetFullFileName(GetFullFileNameResponse& out, const GetFullFileNameMessage& msg) override;
		virtual bool GetLongPathName(GetLongPathNameResponse& out, const GetLongPathNameMessage& msg) override;
		virtual bool GetListDirectoryInfo(ListDirectoryResponse& out, const StringView& dirName, const StringKey& dirKey, DirectoryTableOverlay* overlay) override;
		virtual bool WriteFilesToDisk(ProcessImpl& process, WrittenFile** files, u32 fileCount) override;
		virtual bool AllocFailed(Process& process, const tchar* allocType, u32 error) override;
		virtual void PrintSessionStats(Logger& logger) override;
		virtual bool GetNextProcess(Process& process, bool& outNewProcess, bool& outShouldExit, NextProcessInfo& outNextProcess, u32 prevExitCode, BinaryReader& statsReader, u32 timeoutMs) override;
		virtual bool CustomMessage(Process& process, BinaryReader& reader, BinaryWriter& writer) override;
		virtual bool SHGetKnownFolderPath(Process& process, BinaryReader& reader, BinaryWriter& writer) override;
		virtual bool HostRun(BinaryReader& reader, BinaryWriter& writer) override;
		virtual bool GetSymbols(const tchar* application, bool isLinux, bool isArm, BinaryReader& reader, BinaryWriter& writer);
		virtual bool FlushWrittenFiles(ProcessImpl& process) override;
		virtual bool UpdateEnvironment(ProcessImpl& process, const StringView& reason, bool resetStats) override;
		virtual bool LogLine(ProcessImpl& process, const tchar* line, LogEntryType logType) override;
		
		void TraceSessionUpdate();

		struct InternalProcessStartInfo;
		struct ModuleInfo;

		bool GetCasKeyForFile(CasKey& out, u32 processId, const StringView& fileName, const StringKey& fileNameKey);
		bool ReadModules(List<ModuleInfo>& outModules, bool& outIsShebang, StringBufferBase& outVirtualApplication, u32 processId, StringView application, StringView workingDir);
		bool EnsureBinaryFile(StringBufferBase& out, StringBufferBase& outVirtual, u32 processId, StringView fileName, const StringKey& fileNameKey, StringView applicationDir, StringView workingDir, const u8* loaderPaths, u32 loaderPathsSize);
		bool WriteBinFile(StringBufferBase& out, const StringView& binaryName, const CasKey& casKey, const KeyToString& applicationDir, u32 fileAttributes);
		bool ShouldSend(ProcessImpl& process, StringView path);
		bool SendFileSystemOperations(ProcessImpl& process);
		bool SendFiles(ProcessImpl& process);
		bool SendFile(WrittenFile& file, u32 processId, bool keepMappingInMemory, bool compressed);
		bool SendUpdateDirectoryTable(StackBinaryReader<SendMaxSize>& reader); // Note, reader is sent in to save stack space.
		bool UpdateDirectoryTableFromServer(StackBinaryReader<SendMaxSize>& reader);
		bool SendUpdateNameToHashTable(StackBinaryReader<SendMaxSize>& reader);
		bool UpdateNameToHashTableFromServer(StackBinaryReader<SendMaxSize>& reader);
		void Connect();
		void BuildEnvironmentVariables(BinaryReader& reader);
		bool SendProcessAvailable(Vector<InternalProcessStartInfo>& outStartInfos, Vector<u32>& outCancelled, float availableWeight);
		bool SendGetNextMessage();
		void CancelGetNextMessageQueue();
		bool HandleGetNextResponse(BinaryReader& reader, u32 processId, bool& outNewProcess, bool& outShouldExit, NextProcessInfo& outNextProcess);
		void SendReturnProcess(u32 processId, const tchar* reason, ...);
		bool SendProcessInputs(ProcessImpl& process);
		bool SendProcessFinished(ProcessImpl& process, u32 exitCode);
		void SendPing(u64 memAvail, u64 memTotal);
		void SendLogEntry(StringView text, LogEntryType type = LogEntryType_Info, bool ack = false);
		bool SendRootsHandle(RootsHandle rootsHandle);
		void SendLogFilesToServer(ProcessImpl& pi);
		void GetLogFiles(List<WrittenFile>& outLogFiles, ProcessImpl& pi);
		void GetLogFileName(StringBufferBase& out, const tchar* logFile, const tchar* arguments, u32 processId);
		u32 WriteLogLines(BinaryWriter& writer, ProcessImpl& process);
		bool ParseDirectoryTable();
		bool EntryExists(StringView path, DirTableOffset& outTableOffset);
		void RunSpeedTest(StringView args);

		void ThreadCreateProcessLoop();

		NetworkClient& m_client;
		static constexpr u8 ServiceId = SessionServiceId;

		StringBuffer<128> m_name;
		StringBuffer<MaxPath> m_processWorkingDir;
		u32 m_sessionId = ~0u;
		u32 m_uiLanguage = 0;
		u32 m_defaultPriorityClass;
		Atomic<u32> m_maxIdleSeconds = ~0u;
		u32 m_osVersion = 0;
		u32 m_killRandomIndex = ~0u;
		u32 m_killRandomCounter = 0;
		u8 m_memWaitLoadPercent;
		u8 m_memKillLoadPercent;
		bool m_disableCustomAllocator;
		bool m_useBinariesAsVersion;
		bool m_dedicated = false;
		bool m_useStorage = true;
		bool m_downloadDetoursLib = true;
		bool m_shouldSendLogToServer = false;
		bool m_shouldSendTraceToServer = false;
		bool m_remoteExecutionEnabled = true;
		bool m_useDependencyCrawler = false;
		bool m_useKnownLocalBinaries = true;
		
		Atomic<const tchar*> m_terminationReason = nullptr;
		Atomic<u64> m_terminationTime = 0;
		Atomic<u32> m_maxProcessCount = 0;
		Atomic<u32> m_desiredConnectionCount = 0;
		Atomic<float> m_cpuUsage = 0.0f;

		struct ApplicationEnvironment { Futex lock; TString virtualApplication; TString realApplication; bool isShebang = false; };
		Futex m_handledApplicationEnvironmentsLock;
		UnorderedMap<TString, ApplicationEnvironment> m_handledApplicationEnvironments;

		Futex m_binFileLock;
		UnorderedMap<TString, CasKey> m_writtenBinFiles;

		Futex m_nameToNameLookupLock;
		struct NameRec { TString name; TString virtualName; Futex lock; bool handled = false; };
		UnorderedMap<StringKey, NameRec> m_nameToNameLookup;

		struct HashRec { CasKey key; u64 serverTime = 0; Futex lock; };
		UnorderedMap<StringKey, HashRec> m_nameToHashLookup;
		Futex m_nameToHashLookupLock;
		ReaderWriterLock m_nameToHashMemLock;

		Futex m_directoryTableLock;
		u32 m_directoryTableMemPos = 0;
		bool m_directoryTableError = false;
		struct ActiveUpdateDirectoryEntry;
		ActiveUpdateDirectoryEntry* m_firstEmptyWait = nullptr;
		ActiveUpdateDirectoryEntry* m_firstReadWait = nullptr;

		Event m_waitToSendEvent;
		Thread m_loopThread;
		Atomic<bool> m_loop = true;
		Atomic<bool> m_sendPing = false;
		Atomic<bool> m_allowSpawn = true;

		Function<void(const ProcessHandle&)> m_processFinished;

		SessionSummaryStats m_stats;

		struct GetNextEntry;
		GetNextEntry* m_firstGetNextEntry = nullptr;
		Futex m_getNextEntryLock;

		Atomic<u64> m_bestPing = 0;
		Atomic<u64> m_lastPing = 0;
		u64 m_lastPingSendTime = 0;
		u32 m_pingTimeoutSecondsPrintCallstacks = 0;

		Atomic<u64> m_memAvail = 0;
		Atomic<u64> m_memTotal = 0;

		UnorderedMap<CasKey, Vector<u8>> m_hostRunCache;
		Futex m_hostRunCacheLock;

		u32 m_dirtableParsedPosition = 0;
		Futex m_dirVisitedLock;
		struct DirVisitedEntry { Futex lock; bool handled = false; };
		UnorderedMap<StringKey, DirVisitedEntry> m_dirVisited;
	};
}
