// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaLogWriter.h"

namespace uba
{
	class Config;
	class Storage;
	class Trace;

	struct SessionCreateInfo
	{
		SessionCreateInfo(Storage& s, LogWriter& w = g_consoleLogWriter) : storage(s), logWriter(w) {}

		void Apply(const Config& config);

		Storage& storage;
		LogWriter& logWriter;
		const tchar* rootDir = nullptr;			// Root dir for logs, binaries, temp files
		const tchar* traceName = nullptr;		// Name of trace. This name can be used by UbaVisualizer to watch progress live
		const tchar* traceChannelName = nullptr;// If set, the trace name is announced on this TraceChannel. When launchVisualizer is also set, a new visualizer is only spawned if no listener is detected on the channel; otherwise the existing visualizer picks up the trace via the channel.
		const tchar* traceOutputFile = nullptr; // Output file. Will be written at end of run.
		const tchar* extraInfo = nullptr;		// Extra info that will be stored in the trace info about the session
		bool logToFile = false;					// Set to true to have all processes write a log file with function calls.
		bool useUniqueId = true;				// If true, id of session will be "yymmdd_hhmmss". Otherwise "Debug"
		bool allowCustomAllocator = true;		// Allow detouring of allocator inside processes.
		bool allowLazyCopyOnWrite = true;		// Allow detoured process to map CoW as read-only and then make pages CoW based on need
		bool launchVisualizer = false;			// Launch a UbaVisualizer process (this automatically enable trace)
		bool allowMemoryMaps = true;			// Use memory maps where possible. Session creates memory maps of files that processes use
		bool allowKeepFilesInMemory = true;		// Allow detoured process to keep output files in memory and send them to session through file mapping handle
		bool allowOutputFiles = true;			// Allow (selected) output files to be written to disk after process has ended.
		bool allowSpecialApplications = true;	// Allow uba to expand cmd.exe and call known commands instead of running additional process
		bool suppressLogging = false;			// Suppress all logging produced by detoured processes. Will be made in detoured process to improve performance when needed
		bool shouldWriteToDisk = true;			// Set to false to skip writing output files to disk
		bool writePlaceholders = false;			// If this is true, 1-byte placeholder files will be written where possible
		bool traceEnabled = false;				// Set to true to always create in-memory trace data. Is not needed if traceName, traceOutputFile or launchVisualizer is set
		bool detailedTrace = false;				// Enable detailed trace to include jobs, individual file I/O etc in trace dump
		bool traceChildProcesses = false;		// Trace and visualize child processes so they can be seen in visualizer
		bool traceWrittenFiles = false;			// Will add process output files to trace
		bool storeIntermediateFilesCompressed = false;	// Compiler will write intermediate files (.obj, .pch etc) compressed to disk and will be decompressed when used
		bool readIntermediateFilesCompressed = false;	// Set to true to support reading compressed .obj files. This flag will not compress new obj files
		bool allowLocalDetour = true;			// Allow local processes to be detoured. If this is false it is up to outside logic to register all created/deleted files
		bool extractObjFilesSymbols = false;	// Will extract import/export symbols to a file
		bool treatTempDirAsEmpty = true;		// If this is true, directory table will always see temp as empty. Use sub directories of temp instead
		bool useFakeVolumeSerial = true;		// Fake volume serials reduce the directory table size since serials end up being compressed down to one byte
		bool allowDiscardVirtualMemory = false;	// Set to true to use DiscardVirtualMemory around map/unmap views of file mappings. This seems to speed up perf quite a lot
		bool allowLinkDependencyCrawler = true;	// Enable crawler that reads ahead to find needed obj files for linker
		bool dirTableSnapshotEnabled = true;	// Creates a snapshot of the dir table when everytime it grows by 200kb or more
		int linkCrawlerMaxParallel = -2;		// If allowLinkDependencyCrawler is set, this controls number of decompress in parallel. If negative it is max cores - value
		u64 deleteSessionsOlderThanSeconds = 12 * 60 * 60; // Delete session folders older than 12 hours by default . Set to 0 to not delete or 1 to delete all
		u64 keepOutputFileMemoryMapsThreshold = 256 * 1024; // If allowMemoryMaps is true, output files will be kept in memory if smaller than this size
		u32 traceReserveSizeMb = 128;			// Memory reserved for trace file (in mb)
		u32 writeFilesBottleneck = 16;			// When writing files to disk this number will control how many that can write in parallel
		u32 writeFilesFileMapMaxMb = 100000;	// If file size is smaller than this number, then use memory maps for writing to disk
		u32 writeFilesNoBufferingMinMb = 16;	// If file size is at least this number, then overlapped io with no buffering will be used
		u32 traceIntervalMs = 500;				// milliseconds between each trace entry

		Trace* trace = nullptr;					// If no trace is provided, session will create a new one
	};
}
