// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaHashMap.h"
#include "UbaNetwork.h"
#include "UbaSession.h"
#include "UbaSessionServerCreateInfo.h"

namespace uba
{
	class NetworkServer;
	class Scheduler;
	struct ConnectionInfo;

	class SessionServer final : public Session
	{
	public:
		// Ctor/dtor. "environmnent" should be an array of utf8-written strings. This is really only needed for posix platforms where process is a c# process
		SessionServer(const SessionServerCreateInfo& info, const u8* environment = nullptr, u32 environmentSize = 0);
		~SessionServer();

		// Run process remotely.
		// startInfo contains info about the process to run remotely
		// weight is the expected core usage of the process. If processes are multithreaded it makes sense to increase weight. As an example, in UnrealBuildTool we see cl.exe as 1.5 and clang.exe as 1.0
		// knownInputs is a memory block with null terminated tchar strings followed by an empty null terminated string to end. knownInputsCount is the number of strings in the memory block
		// strings should be absolute or relative to working dir.
		ProcessHandle RunProcessRemote(const ProcessStartInfo& startInfo, float weight = 1.0f, const void* knownInputs = nullptr, u32 knownInputsCount = 0, bool canCrossArchitecture = false, bool canCrossPlatform = false);

		// Will kick off a local process with the same startInfo as the one provided to start the process with id matching raceAgainstRemoteProcessId
		// This can be useful if there are free local cores and we know local machine is faster or network connection to remote is slow
		using ProcessToRaceCallback = Function<bool(ProcessHandle&, ProcessStartInfo&)>;
		ProcessHandle RunProcessRacing(const ProcessToRaceCallback& callback);

		// Disable remote execution. This will tell all clients to stop taking on new processes and disconnect as soon as their current processes are finished
		void DisableRemoteExecution();
		bool IsRemoteExecutionDisabled();
		void ReenableRemoteExecution();
		bool DisableRemoteExecutionOnAgent(const tchar* agentName);

		// This can be used to set a custom cas key based on a file and its inputs. Can be used for non-deterministic outputs to still be able to use cached cas content on clients
		// For example, when a pch is built on the host we can use all the input involved to create the pch and use that as the cas key for the resulting huge file
		// This means that if a remote machine has a pch from an older run where all the input matches, it can reuse the cas content even though the output differs
		void SetCustomCasKeyFromTrackedInputs(const tchar* fileName, const tchar* workingDir, const u8* trackedInputs, u32 trackedInputsBytes);
		bool GetCasKeyFromTrackedInputs(CasKey& out, const tchar* fileName, const tchar* workingDir, const u8* data, u32 dataLen);

		// Callback that will be called when a client is asking for processes to run. All remotes frequently ask for processes to run if they have free process slots
		using RemoteProcessAvailableCallback = Function<void(bool isCrossArchitecture, bool isCrossPlatform)>;
		void SetRemoteProcessSlotAvailableEvent(const RemoteProcessAvailableCallback& remoteProcessSlotAvailableEvent);

		// Callback that is called when process is returned. This could be because a client unexpectedly disconnected or is running out of memory
		void SetRemoteProcessReturnedEvent(const Function<void(Process&)>& remoteProcessReturnedEvent);

		// Callback that is called when process is returned. This could be because a client unexpectedly disconnected or is running out of memory
		void SetNativeProcessCreatedFunc(const Function<void(ProcessImpl&)>& func);

		// Wait for all queued up/active tasks to finish
		void WaitOnAllTasks();

		// Wait for all clients to disconnect.
		void WaitOnAllClients();

		// Can be used to hint the session how many processes (that can run remotely that are left.. session can then start disabling remote execution on clients not needed anymore
		void SetMaxRemoteProcessCount(u32 count);

		// Report an external process just to get it visible in the trace stream/visualizer. Returns an unique id that should be sent in to EndExternalProcess
		u32 BeginExternalProcess(const tchar* description, const tchar* breadcrumbs = TC(""));

		// End external process.
		void EndExternalProcess(u32 id, u32 exitCode);

		// Update progress. Will show in visualizer
		void UpdateProgress(u32 processesTotal, u32 processesDone, u32 errorCount);

		// Add external status information to trace stream. Will show in visualizer
		void UpdateStatus(u32 statusRow, u32 statusColumn, const tchar* statusText, LogEntryType statusType, const tchar* statusLink);

		// Add additional breadcrumbs to process. Can be done while process is running or after it is done
		void AddProcessBreadcrumbs(u32 processId, const tchar* breadcrumbs, bool deleteOld = false);

		// Get the network server used by this session
		NetworkServer& GetServer();

		// Additional network traffic provider. Can be used to add more network stats to trace
		using NetworkTrafficProvider = Function<void(u64& outSent, u64& outReceive, u32& outConnectionCount)>;
		void RegisterNetworkTrafficProvider(u64 key, const NetworkTrafficProvider& provider);
		void UnregisterNetworkTrafficProvider(u64 key);

		// Register mappings for cross architecture helpers (helpers with different architecture will use the mapping)
		bool RegisterCrossArchitectureMapping(const tchar* from, const tchar* to);

		void SetOuterScheduler(Scheduler* scheduler);
		Scheduler* GetOuterScheduler();

		bool GetRemoteTraceEnabled();

		ProcessHandle GetNewestLocalProcess();
		bool WasEverConnected();

	protected:
		struct ClientSession;

		void OnDisconnected(const Guid& clientUid, u32 clientId);
		#define UBA_SESSION_MESSAGE(x) bool Handle##x(const ConnectionInfo& connectionInfo, const WorkContext& workContext, BinaryReader& reader, BinaryWriter& writer);
		UBA_SESSION_MESSAGES
		#undef UBA_SESSION_MESSAGE

		bool StoreCasFile(CasKey& out, const StringKey& fileNameKey, const tchar* fileName);
		bool WriteDirectoryTable(ClientSession& session, BinaryReader& reader, BinaryWriter& writer);
		bool WriteNameToHashTable(BinaryReader& reader, BinaryWriter& writer, u32 requestedSize);

		struct RacingProcessExitInfo;
		class RemoteProcess;

		RemoteProcess* DequeueProcess(ClientSession& session, u32 sessionId, u32 clientId);
		void OnCancelled(RemoteProcess* process);
		void OnRaceLost(RemoteProcess& process);
		ProcessHandle ProcessRemoved(u32 processId);

		ProcessHandle GetProcess(u32 processId);
		TString GetProcessDescription(u32 processId);

		virtual bool ProcessNativeCreated(ProcessImpl& process) override;
		virtual bool CreateFileForRead(CreateFileResponse& out, TrackWorkScope& tws, StringView fileName, StringKey fileNameKey, ProcessImpl& process, const ApplicationRules& rules) override final;
		virtual bool GetOutputFileSizeInternal(u64& outSize, const StringKey& fileNameKey, StringView filePath) override;
		virtual bool GetOutputFileDataInternal(void* outData, const StringKey& fileNameKey, StringView filePath, bool deleteInternalMapping) override;
		virtual bool WriteOutputFileInternal(const StringKey& fileNameKey, StringView filePath, bool deleteInternalMapping) override;
		virtual void FileEntryAdded(StringKey fileNameKey, u64 lastWritten, u64 size) override final;
		virtual bool RunSpecialProgram(ProcessImpl& process, BinaryReader& reader, BinaryWriter& writer) override final;
		virtual void PrintSessionStats(Logger& logger) override final;
		
		struct ReceivedFile;
		bool WriteReceivedFile(ReceivedFile& file, RemoteProcess* process, const ConnectionInfo& connectionInfo);
		bool WriteReceivedFiles(Vector<ReceivedFile>& receivedFiles, RemoteProcess& process, Timer& writeFilesTimer, const ConnectionInfo& connectionInfo);
		void SkipReceivedFiles(Vector<ReceivedFile>& receivedFiles, const ConnectionInfo& connectionInfo);
		void ReleaseReceivedFiles(Vector<ReceivedFile>& receivedFiles);

		void TraceSessionUpdate();

		void WriteRemoteEnvironmentVariables(BinaryWriter& writer);
		bool InitializeNameToHashTable();


		NetworkServer& m_server;
		u32 m_uiLanguage;
		Atomic<u32> m_maxRemoteProcessCount = ~0u;
		bool m_resetCas;
		bool m_remoteExecutionEnabled;
		bool m_nameToHashTableEnabled;

		Vector<tchar> m_remoteEnvironmentVariables;

		static constexpr u8 ServiceId = SessionServiceId;

		ReaderWriterLock m_remoteProcessSlotAvailableEventLock;
		RemoteProcessAvailableCallback m_remoteProcessSlotAvailableEvent;

		ReaderWriterLock m_remoteProcessReturnedEventLock;
		Function<void(Process&)> m_remoteProcessReturnedEvent;

		Function<void(ProcessImpl&)> m_nativeProcessCreatedFunc;

		CriticalSection m_remoteProcessAndSessionLock; // Can be re-entrant.
		List<ProcessHandle> m_queuedRemoteProcesses;
		UnorderedSet<ProcessHandle> m_activeRemoteProcesses;
		u32 m_finishedRemoteProcessCount = 0;
		u32 m_returnedRemoteProcessCount = 0;
		u32 m_availableRemoteSlotCount = 0;
		u32 m_connectionCount = 0;

		Futex m_binKeysLock;
		CasKey m_detoursBinaryKey[2][2]; // [platform][isarm]
		CasKey m_agentBinaryKey[2]; // [isarm]
		#if PLATFORM_LINUX
		CasKey m_stubBinaryKey[2]; // [isarm]
		#endif

		struct ClientSession
		{
			TString name;
			UnorderedSet<CasKey> sentKeys;
			Vector<u32> processesLostRace;
			Futex dirTablePosLock;
			u32 dirTablePos = 0;
			u32 clientId = ~0u;
			u32 processSlotCount = 1;
			u32 usedSlotCount = 0;
			u64 lastPing = 0;
			u64 memAvail = 0;
			u64 memTotal = 0;
			u64 pingTime = 0;
			float cpuLoad = 0;
			bool connected = true;
			bool enabled = true;
			bool dedicated = false;
			bool abort = false;
			bool crashdump = false;
			bool hasNotification = false;
			bool isArm = false;
			bool isLinux = false; // Only used for non-linux
			bool remoteExecutionEnabled = true;
		};
		Vector<ClientSession*> m_clientSessions;

		struct CustomCasKey
		{
			CasKey casKey;
			TString workingDir;
			Vector<u8> trackedInputs;
		};
		Futex m_customCasKeysLock;
		UnorderedMap<StringKey, CustomCasKey> m_customCasKeys;

		UnorderedMap<StringKey, CasKey> m_nameToHashLookup;
		Futex m_nameToHashLookupLock;
		Atomic<bool> m_nameToHashInitialized = false;

		ReaderWriterLock m_receivedFilesLock;
		UnorderedMap<StringKey, CasKey> m_receivedFiles;

		Futex m_fillUpOneAtTheTimeLock;

		Futex m_applicationDataLock;
		struct ApplicationData { Futex lock; Vector<u8> bytes; };
		UnorderedMap<StringKey, ApplicationData> m_applicationData;

		struct NetworkTrafficProviderRec { u64 key; NetworkTrafficProvider provider; };
		Vector<NetworkTrafficProviderRec> m_providers;

		ReaderWriterLock m_crossArchitectureMappingsLock;
		struct CrossArchitectureMapping { TString from; TString to; };
		Vector<CrossArchitectureMapping> m_crossArchitectureMappings;

		bool m_remoteLogEnabled = false;
		bool m_remoteTraceEnabled = false;
		bool m_traceIOEnabled = false;
		bool m_customCasKeysEnabled = false;
		bool m_useFillLock = false;

		Scheduler* m_outerScheduler = nullptr;

		SessionServer(const SessionServer&) = delete;
		void operator=(const SessionServer&) = delete;
	};
}
