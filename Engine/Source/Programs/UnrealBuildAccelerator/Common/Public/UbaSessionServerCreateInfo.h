// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaSessionCreateInfo.h"

namespace uba
{
	class NetworkServer;

	struct SessionServerCreateInfo : SessionCreateInfo
	{
		SessionServerCreateInfo(Storage& s, NetworkServer& c, LogWriter& writer = g_consoleLogWriter) : SessionCreateInfo(s, writer), server(c) {}

		void Apply(const Config& config);

		NetworkServer& server;
		bool resetCas = false;
		bool remoteExecutionEnabled = true;
		bool nameToHashTableEnabled = true;
		bool remoteLogEnabled = false; // If Uba is built in debug, then the logs will be sent back to server
		bool remoteTraceEnabled = false; // If this is true, the agents will run trace and send the .uba file back to server
		bool traceIOEnabled = true; // Enable IO information for storage 
		bool customCasKeysEnabled = false; // Need to set this to true to be able to use SetCustomCasKeyFromTrackedInputs
		bool useFillLock = true; // Will fill one process request at the time. Useful if files that are similar are queued together
	};
}
