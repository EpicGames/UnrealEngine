// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaDefinitions.h"
#include "UbaFileHandle.h"
#include "UbaHash.h"
#include "UbaMap.h"
#include "UbaSharedMemoryView.h"

// Track shared memory allocations to be able to trace biggest allocations
#define UBA_TRACK_SHARED_MEMORY_ALLOCATIONS 0

namespace uba
{
	class Logger;
	class Trace;
	struct BinaryWriter;
	struct FileMappingBackend;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class SharedMemoryAllocator
	{
	public:
		bool Init(u64 capacity, u64 commitStepSize = 16ull*1024*1024, const tchar* tempFile = TC(""), bool tempFileSparse = true);
		void Deinit(bool keepFileMapping); // keepFileMapping can be used to just leak handle for shutdown
		void Reserve(u64 size);

		operator SharedMemoryAllocatorHandle() const;
		u64 GetCapacity();
		u64 GetUsed();

		SharedMemoryHandle CreateHandle(Logger& logger, const tchar* hint, bool priority = false); // Priority means it will have a compressable handle id
		SharedMemoryHandle DuplicateHandle(SharedMemoryHandle handle, const tchar* hint);
		void ClearHandle(SharedMemoryHandle handle);
		void CloseHandle(Logger& logger, SharedMemoryHandle handle, const tchar* hint);

		void ExtendMemory(BinaryWriter& out, SharedMemoryHandle handle, u64 size, const tchar* hint = TC(""), bool zeroMemory = true);
		void ExtendMemory(SharedMemoryHandle handle, u64 size, const tchar* hint = TC(""), bool zeroMemory = true);
		void ExtendMemory(SharedMemoryView& view, SharedMemoryHandle handle, u64 size, const tchar* hint = TC(""), bool zeroMemory = true);
		void TrimMemory(SharedMemoryHandle handle, u64 usedSize);

		u8* ReserveMemory(SharedMemoryHandle handle, u64 size);
		void CommitMemory(SharedMemoryHandle handle, u64 offset, u64 size);
		void DecommitMemory(SharedMemoryHandle handle, u64 offset, u64 size);
		void UnreserveMemory(SharedMemoryHandle handle);

		u64 GetCommitted(SharedMemoryHandle handle);
		u64 GetRefCount(SharedMemoryHandle handle);
		FileMappingBackend& GetFileMappingBackend();

		void WriteViewSlices(BinaryWriter& out, SharedMemoryHandle handle, u64 offset = 0, u64 size = ~0llu);
		bool MapView(SharedMemoryView& view, SharedMemoryHandle handle, const tchar* hint, SharedMemoryMapType mapType = SharedMemoryMapType_ReadOnly, u64 offset = 0, u64 size = ~0llu);


		u8* MapView(SharedMemoryHandle handle, const tchar* hint, SharedMemoryMapType mapType = SharedMemoryMapType_ReadOnly);
		void UnmapView(SharedMemoryHandle handle, u8* memory);


		void TraceStats(Trace& trace, u32 startRow);

		SharedMemoryAllocator(Logger& logger, FileMappingBackend& fileMappingBackend);
		~SharedMemoryAllocator();

		// This is really ugly. But very contained for virtual file mappings
		SharedMemoryHandle RegisterExternalMapping(FileMappingHandle file);
		bool IsExternalMapping(SharedMemoryHandle handle);
		void UnmapExternalMapping(SharedMemoryHandle handle, u8* memory, u64 size);

	private:
		struct InternalHandle;
		struct Slice;

		InternalHandle& GetHandle(SharedMemoryHandle h);
		void ReleaseHandle(SharedMemoryHandle h);
		void ExtendHandle(InternalHandle& handle, u64 size, const tchar* hint, bool zeroMemory);
		void ClearHandle(InternalHandle& handle);
		Slice AllocateNoLock(u64 desiredSize);
		void FreeNoLock(Slice slice);

		Logger& m_logger;
		FileMappingBackend& m_fileMappingBackend;

		u64 m_commitStepSize;
		FileHandle m_tempFile = InvalidFileHandle;
		FileMappingHandle m_fileMapping;
		u64 m_capacity = 0; // Capacity is total reserved address space from system
		u64 m_reserved = 0; // Reserved is the amount of address space used
		u64 m_committed = 0; // Committed is the amount of pages in address space that has been committed
		u8* m_memory = nullptr;

		Atomic<u64> m_used = 0; // Amount of memory used through ExtendMemory

		Futex m_lock;

		Map<u64, u64> m_freeMemory; // memory slots. Key is offset, value is size

		struct SizeAndOffset { u64 size; u64 offset; bool operator<(const SizeAndOffset& o) const { return size != o.size ? size < o.size : offset < o.offset; } };
		Set<SizeAndOffset> m_freeMemoryBySize; // Sorted by size and then offset. To find best fit allocations

		Atomic<u64> m_handleCount = 0;

		InternalHandle* m_priorityHandles[8];
		Atomic<u32> m_priorityHandleCounter = 0;

		SharedMemoryAllocator(const SharedMemoryAllocator&) = delete;
		SharedMemoryAllocator& operator=(const SharedMemoryAllocator&) = delete;

		#if UBA_TRACK_SHARED_MEMORY_ALLOCATIONS
		InternalHandle* m_firstHandle = nullptr;
		Futex m_handleLock;
		CasKey m_lastSent;
		#endif
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void GetMappingString(StringBufferBase& out, SharedMemoryHandle handle, u64 offset, bool canBeFreed);

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
