// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct sockaddr;

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct Socket { u64 internal = ~0ull; bool operator==(const Socket& o) const { return internal == o.internal; } };
	inline constexpr Socket InvalidSocket;

	#if PLATFORM_WINDOWS
	inline SOCKET ToPlatformSocket(Socket socket) { return (SOCKET)socket.internal; }
	inline Socket FromPlatformSocket(SOCKET socket) { return { u64(socket) }; }
	#else
	inline int ToPlatformSocket(Socket socket) { return (int)socket.internal; }
	inline Socket FromPlatformSocket(int socket) { return { u64(socket) }; }
	#endif

	////////////////////////////////////////////////////////////////////////////////////////////////////

	inline constexpr int PollIN      = 0x0001;
	inline constexpr int PollPRI     = 0x0002;
	inline constexpr int PollOUT     = 0x0004;
	inline constexpr int PollERR     = 0x0008;
	inline constexpr int PollHUP     = 0x0010;
	inline constexpr int PollNVAL    = 0x0020;
	inline constexpr int PollRDNORM  = 0x0040;
	inline constexpr int PollRDBAND  = 0x0080;
	inline constexpr int PollWRNORM  = 0x0100;
	inline constexpr int PollWRBAND  = 0x0200;

	struct SendBuf
	{
		u32 len;
		const void* data;
	};

	struct RecvBuf
	{
		u32 len;
		void* data;
	};

	Socket SocketCreate(int family = AF_INET, int type = SOCK_STREAM, int protocol = IPPROTO_TCP);
	int SocketConnect(Socket s, const sockaddr* addr, u32 addrSize, bool isBlocking = true);
	int SocketClose(Socket s);
	int SocketSetBlocking(Socket s, bool blocking);
	int SocketReuseAddr(Socket s);
	int SocketBind(Socket s, const sockaddr* addr, u32 addrSize);
	int SocketListen(Socket s);
	int SocketSetKeepAlive(Socket s, int keepAliveTime, int keepAliveTimeInterval);
	int SocketSetUserTimeout(Socket s, int timeoutMs);
	int SocketGetLastError();
	int SocketSetNoDelay(Socket s);
	int SocketSetLinger(Socket s, int lingerSeconds);
	int SocketSetTimeout(Socket s, int timeoutMs);
	int SocketSetRecvBuf(Socket s, int size);
	int SocketSetSendBuf(Socket s, int size);
	int SocketSetPriority(Socket s);
	int SocketPoll(Socket s, int events, int timeoutMs, int* outRevents);
	int SocketPoll2(Socket s, int events, int timeoutMs, Socket wakeupSocket, int* outRevents);
	Socket SocketCreateWakeupSocket();
	int SocketWakeup(Socket wakeupSocket);
	Socket SocketAccept(Socket s, sockaddr* outAddr, int addrLen);
	int SocketRecv(Socket s, void* data, int len);
	int SocketRecv2(Socket s, RecvBuf* bufs, int bufCount);
	int SocketSend(Socket s, SendBuf* bufs, int bufCount);
	bool SocketShouldPoll(bool* outRetry);
	int SocketCheckConnect(Socket s, bool* outTimedOut);
	int SocketShutdown(Socket s);
	int SocketGetCongestionAlgorithm(Socket s, char* out, int outCapacity);
	int SocketGetRecvBuf(Socket s, int& outSize);
	int SocketGetSendBuf(Socket s, int& outSize);

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
