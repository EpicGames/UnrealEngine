// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

////////////////////////////////////////////////////////////////////////////////////////////////////

Socket SocketCreate(int family, int type, int protocol)
{
#if PLATFORM_WINDOWS
	return FromPlatformSocket(WSASocketW(family, type, protocol, NULL, 0, WSA_FLAG_OVERLAPPED));
#elif PLATFORM_LINUX
	return FromPlatformSocket(::socket(family, type | SOCK_CLOEXEC, protocol));
#else
	return FromPlatformSocket(::socket(family, type, protocol));
#endif
}

int SocketConnect(Socket s, const sockaddr* addr, u32 addrSize, bool isBlocking)
{
	int res = ::connect(ToPlatformSocket(s), addr, addrSize);
	if (isBlocking)
		return res;
#if PLATFORM_WINDOWS
	if (WSAGetLastError() != WSAEWOULDBLOCK)
		return -1;
#else
	if (errno != EINPROGRESS)
		return -1;
#endif
	return 0;
}

int SocketClose(Socket s)
{
#if PLATFORM_WINDOWS
	return closesocket(ToPlatformSocket(s));
#else
	return close(ToPlatformSocket(s));
#endif
}

int SocketSetBlocking(Socket s, bool blocking)
{
#if PLATFORM_WINDOWS
	u_long value = blocking ? 0 : 1;
	return ioctlsocket(ToPlatformSocket(s), FIONBIO, &value);
#else
	int flags = fcntl(ToPlatformSocket(s), F_GETFL, 0);
	if (flags == -1) return -1;
	flags = blocking ? (flags & ~O_NONBLOCK) : (flags | O_NONBLOCK);
	if (fcntl(ToPlatformSocket(s), F_SETFL, flags) != 0)
		return -1;
	return 0;
#endif
}

int SocketReuseAddr(Socket s)
{
	u32 reuseAddr = 1;
	return ::setsockopt(ToPlatformSocket(s), SOL_SOCKET, SO_REUSEADDR, (const char*)&reuseAddr, sizeof reuseAddr);
}

int SocketBind(Socket s, const sockaddr* addr, u32 addrSize)
{
	return bind(ToPlatformSocket(s), addr, (socklen_t)addrSize);
}
int SocketListen(Socket s)
{
	return listen(ToPlatformSocket(s), SOMAXCONN);
}
int SocketSetKeepAlive(Socket s, int keepAliveTime, int keepAliveTimeInterval)
{
	u32 value = 1;
	int res = setsockopt(ToPlatformSocket(s), SOL_SOCKET, SO_KEEPALIVE, (const char*)&value, sizeof(value));
	if (res != 0)
		return res;

#if PLATFORM_WINDOWS	
	struct tcp_keepalive kaSettings;
	DWORD bytesReturned;
	kaSettings.onoff = 1;
	kaSettings.keepalivetime = keepAliveTime * 1000;
	kaSettings.keepaliveinterval = keepAliveTimeInterval * 1000; 
	res = WSAIoctl(ToPlatformSocket(s), SIO_KEEPALIVE_VALS, &kaSettings, sizeof(kaSettings), NULL, 0, &bytesReturned, NULL, NULL);
	if (res != 0)
		return res;
#elif PLATFORM_LINUX
	if (setsockopt(ToPlatformSocket(s), IPPROTO_TCP, TCP_KEEPIDLE, &keepAliveTime, sizeof(int)) < 0)
		return -1;
	if (setsockopt(ToPlatformSocket(s), IPPROTO_TCP, TCP_KEEPINTVL, &keepAliveTimeInterval, sizeof(int)) < 0)
		return -1;
	int keepAliveProbes = KeepAliveProbeCount; // Number of tests before timing out
	if (setsockopt(ToPlatformSocket(s), IPPROTO_TCP, TCP_KEEPCNT, &keepAliveProbes, sizeof(int)) < 0)
		return -1;
#else // PLATFORM_MAC
	if (setsockopt(ToPlatformSocket(s), IPPROTO_TCP, TCP_KEEPALIVE, &keepAliveTime, sizeof(int)) < 0)
		return -1;
#endif
	return 0;
}

int SocketSetUserTimeout(Socket s, int timeoutMs)
{
#if PLATFORM_WINDOWS
	DWORD maxrtSec = (DWORD)((timeoutMs + 999) / 1000);
	return setsockopt(ToPlatformSocket(s), IPPROTO_TCP, TCP_MAXRT, (const char*)&maxrtSec, sizeof(maxrtSec));
#elif PLATFORM_LINUX
	unsigned int ms = (unsigned int)timeoutMs;
	return setsockopt(ToPlatformSocket(s), IPPROTO_TCP, TCP_USER_TIMEOUT, &ms, sizeof(ms));
#else
	(void)s; (void)timeoutMs;
	return 0;
#endif
}

int SocketGetLastError()
{
#if PLATFORM_WINDOWS
	return WSAGetLastError();
#else
	return errno;
#endif
}

int SocketSetNoDelay(Socket s)
{
#if !PLATFORM_MAC
	u32 value = 1;
	return setsockopt(ToPlatformSocket(s), IPPROTO_TCP, TCP_NODELAY, (const char*)&value, sizeof(value));
#else
	return 0;
#endif
}

int SocketSetLinger(Socket s, int lingerSeconds)
{
	struct linger so_linger;
	so_linger.l_onoff = 1;
	so_linger.l_linger = u16(lingerSeconds);
	return setsockopt(ToPlatformSocket(s), SOL_SOCKET, SO_LINGER, (const char*)&so_linger, sizeof(so_linger));
}

int SocketSetTimeout(Socket s, int timeoutMs)
{
#if PLATFORM_WINDOWS
	DWORD timeout = timeoutMs;
#else
	struct timeval timeout;
	timeout.tv_sec = timeoutMs / 1000;
	timeout.tv_usec = (timeoutMs % 1000)*1000;
#endif
	if (setsockopt(ToPlatformSocket(s), SOL_SOCKET, SO_SNDTIMEO, (const char*)&timeout, sizeof timeout) != 0)
		return -1;
	if (setsockopt(ToPlatformSocket(s), SOL_SOCKET, SO_RCVTIMEO, (const char*)&timeout, sizeof timeout) != 0)
		return -1;
	return 0;
}

int SocketSetRecvBuf(Socket s, int size)
{
	int optname = SO_RCVBUF;
#if PLATFORM_LINUX
	optname = SO_RCVBUFFORCE;
#endif
	return setsockopt(ToPlatformSocket(s), SOL_SOCKET, optname, (const char*)&size, sizeof(size));
}

int SocketSetSendBuf(Socket s, int size)
{
	int optname = SO_SNDBUF;
#if PLATFORM_LINUX
	optname = SO_SNDBUFFORCE;
#endif
	return setsockopt(ToPlatformSocket(s), SOL_SOCKET, optname, (const char*)&size, sizeof(size));
}

int SocketSetPriority(Socket s)
{
#if PLATFORM_WINDOWS
	static HANDLE s_qosHandle = []()
		{
			HANDLE h = INVALID_HANDLE_VALUE;
			QOS_VERSION ver = {1, 0};
			QOSCreateHandle(&ver, &h);
			return h;
		}();
	if (s_qosHandle == INVALID_HANDLE_VALUE)
		return -1;
	QOS_FLOWID flowId = 0;
	if (!QOSAddSocketToFlow(s_qosHandle, ToPlatformSocket(s), NULL, QOSTrafficTypeExcellentEffort, QOS_NON_ADAPTIVE_FLOW, &flowId))
		return -1;
	return 0;
#elif PLATFORM_LINUX
	int prio = 6;
	return setsockopt(ToPlatformSocket(s), SOL_SOCKET, SO_PRIORITY, &prio, sizeof(prio));
	//uint8_t tos = 46 << 2; // EF = 0b101110xx
	//if (setsockopt(socket, IPPROTO_IP, IP_TOS, &tos, sizeof(tos)) == -1)
	//	return logger.Error(TC("setsockopt IP_TOS failed (%s)"), LastErrorToText(WSAGetLastError()).data);
#else
	return 0;
#endif
}

int SocketPoll(Socket s, int events, int timeoutMs, int* outRevents)
{
	int realEvents = 0;
	realEvents |= (events & PollIN) ? POLLIN : 0;
	realEvents |= (events & PollPRI) ? POLLPRI : 0;
	realEvents |= (events & PollOUT) ? POLLOUT : 0;
	realEvents |= (events & PollRDNORM) ? POLLRDNORM : 0;
	realEvents |= (events & PollWRNORM) ? POLLWRNORM : 0;
	realEvents |= (events & PollWRBAND) ? POLLWRBAND : 0;

#if PLATFORM_WINDOWS
	WSAPOLLFD p;
	p.fd = ToPlatformSocket(s);
	p.revents = 0;
	p.events = (SHORT)realEvents;
	int res = WSAPoll(&p, 1, timeoutMs);
#else
	pollfd p;
	p.fd = ToPlatformSocket(s);
	p.revents = 0;
	p.events = (short)realEvents;
	int res = poll(&p, 1, timeoutMs);

#endif
	if (!outRevents)
		return res;
	realEvents = 0;
	realEvents |= (p.revents & POLLIN) ? PollIN : 0;
	realEvents |= (p.revents & POLLPRI) ? PollPRI : 0;
	realEvents |= (p.revents & POLLOUT) ? PollOUT : 0;
	realEvents |= (p.revents & POLLERR) ? PollERR : 0;
	realEvents |= (p.revents & POLLHUP) ? PollHUP : 0;
	realEvents |= (p.revents & POLLNVAL) ? PollNVAL : 0;
	realEvents |= (p.revents & POLLRDNORM) ? PollRDNORM : 0;
	realEvents |= (p.revents & POLLWRNORM) ? PollWRNORM : 0;
	realEvents |= (p.revents & POLLWRBAND) ? PollWRBAND : 0;
	*outRevents = realEvents;
	return res;
}

int SocketPoll2(Socket s, int events, int timeoutMs, Socket wakeupSocket, int* outRevents)
{
	int realEvents = 0;
	realEvents |= (events & PollIN) ? POLLIN : 0;
	realEvents |= (events & PollPRI) ? POLLPRI : 0;
	realEvents |= (events & PollOUT) ? POLLOUT : 0;
	realEvents |= (events & PollRDNORM) ? POLLRDNORM : 0;
	realEvents |= (events & PollWRNORM) ? POLLWRNORM : 0;
	realEvents |= (events & PollWRBAND) ? POLLWRBAND : 0;

#if PLATFORM_WINDOWS
	WSAPOLLFD p[2];
	p[0].fd = ToPlatformSocket(s);
	p[0].revents = 0;
	p[0].events = (SHORT)realEvents;
	p[1].fd = ToPlatformSocket(wakeupSocket);
	p[1].revents = 0;
	p[1].events = POLLRDNORM;
	int res = WSAPoll(p, 2, timeoutMs);
#else
	pollfd p[2];
	p[0].fd = ToPlatformSocket(s);
	p[0].revents = 0;
	p[0].events = (short)realEvents;
	p[1].fd = ToPlatformSocket(wakeupSocket);
	p[1].revents = 0;
	p[1].events = POLLIN;
	int res = poll(p, 2, timeoutMs);
#endif
	if (res > 0 && (p[1].revents & POLLIN))
	{
		// Drain wakeup datagrams. Non-blocking socket → recv returns once empty.
		char drain[64];
		while (::recv(ToPlatformSocket(wakeupSocket), drain, sizeof(drain), 0) > 0) {}
	}
	if (!outRevents)
		return res;
	realEvents = 0;
	realEvents |= (p[0].revents & POLLIN) ? PollIN : 0;
	realEvents |= (p[0].revents & POLLPRI) ? PollPRI : 0;
	realEvents |= (p[0].revents & POLLOUT) ? PollOUT : 0;
	realEvents |= (p[0].revents & POLLERR) ? PollERR : 0;
	realEvents |= (p[0].revents & POLLHUP) ? PollHUP : 0;
	realEvents |= (p[0].revents & POLLNVAL) ? PollNVAL : 0;
	realEvents |= (p[0].revents & POLLRDNORM) ? PollRDNORM : 0;
	realEvents |= (p[0].revents & POLLWRNORM) ? PollWRNORM : 0;
	realEvents |= (p[0].revents & POLLWRBAND) ? PollWRBAND : 0;
	*outRevents = realEvents;
	return res;
}

Socket SocketCreateWakeupSocket()
{
	// UDP socket on 127.0.0.1, connected to itself: send/recv have no destination/source bookkeeping.
	// Single fd per blocking thread. Cheaper than a TCP socketpair (no listen/accept handshake, smaller buffers).
#if PLATFORM_WINDOWS
	SOCKET fd = WSASocketW(AF_INET, SOCK_DGRAM, IPPROTO_UDP, NULL, 0, WSA_FLAG_OVERLAPPED);
	if (fd == INVALID_SOCKET)
		return InvalidSocket;
#elif PLATFORM_LINUX
	int fd = ::socket(AF_INET, SOCK_DGRAM | SOCK_CLOEXEC, IPPROTO_UDP);
	if (fd == -1)
		return InvalidSocket;
#else
	int fd = ::socket(AF_INET, SOCK_DGRAM, IPPROTO_UDP);
	if (fd == -1)
		return InvalidSocket;
	int fdFlags = fcntl(fd, F_GETFD, 0);
	if (fdFlags != -1) fcntl(fd, F_SETFD, fdFlags | FD_CLOEXEC);
#endif
	sockaddr_in addr = {};
	addr.sin_family = AF_INET;
	addr.sin_addr.s_addr = htonl(INADDR_LOOPBACK);
	addr.sin_port = 0;
	socklen_t alen = sizeof(addr);
	if (::bind(fd, (sockaddr*)&addr, sizeof(addr)) != 0 || ::getsockname(fd, (sockaddr*)&addr, &alen) != 0 || ::connect(fd, (sockaddr*)&addr, sizeof(addr)) != 0)
	{
#if PLATFORM_WINDOWS
		closesocket(fd);
#else
		close(fd);
#endif
		return InvalidSocket;
	}
#if PLATFORM_WINDOWS
	u_long nb = 1;
	ioctlsocket(fd, FIONBIO, &nb);
#else
	int flags = fcntl(fd, F_GETFL, 0);
	if (flags != -1) fcntl(fd, F_SETFL, flags | O_NONBLOCK);
#endif
	return FromPlatformSocket(fd);
}

int SocketWakeup(Socket wakeupSocket)
{
	char one = 'x';
	// Connected UDP socket → no destination needed. Non-blocking → never stalls. Idempotent: extra
	// bytes accumulate harmlessly until the waiter drains them in SocketPoll2.
	return (int)::send(ToPlatformSocket(wakeupSocket), &one, 1, 0);
}

Socket SocketAccept(Socket s, sockaddr* outAddr, int addrLen)
{
	socklen_t remoteSockAddrLen = addrLen;
#if PLATFORM_LINUX
	return FromPlatformSocket(accept4(ToPlatformSocket(s), outAddr, &remoteSockAddrLen, SOCK_CLOEXEC));
#else
	return FromPlatformSocket(accept(ToPlatformSocket(s), outAddr, &remoteSockAddrLen));
#endif
}

int SocketRecv(Socket s, void* data, int len)
{
	return (int)recv(ToPlatformSocket(s), (char*)data, len, 0);
}

int SocketRecv2(Socket s, RecvBuf* bufs, int bufCount)
{
#if PLATFORM_WINDOWS
	DWORD flags = 0;
	DWORD received = 0;
	int res = WSARecv(ToPlatformSocket(s), (LPWSABUF)bufs, bufCount, &received, &flags, NULL, NULL);
	if (!res)
		return int(received);
	return SOCKET_ERROR;
#else
	iovec bufs2[2];
	for (int i=0; i!=bufCount; ++i)
		bufs2[i] = { (void*)bufs[i].data, bufs[i].len };
	msghdr msg = {};
	msg.msg_iov = bufs2;
	msg.msg_iovlen = bufCount;
	return recvmsg(ToPlatformSocket(s), &msg, 0);
#endif
}

int SocketSend(Socket s, SendBuf* bufs, int bufCount)
{
	if (bufCount == 1)
		return (int)send(ToPlatformSocket(s), (char*)bufs[0].data, bufs[0].len, 0);
#if PLATFORM_WINDOWS
	DWORD bytesSent = 0;
	int res = WSASend(ToPlatformSocket(s), (LPWSABUF)bufs, bufCount, &bytesSent, 0, NULL, NULL);
	if (!res)
		res = int(bytesSent);
	return res;
#else
	iovec bufs2[256]; // 256 is also set in UbaNetworkBackendTcp.cpp
	for (int i=0; i!=bufCount; ++i)
		bufs2[i] = { (void*)bufs[i].data, bufs[i].len };
	return writev(ToPlatformSocket(s), bufs2, bufCount);
#endif

}

bool SocketShouldPoll(bool* outRetry)
{
	*outRetry = false;
#if PLATFORM_WINDOWS
	return WSAGetLastError() == WSAEWOULDBLOCK;
#else
	if (errno == EINTR)
		*outRetry = true;
	return errno == EWOULDBLOCK || errno == EAGAIN;
#endif
}

int SocketCheckConnect(Socket s, bool* outTimedOut)
{
#if !PLATFORM_WINDOWS
	// Before we send anything even though the
	// the socket is writable, but let's make sure
	// the connection is actually valid by getting
	// information about what we've connected to
	struct sockaddr_in junk = { 0 };
	socklen_t length = sizeof(junk);
	if (getpeername(ToPlatformSocket(s), (struct sockaddr *)&junk, &length) != 0)
	{
		if (outTimedOut)
			*outTimedOut = true;
		return -1;
	}

	int sent = (int)send(ToPlatformSocket(s), nullptr, 0, 0);
	if (sent < 0)
	{
		if (errno == ECONNREFUSED || errno == EPIPE)
		{
			if (outTimedOut)
				*outTimedOut = true;
			return -1;
		}
		return -1;
	}
#endif
	return 0;
}

int SocketShutdown(Socket s)
{
	if (s == InvalidSocket)
		return 0;
#if PLATFORM_WINDOWS
	if (shutdown(ToPlatformSocket(s), SD_BOTH) != SOCKET_ERROR)
		return 0;
	if (WSAGetLastError() == WSAENOTCONN)
		return 0;
#else
	if (shutdown(ToPlatformSocket(s), SHUT_RDWR) == 0)
		return 0;
	if (errno == ENOTCONN)
		return 0;
#endif
	return -1;
}

int SocketGetCongestionAlgorithm(Socket s, char* out, int outCapacity)
{
#if PLATFORM_LINUX
	socklen_t len = outCapacity;
	return getsockopt(ToPlatformSocket(s), IPPROTO_TCP, TCP_CONGESTION, out, &len);
#else
	strcpy_s(out, outCapacity, "Default");
	return 0;
#endif
}

int SocketGetRecvBuf(Socket s, int& outSize)
{
	socklen_t optlen = sizeof(int);
	return getsockopt(ToPlatformSocket(s), SOL_SOCKET, SO_RCVBUF, (char*)&outSize, &optlen);
}

int SocketGetSendBuf(Socket s, int& outSize)
{
	socklen_t optlen = sizeof(int);
	return getsockopt(ToPlatformSocket(s), SOL_SOCKET, SO_SNDBUF, (char*)&outSize, &optlen);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
