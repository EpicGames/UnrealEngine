// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	namespace impl
	{
		#include "UbaSocket.inl"
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	#define SOCKET_FUNCTIONS \
		SOCKET_FUNCTION(Create) \
		SOCKET_FUNCTION(Connect) \
		SOCKET_FUNCTION(Close) \
		SOCKET_FUNCTION(SetBlocking) \
		SOCKET_FUNCTION(ReuseAddr) \
		SOCKET_FUNCTION(Bind) \
		SOCKET_FUNCTION(Listen) \
		SOCKET_FUNCTION(SetKeepAlive) \
		SOCKET_FUNCTION(SetUserTimeout) \
		SOCKET_FUNCTION(GetLastError) \
		SOCKET_FUNCTION(Poll) \
		SOCKET_FUNCTION(Accept) \
		SOCKET_FUNCTION(SetNoDelay) \
		SOCKET_FUNCTION(SetLinger) \
		SOCKET_FUNCTION(SetTimeout) \
		SOCKET_FUNCTION(SetRecvBuf) \
		SOCKET_FUNCTION(SetSendBuf) \
		SOCKET_FUNCTION(SetPriority) \
		SOCKET_FUNCTION(Recv) \
		SOCKET_FUNCTION(Recv2) \
		SOCKET_FUNCTION(Send) \
		SOCKET_FUNCTION(ShouldPoll) \
		SOCKET_FUNCTION(CheckConnect) \
		SOCKET_FUNCTION(Shutdown) \
		SOCKET_FUNCTION(GetCongestionAlgorithm) \
		SOCKET_FUNCTION(GetRecvBuf) \
		SOCKET_FUNCTION(GetSendBuf) \
		SOCKET_FUNCTION(Poll2) \
		SOCKET_FUNCTION(CreateWakeupSocket) \
		SOCKET_FUNCTION(Wakeup) \

	#define SOCKET_FUNCTION(func) decltype(impl::Socket##func)* g_socket##func##Func = impl::Socket##func;
	SOCKET_FUNCTIONS
	#undef SOCKET_FUNCTION

	////////////////////////////////////////////////////////////////////////////////////////////////////

	Socket SocketCreate(int family, int type, int protocol) { return g_socketCreateFunc(family, type, protocol); }
	int SocketConnect(Socket s, const sockaddr* addr, u32 addrSize, bool isBlocking) { return g_socketConnectFunc(s, addr, addrSize, isBlocking); }
	int SocketClose(Socket s) { return g_socketCloseFunc(s); }
	int SocketSetBlocking(Socket s, bool blocking) { return g_socketSetBlockingFunc(s, blocking); }
	int SocketReuseAddr(Socket s) { return g_socketReuseAddrFunc(s); }
	int SocketBind(Socket s, const sockaddr* addr, u32 addrSize) { return g_socketBindFunc(s, addr, addrSize); }
	int SocketListen(Socket s) { return g_socketListenFunc(s); }
	int SocketSetKeepAlive(Socket s, int keepAliveTime, int keepAliveTimeInterval) { return g_socketSetKeepAliveFunc(s, keepAliveTime, keepAliveTimeInterval); }
	int SocketSetUserTimeout(Socket s, int timeoutMs) { return g_socketSetUserTimeoutFunc(s, timeoutMs); }
	int SocketGetLastError() { return g_socketGetLastErrorFunc(); }
	int SocketSetNoDelay(Socket s) { return g_socketSetNoDelayFunc(s); }
	int SocketSetLinger(Socket s, int lingerSeconds) { return g_socketSetLingerFunc(s, lingerSeconds); }
	int SocketSetTimeout(Socket s, int timeoutMs) { return g_socketSetTimeoutFunc(s, timeoutMs); }
	int SocketSetRecvBuf(Socket s, int size) { return g_socketSetRecvBufFunc(s, size); }
	int SocketSetSendBuf(Socket s, int size) { return g_socketSetSendBufFunc(s, size); }
	int SocketPoll(Socket s, int events, int timeoutMs, int* outRevents) { return g_socketPollFunc(s, events, timeoutMs, outRevents); }
	int SocketPoll2(Socket s, int events, int timeoutMs, Socket wakeupSocket, int* outRevents) { return g_socketPoll2Func(s, events, timeoutMs, wakeupSocket, outRevents); }
	Socket SocketCreateWakeupSocket() { return g_socketCreateWakeupSocketFunc(); }
	int SocketWakeup(Socket wakeupSocket) { return g_socketWakeupFunc(wakeupSocket); }
	Socket SocketAccept(Socket s, sockaddr* outAddr, int addrLen) { return g_socketAcceptFunc(s, outAddr, addrLen); }
	int SocketRecv(Socket s, void* data, int len) { return g_socketRecvFunc(s, data, len); }
	int SocketRecv2(Socket s, RecvBuf* bufs, int bufCount) { return g_socketRecv2Func(s, bufs, bufCount); }
	int SocketSend(Socket s, SendBuf* bufs, int bufCount) { return g_socketSendFunc(s, bufs, bufCount); }
	int SocketCheckConnect(Socket s, bool* outTimedOut) { return g_socketCheckConnectFunc(s, outTimedOut); }
	bool SocketShouldPoll(bool* outRetry) { return g_socketShouldPollFunc(outRetry); }
	int SocketShutdown(Socket s) { return g_socketShutdownFunc(s); }
	int SocketGetCongestionAlgorithm(Socket s, char* out, int outCapacity) { return g_socketGetCongestionAlgorithmFunc(s, out, outCapacity); } 
	int SocketGetRecvBuf(Socket s, int& outSize) { return g_socketSetRecvBufFunc(s, outSize); }
	int SocketGetSendBuf(Socket s, int& outSize) { return g_socketSetSendBufFunc(s, outSize); }
	int SocketSetPriority(Socket s) { return g_socketSetPriorityFunc(s); };

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool SocketInitWine(Logger& logger, HMODULE wineDll)
	{
		#define SOCKET_FUNCTION(func) \
			auto func##Func = (decltype(Socket##func)*)(void*)GetProcAddress(wineDll, "Socket" #func); \
			if (!func##Func) return logger.Warning(TC("Socket" #func " is not exported from wine dll"));
		SOCKET_FUNCTIONS
		#undef SOCKET_FUNCTION

		#define SOCKET_FUNCTION(func) g_socket##func##Func = func##Func;
		SOCKET_FUNCTIONS
		#undef SOCKET_FUNCTION

		return true;
	}

	bool UsingNativeSockets()
	{
		return g_socketCreateFunc != impl::SocketCreate;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
