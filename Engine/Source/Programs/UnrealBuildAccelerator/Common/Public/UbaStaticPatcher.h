// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if PLATFORM_LINUX

#include "UbaStringBuffer.h"

namespace uba
{
	class Logger;

	// Returns true if `path` is a statically-linked ELF that UBA can
	// detour by injecting the static stub blob — i.e. either:
	//   * a Go-static binary (has internal/runtime/syscall.Syscall6 in
	//     its .symtab so the stub can hook it), OR
	//   * a glibc-static binary (has a usable .symtab AND ≥ 5 of the 9
	//     MVP libc wrappers: __libc_open / __openat / __libc_read /
	//     __libc_write / __libc_close / __libc_lseek / __fxstat / __mmap
	//     / __access — STT_FUNC GLOBAL/WEAK, first byte not 0x0F 0x05).
	//
	// Returns false for everything else: dynamic ELFs (use the normal
	// LD_PRELOAD path), non-ELFs, missing files, plain static ELFs that
	// have neither the Go nor the glibc fingerprint (musl-static and
	// stripped statics land here — we can't hook by name without a
	// symtab), and any parse error. All errors are logged at Info level
	// so this can be called speculatively on every launch.
	bool BinaryRequiresPatching(Logger& logger, StringView inputPath, const u8* inputData, u32 inputSize);

	// Produces a patched copy of a statically-linked ELF binary with the
	// UBA static-detour stub embedded. Internally dispatches to the
	// Go-static path (e_entry rewrite + Syscall6 / runtime.exit baking)
	// or the glibc-static path (Go-style stub injection plus per-libc-
	// wrapper JMP installation and ImportSlot stamping for the target's
	// __errno_location). The stub blob is loaded from disk
	// (UbaStaticStub.bin, shipped alongside UbaCli / UbaAgent).
	//
	// The caller is expected to have already filtered with
	// RequiresPatching(); calling PatchBinary on a binary that doesn't
	// pass that filter will fail.
	//
	// Intended call sites:
	//   - UbaCli -patchbinary=<in> -out=<out>
	//   - UbaSession::CookProcessAndIo (local/native launches)
	//   - UbaSessionServer remote-binary serving path
	bool PatchStaticBinary(Logger& logger, StringView inputPath, const u8* inputData, u32 inputSize, StringView outputPath, StringView stubBlobPath);

	// -------------------------------------------------------------------------
	// Granular predicates / per-flavour patchers.
	//
	// Production code should prefer the unified RequiresPatching /
	// PatchBinary entry points above. These granular variants are kept
	// public for unit tests that distinguish Go-static from glibc-static
	// detection (UbaTestGlibcPatcher / UbaTestStaticPatcher) — the
	// dispatcher swallows that distinction.
	// -------------------------------------------------------------------------

	bool IsGoStaticBinary(Logger& logger, StringView inputPath, const u8* inputData, u32 inputSize);
	bool IsGlibcStaticBinary(Logger& logger, StringView inputPath, const u8* inputData, u32 inputSize);
	bool PatchStaticBinary(Logger& logger, StringView inputPath, const u8* inputData, u32 inputSize, StringView outputPath, StringView stubBlobPath);
	bool PatchGlibcStaticBinary(Logger& logger, StringView inputPath, const u8* inputData, u32 inputSize, StringView outputPath, StringView stubBlobPath);
	bool LooksAlreadyPatched(const u8* data, u32 size);
}

#endif
