// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaProcessStats.h"

namespace uba
{
	class Logger;

	#define UBA_STORAGE_STATS \
		UBA_STORAGE_STAT(Timer, calculateCasKey, 0) \
		UBA_STORAGE_STAT(Timer, copyOrLink, 0) \
		UBA_STORAGE_STAT(Timer, copyOrLinkWait, 0) \
		UBA_STORAGE_STAT(Timer, ensureCas, 0) \
		UBA_STORAGE_STAT(Timer, sendCas, 0) \
		UBA_STORAGE_STAT(Timer, recvCas, 0) \
		UBA_STORAGE_STAT(Timer, compressWrite, 0) \
		UBA_STORAGE_STAT(Timer, compressSend, 0) \
		UBA_STORAGE_STAT(Timer, decompressRecv, 0) \
		UBA_STORAGE_STAT(Timer, decompressToMem, 0) \
		UBA_STORAGE_STAT(Timer, memoryCopy, 30) \
		UBA_STORAGE_STAT(Timer, handleOverflow, 0) \
		UBA_STORAGE_STAT(AtomicU64, sendCasBytesRaw, 0) \
		UBA_STORAGE_STAT(AtomicU64, sendCasBytesComp, 0) \
		UBA_STORAGE_STAT(AtomicU64, recvCasBytesRaw, 0) \
		UBA_STORAGE_STAT(AtomicU64, recvCasBytesComp, 0) \
		UBA_STORAGE_STAT(Timer, createCas, 0) \
		UBA_STORAGE_STAT(AtomicU64, createCasBytesRaw, 0) \
		UBA_STORAGE_STAT(AtomicU64, createCasBytesComp, 0) \

    struct StorageStats
	{
		#define UBA_STORAGE_STAT(type, var, ver) type var;
		UBA_STORAGE_STATS
		#undef UBA_STORAGE_STAT

		void Write(BinaryWriter& writer);
		void Read(BinaryReader& reader, u32 version);
		void Add(const StorageStats& other);
		void Print(Logger& logger, u64 frequency = GetFrequency());
		bool IsEmpty();
		static StorageStats* GetCurrent();

		enum
		{
			#define UBA_STORAGE_STAT(type, var, ver) Bit_##var,
			UBA_STORAGE_STATS
			#undef UBA_STORAGE_STAT
		};
	};

	struct StorageStatsScope
	{
		StorageStatsScope(StorageStats& stats);
		~StorageStatsScope();
		StorageStatsScope(const StorageStatsScope&) = delete;
		void operator=(const StorageStatsScope&) = delete;
		StorageStats& stats;
	};

	#define UBA_SESSION_STATS \
		UBA_SESSION_STAT(Timer, getFileMsg, 0) \
		UBA_SESSION_STAT(Timer, getBinaryMsg, 0) \
		UBA_SESSION_STAT(Timer, sendFileMsg, 0) \
		UBA_SESSION_STAT(Timer, listDirMsg, 0) \
		UBA_SESSION_STAT(Timer, getDirsMsg, 0) \
		UBA_SESSION_STAT(Timer, getHashesMsg, 8) \
		UBA_SESSION_STAT(Timer, deleteFileMsg, 0) \
		UBA_SESSION_STAT(Timer, copyFileMsg, 16) \
		UBA_SESSION_STAT(Timer, createDirMsg, 0) \
		UBA_SESSION_STAT(Timer, waitGetFileMsg, 10) \
		UBA_SESSION_STAT(Timer, createMmapFromFile, 12) \
		UBA_SESSION_STAT(Timer, waitMmapFromFile, 12) \
		UBA_SESSION_STAT(Timer, getLongNameMsg, 31) \
		UBA_SESSION_STAT(Timer, waitBottleneck, 40) \

    struct SessionStats
	{
		#define UBA_SESSION_STAT(type, var, ver) type var;
		UBA_SESSION_STATS
		#undef UBA_SESSION_STAT

		void Write(BinaryWriter& writer);
		void Read(BinaryReader& reader, u32 version);
		void Add(const SessionStats& other);
		void Print(Logger& logger, u64 frequency = GetFrequency());
		bool IsEmpty();
		static SessionStats* GetCurrent();

		enum
		{
			#define UBA_SESSION_STAT(type, var, ver) Bit_##var,
			UBA_SESSION_STATS
			#undef UBA_SESSION_STAT
		};
	};

	struct SessionStatsScope
	{
		SessionStatsScope(SessionStats& stats);
		~SessionStatsScope();
		SessionStatsScope(const SessionStatsScope&) = delete;
		void operator=(const SessionStatsScope&) = delete;
		SessionStats& stats;
	};

	#define UBA_SESSION_SUMMARY_STATS \
		UBA_SESSION_SUMMARY_STAT(Timer, storageRetrieve) \
		UBA_SESSION_SUMMARY_STAT(Timer, storageSend) \
		UBA_SESSION_SUMMARY_STAT(Timer, connectMsg) \
		UBA_SESSION_SUMMARY_STAT(Timer, getApplicationMsg) \
		UBA_SESSION_SUMMARY_STAT(Timer, procAvailableMsg) \
		UBA_SESSION_SUMMARY_STAT(Timer, procFinishedMsg) \
		UBA_SESSION_SUMMARY_STAT(Timer, procReturnedMsg) \
		UBA_SESSION_SUMMARY_STAT(Timer, pingMsg) \
		UBA_SESSION_SUMMARY_STAT(Timer, customMsg) \
		UBA_SESSION_SUMMARY_STAT(u64, waitMemPressure) \
		UBA_SESSION_SUMMARY_STAT(u64, killCount) \

    struct SessionSummaryStats
	{
		SessionStats stats;

		#define UBA_SESSION_SUMMARY_STAT(type, var) type var;
		UBA_SESSION_SUMMARY_STATS
		#undef UBA_SESSION_SUMMARY_STAT

		SessionSummaryStats() : waitMemPressure(0), killCount(0) {}
		void Write(BinaryWriter& writer);
		void Read(BinaryReader& reader, u32 version);
		void Print(Logger& logger, u64 frequency = GetFrequency());
	};

	#define UBA_CACHE_FETCH_STATS \
		UBA_CACHE_STAT(Timer, fetchEntries, 0) \
		UBA_CACHE_STAT(Timer, fetchCasTable, 0) \
		UBA_CACHE_STAT(Timer, normalizeFile, 28) \
		UBA_CACHE_STAT(Timer, testEntry, 0) \
		UBA_CACHE_STAT(Timer, fetchOutput, 0) \
		UBA_CACHE_STAT(AtomicU64, fetchBytesRaw, 26) \
		UBA_CACHE_STAT(AtomicU64, fetchBytesComp, 26) \

    struct CacheFetchStats
	{
		#define UBA_CACHE_STAT(type, var, ver) type var;
		UBA_CACHE_FETCH_STATS
		#undef UBA_CACHE_STAT

		void Write(BinaryWriter& writer);
		void Read(BinaryReader& reader, u32 version);
		void Print(Logger& logger, u64 frequency = GetFrequency());
		bool IsEmpty();
	};

	#define UBA_CACHE_SEND_STATS \
		UBA_CACHE_STAT(Timer, build, 0) \
		UBA_CACHE_STAT(Timer, sendPathTable, 0) \
		UBA_CACHE_STAT(Timer, sendCasTable, 0) \
		UBA_CACHE_STAT(Timer, sendTableWait, 0) \
		UBA_CACHE_STAT(Timer, sendEntry, 0) \
		UBA_CACHE_STAT(Timer, createCas, 0) \
		UBA_CACHE_STAT(TimeAndBytes, sendFile, 0) \
		UBA_CACHE_STAT(TimeAndBytes, sendNormalizedFile, 0) \

    struct CacheSendStats
	{
		#define UBA_CACHE_STAT(type, var, ver) type var;
		UBA_CACHE_SEND_STATS
		#undef UBA_CACHE_STAT

		void Write(BinaryWriter& writer);
		void Read(BinaryReader& reader, u32 version);
		void Print(Logger& logger, u64 frequency = GetFrequency());
		bool IsEmpty();
	};
}
