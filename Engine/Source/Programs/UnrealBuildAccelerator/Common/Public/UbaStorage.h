// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBottleneck.h"
#include "UbaBufferCache.h"
#include "UbaEnvironment.h"
#include "UbaFile.h"
#include "UbaFileMappingBuffer.h"
#include "UbaList.h"
#include "UbaLogger.h"
#include "UbaMemory.h"
#include "UbaSharedMemoryAllocator.h"
#include "UbaStats.h"

#define UBA_TRACK_IS_EXECUTABLE !PLATFORM_WINDOWS

namespace uba
{
	class Config;
	class FileAccessor;
	class Trace;
	class WorkManager;
	struct DirectoryEntry;
	extern CasKey EmptyFileKey;
	
	struct MappedView2 : MappedView
	{
		MappedView2() = default;
		MappedView2(const MappedView2&) = default;
		MappedView2(const MappedView& v) { *(MappedView*)this = v; }
		bool isCompressed = true;
	};

	class Storage
	{
	public:
		virtual ~Storage() {}
		virtual bool StoreCompressed() const = 0;
		virtual void PrintSummary(Logger& logger) = 0;
		virtual bool Reset() = 0;
		virtual bool SaveCasTable(bool deleteIsRunningfile, bool deleteDropped = true) = 0;
		virtual u64 GetStorageCapacity() = 0;
		virtual u64 GetStorageUsed() = 0;
		virtual bool GetZone(StringBufferBase& out) = 0;
		virtual bool HasProxy(u32 clientId) { return false; }

		virtual bool DecompressFileToMemory(const tchar* fileName, FileHandle fileHandle, u8* dest, u64 decompressedSize, const tchar* writeHint, u64 fileStartOffset) = 0;
		virtual bool DecompressMemoryToMemory(const u8* compressedData, u64 compressedSize, u8* writeData, u64 decompressedSize, const tchar* readHint, const tchar* writeHint) = 0;
		virtual bool CreateDirectory(const tchar* dir, bool* outAlreadyExists = nullptr) = 0;
		virtual bool DeleteCasForFile(const tchar* file) = 0;
		virtual bool GetFileSize(u64& outSize, const CasKey& casKey, const tchar* hint) = 0;
		virtual bool GetFileData(void* outData, const CasKey& casKey, const tchar* hint) = 0;

		struct RetrieveResult { CasKey casKey; u64 size = 0; MappedView2 view; };
		virtual bool RetrieveCasFile(RetrieveResult& out, const CasKey& casKey, StringKey keyHint, const tchar* hint, FileMappingBuffer* mappingBuffer = nullptr, bool createIndependentMapping = false, u64 memoryMapAlignment = 1, bool allowProxy = true, u32 clientId = 0) = 0;

		virtual bool AddRef(const CasKey& casKey) = 0;
		virtual bool ReleaseRef(const CasKey& casKey, bool dropIfZero) = 0;

		struct CachedFileInfo { CasKey casKey; };
		virtual bool VerifyAndGetCachedFileInfo(CachedFileInfo& out, StringKey fileNameKey, u64 verifiedLastWriteTime, u64 verifiedSize) = 0;
		virtual bool InvalidateCachedFileInfo(StringKey fileNameKey) = 0;

		virtual bool StoreCasFile(CasKey& out, const tchar* fileName, const CasKey& casKeyOverride, bool deferCreation) = 0;
		virtual bool StoreCasFileClient(CasKey& out, StringKey fileNameKey, const tchar* fileName, SharedMemoryHandle memoryHandle, u64 memoryOffset, u64 fileSize, const tchar* hint, bool keepMappingInMemory = false, bool storeCompressed = true) = 0;
		virtual bool DropCasFile(const CasKey& casKey, bool forceDelete, const tchar* hint, bool reduceRefCount = false) = 0;
		virtual bool ReportBadCasFile(const CasKey& casKey) = 0;
		virtual bool CalculateCasKey(CasKey& out, StringKey fileNameKey, const tchar* fileName) = 0;

		using FormattingFunc = Function<bool(MemoryBlock& destData, const void* sourceData, u64 sourceSize, const tchar* hint)>;
		virtual bool CopyOrLink(const CasKey& casKey, const tchar* destination, u32 fileAttributes, bool writeCompressed = false, u32 clientId = 0, const FormattingFunc& formattingFunc = {}, bool isTemp = false, bool allowHardLink = true) = 0;
		virtual bool WritePlaceholder(const CasKey& casKey, const tchar* destination, u32 fileAttributes, u32 clientId) = 0;
		virtual bool FakeCopy(const CasKey& casKey, const tchar* destination, u64 size, u64 lastWritten, u32 fileAttributes, bool deleteExisting = true) = 0;
		virtual bool SkipWrite(const CasKey& casKey, u32 clientId) { return true; } 
		virtual bool GetCasFileName(StringBufferBase& out, const CasKey& casKey) = 0;

		virtual MappedView2 MapView(const CasKey& casKey, const tchar* hint) = 0;
		virtual void UnmapView(const MappedView& view, const tchar* hint) = 0;

		virtual void ReportFileWrite(StringKey fileNameKey, const tchar* fileName) = 0;
		
		virtual StorageStats& Stats() = 0;
		virtual void AddStats(StorageStats& stats) = 0;
		virtual void TraceUpdate(Trace& trace, u32 startRow) = 0;

		virtual void SetTrace(Trace* trace, bool detailed) {}
		virtual void Ping() {}

		virtual SharedMemoryAllocator& GetSharedMemory() = 0;

		struct WriteResult
		{
			SharedMemoryHandle memoryHandle;
			u64 memoryOffset = InvalidValue;
			u64 storedSize = InvalidValue; // Stored size of the destination
			u64 sourceStoredSize = InvalidValue; // Stored size of the source used to write compressed file. Can be both compressed and uncompressed
			bool isIndependent = false;
		};
		virtual bool WriteCompressedFile(WriteResult& out, const tchar* from, FileHandle readHandle, u8* readMem, u64 fileSize, const tchar* toFile, const void* header, u64 headerSize, u64 lastWriteTime = 0) = 0;

		struct ExternalFileMapping
		{
			SharedMemoryHandle handle; // Will close handle so provider needs to DuplicateHandle when setting it in ExternalFileMapping
			u64 offset = 0;
			u64 size = 0;
			u64 lastWriteTime = 0;
			u64 storedSize = InvalidValue; // Size stored on disk. Can be compressed or uncompressed
			bool createIndependentMapping = false;
			bool dropCasAfterUse = false;
		};
		using ExternalFileMappingsProvider = Function<bool(ExternalFileMapping& out, StringKey fileNameKey, const tchar* fileName)>;
		virtual void RegisterExternalFileMappingsProvider(ExternalFileMappingsProvider&& provider) {}
		virtual void DeleteExternalFileMapping(StringKey fileNameKey, const tchar* fileName) {}
	};

	struct StorageCreateInfo
	{
		StorageCreateInfo(const tchar* rootDir_, LogWriter& w, WorkManager& wm);

		void Apply(const Config& config);

		LogWriter& writer;
		WorkManager& workManager;
		const tchar* rootDir;
		u64 casCapacityBytes = 20llu * 1024 * 1024 * 1024;
		u32 fileEntryTableWipeThreshold = 1'000'000;
		u32 maxParallelCopyOrLink = 1000;
		u32 deleteCasBatchSize = 10;
		bool storeCompressed = true;
		bool manuallyHandleOverflow = false;
		bool asyncUnmapViewOfFile = true;
		bool writeToDisk = true;
		bool allowDeleteVerified = false;
		bool allowDeferredDeletes = true;
		bool allowMemfd = true;
		bool skipCleanupOnShutdown = false;
		bool allowUnDropOfCas = true;
		bool sideStepWineForSharedMemory = true;
		u64 sharedMemoryCapacity = 1024ull*1024*1024*1024; // 1TB
		u64 sharedMemoryCommitSize = 64ull*1024*1024;
		u64 sharedMemoryInitialCommit = 0ull;
		const tchar* sharedMemoryTempFile = TC("");
		bool sharedMemoryTempFileSparse = true;
		MutexHandle exclusiveMutex = InvalidMutexHandle; // Set this to hand over pre-exclusive access to storage. Storage will release mutex on shutdown
		u8 casCompressor = 0;
		u8 casCompressionLevel = 0;
	};

	static constexpr u64 BufferSlotSize = 16*1024*1024;
	static constexpr u64 BufferSlotHalfSize = BufferSlotSize/2; // This must be three times a msg size or more.

	class StorageImpl : public Storage
	{
	public:
		StorageImpl(const StorageCreateInfo& info, const tchar* logPrefix = TC("UbaStorage"));
		virtual ~StorageImpl();

		bool LoadCasTable(bool logStats = true, bool alwaysCheckAllFiles = false, bool* outWasTerminated = nullptr);
		bool CheckCasContent(Vector<TString>* badFiles = nullptr);
		bool CheckFileTable(const tchar* searchPath);
		const tchar* GetTempPath();

		virtual bool SaveCasTable(bool deleteIsRunningfile, bool deleteDropped = true) override final;
		virtual u64 GetStorageCapacity() override final;
		virtual u64 GetStorageUsed() override final;
		virtual bool GetZone(StringBufferBase& out) override;
		virtual bool Reset() override final;
		bool DeleteAllCas();
		bool IsCasTableLoaded() const { return m_casTableLoaded; }

		virtual bool StoreCompressed() const final { return m_storeCompressed; }
		virtual void PrintSummary(Logger& logger) override;

		virtual bool DecompressFileToMemory(const tchar* fileName, FileHandle fileHandle, u8* dest, u64 decompressedSize, const tchar* writeHint, u64 fileStartOffset) override final;
		virtual bool CreateDirectory(const tchar* dir, bool* outAlreadyExists = nullptr) override final;
		virtual bool DeleteCasForFile(const tchar* file) override final;
		virtual bool GetFileSize(u64& outSize, const CasKey& casKey, const tchar* hint) override final;
		virtual bool GetFileData(void* outData, const CasKey& casKey, const tchar* hint) override final;
		virtual bool RetrieveCasFile(RetrieveResult& out, const CasKey& casKey, StringKey keyHint, const tchar* hint, FileMappingBuffer* mappingBuffer = nullptr, bool createIndependentMapping = false, u64 memoryMapAlignment = 1, bool allowProxy = true, u32 clientId = 0) override;
		virtual bool AddRef(const CasKey& casKey) override final;
		virtual bool ReleaseRef(const CasKey& casKey, bool dropIfZero) override final;
		virtual bool VerifyAndGetCachedFileInfo(CachedFileInfo& out, StringKey fileNameKey, u64 verifiedLastWriteTime, u64 verifiedSize) override final;
		virtual bool InvalidateCachedFileInfo(StringKey fileNameKey) override final;
		virtual bool StoreCasFile(CasKey& out, const tchar* fileName, const CasKey& casKeyOverride, bool deferCreation) override;
		virtual bool StoreCasFileClient(CasKey& out, StringKey fileNameKey, const tchar* fileName, SharedMemoryHandle memoryHandle, u64 memoryOffset, u64 fileSize, const tchar* hint, bool keepMappingInMemory = false, bool storeCompressed = true) override;
		virtual bool DropCasFile(const CasKey& casKey, bool forceDelete, const tchar* hint, bool reduceRefCount = false) override final;
		virtual bool ReportBadCasFile(const CasKey& casKey) override final;
		virtual bool CalculateCasKey(CasKey& out, StringKey fileNameKey, const tchar* fileName) override final;
		virtual bool CopyOrLink(const CasKey& casKey, const tchar* destination, u32 fileAttributes, bool writeCompressed = false, u32 clientId = 0, const FormattingFunc& formattingFunc = {}, bool isTemp = false, bool allowHardLink = true) override;
		virtual bool WritePlaceholder(const CasKey& casKey, const tchar* destination, u32 fileAttributes, u32 clientId) override;
		virtual bool FakeCopy(const CasKey& casKey, const tchar* destination, u64 size, u64 lastWritten, u32 fileAttributes, bool deleteExisting = true) override final;
		virtual bool GetCasFileName(StringBufferBase& out, const CasKey& casKey) override;
		virtual MappedView2 MapView(const CasKey& casKey, const tchar* hint) override;
		virtual void UnmapView(const MappedView& view, const tchar* hint) override final;
		virtual SharedMemoryAllocator& GetSharedMemory() override final;

		virtual void ReportFileWrite(StringKey fileNameKey, const tchar* fileName) override final;
		
		virtual StorageStats& Stats() final;
		virtual void AddStats(StorageStats& stats) override final;
		virtual void TraceUpdate(Trace& trace, u32 startRow) override final;

		struct CasEntry;
		struct CasEntryWriteLock;
		struct DeferredSource;

		virtual bool StoreCasFile(CasKey& out, const tchar* fileName, const CasKey& casKeyOverride, bool deferCreation, CasEntry** outEntry);
		virtual bool HasCasFile(const CasKey& casKey, CasEntry** out, bool useMapping, bool allowFail);

		bool EnsureCasFile(const CasKey& casKey, const tchar* fileName);
		CasKey CalculateCasKey(const tchar* fileName, FileHandle fileHandle, u64 fileSize, bool storeCompressed);
		CasKey CalculateCasKey(const u8* fileMem, u64 fileSize, bool storeCompressed);
		bool StoreCasKey(CasKey& out, const tchar* fileName, const CasKey& casKeyOverride);
		bool StoreCasKey(CasKey& out, const StringKey& fileNameKey, const tchar* fileName, const CasKey& casKeyOverride, bool showErrors = true);
		bool IsFileVerified(const StringKey& fileNameKey);
		void ReportFileInfoWeak(const StringKey& fileNameKey, u64 verifiedLastWriteTime, u64 verifiedSize);
		bool HasBeenSeen(const CasKey& casKey);
		void ReleaseCasEntry(CasEntry& casEntry, bool dropIfZero);
		void ReleaseCasEntryNoLock(CasEntry& casEntry, bool dropIfZero);

		struct WriteCasResult : WriteResult
		{
			bool dropAfterUse = false;
		};

		virtual void RegisterExternalFileMappingsProvider(ExternalFileMappingsProvider&& provider) override final;
		virtual void DeleteExternalFileMapping(StringKey fileNameKey, const tchar* fileName) override final;
		virtual bool WriteCompressedFile(WriteResult& out, const tchar* from, FileHandle readHandle, u8* readMem, u64 fileSize, const tchar* toFile, const void* header, u64 headerSize, u64 lastWriteTime = 0) final;
		bool WriteCasFileNoCheck(WriteCasResult& out, const StringKey& fileNameKey, const tchar* fileName, const CasKey& casKey, const tchar* casFile, bool storeCompressed, bool useMapping, DeferredSource* deferredSource, bool allowFail);
		bool WriteCasFile(WriteCasResult& out, const tchar* fileName, const CasKey& casKey, bool useMapping, DeferredSource* deferredSouirce, bool allowFail);
		bool VerifyExisting(bool& outReturnValue, CasEntryWriteLock& casEntryLock, const CasKey& casKey, CasEntry& casEntry, StringBufferBase& casFile);
		bool AddCasFile(StringKey fileNameKey, const tchar* fileName, const CasKey& casKey, bool deferCreation, u64 fileSize, u64 lastWriteTime, CasEntry** outEntry, bool dropCasAfterUse);
		void CasEntryAccessed(const CasKey& casKey);
		virtual bool IsDisallowedPath(const tchar* fileName);
		virtual bool DecompressMemoryToMemory(const u8* compressedData, u64 compressedSize, u8* writeData, u64 decompressedSize, const tchar* readHint, const tchar* writeHint) override final;

		void CasEntryAccessed(CasEntry& entry);
		void CasEntryWritten(CasEntry& entry, u64 size);
		void CasEntryDeleted(CasEntry& entry, u64 size);
		void AttachEntry(CasEntry& entry);
		void DetachEntry(CasEntry& entry);
		void TraverseAllCasFiles(const tchar* dir, const Function<void(const StringBufferBase& fullPath, const DirectoryEntry& e)>& func, bool allowParallel = false);
		void TraverseAllCasFiles(const Function<void(const CasKey& key, u64 size)>& func, bool allowParallel = false);
		bool CheckAllCasFiles(u64 checkContentOfFilesNewerThanTime = ~u64(0));
		void HandleOverflow(UnorderedSet<CasKey>* outDeletedFiles);
		void DeleteOldestCasEntries();
		
		static MutexHandle GetExclusiveAccess(Logger& logger, const StringView& rootDir, bool reportError);
		bool DeleteIsRunningFile();
		bool TouchIsRunningFile();

		struct FileEntry;
		FileEntry& GetOrCreateFileEntry(const StringKey& fileNameKey);
		CasEntry& GetOrCreateCasEntry(const CasKey& casKey);
		bool GetFileEntry(FileEntry*& out, const tchar* file);

		MutableLogger m_logger;
		WorkManager& m_workManager;
		SharedMemoryAllocator m_sharedMemory;
		BufferCache m_bufferSlots;

		StringBuffer<> m_rootDir;
		StringBuffer<> m_tempPath;

		ReaderWriterLock m_fileTableLookupLock;
		struct FileEntry
		{
			Futex lock;
			CasKey casKey;
			u64 size = 0;
			u64 lastWritten = 0;
			bool verified = false;
			bool isTemp = false;

			#if UBA_TRACK_IS_EXECUTABLE
			bool isExecutable = false;
			#endif
		};
		UnorderedMap<StringKey, FileEntry> m_fileTableLookup;

		ReaderWriterLock m_casLookupLock;

		struct CasEntryReadLock
		{
			CasEntryReadLock(CasEntry& e);
			~CasEntryReadLock();
			CasEntry& entry;
			bool active;
		};

		struct CasEntryWriteLock
		{
			CasEntryWriteLock(CasEntry& e);
			~CasEntryWriteLock();
			void Enter();
			void Leave();
			CasEntry& entry;
			bool active;
		};

		struct CasEntry
		{
			CasEntry(const CasKey& k) : key(k) {}
			ReaderWriterLock internalLock;
			CasKey key;
			CasEntry* prevAccessed = nullptr;
			CasEntry* nextAccessed = nullptr;
			u64 size = 0; // Stored size of cas entry. Can be both compressed or uncompressed
			Atomic<u16> refCount = 0; // This is set while file is being read or queued to be written over network
			bool verified = false; // This flag needs to be set for below flags to be reliable. if this is false below flags are assumptions
			bool exists = false; // File exists on disk
			bool dropped = false; // This file is not seen anymore. will be deleted when refCount reaches zero or during shutdown
			bool beingWritten = false; // This is set while file is being written (when coming from network)..
			bool disallowed = false; // This is set if cas is created from disallowed file

			SharedMemoryHandle memoryHandle;
			u64 memoryOffset = 0;
			u64 memorySize = 0;
		};

		UnorderedMap<CasKey, CasEntry> m_casLookup;
		UnorderedSet<CasKey>* m_trackedDeletes = nullptr;
		Futex m_accessLock;
		CasEntry* m_newestAccessed = nullptr;
		CasEntry* m_oldestAccessed = nullptr;
		u64 m_casTotalBytes = 0;
		u64 m_casQueuedDeleteBytes = 0;
		u64 m_casMaxBytes = 0;
		u64 m_casCapacityBytes = 0;
		u64 m_casEvictedBytes = 0;
		u32 m_casEvictedCount = 0;
		u64 m_casDroppedBytes = 0;
		u32 m_casDroppedCount = 0;
		u32 m_fileEntryTableWipeThreshold = 0;
		u32 m_deleteCasBatchSize;
		bool m_overflowReported = false;
		bool m_storeCompressed = false;
		bool m_manuallyHandleOverflow = false;
		bool m_asyncUnmapViewOfFile = true;
		bool m_allowDeleteVerified = false;
		bool m_allowDeferredDeletes = true;
		bool m_skipCleanupOnShutdown = false;
		bool m_allowUnDropOfCas = true;
		bool m_writeToDisk;

		MutexHandle m_exclusiveMutex = InvalidMutexHandle;

		ExternalFileMappingsProvider m_externalFileMappingsProvider;

		Bottleneck m_maxParallelCopyOrLinkBottleneck;

		Futex m_casTableLoadSaveLock;
		bool m_casTableLoaded = false;
		u32 m_casTableVersion = 0;

		FileMappingBuffer m_casDataBuffer;

		Futex m_deferredCasCreationLookupLock;
		struct DeferredSource { CasKey key; TString fileName; u64 fileSize = InvalidValue; u64 lastWriteTime = 0; bool manualDelete = false; };
		using NameToCas = UnorderedMap<StringKey, DeferredSource>;
		struct DeferredCasCreation { Futex lock; List<StringKey> sources; u32 refCount = 0; };
		UnorderedMap<CasKey, DeferredCasCreation> m_deferredCasCreationLookup;
		NameToCas m_deferredCasCreationLookupByName;

		DirectoryCache m_dirCache;

		u8 m_casCompressor;
		u8 m_casCompressionLevel;

		StorageStats m_stats;

		StorageImpl(const StorageImpl&) = delete;
		void operator=(const StorageImpl&) = delete;
	};
}
