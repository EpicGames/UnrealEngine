// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaNetworkClient.h"
#include "UbaStorage.h"

namespace uba
{
	class NetworkClient;

	using StartProxyCallback = bool(void* userData, u16 port, const Guid& storageServerUid);
	using GetProxyBackendCallback = NetworkBackend&(void* userData, const tchar* proxyHost);

	struct StorageClientCreateInfo : StorageCreateInfo
	{
		StorageClientCreateInfo(NetworkClient& c, const tchar* rootDir_) : StorageCreateInfo(rootDir_, c.GetLogWriter(), c), client(c) {}

		void Apply(const Config& config);

		NetworkClient& client;
		const tchar* zone = TC("");
		u16 proxyPort = DefaultStorageProxyPort;
		const tchar* proxyAddress = TC("");
		bool sendCompressed = true;
		bool allowProxy = true;
		bool sendOneBigFileAtTheTime = true;
		bool fileParallelSend = false;
		bool checkExistsOnServer = false; // There is no point having this enabled if server is not writing received cas files to disk (which is default)
		bool resendCas = false; // Will try to send created file everytime even if the cas has been seen before
		
		u32 proxyConnectionCount = 4;
		GetProxyBackendCallback* getProxyBackendCallback = nullptr;
		void* getProxyBackendUserData = nullptr;
		StartProxyCallback* startProxyCallback = nullptr;
		void* startProxyUserData = nullptr;
	};


	class StorageClient final : public StorageImpl
	{
	public:
		StorageClient(const StorageClientCreateInfo& info);
		~StorageClient();

		bool Start();

		bool IsUsingProxy();
		void StopProxy();

		using DirVector = Vector<TString>;
		bool PopulateCasFromDirs(const DirVector& directories, u32 workerCount, const Function<bool()>& shouldExit = {});

		virtual bool GetCasFileName(StringBufferBase& out, const CasKey& casKey) override;

		virtual MappedView2 MapView(const CasKey& casKey, const tchar* hint) override;

		virtual bool GetZone(StringBufferBase& out) override;
		virtual bool RetrieveCasFile(RetrieveResult& out, const CasKey& casKey, StringKey keyHint, const tchar* hint, FileMappingBuffer* mappingBuffer = nullptr, bool createIndependentMapping = false, u64 memoryMapAlignment = 1, bool allowProxy = true, u32 clientId = 0) override;
		virtual bool StoreCasFile(CasKey& out, const tchar* fileName, const CasKey& casKeyOverride, bool deferCreation) override;
		virtual bool StoreCasFileClient(CasKey& out, StringKey fileNameKey, const tchar* fileName, SharedMemoryHandle mappingFile, u64 mappingOffset, u64 fileSize, const tchar* hint, bool keepMappingInMemory = false, bool storeCompressed = true) override;
		virtual bool HasCasFile(const CasKey& casKey, CasEntry** out, bool useMapping, bool allowFail) override;
		virtual void Ping() override;
		virtual void PrintSummary(Logger& logger) override;

	private:
		bool PopulateCasFromDirsRecursive(const tchar* dir, WorkManager& workManager, UnorderedSet<u64>& seenIds, Futex& seenIdsLock, const Function<bool()>& shouldExit);

		NetworkClient& m_client;
		bool m_sendCompressed;
		bool m_allowProxy;
		bool m_sendOneBigFileAtTheTime;
		bool m_fileParallelSend;
		bool m_checkExistsOnServer;
		bool m_resendCas;

		Guid m_storageServerUid;

		TString m_zone;

		ReaderWriterLock m_localStorageFilesLock;
		struct LocalFile { CasEntry casEntry; TString fileName; Event hasBeenSent; LocalFile() : casEntry(CasKeyZero) {} };
		UnorderedMap<CasKey, LocalFile> m_localStorageFiles;

		Futex m_sendOneAtTheTimeLock;
		Futex m_retrieveOneBatchAtTheTimeLock;

		static constexpr u8 ServiceId = StorageServiceId;

		struct ProxyClient;
		Futex m_proxyClientLock;
		ProxyClient* m_proxyClient = nullptr;
		u64 m_proxyClientKeepAliveTime = 0;

		GetProxyBackendCallback* m_getProxyBackendCallback = nullptr;
		void* m_getProxyBackendUserData = nullptr;
		StartProxyCallback* m_startProxyCallback = nullptr;
		void* m_startProxyUserData = nullptr;
		u32 m_proxyConnectionCount;
		u16 m_proxyPort = 0;
		TString m_proxyAddress;
	};

}
