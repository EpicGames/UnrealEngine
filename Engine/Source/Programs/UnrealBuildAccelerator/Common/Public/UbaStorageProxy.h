// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaHash.h"
#include "UbaLogger.h"
#include "UbaNetwork.h"
#include "UbaUnorderedMap.h"

namespace uba
{
	class Config;
    class NetworkClient;
	class NetworkServer;
	class StorageImpl;
	struct BinaryReader;
	struct BinaryWriter;
	struct ConnectionInfo;
	struct MessageInfo;
	struct WorkContext;

	struct StorageProxyCreateInfo
	{
		NetworkServer& server;
		NetworkClient& client;
		Guid storageServerUid;
		const tchar* name = TC("");
		StorageImpl* localStorage = nullptr;
		bool useLocalStorage = false; // Use local storage to populate proxy

		void Apply(Config& config, const tchar* tableName = TC("StorageProxy"));
	};

	class StorageProxy
	{
	public:
		StorageProxy(const StorageProxyCreateInfo& info);
		~StorageProxy();
		void PrintSummary();

		u32 GetActiveFetchCount();

	protected:
		u16 PopId();
		void PushId(u16 id);
		bool HandleMessage(const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer);
		bool HandleConnect(const ConnectionInfo& connectionInfo, BinaryReader& reader, BinaryWriter& writer);

		struct BeginMessage;
		bool HandleFetchBegin(const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer);
		void HandleFetchBeginReceived(BeginMessage& m, bool error);

		struct SegmentMessage;
		bool HandleFetchSegment(const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer);
		void HandleFetchSegmentReceived(SegmentMessage& mif);

		bool HandleDefault(MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer);

		struct FileEntry;
		bool GetFileFromLocalStorage(u32 clientId, const CasKey& casKey, FileEntry& file, ScopedFutex& fileLock);
		bool SendFetchBeginResponse(u32 clientId, FileEntry& file, BinaryWriter& writer, bool writeData = true);
		bool UpdateFetch(u32 clientId, u16 fetchId, u64 segmentSize);
		bool SendEnd(const CasKey& key);

		static constexpr u8 ServiceId = StorageServiceId;

		NetworkServer& m_server;
		NetworkClient& m_client;
		StorageImpl* m_localStorage;

		MutableLogger m_logger;

		Guid m_storageServerUid;

		TString m_name;

		Atomic<u32> m_inProcessClientId = 0;

		struct FileEntry
		{
			Futex lock;
			u8* memory = nullptr;
			u64 size = 0;
			Atomic<u64> received = 0;
			CasKey casKey;
			u32 trackId = 0;
			u16 fetchId = 0;
			bool storeCompressed = false;
			bool sendEnd = false;
			bool error = false;
			bool disallowed = false;
			bool available = false;
			BeginMessage* beginMessage = nullptr;
			Vector<SegmentMessage*> segmentMessages;
		};

		Futex m_filesLock;
		UnorderedMap<CasKey, FileEntry> m_files;

		struct ActiveFetch
		{
			FileEntry* file = nullptr;
			u64 fetchedSize = 0;
			u32 clientId = ~0u;
			u32 connectionId = 0;
		};

		ReaderWriterLock m_activeFetchesLock;
		UnorderedMap<u16, ActiveFetch> m_activeFetches;

		ReaderWriterLock m_largeFileLock;

		Vector<u16> m_availableIds;
		u16 m_availableIdsHigh = 1;

		bool m_useLocalStorage;
	};
}
