// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaNetworkServer.h"
#include "UbaStorage.h"

#define UBA_TRACK_FETCH_STORE_HINTS 0

namespace uba
{
	struct StorageServerCreateInfo : StorageCreateInfo
	{
		StorageServerCreateInfo(NetworkServer& s, const tchar* rootDir_, LogWriter& writer) : StorageCreateInfo(rootDir_, writer, s), server(s) {}

		void Apply(const Config& config);

		NetworkServer& server;
		bool allowHintAsFallback = true; // Will fallback to file system to recalculate cas if cas content does not exist anymore
		bool writeReceivedCasFilesToDisk = false; // Write received files to disk. This is only useful if we expect same file to be written multiple times (requires deterministic outputs ofc)
		bool createIndependentMappings = true; // Set to true if we want every store to end up in its own file mapping. This will use more memory but makes it possible to delete files
		bool dropIndependentMappingsOnCopy = true; // Will drop independent mappings once they have been copyorlink to destination
		const tchar* zone = TC("");
	};

	class StorageServer final : public StorageImpl
	{
	public:
		StorageServer(const StorageServerCreateInfo& info);
		~StorageServer();

		bool RegisterDisallowedPath(const tchar* path);

		// Get the network server used by this session
		NetworkServer& GetServer() { return m_server; }

		using StorageImpl::StoreCasFile;

		void WaitForActiveWork();
		bool WaitForCasFile(const CasKey& casKey, const tchar* hint, u32 clientId = 0);

		virtual bool CopyOrLink(const CasKey& casKey, const tchar* destination, u32 fileAttributes, bool writeCompressed = false, u32 clientId = 0, const FormattingFunc& formattingFunc = {}, bool isTemp = false, bool allowHardLink = true) override;
		virtual bool WritePlaceholder(const CasKey& casKey, const tchar* destination, u32 fileAttributes, u32 clientId) override;
		virtual bool SkipWrite(const CasKey& casKey, u32 clientId) override;

	protected:

		virtual bool GetZone(StringBufferBase& out) override;
		virtual bool RetrieveCasFile(RetrieveResult& out, const CasKey& casKey, StringKey keyHint, const tchar* hint, FileMappingBuffer* mappingBuffer = nullptr, bool createIndependentMapping = false, u64 memoryMapAlignment = 1, bool allowProxy = true, u32 clientId = 0) override;
		virtual bool IsDisallowedPath(const tchar* fileName) override;
		virtual void SetTrace(Trace* trace, bool detailed) override;
		virtual bool HasProxy(u32 clientId) override;

		void OnDisconnected(u32 clientId);
		void ReleaseActiveRefCount(const CasKey& casKey, u32 clientId);
		void UpdateWaitEntries(const CasKey& casKey, bool success);
		bool HandleMessage(const ConnectionInfo& connectionInfo, const WorkContext& workContext, u8 messageType, BinaryReader& reader, BinaryWriter& writer);
		bool WaitForWritten(CasEntry& casEntry, CasEntryWriteLock& casEntryLock, const ConnectionInfo& connectionInfo, const tchar* hint);
		bool VerifyExists(bool& outExists, CasEntry& casEntry, CasEntryWriteLock& casEntryLock, const CasKey& casKey);
		bool GetProxy(BinaryWriter& writer, u32 clientId, bool writeCasHeader);

		u16 PopId();
		void PushId(u16 id);

		struct Info;
		void GetProxyName(StringBufferBase& out, const Info& info);

		NetworkServer& m_server;
		bool m_traceFetch = false;
		bool m_traceStore = false;

		Guid m_uid;

		static constexpr u8 ServiceId = StorageServiceId;


		Futex m_waitEntriesLock;

		struct WaitEntry
		{
			WaitEntry() : done(EventResetType_Manual) {}
			Event done;
			bool success = false;
			u32 refCount = 0;
		};
		UnorderedMap<CasKey, WaitEntry> m_waitEntries;


		struct ActiveStore
		{
			u32 clientId = ~0u;
			MappedView mappedView;
			FileAccessor* fileAccessor = nullptr;
			CasEntry* casEntry = nullptr;
			Atomic<u64> totalWritten = 0;
			Atomic<u64> recvCasTime = 0;
			u64 fileSize = 0;
			u64 actualSize = 0;

			#if UBA_TRACK_FETCH_STORE_HINTS
			TString hint;
			#endif
		};

		ReaderWriterLock m_activeStoresLock;
		UnorderedMap<u16, ActiveStore> m_activeStores;


		struct ActiveFetch
		{
			u32 clientId = ~0u;
			Atomic<u64> left = 0;
			CasEntry* casEntry;
			Atomic<u64> sendCasTime = 0;

			FileHandle readFileHandle = InvalidFileHandle;
			FileMappingHandle fileMappingHandle;
			MappedView mappedView;
			const u8* memoryBegin = nullptr;
			const u8* memoryPos = nullptr;
			bool ownsMapping = false;

			#if UBA_TRACK_FETCH_STORE_HINTS
			TString hint;
			#endif

			void Release(StorageServer& server, const tchar* reason);
		};
		ReaderWriterLock m_activeFetchesLock;
		UnorderedMap<u16, ActiveFetch> m_activeFetches;
		Atomic<u32> m_activeUnmap = 0;

		ReaderWriterLock m_activeRefCountsLock;
		UnorderedMap<u32, List<CasEntry*>> m_activeRefCounts;

		Futex m_availableIdsLock;
		Vector<u16> m_availableIds;
		u16 m_availableIdsHigh = 1;

		struct ProxyEntry
		{
			u32 clientId = ~0u;
			TString zone;
			TString host;
			u16 port = 0;
			UnorderedSet<u32> failedClients;
		};
		ReaderWriterLock m_proxiesLock;
		UnorderedMap<StringKey, ProxyEntry> m_proxies;

		TString m_zone;

		struct Info
		{
			TString zone;
			TString internalAddress;
			u64 storageSize = 0;
			u16 proxyPort = 0;
		};
		ReaderWriterLock m_connectionInfoLock;
		UnorderedMap<u32, Info> m_connectionInfo;

		Trace* m_trace = nullptr;

		Vector<TString> m_disallowedPaths;

		bool m_allowHintAsFallback;
		bool m_writeReceivedCasFilesToDisk;
		bool m_createIndependentMappings;
		bool m_dropIndependentMappingsOnCopy;
	};
}
