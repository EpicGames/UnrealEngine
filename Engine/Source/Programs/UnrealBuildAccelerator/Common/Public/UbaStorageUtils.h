// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaFile.h"
#include "UbaHash.h"
#include "UbaSharedMemoryView.h"
#include <oodle2.h>

namespace uba
{
	class FileAccessor;
	class NetworkClient;
	class SharedMemoryAllocator;
	class WorkManager;
	struct BufferCache;
	struct StorageStats;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	static constexpr u8 DefaultCompressor = OodleLZ_Compressor_Kraken;
	static constexpr u8 DefaultCompressionLevel = OodleLZ_CompressionLevel_SuperFast;

	u8 GetCompressor(const tchar* str);
	u8 GetCompressionLevel(const tchar* str);

	CasKey CalculateCasKey(const u8* fileMem, u64 fileSize, bool storeCompressed, WorkManager* workManager, const tchar* hint);
	bool SendBatchMessages(Logger& logger, NetworkClient& client, u16 fetchId, u8* slot, u64 capacity, u64 left, u32 messageMaxSize, u32& readIndex, u32& responseSize, const Function<bool()>& runInWaitFunc = {}, const tchar* hint = TC(""), u32* outError = nullptr, const Function<bool(u64)>& progressFunc = {});
	bool SendFile(Logger& logger, NetworkClient& client, WorkManager* workManager, const CasKey& casKey, const u8* sourceMem, u64 sourceSize, const tchar* hint);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct FileSender
	{
		bool SendFileCompressed(const CasKey& casKey, const tchar* fileName, const u8* sourceMem, u64 sourceSize, const tchar* hint);

		Logger& m_logger;
		NetworkClient& m_client;
		BufferCache& m_bufferSlots;
		StorageStats& m_stats;
		Futex& m_sendOneAtTheTimeLock;
		u8 m_casCompressor = DefaultCompressor;
		u8 m_casCompressionLevel = DefaultCompressionLevel;
		bool m_sendOneBigFileAtTheTime = true;
		bool m_parallelSend = false;
		u64 m_bytesSent = 0;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct FileFetcher
	{
		bool RetrieveFile(Logger& logger, NetworkClient& client, const CasKey& casKey, const tchar* destination, bool writeCompressed, MemoryBlock* destinationMem = nullptr, u32 attributes = DefaultAttributes());

		BufferCache& m_bufferSlots;
		StorageStats& m_stats;
		StringBuffer<> m_tempPath;
		bool m_errorOnFail = true;

		u64 lastWritten = 0;
		u64 sizeOnDisk = 0;
		u64 bytesReceived = 0;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct LinearWriter
	{
		virtual bool Write(const void* data, u64 dataLen) = 0;
		virtual u64 GetWritten() = 0;
		virtual const tchar* GetHint() = 0;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct CompressWriter
	{
		CompressWriter(Logger& l, SharedMemoryAllocator& a, BufferCache& bs, WorkManager& wm, StorageStats& s, u8 cc = DefaultCompressor, u8 ccl = DefaultCompressionLevel, bool avof = false);

		bool CompressToFile(u64& outCompressedSize, const tchar* from, FileHandle readHandle, u8* readMem, u64 fileSize, const tchar* toFile, const void* header, u64 headerSize, u64 lastWriteTime, const tchar* tempPath);
		bool CompressToMapping(SharedMemoryHandle& outMappingHandle, u64& outMappingSize, u8* readMem, u64 fileSize, const tchar* hint);
		bool CompressToMappingFromFile(SharedMemoryHandle& outMappingHandle, u64& outMappingSize, FileHandle readHandle, u64 fileSize, const tchar* hint);
		bool CompressFromMem(LinearWriter& destination, u32 workCount, const u8* uncompressedData, u64 fileSize, u64 maxUncompressedBlock, u64& totalWritten);
		bool CompressFromMemOrFile(LinearWriter& destination, const tchar* from, FileHandle readHandle, u8* readMem, u64 fileSize);

		Logger& m_logger;
		SharedMemoryAllocator& m_allocator;
		BufferCache& m_bufferSlots;
		WorkManager& m_workManager;
		StorageStats& m_stats;
		u8 m_casCompressor;
		u8 m_casCompressionLevel;
		bool m_asyncUnmapViewOfFile;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct LinearWriterFile final : public LinearWriter
	{
		LinearWriterFile(FileAccessor& f);
		virtual bool Write(const void* data, u64 dataLen) override;
		virtual u64 GetWritten() override;
		virtual const tchar* GetHint() override;
		FileAccessor& file;
		u64 written = 0;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct LinearWriterMem final : public LinearWriter
	{
		LinearWriterMem(u8* d, u64 c);
		virtual bool Write(const void* data, u64 dataLen) override;
		virtual u64 GetWritten() override;
		virtual const tchar* GetHint() override;
		u8* pos;
		u64 size;
		u64 capacity;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct DecompressWriter
	{
		bool DecompressMemoryToMemory(const u8* compressedData, u64 compressedSize, u8* writeData, u64 decompressedSize, const tchar* readHint, const tchar* writeHint);
		bool DecompressMemoryToFile(const u8* compressedData, FileAccessor& destination, u64 decompressedSize, bool useNoBuffering);
		bool DecompressFileToMemory(const tchar* fileName, FileHandle fileHandle, u8* dest, u64 decompressedSize, const tchar* writeHint, u64 fileStartOffset);

		Logger& m_logger;
		BufferCache& m_bufferSlots;
		WorkManager& m_workManager;
		StorageStats& m_stats;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////
}