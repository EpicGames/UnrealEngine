// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBinaryReaderWriter.h"
#include "UbaEvent.h"
#include "UbaFunctional.h"

#define UBA_TRACK_THREADS 0

namespace uba
{
	class Event;
	using TraverseThreadFunc = Function<void(u32 tid, void** callstack, u32 callstackCount, const tchar* description)>;
	using TraverseThreadErrorFunc = Function<void(const StringView& error)>;

	class Thread
	{
	public:
		Thread();
		Thread(Function<u32()>&& func, const tchar* description = nullptr);
		~Thread();
		void Start(Function<u32()>&& func, const tchar* description = nullptr);
		bool Wait(u32 milliseconds = ~0u, Event* wakeupEvent = nullptr);
		bool IsInside() const;
		void* GetNativeHandle() const { return m_handle; }

	private:
		Function<u32()>	m_func;
		void* m_handle = nullptr;
		ReaderWriterLock m_funcLock;

		#if !PLATFORM_WINDOWS
		Event m_finished;
		#endif

		#if UBA_TRACK_THREADS
		Thread* m_next = nullptr;
		Thread* m_prev = nullptr;
		#endif

		Thread(const Thread&) = delete;
		void operator=(const Thread&) = delete;

		friend bool TraverseAllThreads(const TraverseThreadFunc& func, const TraverseThreadErrorFunc& errorFunc);
	};

	bool AlternateThreadGroupAffinity(void* nativeThreadHandle);

	// Distribute a process across processor groups via CPU sets + ideal-processor hint. Returns false
	// pre-Win10-1607 / when CpuSet APIs are unavailable; caller falls back to GROUP_AFFINITY.
	bool DistributeProcessSoft(void* nativeProcessHandle, void* nativeMainThreadHandle);

	bool TraverseAllThreads(const TraverseThreadFunc& func, const TraverseThreadErrorFunc& errorFunc);
}
