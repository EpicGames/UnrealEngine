// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaProcessStats.h"
#include "UbaThread.h"
#include "UbaTraceChannel.h"
#include "UbaUnorderedMap.h"
#include "UbaWorkManager.h"

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct ProcessLogLine;
	struct ProcessStats;

	#define UBA_TRACE_TYPES \
		UBA_TRACE_TYPE(SessionAdded) \
		UBA_TRACE_TYPE(SessionUpdate) \
		UBA_TRACE_TYPE(ProcessAdded) \
		UBA_TRACE_TYPE(ProcessExited) \
		UBA_TRACE_TYPE(ProcessReturned) \
		UBA_TRACE_TYPE(FileFetchBegin) \
		UBA_TRACE_TYPE(FileFetchEnd) \
		UBA_TRACE_TYPE(FileStoreBegin) \
		UBA_TRACE_TYPE(FileStoreEnd) \
		UBA_TRACE_TYPE(Summary) \
		UBA_TRACE_TYPE(WorkBegin) \
		UBA_TRACE_TYPE(WorkEnd) \
		UBA_TRACE_TYPE(String) \
		UBA_TRACE_TYPE(SessionSummary) \
		UBA_TRACE_TYPE(ProcessEnvironmentUpdated) \
		UBA_TRACE_TYPE(SessionDisconnect) \
		UBA_TRACE_TYPE(ProxyCreated) \
		UBA_TRACE_TYPE(ProxyUsed) \
		UBA_TRACE_TYPE(FileFetchLight) \
		UBA_TRACE_TYPE(FileStoreLight) \
		UBA_TRACE_TYPE(StatusUpdate) \
		UBA_TRACE_TYPE(SessionNotification) \
		UBA_TRACE_TYPE(CacheBeginFetch) \
		UBA_TRACE_TYPE(CacheEndFetch) \
		UBA_TRACE_TYPE(CacheBeginWrite) \
		UBA_TRACE_TYPE(CacheEndWrite) \
		UBA_TRACE_TYPE(ProgressUpdate) \
		UBA_TRACE_TYPE(RemoteExecutionDisabled) \
		UBA_TRACE_TYPE(FileFetchSize) \
		UBA_TRACE_TYPE(ProcessBreadcrumbs) \
		UBA_TRACE_TYPE(WorkHint) \
		UBA_TRACE_TYPE(DriveUpdate) \
		UBA_TRACE_TYPE(CacheSummary) \
		UBA_TRACE_TYPE(TaskBegin) \
		UBA_TRACE_TYPE(TaskHint) \
		UBA_TRACE_TYPE(TaskEnd) \
		UBA_TRACE_TYPE(SchedulerUpdate) \
		UBA_TRACE_TYPE(SchedulerKillProcess) \
		UBA_TRACE_TYPE(SessionInfo) \
		UBA_TRACE_TYPE(StringAdd) \
		UBA_TRACE_TYPE(StatusUpdate2) \
		UBA_TRACE_TYPE(TableUpdate) \
		UBA_TRACE_TYPE(ProcessRace) \

	enum TraceType : u8
	{
		#define UBA_TRACE_TYPE(name) TraceType_##name,
		UBA_TRACE_TYPES
		#undef UBA_TRACE_TYPE
	};

	static constexpr u32 TraceVersion = 53;
	static constexpr u32 TraceReadCompatibilityVersion = 6;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class Trace final : public WorkTracker
	{
	public:
		Trace(LogWriter& logWriter);
		~Trace();

		bool StartWriteAndThread(const tchar* namedTrace, u64 traceMemCapacity, bool logStart, const tchar* channelName = TC("Default"));

		void StartThread();
		void StopThread();

		void SetThreadUpdateCallback(Function<void()>&& callback, u32 traceIntervalMs = 500);


		bool StartWrite(const tchar* namedTrace, u64 traceMemCapacity, const tchar* channelName = TC("Default"));

		// Writes the current state of the trace to the specified output file without stopping anything
		bool Write(const tchar* writeFileName, bool writeSummary);

		// Stops the trace and writes it to the specified output file. This releases the internal memory buffer.
		bool StopWrite(const tchar* writeFileName);


		const TString& GetNamedTrace() { return m_namedTrace; }
		bool IsWriting() const { return m_memoryBegin != nullptr; }
		TraceChannel& GetChannel() { return m_channel; }

		void SessionAdded(u32 sessionId, u32 clientId, StringView name, StringView machineId);
		void SessionInfo(u32 sessionId, StringView info);
		void SessionUpdate(u32 sessionId, u32 connectionCount, u64 send, u64 recv, u64 lastPing, u64 memAvail, u64 memTotal, float cpuLoad);
		void SessionNotification(u32 sessionId, const tchar* text);
		void SessionSummary(u32 sessionId, const u8* data, u64 dataSize);
		void SessionDisconnect(u32 sessionId);
		void ProcessAdded(u32 sessionId, u32 processId, StringView description, StringView breadcrumbs);
		void ProcessEnvironmentUpdated(u32 processId, StringView reason, u32 prevExitCode, const u8* prevStats, u64 prevStatsSize, StringView breadcrumbs);
		void ProcessExited(u32 processId, u32 exitCode, const u8* data, u64 dataSize, const Vector<ProcessLogLine>& logLines);
		void ProcessExited(u32 processId, u32 exitCode);
		void ProcessReturned(u32 processId, StringView reason);
		void ProcessAddBreadcrumbs(u32 processId, StringView breadcrumbs, bool deleteOld);
		void ProcessRace(u32 processId, u32 remoteProcess);
		void ProxyCreated(u32 clientId, const tchar* proxyName);
		void ProxyUsed(u32 clientId, const tchar* proxyName);
		void FileFetchLight(u32 clientId, const CasKey& key, u64 fileSize);
		void FileFetchBegin(u32 clientId, const CasKey& key, StringView hint);
		void FileFetchSize(u32 clientId, const CasKey& key, u64 size);
		void FileFetchEnd(u32 clientId, const CasKey& key);
		void FileStoreBegin(u32 clientId, const CasKey& key, u64 size, StringView hint, bool detailed);
		void FileStoreEnd(u32 clientId, const CasKey& key);
		void WorkBegin(u32 workIndex, StringView desc, const Color& color);
		void WorkHint(u32 workIndex, StringView hint, u64 startTime);
		void WorkEnd(u32 workIndex);
		void ProgressUpdate(u32 processesTotal, u32 processesDone, u32 errorCount);
		void StatusUpdate(u32 statusRow, u32 statusColumn, StringView statusText, LogEntryType statusType, StringView statusLink = {});
		u32  StringAdd(StringView string);
		void StatusUpdate2(u32 statusRow, u32 statusColumn, u32 stringId);
		void DriveUpdate(const tchar drive, u8 busyPercent, u32 readCount, u64 readBytes, u32 writeCount, u64 writeBytes);
		void RemoteExecutionDisabled();
		void SchedulerUpdate(bool spawnPaused);
		void SchedulerKillProcess(u32 processId, StringView reason);

		void CacheBeginFetch(u32 fetchId, const tchar* description);
		void CacheEndFetch(u32 fetchId, bool success, const u8* data, u64 dataSize);
		void CacheBeginWrite(u32 processId);
		void CacheEndWrite(u32 processId, bool success, const u8* data, u64 dataSize);
		void CacheSummary(const u8* data, u64 dataSize);

		void TaskBegin(u32 processId, StringView description, StringView details, const Color& color);
		void TaskHint(u32 processId, StringView hint);
		void TaskEnd(u32 processId, bool success);

		void TableUpdate(u32 name, u32 rowCharWidth, u32 rowMaxCount, const u32* items, u32 itemCount);

		virtual u32 TrackWorkStart(const StringView& desc, const Color& color) final override;
		virtual void TrackWorkHint(u32 id, const StringView& hint, u64 startTime = 0) final override;
		virtual void TrackWorkEnd(u32 id) final override;

		void WriteSessionSummary(const Function<void(Logger& logger)>& summaryFunc);
		static void WriteSummaryText(BinaryWriter& writer, const Function<void(Logger& logger)>& summaryFunc);

		float UpdateCpuLoad();

		// Backstop publish hook. Called from SessionServer::TraceSessionUpdate so the size
		// header keeps advancing under steady write pressure even when no single writer
		// happens to be the last-out.
		void PublishIfQuiet();

		// Snapshot of the published size header. Drives PublishIfQuiet first so callers
		// streaming the trace to remote consumers see a fresh value.
		u32 GetPublishedSize();

	private:
		struct TraceWriter;
		void ThreadLoop();
		void FreeMemory();
		bool CommitTo(u64 endPosition); // Slow path. Caller must hold m_memoryLock.
		void PublishHeaderTo(u64 desired);
		u32 AddString(StringView string);

		LoggerWithWriter m_logger;
		TString m_namedTrace;
		TraceChannel m_channel;
		Futex m_memoryLock; // Only held during the page-commit slow path.
		FileMappingHandle m_memoryHandle;
		u8* m_memoryBegin = nullptr;
		Atomic<u64> m_memoryReserved = 0;
		Atomic<u64> m_memoryCommitted = 0;
		Atomic<u64> m_firstAbortPos = ~u64(0);
		Atomic<u32> m_inflightWriters = 0;
		u64 m_memoryCapacity = 0;
		u64 m_startTime = ~u64(0);

		Futex m_stringsLock;
		UnorderedMap<StringKey, u32> m_strings;

		Atomic<u32> m_workCounter = 0;

		Futex m_threadCallbackLock;
		Function<void()> m_threadCallback;
		u32 m_threadIntervalMs = 500;
		Event m_threadEvent;
		Thread m_thread;

		u64 m_previousTotalCpuTime = 0;
		u64 m_previousIdleCpuTime = 0;
		float m_cpuLoad = 0;

		friend class NetworkServer;
		friend class SessionServer;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct OwnerInfo
	{
		const tchar* id;
		u32 pid;
	};

	const OwnerInfo& GetOwnerInfo();

	////////////////////////////////////////////////////////////////////////////////////////////////////
}