// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaEnvironment.h"
#include "UbaFileMappingHandle.h"
#include "UbaLogger.h"

#define UBA_USE_TRACE_CHANNEL_FILE !PLATFORM_WINDOWS

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	class TraceChannel
	{
	public:
		TraceChannel(Logger& logger);
		~TraceChannel();

		bool Init(const tchar* channelName = TC("Default"));
		void Deinit();
		bool Write(const tchar* traceName, const tchar* ifMatching = nullptr);
		bool Read(StringBufferBase& outTraceName);
		bool IsInitialized();

		// Register that the current process is actively listening on this channel.
		// The registration is held for the lifetime of the TraceChannel and is
		// automatically released by the OS if the process exits.
		bool StartListening();

		// True if some process has called StartListening on this channel and is still alive.
		bool IsListenerActive();

		Logger& m_logger;

		#if UBA_USE_TRACE_CHANNEL_FILE
		TString m_channelFile;
		TString m_listenerFile;
		struct timespec m_lastModified;
		#else
		TString m_channelName;
		MutexHandle m_mutex = InvalidMutexHandle;
		MutexHandle m_listenerMutex = InvalidMutexHandle;
		FileMappingHandle m_memHandle;
		void* m_mem = nullptr;
		#endif
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

}
