// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaEvent.h"
#include "UbaFunctional.h"
#include "UbaList.h"
#include "UbaStringBuffer.h"
#include "UbaTimer.h"

#include <memory>

#define UBA_TRACK_WORK 1

namespace uba
{
	class Thread;
	struct Bottleneck;
	struct BottleneckTicket;
	struct StringView;
	struct TrackWorkScope;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	using Color = u32;
	constexpr inline Color ToColor(u8 r, u8 g, u8 b) { return (r << 16) + (g << 8) + b; }
	constexpr Color ColorWhite = ToColor(255, 255, 255);
	constexpr Color ColorWork = ToColor(70, 70, 100);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class WorkTracker
	{
	public:
		virtual u32 TrackWorkStart(const StringView& desc, const Color& color) = 0;
		virtual void TrackWorkHint(u32 id, const StringView& hint, u64 startTime = 0) = 0;
		virtual void TrackWorkEnd(u32 id) = 0;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct WorkContext
	{
		TrackWorkScope& tracker;
		Thread* thread = nullptr;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	using WorkFunction = Function<void(const WorkContext& context)>;

	enum WorkPriority : u8
	{
		WorkPriority_High,
		WorkPriority_Normal,
		WorkPriority_Low,
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class WorkManager : public WorkTracker
	{
	public:
		virtual void AddWork(WorkFunction&& work, u32 count, const tchar* desc, const Color& color = ColorWork, WorkPriority priority = WorkPriority_Normal, Bottleneck* bottleneck = nullptr) = 0;
		virtual u32 GetWorkerCount() = 0;
		virtual void DoWork(u32 count = 1) = 0;
		virtual bool FlushWork(u32 timeoutMs = 0) = 0;

		u32 TrackWorkStart(const StringView& desc, const Color& color);
		void TrackWorkHint(u32 id, const StringView& hint, u64 startTime = 0);
		void TrackWorkEnd(u32 id);

		void SetWorkTracker(WorkTracker* workTracker);
		WorkTracker* GetWorkTracker() { return m_workTracker; }

		template<u32 BatchSize = 1, typename TContainer, typename TFunc>
		void ParallelFor(u32 workCount, TContainer& container, TFunc&& func, const StringView& description = AsView(TC("")), WorkPriority priority = WorkPriority_Normal, Bottleneck* bottleneck = nullptr);

	protected:
		Atomic<WorkTracker*> m_workTracker = nullptr;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct Work { WorkFunction func; Bottleneck* bottleneck; TString desc; Work* next; };

	struct WorkQueue
	{
		~WorkQueue();
		void Enqueue(WorkFunction&& func, u32 count, const tchar* desc, const Color& color, WorkPriority priority, Bottleneck* bottleneck, bool trackWork);
		Work* Dequeue(BottleneckTicket& ticket);
		void Finish(Work* work, BottleneckTicket& ticket);
		bool IsEmpty();

		bool IsEmptyNoLock();

		Futex lock;
		Work* first[3] = { 0 };
		Work* last[3] = { 0 };
		Work* free = nullptr;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class WorkManagerImpl : public WorkManager
	{
	public:
		WorkManagerImpl(u32 maxWorkerCount, const tchar* workerDesc = TC("UbaWrk"), u32 precreateWorkerCount = 1);
		virtual ~WorkManagerImpl();
		virtual void AddWork(WorkFunction&& work, u32 count, const tchar* desc, const Color& color = ColorWork, WorkPriority priority = WorkPriority_Normal, Bottleneck* bottleneck = nullptr) override final;
		virtual u32 GetWorkerCount() override final;
		virtual void DoWork(u32 count = 1) override final;
		virtual bool FlushWork(u32 timeoutMs = 0) override final;

	private:
		struct Worker;
		void PushWorkerNoLock(Worker* worker);
		Worker* PopWorkerNoLock();

		u32 m_maxWorkerCount;
		TString m_workerDesc;
		List<Worker> m_workers;
		WorkQueue m_workQueue;
		Atomic<u32> m_activeWorkerCount = 0;

		Futex m_availableWorkersLock;
		Worker* m_firstAvailableWorker = nullptr;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct TrackWorkScope
	{
		TrackWorkScope();
		TrackWorkScope(WorkTracker& t, StringView desc, const Color& color = ColorWork);
		void AddHint(StringView hint, u64 startTime = 0);
		~TrackWorkScope();
		static TrackWorkScope* GetActiveScope();
		WorkTracker* tracker;
		TrackWorkScope* outerScope;
		u32 id;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct TrackHintScope
	{
		TrackHintScope(TrackWorkScope& t, StringView h) : tws(t), hint(h), startTime(GetTime()) {}
		~TrackHintScope() { tws.AddHint(hint, startTime); }
		TrackWorkScope& tws;
		StringView hint;
		u64 startTime;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	template<u32 BatchSize, typename TContainer, typename TFunc>
	void WorkManager::ParallelFor(u32 workCount, TContainer& container, TFunc&& func, const StringView& description, WorkPriority priority, Bottleneck* bottleneck)
	{
		#if !defined( __clang_analyzer__ ) // Static analyzer claims context can leak but it can only leak when process terminates (and then it doesn't matter)

		typedef typename TContainer::iterator iterator;

		auto size = container.size();

		if (size <= BatchSize)
		{
			#if UBA_TRACK_WORK
			TrackWorkScope tws(*this, description, ColorWork);
			#else
			TrackWorkScope tws;
			#endif
			WorkContext wc{tws};
			for (iterator i=container.begin(), e=container.end(); i!=e; i++)
				func(wc, i);
			return;
		}

		workCount = Min(workCount, u32(size - 1)/BatchSize);

		EventSlim doneEvent(EventResetType_Manual);

		struct Context
		{
			iterator it;
			iterator end;
			u32 activeCount;
			bool isDone;
			Futex lock;
			EventSlim* doneEvent;
		};

		// shared_ptr lifetime is what makes this race-safe: ReaderWriterLock::Leave()
		// releases the lock state before WakeWaitingThreads runs, so another thread
		// can acquire+release+free the lock while we're still mid-wake. Holding the
		// Context via shared_ptr ensures its destruction is deferred until every
		// lambda copy is destroyed, which is sequenced after every SCOPED_FUTEX
		// destructor (and therefore every WakeWaitingThreads) has fully unwound.
		auto context = std::make_shared<Context>();
		context->it = container.begin();
		context->end = container.end();
		context->activeCount = 0;
		context->isDone = false;
		context->doneEvent = &doneEvent;

		auto work = [context, funcCopy = func](const WorkContext& wc) mutable
			{
				iterator itBatch[BatchSize];

				u32 active = 0;
				while (true)
				{
					SCOPED_FUTEX(context->lock, l);
					context->activeCount -= active;
					context->isDone = context->it == context->end;
					if (context->isDone)
					{
						if (context->activeCount == 0 && context->doneEvent)
						{
							context->doneEvent->Set();
							context->doneEvent = nullptr;
						}
						return;
					}

					itBatch[0] = context->it++;
					active = 1;

					if constexpr (BatchSize > 1)
						for (;active < BatchSize && context->it != context->end;)
							itBatch[active++] = context->it++;

					context->activeCount += active;
					l.Leave();

					funcCopy(wc, itBatch[0]);

					if constexpr (BatchSize > 1)
						for (u32 i=1; i!=active; ++i)
							funcCopy(wc, itBatch[i]);
				}
			};

		// Copy `work` for AddWork so std::move doesn't null out main thread's
		// shared_ptr capture. Each lambda instance carries its own shared_ptr ref
		// — AddWork's Function<> holds one, main keeps one.
		auto workCopy = work;
		AddWork(std::move(workCopy), workCount, description.data, ColorWork, priority, bottleneck);

		{
			#if UBA_TRACK_WORK
			TrackWorkScope tws(*this, description, ColorWork);
			#else
			TrackWorkScope tws;
			#endif
			work({tws});
		}

		doneEvent.IsSet();

		#endif
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct IndexContainer
	{
		struct iterator
		{
			iterator() : index(0) {}
			iterator(u32 i) : index(i) {}
			iterator operator++(int) { return iterator(index++); }
			bool operator==(const iterator& o) const { return index == o.index; }
			u32 index;
		};
		IndexContainer(u64 c) : count(u32(c)) {}
		iterator begin() const { return iterator(0); }
		iterator end() const { return iterator(count); }
		u32 size() const { return count; }
		u32 count;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
