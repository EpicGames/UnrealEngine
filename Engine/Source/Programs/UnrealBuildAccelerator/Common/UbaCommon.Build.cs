// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaCommon : ModuleRules
{
	public UbaCommon(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivatePCHHeaderFile = "../Core/Public/UbaCorePch.h";

		bRequiresPlatformSDK = true;

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		StaticAnalyzerDisabledCheckers.Clear();

		PublicDependencyModuleNames.AddRange(new string[] {
			"UbaCore",
		});

		// External modules
		PrivateDependencyModuleNames.AddRange(new string[] {
			"UbaMimalloc",
		});

		PublicDependencyModuleNames.AddRange(new string[] {
			"OodleDataCompression",
		});

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
				"Detours",
			});
		}
		else if (Target.Platform.IsInGroup(UnrealPlatformGroup.Linux))
		{
			PublicSystemLibraries.Add("dl");
		}

		if (Target.Platform != UnrealTargetPlatform.Win64)
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
				"OpenSSL",
			});
		}

		PrivateDefinitions.AddRange(new string[] {
			"_CONSOLE",
		});
	}
}
