LARGE FEATURES:
* Add support for running separate process handling storage (UbaStorageService.exe). Agents and hosts should be able to connect to this service
	* This would enable so multiple agents/hosts can run on the same machine sharing same storage (for example to split up cores with higher granularity, say 8 cores each)
* Change cas storage to use huge sparse files instead of one file per cas. This is to try to reduce I/O contention. Will also make cleanup super fast

MEDIUM FEATURES:
* Add visual studio panel for visualizer
* Add "racing" where Executor starts taking on jobs when it is stuck depending on work being handled by remotes. Executor needs to make a judgement when to start the race
* Try building without writing to disk (everything in memory)
* Startup from non-graceful shutdown needs to be faster. (checking cas multithreaded)
* Build all remotely (pch, linking etc etc)
* Pch caching for clang based on input (don't know why this does not work)

SMALL FEATURES/IMPROVEMENTS:
* Linux - Fix opendir on linux. Right now it is wrong for remote processes but clang etc seems to not be using it
* Linux - Test crashing a process and see if we can catch the dump
* UbaVisualizer run network on different thread.. to not hang everything when network stalls
* Compact view of ubavisualizer
* Change so UbaExecutor use callbacks for main jobs instead of .net long running worker and add kill-on-mem
* Improve wait on mem message to include how much memory is used by other programs
* Test UbaAgent -binasversion.. seems to be bugs
* Change so visualizer defaults to -listen
* Fix so UbaVisualizer copy itself to temp and run from there to prevent locking of file in p4 source tree
* Add option to name SessionServer... so it is written to Trace
* Gracefully handle out of disk space on remotes (return all processes and disconnect)
* Compress non-file messages (Dir table, Hash table, etc) to reduce bandwidth usage (maybe this is minimal)
* Let storage proxy use stored files from StorageClient if StorageClient exists
* Add detoured process std output (errors) to trace stream (and be able to show in visualizer)
* Connect Visualizer live to horde builds
* Add date/time to trace dump and show in Visualizer when opening files
* Show more info in visualizer about the machine building
* Show loading cas table time in Visualizer!
