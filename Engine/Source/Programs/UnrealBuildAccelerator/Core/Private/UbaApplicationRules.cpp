// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaApplicationRules.h"
#include "UbaPlatform.h"

namespace uba
{
	bool GlobalRules::FileCanBeCompressed(const StringView& fileName)
	{
		if constexpr (!IsWindows)
			return false;
		return fileName.EndsWith(TCV(".obj"))
			|| (fileName.EndsWith(TCV(".o")) && !fileName.EndsWith(TCV(".native.o"))) // We can't compress the .native.o because thinlto distributed does messy things
			|| fileName.EndsWith(TCV(".pch"))
			|| fileName.EndsWith(TCV(".gch"))
			|| fileName.EndsWith(TCV(".ifc"))
			;
	}

	u8 GlobalRules::GetUsedCountBeforeFree(const StringView& fileName)
	{
		if (!fileName.EndsWith(TCV(".obj")))
			return 0;
		if (fileName.EndsWith(TCV(".h.obj")))
			return 255;
		return 2;
	}

	GlobalRules g_globalRules;

	
	bool SupportsLinkerPlaceholder(StringView fileName)
	{
		return fileName.EndsWith(TCV(".pdb"))
			|| fileName.EndsWith(TCV(".exe"))
			|| fileName.EndsWith(TCV(".dll"));
	}


	class ApplicationRulesDotnet : public ApplicationRules
	{
		using Super = ApplicationRules;

	public:
		virtual bool CanDetour(StringView file, bool isRunningRemote) const override
		{
			return !isRunningRemote || !file.Contains(',');
		}
	};

	class ApplicationRulesVC : public ApplicationRules
	{
		using Super = ApplicationRules;

	public:
		virtual bool AllowDetach() const override
		{
			return true;
		}

		virtual u64 FileTypeMaxSize(const StringView& file, bool isSystemOrTempFile) const override
		{
			if (file.EndsWith(TCV(".pdb")))
				return 14ull * 1024 * 1024 * 1024; // This is ridiculous
			if (file.EndsWith(TCV(".json")) || file.EndsWith(TCV(".exp")) || file.EndsWith(TCV(".sarif")) || file.EndsWith(TCV(".res")))
				return 32 * 1024 * 1024;
			if (file.EndsWith(TCV(".ifc")) || file.EndsWith(TCV(".obj")) || (isSystemOrTempFile && file.Contains(TCV("_cl_")))) // There are _huge_ obj files when building with -stresstestunity
				return 1024 * 1024 * 1024;
			return Super::FileTypeMaxSize(file, isSystemOrTempFile);
		}
		virtual u32 FileTypeCommitSize(StringView file) const
		{
			if (file.EndsWith(TCV(".pch")))
				return 8 * 1024 * 1024;
			if (file.EndsWith(TCV(".sarif")) || file.EndsWith(TCV(".json")) || file.EndsWith(TCV(".h")))
				return 64 * 1024;
			return Super::FileTypeCommitSize(file);
		}

		virtual bool IsThrowAway(StringView fileName, bool isRunningRemote) const override
		{
			return fileName.Contains(TCV("vctip_")) || Super::IsThrowAway(fileName, isRunningRemote);
		}
		virtual bool KeepInMemory(StringView fileName, StringView systemTemp, bool isRunningRemote, bool isWrite) const override
		{
			if (fileName.Contains(TCV("\\vctip_")))
				return true;
			if (fileName.Contains(systemTemp) && !fileName.EndsWith(TCV(".pgd"))) // pgd is copied into temp folder and detoured CopyFileExW is missing implementation to copy to memory
				return true;
			return Super::KeepInMemory(fileName, systemTemp, isRunningRemote, isWrite);
		}

		virtual bool IsExitCodeSuccess(u32 exitCode) const override
		{
			return exitCode == 0;
		}
	};

	class ApplicationRulesClExe : public ApplicationRulesVC
	{
	public:
		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return fileName.EndsWith(TCV(".obj"))
				|| fileName.EndsWith(TCV(".dep.json"))
				|| fileName.EndsWith(TCV(".sarif"))
				|| fileName.EndsWith(TCV(".pch"))
				|| fileName.EndsWith(TCV(".ifc"))
				|| fileName.EndsWith(TCV(".rc2.res")) // Not really an obj file.. 

				|| fileName.Contains(TCV("\\_cl_")); // This file is needed when cl.exe spawns link.exe
				;
		}

		virtual bool IsRarelyRead(const StringView& file) const override
		{
			return file.EndsWith(TCV(".cpp"))
				|| file.EndsWith(TCV(".cc"))
				|| (file.EndsWith(TCV(".rsp")) && !file.EndsWith(TCV("Shared.rsp")))
				|| file.EndsWith(TCV(".i")) // .i is preprocessed source code (for static analysis)
				;
		}

		virtual bool IsMaybeReadAfterWritten(const StringView& fileName) const override
		{
			return !(fileName.EndsWith(TCV(".dep.json"))
				|| fileName.EndsWith(TCV(".sarif"))
				|| fileName.EndsWith(TCV(".exe"))
				|| fileName.EndsWith(TCV(".dll")));
		}

		virtual bool ShouldExtractSymbols(const StringView& fileName) const override
		{
			return fileName.EndsWith(TCV(".obj"));
		}

		virtual bool ShouldDevirtualizeFile(const StringView& fileName, bool& outEscapeSpaces) const override
		{
			outEscapeSpaces = false;
			return fileName.EndsWith(TCV(".dep.json"));
		}

		virtual DependencyCrawlerType GetDependencyCrawlerType() const override
		{
			return DependencyCrawlerType_MsvcCompiler;
		}

		virtual bool CanDependOnCompressedFiles()
		{
			return true; // .pch
		}

		virtual bool CanExist(StringView file) const override
		{
			return !file.EndsWith(TCV("vctip.exe")); // This is hacky but we don't want to start vctip.exe
		}

		virtual bool SupportsRacing() const override
		{
			return true;
		}

		virtual bool ShouldWriteFilesBeforeWaitOnExit() const override
		{
			return true;
		}
	};

	class ApplicationRulesVcLink : public ApplicationRulesVC
	{
		using Super = ApplicationRulesVC;
	public:
		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return fileName.EndsWith(TCV(".lib"))
				|| fileName.EndsWith(TCV(".exp"))
				|| fileName.EndsWith(TCV(".pdb"))
				|| fileName.EndsWith(TCV(".dll"))
				|| fileName.EndsWith(TCV(".exe"))
				|| fileName.EndsWith(TCV(".rc2.res")) // Not really an obj file.. 

				// temp files
				|| fileName.Contains(TCV("lnk{")) // This file is shared from link.exe to mt.exe and rc.exe so we need to put it shared memory
			 	|| fileName.Contains(TCV("\\_cl_")) // When link.exe is spawned by cl.exe we might use this which is in shared memory
			 	|| fileName.EndsWith(TCV(".manifest")); // lld-link.exe is using a different name for files shared with child processes
		}

		virtual bool IsThrowAway(StringView fileName, bool isRunningRemote) const override
		{
			return fileName.Contains(TCV(".sup.")); // .sup.lib/exp are throw-away files that we don't want created
		}

		virtual bool CanExist(StringView file) const override
		{
			return !file.EndsWith(TCV("vctip.exe")); // This is hacky but we don't want to start vctip.exe
		}

		virtual u64 FileTypeMaxSize(const StringView& file, bool isSystemOrTempFile) const override
		{
			if (file.Contains(TCV("lnk{")))
				return 32 * 1024 * 1024;
			if (file.EndsWith(TCV(".lib"))) // This is usually for import lib which is tiny but not sure if we output any full libs
				return 2ull * 1024 * 1024 * 1024;
			if (file.EndsWith(TCV(".dll")) || file.EndsWith(TCV(".exe")))
				return 4ull * 1024 * 1024 * 1024;
			return Super::FileTypeMaxSize(file, isSystemOrTempFile);
		}

		virtual bool IsRarelyRead(const StringView& file) const override
		{
			return file.EndsWith(TCV(".exp"))
				|| file.EndsWith(TCV(".dll.rsp"))
				|| file.EndsWith(TCV(".lib.rsp"))
				|| file.EndsWith(TCV(".ilk"))
				|| file.EndsWith(TCV(".pdb"));
		}

		virtual bool AllowStorageProxy(const StringView& file) const override
		{
			if (file.EndsWith(TCV(".obj")))
				return false;
			return Super::AllowStorageProxy(file);
		}

		virtual bool IsMaybeReadAfterWritten(const StringView& fileName) const override
		{
			return !(fileName.EndsWith(TCV(".pdb"))
				|| fileName.EndsWith(TCV(".exe"))
				//|| fileName.EndsWith(TCV(".exp"))  // This is used by clang
				|| fileName.EndsWith(TCV(".dll")));
		}

		virtual bool CanDependOnCompressedFiles() override
		{
			return true; // Yes, .obj files
		}

		virtual DependencyCrawlerType GetDependencyCrawlerType() const override
		{
			return DependencyCrawlerType_MsvcLinker;
		}
	};

	class ApplicationRulesLinkExe : public ApplicationRulesVcLink
	{
		using Super = ApplicationRulesVcLink;

		virtual bool CloseFileMappingAfterUse(StringView fileName) const override
		{
			return true;
		}

		virtual bool SupportsPlaceholder(StringView fileName) const override { return SupportsLinkerPlaceholder(fileName); }

		virtual bool Time64AlwaysReturnZero() const override
		{
			return true;
		}

		virtual bool ShouldWriteFilesBeforeWaitOnExit() const override
		{
			return true;
		}
	};

	class ApplicationCvtresExe : public ApplicationRulesLinkExe { virtual bool Time64AlwaysReturnZero() const override { return true; } };

	class ApplicationRulesLldLinkExe : public ApplicationRulesVcLink
	{
		using Super = ApplicationRulesVcLink;

		virtual u64 FileTypeMaxSize(const StringView& file, bool isSystemOrTempFile) const override
		{
			if (file.Contains(TCV(".pdb.tmp")))
				return 14ull * 1024 * 1024 * 1024; // This is ridiculous
			return ApplicationRulesVcLink::FileTypeMaxSize(file, isSystemOrTempFile);
		}

		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return false//Super::IsOutputFile(fileName)
				|| fileName.Contains(TCV(".exe.tmp"))
				|| fileName.Contains(TCV(".dll.tmp"))
				|| fileName.Contains(TCV(".pdb.tmp"))
				|| (fileName.EndsWith(TCV(".manifest")) && fileName.Contains(systemTemp)) // Used between lld-link.exe and mt.exe (test compiling AutoRTFMEngineTests)
				;
		}

		virtual bool CloseFileMappingAfterUse(StringView fileName) const override
		{
			return true;
		}

		virtual bool SupportsPlaceholder(StringView fileName) const override { return SupportsLinkerPlaceholder(fileName); }
	};

	class ApplicationRulesRadLinkExe : public ApplicationRulesVcLink
	{
		using Super = ApplicationRulesVcLink;

		virtual u64 FileTypeMaxSize(const StringView& file, bool isSystemOrTempFile) const override
		{
			if (file.Contains(TCV(".pdb.tmp")) || file.EndsWith(TCV(".pdb")))
				return 14ull * 1024 * 1024 * 1024; // This is ridiculous
			return ApplicationRulesVcLink::FileTypeMaxSize(file, isSystemOrTempFile);
		}

		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return false//Super::IsOutputFile(fileName)
				|| fileName.EndsWith(TCV(".exe"))
				|| fileName.EndsWith(TCV(".dll"))
				|| fileName.EndsWith(TCV(".pdb"))
				|| fileName.Contains(TCV(".exe.tmp"))
				|| fileName.Contains(TCV(".dll.tmp"))
				|| fileName.Contains(TCV(".pdb.tmp"));
		}

		virtual bool CloseFileMappingAfterUse(StringView fileName) const
		{
			return true;
		}

		virtual bool SupportsPlaceholder(StringView fileName) const override { return SupportsLinkerPlaceholder(fileName); }

		virtual bool ShouldWriteFilesBeforeWaitOnExit() const override
		{
			return true;
		}
	};

	// ==== Clang tool chain ====

	class ApplicationRulesClang : public ApplicationRules
	{
	public:
		using Super = ApplicationRules;

		virtual bool IsExitCodeSuccess(u32 exitCode) const override
		{
			return exitCode == 0;
		}

		virtual u64 FileTypeMaxSize(const StringView& file, bool isSystemOrTempFile) const override
		{
			if (file.EndsWith(TCV(".obj")) || file.EndsWith(TCV(".o")) || file.EndsWith(TCV(".o.tmp")) || file.EndsWith(TCV(".obj.tmp")))
				return 1024 * 1024 * 1024; // There are huge obj files when building with -stresstestunity
			if (file.EndsWith(TCV(".d")) || file.EndsWith(TCV(".d.tmp")))
				return 32 * 1024 * 1024;
			return Super::FileTypeMaxSize(file, isSystemOrTempFile);
		}
	};

	class ApplicationRulesClangPlusPlusExe : public ApplicationRulesClang
	{
	public:
		using Super = ApplicationRulesClang;

		virtual bool AllowDetach() const override
		{
			return true;
		}

		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return fileName.EndsWith(TCV(".d"))
				|| fileName.EndsWith(TCV(".d.tmp"))
				|| fileName.EndsWith(TCV(".o"))
				|| fileName.EndsWith(TCV(".o.tmp"))
				|| fileName.EndsWith(TCV(".gch"))
				|| fileName.EndsWith(TCV(".gch.tmp"))
				|| fileName.EndsWith(TCV(".obj"))
				|| fileName.EndsWith(TCV(".obj.tmp"))
				|| fileName.EndsWith(TCV(".pch"))
				;
		}

		virtual bool IsRarelyRead(const StringView& file) const override
		{
			return file.EndsWith(TCV(".cpp"))
				|| file.EndsWith(TCV(".cc"))
				|| (file.EndsWith(TCV(".rsp")) && !file.EndsWith(TCV("Shared.rsp")));
		}

		virtual bool IsMaybeReadAfterWritten(const StringView& fileName) const override
		{
			return !(fileName.EndsWith(TCV(".d")) || fileName.EndsWith(TCV(".d.tmp")));
		}

		virtual bool ShouldExtractSymbols(const StringView& fileName) const override
		{
			return fileName.EndsWith(TCV(".obj")) || fileName.EndsWith(TCV(".o"));
		}

		virtual bool ShouldDevirtualizeFile(const StringView& fileName, bool& outEscapeSpaces) const override
		{
			outEscapeSpaces = true;
			return fileName.EndsWith(TCV(".d")) || fileName.EndsWith(TCV(".d.tmp"));
		}

		virtual DependencyCrawlerType GetDependencyCrawlerType() const override
		{
			return DependencyCrawlerType_ClangCompiler;
		}

		virtual bool CanDependOnCompressedFiles() override
		{
			return true; // Yes, .pch
		}

		virtual bool SupportsRacing() const override
		{
			return true;
		}
	};

	class ApplicationRulesClangClExe : public ApplicationRulesClangPlusPlusExe
	{
		using Super = ApplicationRulesClangPlusPlusExe;
	public:
		virtual DependencyCrawlerType GetDependencyCrawlerType() const override
		{
			return DependencyCrawlerType_MsvcCompiler;
		}

		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return Super::IsOutputFile(fileName, systemTemp) || fileName.Contains(TCV("response-")); // file used by internally spawned clang-cl.exe. Note this is only generated when command line is very long
		}

		virtual bool KeepInMemory(StringView fileName, StringView systemTemp, bool isRunningRemote, bool isWrite) const override
		{
			return fileName.count > systemTemp.count && fileName.StartsWith(systemTemp);
		}
	};

	class ApplicationRulesLdLLdExe : public ApplicationRulesClang
	{
		using Super = ApplicationRulesClang;

		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return fileName.Contains(TCV(".tmp")); // both .so.tmp and .tmp123456
		}

		virtual u64 FileTypeMaxSize(const StringView& file, bool isSystemOrTempFile) const override
		{
			return 14ull * 1024 * 1024 * 1024; // This is ridiculous (needed for asan targets)
		}

		virtual bool CanDependOnCompressedFiles() override
		{
			return true;
		}

		virtual bool CloseFileMappingAfterUse(StringView fileName) const override
		{
			return true;
		}
	};

	class ApplicationRulesLlvmObjCopyExe : public ApplicationRulesClang
	{
		using Super = ApplicationRulesClang;

		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return fileName.Contains(TCV(".temp-stream-"));
		}

		virtual u64 FileTypeMaxSize(const StringView& file, bool isSystemOrTempFile) const override
		{
			if (file.Contains(TCV(".temp-stream-")))
				return 14ull * 1024 * 1024 * 1024; // This is ridiculous (needed for asan targets)
			return Super::FileTypeMaxSize(file, isSystemOrTempFile);
		}
	};

	class ApplicationRulesDumpSymsExe : public ApplicationRulesClang
	{
		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return false; // fileName.EndsWith(TCV(".psym")); With psym as output file the BreakpadSymbolEncoder fails to output a .sym file
		}
	};

	class ApplicationRulesISPCExe : public ApplicationRules
	{
		using Super = ApplicationRules;

		virtual bool AllowDetach() const override
		{
			return true;
		}

		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return fileName.Contains(TCV(".generated.dummy"))
				|| fileName.EndsWith(TCV(".ispc.bc"))
				|| fileName.EndsWith(TCV(".ispc.txt"))
				|| fileName.EndsWith(TCV(".obj"))
				|| fileName.EndsWith(TCV(".o")); // Used when compiling for linux
		}

		virtual u64 FileTypeMaxSize(const StringView& file, bool isSystemOrTempFile) const override
		{
			if (file.Contains(TCV(".generated.dummy")) || file.EndsWith(TCV(".ispc.txt")))
				return 32ull * 1024 * 1024;
			if (file.EndsWith(TCV(".obj")) || file.EndsWith(TCV(".o")))
				return 128ull * 1024 * 1024;
			return Super::FileTypeMaxSize(file, isSystemOrTempFile);
		}

		virtual bool ShouldExtractSymbols(const StringView& fileName) const override
		{
			return fileName.EndsWith(TCV(".obj")) || fileName.EndsWith(TCV(".o"));
		}

		virtual bool ShouldDevirtualizeFile(const StringView& fileName, bool& outEscapeSpaces) const override
		{
			return fileName.EndsWith(TCV(".ispc.txt"));
		}

		virtual bool SupportsRacing() const override
		{
			return true;
		}
	};

	static bool IsPlatform12ThrowAway(StringView fileName)
	{
		return fileName.EndsWith(TCV("-telemetry.json")) || fileName.EndsWith(TCV(".crash.json"));
	}

	class ApplicationRulesClangPlusPlusExePlatform1 : public ApplicationRulesClangPlusPlusExe
	{
		using Super = ApplicationRulesClangPlusPlusExe;

		virtual bool IsThrowAway(StringView fileName, bool isRunningRemote) const override
		{
			return IsPlatform12ThrowAway(fileName) || Super::IsThrowAway(fileName, isRunningRemote);
		}
	};

	class ApplicationRulesLdExePlatform1 : public ApplicationRules
	{
		using Super = ApplicationRules;

		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return fileName.Contains(TCV("thinlto-")); // Used by a clang based platform's link time optimization pass. Shared from lto process back to linker process
		}

		virtual bool KeepInMemory(StringView fileName, StringView systemTemp, bool isRunningRemote, bool isWrite) const override
		{
			return Super::KeepInMemory(fileName, systemTemp, isRunningRemote, isWrite)
				|| fileName.Contains(TCV("thinlto-"));// Used by a clang based platform's link time optimization pass. Shared from lto process back to linker process
		}

		virtual bool CanDependOnCompressedFiles() override
		{
			return true;
		}

		virtual bool CloseFileMappingAfterUse(StringView fileName) const override
		{
			return true;
		}

		virtual bool SupportsPlaceholder(StringView fileName) const override { return fileName.EndsWith(TCV(".self")); }
	};

	class ApplicationRulesClangPlusPlusExePlatform2 : public ApplicationRulesClangPlusPlusExe
	{
		using Super = ApplicationRulesClangPlusPlusExe;

		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return fileName.Contains(TCV(".self")) || Super::IsOutputFile(fileName, systemTemp);
		}

		virtual bool KeepTempOutputFile(const StringView& fileName) const override
		{
			return fileName.Contains(TCV(".native."));
		}

		virtual bool IsThrowAway(StringView fileName, bool isRunningRemote) const override
		{
			return Super::IsThrowAway(fileName, isRunningRemote)
				|| IsPlatform12ThrowAway(fileName); // Seems like these can collide when running distributed thinlto
		}
	};

	class ApplicationRulesLldExePlatform2 : public ApplicationRules
	{
		using Super = ApplicationRules;

		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return fileName.Contains(TCV(".self"));
		}

		virtual bool IsThrowAway(StringView fileName, bool isRunningRemote) const override
		{
			return Super::IsThrowAway(fileName, isRunningRemote)
				|| IsPlatform12ThrowAway(fileName); // Seems like these can collide when running distributed thinlto
		}

		virtual bool CanDependOnCompressedFiles() override
		{
			return true;
		}

		virtual bool CloseFileMappingAfterUse(StringView fileName) const override
		{
			return true;
		}

		virtual bool SupportsPlaceholder(StringView fileName) const override { return fileName.EndsWith(TCV(".self")); }
	};

	class ApplicationRulesISPCExePlatform2 : public ApplicationRulesISPCExe
	{
		using Super = ApplicationRulesISPCExe;
		virtual bool IsThrowAway(StringView fileName, bool isRunningRemote) const override
		{
			return Super::IsThrowAway(fileName, isRunningRemote)
				|| IsPlatform12ThrowAway(fileName); // Seems like these can collide when running distributed thinlto
		}
	};

	class ApplicationRulesStageSplitMergeApplication : public ApplicationRules
	{
		using Super = ApplicationRules;

		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return fileName.EndsWith(TCV(".split"));
		}

		virtual bool AllowStorageProxy(const StringView& file) const override
		{
			return !file.EndsWith(TCV(".split"));
		}

		virtual bool SendFileCompressedFromClient(const StringView& fileName) const override
		{
			return !fileName.EndsWith(TCV(".split"));
		}

		virtual bool KeepInMemory(StringView fileName, StringView systemTemp, bool isRunningRemote, bool isWrite) const override
		{
			if (isWrite && fileName.Contains(TCV("\\Split\\")) && fileName.EndsWith(TCV(".split")))
				return true;
			return false;
		}

		virtual bool IsThrowAway(StringView fileName, bool isRunningRemote) const override
		{
			if (fileName.Contains(TCV("\\Split\\")) && fileName.EndsWith(TCV(".split")))
				return true;
			return Super::IsThrowAway(fileName, isRunningRemote);
		}

		virtual bool ReportAllExceptions() const override
		{
			return true;
		}
	};

	// ====

	class ApplicationRulesUBTDll : public ApplicationRules
	{
		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return false;
			// TODO: These does not work when UnrealBuildTool creates these files multiple times in a row (building multiple targets)
			// ... on output they get stored as file mappings.. and next execution of ubt opens them for write (writing file mappings not implemented right now)
			//return fileName.EndsWith(TCV(".modules"))
			//	|| fileName.EndsWith(TCV(".target"))
			//	|| fileName.EndsWith(TCV(".version"));
		}
	};

	class ApplicationRulesPVSStudio : public ApplicationRules
	{
		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return fileName.EndsWith(TCV(".PVS-Studio.log"))
				|| fileName.EndsWith(TCV(".pvslog"))
				|| fileName.EndsWith(TCV(".stacktrace.txt"));
		}
		
		virtual bool IsRarelyRead(const StringView& file) const override
		{
			return file.EndsWith(TCV(".i"))
				|| file.EndsWith(TCV(".PVS-Studio.log"))
				|| file.EndsWith(TCV(".pvslog"))
				|| file.EndsWith(TCV(".stacktrace.txt"));
		}

#if PLATFORM_WINDOWS
		virtual void RepairMalformedLibPath(const tchar* path) const override
		{
			// There is a bug where the path passed into wsplitpath_s is malformed and not null terminated correctly
			const tchar* pext = TStrstr(path, TC(".dll"));
			if (pext == nullptr) pext = TStrstr(path, TC(".DLL"));
			if (pext == nullptr) pext = TStrstr(path, TC(".exe"));
			if (pext == nullptr) pext = TStrstr(path, TC(".EXE"));
			if (pext != nullptr && *(pext + 4) != 0) *(const_cast<tchar*>(pext + 4)) = 0;
		}
#endif // #if PLATFORM_WINDOWS
	};

	class ApplicationRulesIWYU : public ApplicationRulesClangClExe
	{
		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return fileName.EndsWith(TCV(".d"))
				|| fileName.EndsWith(TCV(".d.tmp"))
				|| fileName.EndsWith(TCV(".iwyu"))
				;
		}
	};

	class ApplicationRulesShaderCompileWorker : public ApplicationRules
	{
		virtual bool IsRarelyRead(const StringView& file) const override
		{
			return file.Contains(TCV(".uba."));
		}

		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return fileName.EndsWith(TCV(".tmp")) || fileName.EndsWith(TCV(".out"));
		}

		virtual bool AllowDetach() const override
		{
			return true;
		}

		virtual bool IsInvisible(const StringView& fileName) const override
		{
			return fileName.EndsWith(TCV(".tmp")) || fileName.EndsWith(TCV(".out"));
		}
	};

	class ApplicationRulesUbaObjTool : public ApplicationRules
	{
		virtual bool AllowDetach() const override
		{
			return true;
		}

		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return fileName.EndsWith(TCV(".obj"))
				|| fileName.EndsWith(TCV(".exp"));
		}

		virtual bool CanDependOnCompressedFiles() override
		{
			return true;
		}

		virtual DependencyCrawlerType GetDependencyCrawlerType() const override
		{
			return DependencyCrawlerType_MsvcLinker;
		}

		virtual bool ReportAllExceptions() const override
		{
			return true;
		}

		virtual bool CloseFileMappingAfterUse(StringView fileName) const
		{
			return true;
		}
	};

	class ApplicationRulesUbaTestApp : public ApplicationRules
	{
		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return fileName.EndsWith(TCV(".out"));
		}

		virtual bool IsInvisible(const StringView& fileName) const override
		{
			return fileName.EndsWith(TCV(".out"));
		}
	};

	class ApplicationRulesIcxExe : public ApplicationRulesClangClExe
	{
		using Super = ApplicationRulesClangClExe;
		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const override
		{
			return Super::IsOutputFile(fileName, systemTemp) || (fileName.count > systemTemp.count && fileName.StartsWith(systemTemp));
		}
	};

	class ApplicationRulesDumpbin : public ApplicationRules
	{
		virtual bool CanDependOnCompressedFiles() override
		{
			return true;
		}
	};

	template<typename T, size_t size>
	constexpr u32 ApplicationHash(T (&buffer)[size])
	{
		u32 hash = 5381;
		for (size_t i=0, e=size-1; i!=e; ++i)
			hash = ((hash << 5) + hash) + buffer[i];
		return hash;
	}

	u32 GetApplicationHash(const StringView& str)
	{
		u32 hash = 5381;
		for (size_t i=0; i!=str.count; ++i)
			hash = ((hash << 5) + hash) + str.data[i];
		return hash;
	}

	const RulesRec* GetApplicationRules()
	{
		// TODO: Add support for data driven rules.
		// Note, they need to be possible to serialize from server to client and then from client to each detoured process
		// ALL HASHES LOWERCASE

		static RulesRec rules[]
		{
			{ ApplicationHash(TC("")),							new ApplicationRules() },
			{ ApplicationHash(TC("dotnet")),					new ApplicationRulesDotnet() }, // Fallback for non-specificied dotnet applications
#if PLATFORM_WINDOWS
			{ ApplicationHash(TC("cl.exe")),					new ApplicationRulesClExe() },	 // Must be index 2
			{ ApplicationHash(TC("link.exe")),					new ApplicationRulesLinkExe() }, // Must be index 3
			{ ApplicationHash(TC("lib.exe")),					new ApplicationRulesVcLink() },
			{ ApplicationHash(TC("cvtres.exe")),				new ApplicationCvtresExe() },
			{ ApplicationHash(TC("mt.exe")),					new ApplicationRulesVcLink() },
			{ ApplicationHash(TC("rc.exe")),					new ApplicationRulesVcLink() },
			{ ApplicationHash(TC("lld-link.exe")),				new ApplicationRulesLldLinkExe() },
			{ ApplicationHash(TC("clang++.exe")),				new ApplicationRulesClangPlusPlusExe() },
			{ ApplicationHash(TC("clang-cl.exe")),				new ApplicationRulesClangClExe() },
			{ ApplicationHash(TC("verse-clang-cl.exe")),		new ApplicationRulesClangClExe() },
			{ ApplicationHash(TC("epicclang-cl.exe")),			new ApplicationRulesClangClExe() },
			{ ApplicationHash(TC("ispc.exe")),					new ApplicationRulesISPCExe() },
			{ ApplicationHash(TC("radlink.exe")),				new ApplicationRulesRadLinkExe() },
			{ 3340509542,										new ApplicationRulesClangPlusPlusExePlatform1() },
			{ 4113554641,										new ApplicationRulesLdExePlatform1() }, // Must be index 15
			{ 1752955744,										new ApplicationRulesLdExePlatform1() },
			{ 238360161,										new ApplicationRulesClangPlusPlusExePlatform2() },
			{ 2119756440,										new ApplicationRulesLldExePlatform2() },
			{ 3206998955,										new ApplicationRulesISPCExePlatform2() },
			{ 2898035017,										new ApplicationRulesStageSplitMergeApplication() },
			{ ApplicationHash(TC("dump_syms.exe")),				new ApplicationRulesDumpSymsExe() },
			{ ApplicationHash(TC("ld.lld.exe")),				new ApplicationRulesLdLLdExe() },
			{ ApplicationHash(TC("llvm-objcopy.exe")),			new ApplicationRulesLlvmObjCopyExe() },
			{ ApplicationHash(TC("unrealbuildtool.dll")),		new ApplicationRulesUBTDll() },
			{ ApplicationHash(TC("pvs-studio.exe")),			new ApplicationRulesPVSStudio() },
			{ ApplicationHash(TC("ubaobjtool.exe")),			new ApplicationRulesUbaObjTool() },
			{ ApplicationHash(TC("shadercompileworker.exe")),	new ApplicationRulesShaderCompileWorker() },
			{ ApplicationHash(TC("instr-clang-cl.exe")),		new ApplicationRulesClangClExe() },
			{ ApplicationHash(TC("include-what-you-use.exe")),	new ApplicationRulesIWYU() },
			{ ApplicationHash(TC("ubatestapp.exe")),			new ApplicationRulesUbaTestApp() },
			{ ApplicationHash(TC("pgocvt.exe")),				new ApplicationRulesLinkExe() },
			{ ApplicationHash(TC("pgomgr.exe")),				new ApplicationRulesVcLink() },
			{ ApplicationHash(TC("icx.exe")),					new ApplicationRulesIcxExe() },
			{ ApplicationHash(TC("dumpbin.exe")),				new ApplicationRulesDumpbin() },
			//{ L"MSBuild.dll"),				new ApplicationRules() },
			//{ L"BreakpadSymbolEncoder.exe"),	new ApplicationRulesClang() },
			//{ L"cmd.exe"),		new ApplicationRules() },
#else
			{ ApplicationHash(TC("clang++")),					new ApplicationRulesClangPlusPlusExe() },
			{ ApplicationHash(TC("clang")),						new ApplicationRulesClangPlusPlusExe() },
			{ ApplicationHash(TC("verse-clang++")),				new ApplicationRulesClangPlusPlusExe() },
			{ ApplicationHash(TC("verse-clang")),				new ApplicationRulesClangPlusPlusExe() },
			{ ApplicationHash(TC("epicclang++")),				new ApplicationRulesClangPlusPlusExe() },
			{ ApplicationHash(TC("ld.lld")),					new ApplicationRulesLdLLdExe() },
			{ ApplicationHash(TC("ispc")),						new ApplicationRulesISPCExe() },
			{ ApplicationHash(TC("shadercompileworker")),		new ApplicationRulesShaderCompileWorker() },
			{ ApplicationHash(TC("ubatestapp")),				new ApplicationRulesUbaTestApp() },
			{ 851054321,										new ApplicationRulesClangPlusPlusExePlatform2() },
			{ 138829736,										new ApplicationRulesLldExePlatform2() },
			{ 286314299,										new ApplicationRulesISPCExe() },
			{ 3836757465,										new ApplicationRulesStageSplitMergeApplication() },
#endif
			{ 0, nullptr }
		};

		static bool initRules = []()
			{
				u32 index = 0;
				for (RulesRec* rec = rules; rec->hash; ++rec)
					rec->rules->index = index++;
				return true;
			}();

		return rules;
	}
}
