// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaEvent.h"
#include "UbaDefinitions.h"
#include "UbaPlatform.h"
#include "UbaStringBuffer.h"
#include "UbaSynchronization.h"

#define UBA_TEST_WAIT_QUALITY 0

namespace uba
{
	struct ResetTypeWithPadding
	{
		u8 padding;
		EventResetType value;
	};

	struct EventHeader
	{
		EventHeader() { m_next = nullptr; }
		union
		{
			ResetTypeWithPadding m_resetType;
			EventHeader* m_next;
		};
	};

	template <typename TsT>
	inline u64 ToNanoSeconds(const TsT& ts)
	{
		return u64(ts.tv_sec) * 1'000'000'000ull + u64(ts.tv_nsec);
	}

	template <typename TsT>
	inline TsT ToTimeSpecFromNano(u64 ns)
	{
		TsT ts;
		ts.tv_sec  = decltype(ts.tv_sec)(ns / 1'000'000'000ull);
		ts.tv_nsec = decltype(ts.tv_nsec)(ns % 1'000'000'000ull);
		return ts;
	}
}

//#include "UbaEventPThread.inl"
#include "UbaEventMachSemaphore.inl"
#include "UbaEventDispatchSemaphore.inl"
#include "UbaEventPosixSemaphore.inl"
#include "UbaEventFutex.inl"
#include "UbaEventUlock.inl"
#include "UbaEventWin32.inl"


namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	// Local event type
	using SharedEventDefaultImpl =
	#if PLATFORM_WINDOWS
		EventWin32;
	#elif PLATFORM_LINUX
		EventFutex;
	#else
		EventUlock;
	#endif

	// Semaphore based IPC event.
	using SharedEventSemaphoreImpl = 

	#if UBA_USE_NATIVE_MAC_SEMAPHORES
		EventMachSemaphore;
	#elif !PLATFORM_WINDOWS
		EventPosixSemaphore;
	#else
		EventWin32;
	#endif

	// Default local event
	using LocalEventDefaultImpl =
	#if PLATFORM_WINDOWS
		EventWin32;
	#elif PLATFORM_LINUX
		EventFutex;
	#else
		EventUlock;
	#endif

	// Semaphore basedlocal event
	using LocalEventSemaphoreImpl = 
		EventDispatchSemaphore;

	bool g_useUseSemaphore = UseSemaphoreForLocalEvents();
	Futex g_firstEventLock[2];
	EventHeader* g_firstEvent[2];

	////////////////////////////////////////////////////////////////////////////////////////////////////

	Event::Event(EventResetType resetType)
	{
		Create(resetType);
	}

	Event::~Event()
	{
		Destroy();
	}

	bool Event::Create(EventResetType resetType)
	{
		EventHeader* impl = nullptr;
		SCOPED_FUTEX(g_firstEventLock[resetType], lock);
		auto*& first = g_firstEvent[resetType];
		if (first)
		{
			impl = (EventHeader*)first;
			first = impl->m_next;
		}
		lock.Leave();

		if (impl)
		{
			m_impl = impl;
			impl->m_resetType.value = resetType;
			Reset();
			return true;
		}
		if (g_useUseSemaphore)
		{
			impl = new LocalEventSemaphoreImpl();
			((LocalEventSemaphoreImpl*)impl)->Create(resetType);
		}
		else
		{
			impl = new LocalEventDefaultImpl();
			((LocalEventDefaultImpl*)impl)->Create(resetType);
		}
		m_impl = impl;
		return true;
	}

	void Event::Destroy()
	{
		if (!m_impl)
			return;
		auto impl = (EventHeader*)m_impl;
		m_impl = nullptr;
		auto resetType = impl->m_resetType.value;
		auto*& first = g_firstEvent[resetType];
		SCOPED_FUTEX(g_firstEventLock[resetType], l);
		impl->m_next = first;
		first = impl;
	}

	void Event::Set()
	{
		if (!m_impl)
			return;
		if (g_useUseSemaphore)
			((LocalEventSemaphoreImpl*)m_impl)->Set();
		else
			((LocalEventDefaultImpl*)m_impl)->Set();
	}

	void Event::Reset()
	{
		if (!m_impl)
			return;
		if (g_useUseSemaphore)
			((LocalEventSemaphoreImpl*)m_impl)->Reset();
		else
			((LocalEventDefaultImpl*)m_impl)->Reset();
	}

	bool Event::IsCreated()
	{
		return m_impl != nullptr;
	}

	bool Event::IsSet(u32 timeOutMs)
	{
		if (!m_impl)
			return false;
		if (g_useUseSemaphore)
			return ((LocalEventSemaphoreImpl*)m_impl)->IsSet(timeOutMs);
		else
			return ((LocalEventDefaultImpl*)m_impl)->IsSet(timeOutMs);
	}

	void* Event::GetHandle()
	{
		#if PLATFORM_WINDOWS
		if (!m_impl || g_useUseSemaphore)
			return nullptr;
		return ((LocalEventDefaultImpl*)m_impl)->GetHandle();
		#else
		UBA_ASSERTF(false, "Event::GetHandle not available");
		return nullptr;
		#endif
	}

	void Event::ClearCache()
	{
		for (u32 i = 0; i != 2; ++i)
		{
			auto it = g_firstEvent[i];
			g_firstEvent[i] = nullptr;;
			while (it)
			{
				auto next = it->m_next;
				if (g_useUseSemaphore)
				{
					((LocalEventSemaphoreImpl*)it)->Destroy();
					delete (LocalEventSemaphoreImpl*)it;
				}
				else
				{
					((LocalEventDefaultImpl*)it)->Destroy();
					delete (LocalEventDefaultImpl*)it;
				}
				it = next;
			}
		}
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	SharedEvent::SharedEvent()
	{
		new (m_data) EventHeader();
		*((u8*)m_data) = 0;
		static_assert(sizeof(m_data) >= sizeof(SharedEventDefaultImpl));
		static_assert(sizeof(m_data) >= sizeof(SharedEventSemaphoreImpl));
		#if PLATFORM_LINUX
		static_assert(sizeof(m_data) >= sizeof(EventFutex));
		#endif
	}

	SharedEvent::~SharedEvent()
	{
		Destroy();
	}

	bool SharedEvent::Create(EventResetType resetType)//, bool shared)
	{
		// Tag byte layout at offset 0:
		//   0 = uninitialized / destroyed (owned by SharedEvent's ctor + Destroy)
		//   1 = EventWin32 / EventPThread (SharedEventDefaultImpl elsewhere; stamped here)
		//   2 = SharedEventSemaphoreImpl (Create(name, isOwner) path; stamped there)
		//   3 = EventFutex (Linux default; written by EventFutex's own ctor so the
		//       static-detour stub — which ships its own SharedEvent::Set/IsSet for
		//       tag 3 only — can identify shm slots without a higher-level handshake)
		UBA_ASSERT(*((u8*)m_data) == 0);
		new (m_data) SharedEventDefaultImpl();
		#if !PLATFORM_LINUX
		// EventWin32/EventPThread don't self-stamp; we still need a non-zero,
		// non-2 tag so Destroy/Set/IsSet route to the default impl.
		*((u8*)m_data) = 1;
		#endif
		if (!((SharedEventDefaultImpl&)m_data).Create(resetType, true))
			return false;
		UBA_ASSERT(*((u8*)m_data) != 0 && *((u8*)m_data) != 2);
		return true;
	}

	bool SharedEvent::Create(StringView name, bool isOwner)
	{
		UBA_ASSERT(*((u8*)m_data) == 0);
		new (m_data) SharedEventSemaphoreImpl();
		*((u8*)m_data) = 2;
		return ((SharedEventSemaphoreImpl&)m_data).Create(name, isOwner);
	}

	void SharedEvent::Destroy()
	{
		if (*((u8*)m_data) == 0)
			return;

		if (IsSemaphore())
		{
			((SharedEventSemaphoreImpl&)m_data).Destroy();
			((SharedEventSemaphoreImpl&)m_data).~SharedEventSemaphoreImpl();
		}
		else
		{
			((SharedEventDefaultImpl&)m_data).Destroy();
			((SharedEventDefaultImpl&)m_data).~SharedEventDefaultImpl();
		}
		*((u8*)m_data) = 0;
	}

	void SharedEvent::Set()
	{
		if (*((u8*)m_data) == 0)
			return;

		if (IsSemaphore())
			((SharedEventSemaphoreImpl&)m_data).Set();
		else
			((SharedEventDefaultImpl&)m_data).Set();
	}

	bool SharedEvent::IsSet(u32 timeOutMs)
	{
		if (*((u8*)m_data) == 0)
			return false;

		if (IsSemaphore())
			return ((SharedEventSemaphoreImpl&)m_data).IsSet(timeOutMs);
		else
			return ((SharedEventDefaultImpl&)m_data).IsSet(timeOutMs);
	}

	bool SharedEvent::IsSemaphore()
	{
		return *((u8*)m_data) == 2;
	}

	void SharedEvent::ToString(StringBufferBase& out)
	{
		UBA_ASSERT(IsSemaphore());
		((SharedEventSemaphoreImpl&)m_data).ToString(out);
	}

	void* SharedEvent::GetHandle()
	{
		UBA_ASSERT(!IsSemaphore());
		#if PLATFORM_WINDOWS || PLATFORM_LINUX
		return ((SharedEventDefaultImpl&)m_data).GetHandle();
		#else
		UBA_ASSERTF(false, "Event::GetHandle not available");
		return nullptr;
		#endif
	}
}
