// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaDefinitions.h"

//#if UBA_USE_NATIVE_MAC_SEMAPHORES
#if PLATFORM_LINUX
#include <semaphore.h>
#elif PLATFORM_MAC
#include <dispatch/dispatch.h>
#endif

namespace uba
{
	struct EventDispatchSemaphore : EventHeader
	{
		EventDispatchSemaphore() = default;

		EventDispatchSemaphore(EventResetType resetType)
		{
			Create(resetType);
		}

		~EventDispatchSemaphore()
		{
			Destroy();
		}

		void Create(EventResetType resetType)
		{
			m_resetType.value = resetType;
			m_triggered = TriggerType_None;

			#if PLATFORM_WINDOWS
			m_sem = CreateSemaphoreExW(nullptr, 0, 100000, nullptr, 0, SEMAPHORE_MODIFY_STATE | SYNCHRONIZE);
			#elif PLATFORM_LINUX
			m_sem = new sem_t;
			sem_init(m_sem, 0, 0);
			#else
			m_sem = dispatch_semaphore_create(0);
			#endif
 		}

		void Destroy()
		{
			if (m_sem)
			{
				#if PLATFORM_WINDOWS
				//CloseHandle(m_sem);
				#elif PLATFORM_LINUX
				sem_destroy(m_sem);
				delete m_sem;
				#else
				//dispatch_release(m_sem);
				#endif
				m_sem = 0;
			}
		}

		void Set()
		{
			UBA_ASSERT(m_sem);

			SCOPED_FUTEX(m_lock, lock);

			if (m_resetType.value == EventResetType_Manual)
			{
				m_triggered = TriggerType_All;

				// Release semaphore for all waiters... also fine releasing semaphore more than needed since it is a manual reset.
				for (Waiter* it=m_oldestWaiter; it; it = it->next)
					Signal();
			}
			else
			{
				if (m_oldestWaiter)
					Signal();
				else
					m_triggered = TriggerType_One;
			}
		}

		void Reset()
		{
			UBA_ASSERT(m_sem);

			UBA_ASSERT(!m_oldestWaiter);

			Drain();

			m_triggered = TriggerType_None;
		}

		bool IsSet(u64 timeoutMs = ~0u)
		{
			UBA_ASSERT(m_sem);

			SCOPED_FUTEX(m_lock, lock);
			if (m_triggered == TriggerType_All)
				return true;

			if (timeoutMs == 0 && m_triggered == TriggerType_None)
				return false;

			Waiter waiter;
			if (m_newestWaiter)
			{
				m_newestWaiter->next = &waiter;
				waiter.prev = m_newestWaiter;
			}
			else if (m_triggered == TriggerType_One) // If we are the first waiter and one is set, we take that trigger
			{
				m_triggered = TriggerType_None;
				return true;
			}
			else
				m_oldestWaiter = &waiter;

			m_newestWaiter = &waiter;

			// Unlock and wait for semaphore
			lock.Leave();

			bool success = Wait(timeoutMs);

			// Take lock and disconnect waiter
			lock.Enter();

			if (waiter.prev)
				waiter.prev->next = waiter.next;
			else
				m_oldestWaiter = waiter.next;

			if (waiter.next)
				waiter.next->prev = waiter.prev;
			else
				m_newestWaiter = waiter.prev;

			return success || m_triggered == TriggerType_All;
		}

		void Signal()
		{
			#if PLATFORM_WINDOWS
			LONG prev = 0;
			ReleaseSemaphore(m_sem, 1, &prev);
			#elif PLATFORM_LINUX
			sem_post(m_sem);
			#else
			dispatch_semaphore_signal(m_sem);
			#endif
		}

		bool Wait(u64 timeoutMs)
		{
			#if PLATFORM_WINDOWS
			DWORD timeout = INFINITE;
			if (timeoutMs != ~0u)
				timeout = DWORD(timeoutMs);
			
			DWORD r = WaitForSingleObject(m_sem, timeout);
			return r == WAIT_OBJECT_0;
			#elif PLATFORM_LINUX
			if (timeoutMs == ~0u)
				timeoutMs = 24llu*60*60*1000; // 24hrs
			u64 timeoutNs = u64(timeoutMs) * 1'000'000;
			struct timespec now;
			clock_gettime(CLOCK_REALTIME, &now);
			u64 absTimeNano = ToNanoSeconds(now) + timeoutNs;
			auto absTime = ToTimeSpecFromNano<timespec>(absTimeNano);
			while (sem_timedwait(m_sem, &absTime) == -1)
			{
				if (errno == EINTR)
					continue;
				UBA_ASSERTF(errno == ETIMEDOUT, "sem_wait failed");
				return false;
			}
			return true;
			#else
			if (timeoutMs == ~0u)
				timeoutMs = 24llu*60*60*1000; // 24hrs
			dispatch_time_t t = dispatch_time(DISPATCH_TIME_NOW, (int64_t)timeoutMs * 1000000ll);
			return dispatch_semaphore_wait(m_sem, t) == 0;
			#endif
		}

		void Drain()
		{
			#if PLATFORM_WINDOWS
			while (WaitForSingleObject(m_sem, 0) == WAIT_OBJECT_0)
			#elif PLATFORM_LINUX
			while (sem_trywait(m_sem) == 0)
			#else
			while (dispatch_semaphore_wait(m_sem, DISPATCH_TIME_NOW) == 0)
			#endif
			{				
			}
		}

		void* GetHandle()
		{
			return nullptr;
		}

		struct Waiter
		{
			Waiter* next = nullptr;
			Waiter* prev = nullptr;
		};
		Waiter* m_oldestWaiter = nullptr;
		Waiter* m_newestWaiter = nullptr;

		enum TriggerType : u8 { TriggerType_None, TriggerType_One, TriggerType_All };

		TriggerType m_triggered = TriggerType_None;

		Futex m_lock;

#if PLATFORM_WINDOWS
		HANDLE m_sem = 0;
#elif PLATFORM_LINUX
		sem_t* m_sem;
#else
		dispatch_semaphore_t m_sem = 0;
#endif
	};
}

//#endif
