// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaDefinitions.h"

#if UBA_USE_NATIVE_MAC_SEMAPHORES
#include <mach/mach.h>
#include <mach/semaphore.h>

namespace uba
{
	struct EventMachSemaphore : EventHeader
	{
		~EventMachSemaphore()
		{
			Destroy();
		}

		bool Create(StringView name, bool isOwner)
		{
			UBA_ASSERT(!m_sem && !m_isOwner);
			if (isOwner)
			{
				kern_return_t kr;
				kr = semaphore_create(mach_task_self(), &m_sem, SYNC_POLICY_FIFO, 0);
				UBA_ASSERT(kr == KERN_SUCCESS);
				mach_port_insert_right(mach_task_self(), m_sem, m_sem, MACH_MSG_TYPE_MAKE_SEND);
				m_isOwner = true;
				//printf("CREATE OWNER WITH %u\n", m_sem);
			}
			else
			{
				u64 sem;
				Parse(sem, name.data, name.count);
				m_sem = (semaphore_t)sem;
				//printf("CREATE CLIENT WITH %u\n", m_sem);
			}
			return true;
		}

		void Destroy()
		{
			if (!m_sem)
				return;
			if (m_isOwner)
				semaphore_destroy(mach_task_self(), m_sem);
		}

		void Set()
		{
			semaphore_signal(m_sem);
		}

		bool IsSet(u32 timeoutMs = ~0u)
		{
			if (timeoutMs == ~0u)
				return semaphore_wait(m_sem) == KERN_SUCCESS;
			mach_timespec_t ts = ToTimeSpecFromNano<mach_timespec_t>(u64(timeoutMs) * 1'000'000);
			kern_return_t kr = semaphore_timedwait(m_sem, ts);
			return kr == KERN_SUCCESS;
		}

		void ToString(StringBufferBase& out)
		{
			out.AppendValue(m_sem);
		}

		bool m_isOwner = false;
		semaphore_t m_sem = MACH_PORT_NULL;
	};
}

#endif
