// Copyright Epic Games, Inc. All Rights Reserved.

#if !PLATFORM_WINDOWS

#include <sys/time.h>
#include <new>
#include <semaphore.h>

namespace uba
{
	struct EventPosixSemaphore : EventHeader
	{
	public:
		~EventPosixSemaphore()
		{
			Destroy();
		}

		bool Create(StringView name, bool isOwner)
		{
			UBA_ASSERT(!m_snd && !m_isOwner);
			StringBuffer<> n1(name);
			StringBuffer<> n2(name);
			n1.Append(TCV("_cl"));
			n2.Append(TCV("_sv"));
		
			const tchar* sndName = n1.data;
			const tchar* rvcName = n2.data;
			int flags = 0;
			mode_t mode = 0600;

			if (isOwner)
			{
				flags |= O_CREAT;
				sndName = n2.data;
				rvcName = n1.data;
				m_name = TStrdup(name.data);
				m_isOwner = true;
			};

			m_snd = sem_open(sndName, flags, mode, 0);
			m_rcv = sem_open(rvcName, flags, mode, 0);

			UBA_ASSERTF(m_snd != SEM_FAILED, "sem_open failed");
			UBA_ASSERTF(m_rcv != SEM_FAILED, "sem_open failed");
			return true;
		}

		void Destroy()
		{
			if (!m_snd)
				return;
			sem_close(m_snd);
			m_snd = nullptr;

			sem_close(m_rcv);
			m_rcv = nullptr;

			if (m_isOwner)
			{
				StringBuffer<> n1(m_name);
				StringBuffer<> n2(m_name);
				n1.Append(TCV("_cl"));
				n2.Append(TCV("_sv"));
				sem_unlink(n1.data);
				sem_unlink(n2.data);
				free(m_name);
				m_isOwner = false;
			}
		}

		void Set()
		{
			if (!sem_post(m_snd))
				return;
			UBA_ASSERTF(false, "sem_post failed");
		}

		bool IsSet(u32 timeoutMs = ~0u)
		{
			if (timeoutMs == ~0u)
			{
				while (sem_wait(m_rcv) == -1)
				{
					if (errno == EINTR)
						continue;
					UBA_ASSERTF(false, "sem_wait failed");
				}
				return errno == 0;
			}

			u64 timeoutNs = u64(timeoutMs) * 1'000'000;

			struct timespec now;
			clock_gettime(CLOCK_REALTIME, &now);

			u64 absTimeNano = ToNanoSeconds(now) + timeoutNs;

			#if PLATFORM_LINUX
			auto absTime = ToTimeSpecFromNano<timespec>(absTimeNano);
			while (sem_timedwait(m_rcv, &absTime) == -1)
			{
				if (errno == EINTR)
					continue;
				if (errno == ETIMEDOUT)
					return false;
				UBA_ASSERTF(false, "sem_wait failed");
			}
			#else

			u64 sleepNs = 50 * 1000; // 50 µs
			const u64 sleepCapNs = 10 * 1000 * 1000; // 10 ms

			while (true)
			{
				if (sem_trywait(m_rcv) == 0)
					return true;
				if (errno != EAGAIN)
					return false;

				clock_gettime(CLOCK_REALTIME, &now);
				u64 nowNano = ToNanoSeconds(now);
				if (nowNano > absTimeNano)
					return false;

				u64 remaining = absTimeNano - nowNano;
				u64 nap = sleepNs < remaining ? sleepNs : remaining;
				auto deltaNs = ToTimeSpecFromNano<timespec>(nap);
				nanosleep(&deltaNs, nullptr);

				if (sleepNs < sleepCapNs)
				{
					sleepNs <<= 1;
					if (sleepNs > sleepCapNs)
						sleepNs = sleepCapNs;
				}
			}
			#endif

			return true;
		}

		void ToString(StringBufferBase& out)
		{
			out.Append(m_name);
		}

	private:
		bool m_isOwner = false;
		sem_t* m_snd = nullptr;
		sem_t* m_rcv = nullptr;
		tchar* m_name = nullptr;
	};
}

#endif