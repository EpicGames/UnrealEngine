// Copyright Epic Games, Inc. All Rights Reserved.

#if PLATFORM_MAC

#include <errno.h>
#include <stdint.h>
#include <time.h>

// Darwin private SPI. Stable since macOS 10.12 (__ulock_wait) and 11.0
// (__ulock_wait2) — the same kernel primitive that os_unfair_lock, libc++'s
// std::atomic<T>::wait(), libdispatch, and libpthread's cond-var lower to.
// <sys/ulock.h> is not in the public SDK so we declare the prototypes here.
extern "C"
{
	int __ulock_wait2(uint32_t operation, void* addr, uint64_t value, uint64_t timeout_ns, uint64_t value2);
	int __ulock_wake (uint32_t operation, void* addr, uint64_t wake_value);
}

// Ops from xnu/bsd/sys/ulock.h. The SHARED variant keys waits on
// <vm_object, offset> rather than <task, address>, so cross-process waiters
// on the same pager-backed (SHM) page wake each other — the same role
// FUTEX_WAIT without FUTEX_PRIVATE_FLAG plays on Linux.
#ifndef UL_COMPARE_AND_WAIT_SHARED
#define UL_COMPARE_AND_WAIT_SHARED 3
#endif
#ifndef ULF_WAKE_ALL
#define ULF_WAKE_ALL 0x00000100
#endif
#ifndef ULF_NO_ERRNO
#define ULF_NO_ERRNO 0x01000000
#endif

namespace uba
{
	// EventUlock
	//
	// macOS counterpart to EventFutex (Linux). Built on __ulock_wait2 /
	// __ulock_wake with UL_COMPARE_AND_WAIT_SHARED so an Event in shared
	// memory can be waited on in one process and Set() in another.
	//
	// The field layout, state machine, and memory ordering mirror EventFutex
	// exactly. The SharedEvent 13-u64 slot is therefore bit-compatible
	// between Linux (EventFutex) and Mac (EventUlock), keeping the
	// DetoursStatic stub's raw-syscall view valid on both OSes if/when the
	// stub grows a Mac ulock backend.
	//
	// State machine (single u32 primary word `m_state`):
	//
	//   Reset     = 0  (no waiters)
	//   Signaled  = 2  (manual-reset only; Set was called)
	//   Destroyed = 3  (Destroy() was called; all waits return false)
	//
	// Auto-reset uses a second u32 `m_autoCount` as both the wait word and
	// a monotonic token counter — exactly as EventFutex does — so Set()
	// becomes a single fetch_add + wake-one, and the waiter consumes one
	// token via CAS.
	//
	// Notes on __ulock_wait2:
	//  - The comparison value is the *expected* u32 value at *addr. The
	//    kernel re-checks atomically after enqueueing the waiter, closing
	//    the classic load-then-wait race (same contract as FUTEX_WAIT).
	//  - timeout_ns == 0 means infinite.
	//  - With ULF_NO_ERRNO the return value is >= 0 on success (roughly
	//    the remaining-waiters count) or -errno on error, without touching
	//    the thread's errno. We use that everywhere to avoid errno TLS
	//    traffic in hot paths.

	enum EventUlockState : u32
	{
		EventUlockState_Reset     = 0,
		EventUlockState_Signaled  = 2,
		EventUlockState_Destroyed = 3,
	};

	struct EventUlock : EventHeader
	{
		EventUlock() = default;
		~EventUlock() { Destroy(); }

		bool Create(EventResetType rt, bool /*shared*/ = true)
		{
			static_assert(Atomic<u32>::is_always_lock_free, "atomic<u32> not lock free; can't live in shared mem");

			UBA_ASSERTF(m_initialized.load(std::memory_order_acquire) == 0, "EventUlock already created");

			m_resetType.value = rt;
			m_state.store(EventUlockState_Reset, std::memory_order_relaxed);
			m_autoCount.store(0, std::memory_order_relaxed);
			m_initialized.store(1, std::memory_order_release);
			// Wake any cross-process waiter parked in IsSet() that observed
			// m_initialized==0 before we got here. Common case (no waiters)
			// is one cheap syscall on the host's process-create path.
			WakeAll(&m_initialized);
			return true;
		}

		void Destroy()
		{
			if (m_initialized.exchange(0, std::memory_order_acq_rel) == 0)
				return;

			// Publish Destroyed and wake every waiter so their loops exit
			// with `false`. SHARED op so waiters in peer processes wake too.
			m_state.store(EventUlockState_Destroyed, std::memory_order_release);
			WakeAll(&m_state);

			// Also kick the auto-reset counter in case a waiter is parked there.
			m_autoCount.fetch_add(1, std::memory_order_release);
			WakeAll(&m_autoCount);

			// Wake anyone parked on m_initialized in IsSet's pre-init wait —
			// they re-check m_state and return false (Destroyed).
			WakeAll(&m_initialized);
		}

		void Set()
		{
			if (m_initialized.load(std::memory_order_acquire) == 0)
				return;

			if (m_resetType.value == EventResetType_Manual)
			{
				// Manual reset: latch the signal, release everybody.
				u32 prev = m_state.exchange(EventUlockState_Signaled, std::memory_order_release);
				if (prev != EventUlockState_Signaled)
					WakeAll(&m_state);
				return;
			}

			// Auto reset: publish a new token, hand it to one waiter.
			m_autoCount.fetch_add(1, std::memory_order_release);
			WakeOne(&m_autoCount);
		}

		void Reset()
		{
			if (m_initialized.load(std::memory_order_acquire) == 0)
				return;

			if (m_resetType.value == EventResetType_Manual)
			{
				u32 expected = EventUlockState_Signaled;
				m_state.compare_exchange_strong(expected, EventUlockState_Reset,
					std::memory_order_acq_rel, std::memory_order_relaxed);
			}
			else
			{
				m_autoCount.store(0, std::memory_order_release);
			}
		}

		bool IsSet(u32 timeoutMs = ~0u)
		{
			// Cross-process startup race: a waiter can observe shared memory
			// before the producer's Create() store has propagated. Returning
			// false instantly here turns callers like BinaryWriter::Flush into
			// 100%-CPU spin loops. Park on m_initialized so we burn no CPU.
			//
			// Crucially we do NOT check m_state==Destroyed in this pre-init
			// wait — if the slot was reused, m_state still reads as Destroyed
			// from the previous owner's Destroy() until the new Create()'s
			// (relaxed) store of Reset is observed. Bailing on stale Destroyed
			// is what was causing the spin. Once m_initialized==1, the inner
			// WaitManual/WaitAuto's Destroyed check covers genuine destruction.
			// A genuinely-destroyed event that never gets re-created will exit
			// here via timeout, which is the right behavior for a wait.
			if (m_initialized.load(std::memory_order_acquire) == 0)
			{
				u64 deadlineNs = 0;
				if (timeoutMs != ~0u && timeoutMs != 0)
					deadlineNs = MonotonicNs() + u64(timeoutMs) * 1'000'000ull;

				while (m_initialized.load(std::memory_order_acquire) == 0)
				{
					if (timeoutMs == 0)
						return false;

					u64 timeoutNs = 0;
					if (timeoutMs != ~0u)
					{
						u64 nowNs = MonotonicNs();
						if (nowNs >= deadlineNs)
							return false;
						timeoutNs = deadlineNs - nowNs;
					}

					int r = __ulock_wait2(UL_COMPARE_AND_WAIT_SHARED | ULF_NO_ERRNO,
						&m_initialized, 0ull, timeoutNs, 0);
					if (r >= 0)
						continue;
					int e = -r;
					if (e == ETIMEDOUT)
						return false;
					if (e == EINTR || e == EFAULT)
						continue;
				}

				// Adjust remaining timeout for the inner wait.
				if (timeoutMs != ~0u && timeoutMs != 0)
				{
					u64 nowNs = MonotonicNs();
					timeoutMs = nowNs >= deadlineNs ? 0u : u32((deadlineNs - nowNs) / 1'000'000ull);
				}
			}

			if (m_resetType.value == EventResetType_Manual)
				return WaitManual(timeoutMs);
			else
				return WaitAuto(timeoutMs);
		}

		void* GetHandle()
		{
			return nullptr;
		}

	private:
		bool WaitManual(u32 timeoutMs)
		{
			u64 deadlineNs = 0;
			if (timeoutMs != ~0u && timeoutMs != 0)
				deadlineNs = MonotonicNs() + u64(timeoutMs) * 1'000'000ull;

			for (;;)
			{
				u32 s = m_state.load(std::memory_order_acquire);
				if (s == EventUlockState_Signaled)
					return true;
				if (s == EventUlockState_Destroyed)
					return false;

				if (timeoutMs == 0)
					return false;

				u64 timeoutNs = 0; // 0 == infinite for __ulock_wait2
				if (timeoutMs != ~0u)
				{
					u64 nowNs = MonotonicNs();
					if (nowNs >= deadlineNs)
						return false;
					timeoutNs = deadlineNs - nowNs;
				}

				// Kernel atomically re-checks *m_state == s, then parks.
				int r = __ulock_wait2(UL_COMPARE_AND_WAIT_SHARED | ULF_NO_ERRNO,
					&m_state, u64(s), timeoutNs, 0);
				if (r >= 0)
					continue; // woken, or value already changed — re-check state
				int e = -r;
				if (e == ETIMEDOUT)
					return false;
				if (e == EINTR || e == EFAULT)
					continue;
				// Unknown error: fall through and re-check state.
			}
		}

		bool WaitAuto(u32 timeoutMs)
		{
			u64 deadlineNs = 0;
			if (timeoutMs != ~0u && timeoutMs != 0)
				deadlineNs = MonotonicNs() + u64(timeoutMs) * 1'000'000ull;

			for (;;)
			{
				// Must observe Destroyed on every pass, else a dead event
				// could put us to sleep forever.
				if (m_state.load(std::memory_order_acquire) == EventUlockState_Destroyed)
					return false;

				u32 c = m_autoCount.load(std::memory_order_acquire);
				if (c != 0)
				{
					// Try to consume one token.
					if (m_autoCount.compare_exchange_strong(c, c - 1,
						std::memory_order_acquire, std::memory_order_relaxed))
					{
						return true;
					}
					// Lost the race to another waiter: retry from the top.
					continue;
				}

				if (timeoutMs == 0)
					return false;

				u64 timeoutNs = 0;
				if (timeoutMs != ~0u)
				{
					u64 nowNs = MonotonicNs();
					if (nowNs >= deadlineNs)
						return false;
					timeoutNs = deadlineNs - nowNs;
				}

				int r = __ulock_wait2(UL_COMPARE_AND_WAIT_SHARED | ULF_NO_ERRNO,
					&m_autoCount, 0ull, timeoutNs, 0);
				if (r >= 0)
					continue;
				int e = -r;
				if (e == ETIMEDOUT)
					return false;
				if (e == EINTR || e == EFAULT)
					continue;
			}
		}

		static void WakeOne(Atomic<u32>* addr)
		{
			__ulock_wake(UL_COMPARE_AND_WAIT_SHARED | ULF_NO_ERRNO, addr, 0);
		}

		static void WakeAll(Atomic<u32>* addr)
		{
			__ulock_wake(UL_COMPARE_AND_WAIT_SHARED | ULF_WAKE_ALL | ULF_NO_ERRNO, addr, 0);
		}

		static u64 MonotonicNs()
		{
			struct timespec ts;
			clock_gettime(CLOCK_MONOTONIC, &ts);
			return u64(ts.tv_sec) * 1'000'000'000ull + u64(ts.tv_nsec);
		}

	private:
		// EventHeader occupies the first 8 bytes (tag byte + m_resetType).
		// All three atomics are u32 so __ulock_wait2 can wait on each. Bool-
		// padding would have left the dword at &m_initialized with undefined
		// upper bytes, which UL_COMPARE_AND_WAIT can't compare against zero.
		Atomic<u32>  m_initialized { 0 };
		Atomic<u32>  m_state       { EventUlockState_Reset };
		Atomic<u32>  m_autoCount   { 0 };
	};

	// Sanity: must fit in the shared 13-u64 slot and match EventFutex's size
	// so SharedEvent storage is cross-OS bit-compatible at this tag.
	static_assert(sizeof(EventUlock) <= 13 * sizeof(u64), "EventUlock too large for SharedEvent::m_data");
}

#endif // PLATFORM_MAC
