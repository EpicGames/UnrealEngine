// Copyright Epic Games, Inc. All Rights Reserved.

#if PLATFORM_WINDOWS

namespace uba
{
	struct EventWin32 : EventHeader
	{
		HANDLE m_handle;

		bool Create(EventResetType rt, bool = true)
		{
			m_resetType.value = rt;
			m_handle = CreateEvent(nullptr, rt == EventResetType_Manual, false, NULL);
			return m_handle != 0;
		}

		void Destroy()
		{
			CloseHandle(m_handle);
			m_handle = 0;
		}

		void Set()
		{
			SetEvent(m_handle);
		}

		void Reset()
		{
			ResetEvent(m_handle);
		}

		bool IsSet(u32 timeOutMs)
		{
			return WaitForSingleObject(m_handle, timeOutMs) == 0;
		}

		bool Create(StringView name, bool isOwner)
		{
			UBA_ASSERT(false);
			return false;
		}

		void ToString(StringBufferBase& out)
		{
			UBA_ASSERT(false);
		}

		void* GetHandle()
		{
			return m_handle;
		}
	};

}

#endif