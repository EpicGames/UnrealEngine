// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaHash.h"
#include <blake3.h>

namespace uba
{
	StringKeyHasher::StringKeyHasher()
	{
		static_assert(sizeof(hasher) == sizeof(blake3_hasher));
		blake3_hasher_init((blake3_hasher*)&hasher);
	}

	void StringKeyHasher::Update(const StringView& str)
	{
		Update(str.data, str.count);
	}
		
	void StringKeyHasher::Update(const tchar* str, u64 strLen)
	{
		CHECK_PATH(StringView(str, u32(strLen)));
		if (strLen != 0)
			blake3_hasher_update((blake3_hasher*)&hasher, str, strLen * sizeof(tchar));
	}

	void StringKeyHasher::UpdateNoCheck(const StringView& str)
	{
		if (str.count != 0)
			blake3_hasher_update((blake3_hasher*)&hasher, str.data, str.count * sizeof(tchar));
	}

	StringKey ToStringKey(const tchar* str, u64 strLen)
	{
		CHECK_PATH(StringView(str, u32(strLen)));
		blake3_hasher hasher;
		blake3_hasher_init(&hasher);
		blake3_hasher_update(&hasher, str, strLen * sizeof(tchar));
		uint8_t output[BLAKE3_OUT_LEN];
		blake3_hasher_finalize(&hasher, output, BLAKE3_OUT_LEN);
		return (StringKey&)output;
	}

	StringKey ToStringKey(const StringView& b)
	{
		return ToStringKey(b.data, b.count);
	}

#if !UBA_STUB_BUILD
	StringKey ToStringKeyLower(const tchar* str, u64 strLen)
	{
		StringBuffer<> temp;
		temp.Append(str, strLen).MakeLower();
		return ToStringKey(temp.data, temp.count);
	}

	StringKey ToStringKeyLower(const StringView& b)
	{
		return ToStringKeyLower(b.data, b.count);
	}
#endif
	StringKey ToStringKey(const StringKeyHasher& hasher, const tchar* str, u64 strLen)
	{
		CHECK_PATH(StringView(str, u32(strLen)));
		StringKeyHasher temp(hasher);
		blake3_hasher_update((blake3_hasher*)&temp.hasher, str, strLen * sizeof(tchar));
		uint8_t output[BLAKE3_OUT_LEN];
		blake3_hasher_finalize((blake3_hasher*)&temp.hasher, output, BLAKE3_OUT_LEN);
		return (StringKey&)output;
	}

	StringKey ToStringKey(const StringKeyHasher& hasher)
	{
		StringKeyHasher temp(hasher);
		uint8_t output[BLAKE3_OUT_LEN];
		blake3_hasher_finalize((blake3_hasher*)&temp.hasher, output, BLAKE3_OUT_LEN);
		return (StringKey&)output;
	}

	StringKey ToStringKeyNoCheck(const tchar* str, u64 strLen)
	{
		blake3_hasher hasher;
		blake3_hasher_init(&hasher);
		blake3_hasher_update(&hasher, str, strLen * sizeof(tchar));
		uint8_t output[BLAKE3_OUT_LEN];
		blake3_hasher_finalize(&hasher, output, BLAKE3_OUT_LEN);
		return (StringKey&)output;
	}

	StringKey ToStringKeyRaw(const void* data, u64 dataLen)
	{
		blake3_hasher hasher;
		blake3_hasher_init(&hasher);
		blake3_hasher_update(&hasher, data, dataLen);
		uint8_t output[BLAKE3_OUT_LEN];
		blake3_hasher_finalize(&hasher, output, BLAKE3_OUT_LEN);
		return (StringKey&)output;
	}

	CasKeyHasher::CasKeyHasher()
	{
		static_assert(sizeof(hasher) == sizeof(blake3_hasher));
		blake3_hasher_init((blake3_hasher*)&hasher);
	}

	CasKeyHasher& CasKeyHasher::Update(const void* data, u64 bytes)
	{
		blake3_hasher_update((blake3_hasher*)&hasher, data, bytes);
		return *this;
	}

	CasKey ToCasKey(const CasKeyHasher& hasher, bool compressed)
	{
		CasKeyHasher temp(hasher);
		uint8_t output[BLAKE3_OUT_LEN];
		blake3_hasher_finalize((blake3_hasher*)&temp.hasher, output, BLAKE3_OUT_LEN);
		output[19] = compressed ? IsCompressedMask : EmptyMask;
		return (CasKey&)output;
	}

	void BloomFilter::Add(const StringKey& key)
	{
		auto setBit = [&](u32 hash)
			{
				u32 bit = hash % (sizeof(bytes) * 8);
				bytes[bit / 8] |= (1 << (bit % 8));
			};
		setBit(u32(key.a));
		setBit(u32(key.a >> 32));
		setBit(u32(key.b));
		setBit(u32(key.b >> 32));
	}

	bool BloomFilter::IsGuaranteedMiss(const StringKey& key) const
	{
		auto notSet = [&](u32 hash) -> bool
			{
				u32 bit = hash % (sizeof(bytes) * 8);
				return (bytes[bit / 8] & (1 << (bit % 8))) == 0;
			};

		return notSet(u32(key.a))
			|| notSet(u32(key.a >> 32))
			|| notSet(u32(key.b))
			|| notSet(u32(key.b >> 32));
	}

	constexpr BloomFilter g_empty;

	bool BloomFilter::IsEmpty() const
	{
		return memcmp(g_empty.bytes, bytes, sizeof(bytes)) == 0;
	}
}
