// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaMemory.h"
#include "UbaPlatform.h"
#include "UbaStringBuffer.h"

#if PLATFORM_LINUX
#include <linux/mman.h>
#if defined(MAP_HUGE_2MB)
#define UBA_SUPPORTS_HUGE_PAGES 1
#endif
#endif

#if !defined(UBA_SUPPORTS_HUGE_PAGES)
#define UBA_SUPPORTS_HUGE_PAGES 0
#endif

namespace uba
{
	constexpr u64 MemoryBlock_ReserveAlign = 1 * 1024 * 1024;


	MemoryBlock::MemoryBlock(const tchar* name_, u64 reserveSize_, void* baseAddress_)
	{
		name = name_;
		Init(reserveSize_, baseAddress_, false);
	}

	MemoryBlock::MemoryBlock(const tchar* name_, u8* baseAddress_)
	{
		name = name_;
		memory = baseAddress_;
	}

	MemoryBlock::~MemoryBlock()
	{
		Deinit();
	}

	bool MemoryBlock::Init(u64 reserveSize_, void* baseAddress_, bool useHugePages)
	{
		#if PLATFORM_WINDOWS
		reserveSize = AlignUp(reserveSize_, MemoryBlock_ReserveAlign);
		memory = (u8*)VirtualAlloc(baseAddress_, reserveSize, MEM_RESERVE, PAGE_READWRITE); // Max size of obj file?
		if (!memory)
			FatalError(1347, TC("Failed to reserve virtual memory in %s (%u)"), name, GetLastError());
		#else

		int flags = MAP_PRIVATE | MAP_ANONYMOUS;
		int reserveAlign = MemoryBlock_ReserveAlign;
		#if UBA_SUPPORTS_HUGE_PAGES
		if (useHugePages)
		{
			flags |= MAP_HUGETLB | MAP_HUGE_2MB;
			reserveAlign = 2 * 1024 * 1024;
		}
		#endif

		reserveSize = AlignUp(reserveSize_, reserveAlign);
		memory = (u8*)mmap(baseAddress_, reserveSize, PROT_READ | PROT_WRITE, flags, -1, 0);

		if (memory == MAP_FAILED)
		{
			if (useHugePages)
				return false;
			FatalError(1347, "mmap failed to reserve %llu bytes in %s (asking for %llu) (%s)", reserveSize, name, reserveSize_, strerror(errno));
		}
		#endif

		if (baseAddress_ && baseAddress_ != memory)
			FatalError(9881, TC("Failed to reserve virtual memory at specific address in %s (%u)"), name, GetLastError());
		return true;
	}

	void MemoryBlock::Deinit()
	{
		if (!memory)
			return;

		#if PLATFORM_WINDOWS
		VirtualFree(memory, 0, MEM_RELEASE);
		#else
		if (munmap(memory, reserveSize) == -1)
			FatalError(9885, "munmap failed to free %llu bytes in %s (%s)", reserveSize, name, strerror(errno));
		#endif
		memory = nullptr;
	}

	void* MemoryBlock::Allocate(u64 bytes, u64 alignment, const tchar* hint)
	{
		if (!memory)
			return aligned_alloc(alignment, bytes);

		SCOPED_FUTEX(lock, l);
		return AllocateNoLock(bytes, alignment, hint);
	}

	void* MemoryBlock::AllocateNoLock(u64 bytes, u64 alignment, const tchar* hint, bool lockMemory)
	{
		u64 startPos = AlignUp(writtenSize, alignment);
		u64 newPos = startPos + bytes;
		
		if (newPos > reserveSize)
			FatalError(9882, TC("Ran out of reserved virtual address space in %s. Requesting %llu, Alignment %llu, Reserved %llu, Need %llu (%s)"), name, bytes, alignment, reserveSize, newPos, hint);

		#if PLATFORM_WINDOWS
		if (newPos > committedSize)
		{
			u64 toCommit = AlignUp(newPos - committedSize, MemoryBlock_ReserveAlign);
			if (committedSize + toCommit > reserveSize)
				toCommit = reserveSize - committedSize;
			void* addr = memory + committedSize;
			if (!VirtualAlloc(addr, toCommit, MEM_COMMIT, PAGE_READWRITE))
				FatalError(9883, TC("Failed to commit virtual memory for memory block in %s. Total size %llu (%u) (%s)"), name, committedSize + toCommit, GetLastError(), hint);
			if (lockMemory)
				VirtualLock(addr, toCommit);
			committedSize += toCommit;
		}
		#endif

		void* ret = memory + startPos;
		writtenSize = newPos;
		return ret;
	}

	void* MemoryBlock::CommitNoLock(u64 bytes, const tchar* hint)
	{
		#if PLATFORM_WINDOWS
		u64 newPos = Min(writtenSize + bytes, reserveSize);
		if (newPos <= committedSize)
			return memory + writtenSize;
		u64 toCommit = AlignUp(newPos - committedSize, MemoryBlock_ReserveAlign);
		if (committedSize + toCommit > reserveSize)
			toCommit = reserveSize - committedSize;
		if (!VirtualAlloc(memory + committedSize, toCommit, MEM_COMMIT, PAGE_READWRITE))
			FatalError(9883, TC("Failed to commit virtual memory for memory block in %s. Total size %llu (%u) (%s)"), name, committedSize + toCommit, GetLastError(), hint);
		committedSize += toCommit;
		#endif
		return memory + writtenSize;
	}

	bool MemoryBlock::TryGrowTail(void* p, u64 oldBytes, u64 newBytes)
	{
		if (!memory)
			return false;
		SCOPED_FUTEX(lock, l);
		return TryGrowTailNoLock(p, oldBytes, newBytes);
	}

	bool MemoryBlock::TryGrowTailNoLock(void* p, u64 oldBytes, u64 newBytes)
	{
		// Only succeed when the allocation ends exactly at the bump tail.
		if ((u8*)p + oldBytes != memory + writtenSize)
			return false;
		if (newBytes <= oldBytes)
			return true;
		u64 delta = newBytes - oldBytes;
		u64 newPos = writtenSize + delta;
		if (newPos > reserveSize)
			return false;

		#if PLATFORM_WINDOWS
		if (newPos > committedSize)
		{
			u64 toCommit = AlignUp(newPos - committedSize, MemoryBlock_ReserveAlign);
			if (committedSize + toCommit > reserveSize)
				toCommit = reserveSize - committedSize;
			if (!VirtualAlloc(memory + committedSize, toCommit, MEM_COMMIT, PAGE_READWRITE))
				return false;
			committedSize += toCommit;
		}
		#endif

		writtenSize = newPos;
		return true;
	}

	void MemoryBlock::Free(void* p)
	{
		if (!memory)
			aligned_free(p);
	}

	StringView MemoryBlock::Strdup(const StringView& str)
	{
		tchar* mem = (tchar*)Allocate((str.count + 1) * sizeof(tchar), alignof(tchar), TC("Strdup"));
		memcpy(mem, str.data, str.count * sizeof(tchar));
		mem[str.count] = 0;
		return StringView(mem, str.count);
	}

	tchar* MemoryBlock::Strdup(const tchar* str)
	{
		return (tchar*)Strdup(ToView(str)).data;
	}

	void MemoryBlock::Swap(MemoryBlock& other)
	{
		u8* m = memory;
		u64 rs = reserveSize;
		u64 ws = writtenSize;
		u64 ms = committedSize;

		memory = other.memory;
		reserveSize = other.reserveSize;
		writtenSize = other.writtenSize;
		committedSize = other.committedSize;

		other.memory = m;
		other.reserveSize = rs;
		other.writtenSize = ws;
		other.committedSize = ms;
	}

	bool SupportsHugePages()
	{
		return UBA_SUPPORTS_HUGE_PAGES;
	}
	u64 GetHugePageCount()
	{
#if UBA_SUPPORTS_HUGE_PAGES
		FILE* f = fopen("/proc/sys/vm/nr_hugepages", "r");
		if (!f)
			return 0;
		u32 pageCount = 0;
		fscanf(f, "%u", &pageCount);
		fclose(f);
		return pageCount;
#else
		return 0;
#endif
	}
}
