// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaParkingLot.h"

#if UBA_USE_PARKINGLOT

#include "UbaAlgorithm.h"
#include "UbaPlatform.h"
#include "UbaTimer.h"
#include "UbaVector.h"

#include "UbaStringBuffer.h" // for UBA_ASSERTF

#include <atomic>

#if PLATFORM_LINUX
#include <linux/futex.h>
#include <sys/syscall.h>
#include <time.h>
#include <unistd.h>
#endif

#if !(PLATFORM_WINDOWS || PLATFORM_LINUX)
#include <chrono>
#include <condition_variable>
#include <mutex>
#endif

namespace uba::ParkingLot::Private
{
#if PLATFORM_LINUX
	template<typename TimeType>
	inline static const struct timespec* MsToTimeSpec(TimeType timeMs, struct timespec& outTime)
	{
		if (timeMs != 0xffffffff)
		{
			TimeType sec = timeMs / 1'000;
			outTime.tv_sec = sec;
			outTime.tv_nsec = (timeMs - sec * 1'000) * 1'000'000;
			return &outTime;
		}
		return nullptr;
	}

	inline static u64 TimeSpecToMs(const struct timespec& time)
	{
		return u64(time.tv_sec) * 1'000 + u64(time.tv_nsec) / 1'000'000;
	}
#endif

	class PlatformManualResetEvent
	{
	public:
		PlatformManualResetEvent() = default;
		PlatformManualResetEvent(const PlatformManualResetEvent&) = delete;
		PlatformManualResetEvent& operator=(const PlatformManualResetEvent&) = delete;

		inline void Reset()
		{
			#if PLATFORM_WINDOWS || PLATFORM_LINUX
			state.store(0, std::memory_order_relaxed);
			#else
			std::unique_lock selfLock(lock);
			shouldWait = true;
			#endif
		}

		inline bool Poll()
		{
			#if PLATFORM_WINDOWS || PLATFORM_LINUX
			return !!state.load(std::memory_order_acquire);
			#else
			std::unique_lock selfLock(lock);
			return !shouldWait;
			#endif
		}

		inline void Wait()
		{
			#if PLATFORM_WINDOWS || PLATFORM_LINUX
			if (!state.load(std::memory_order_acquire))
				WaitSlow();
			#else
			std::unique_lock selfLock(lock);
			condition.wait(selfLock, [this] { return !shouldWait; });
			#endif
		}

		inline bool WaitFor(u32 waitTime)
		{
			#if PLATFORM_WINDOWS || PLATFORM_LINUX
			return state.load(std::memory_order_acquire) || WaitForSlow(waitTime);
			#else
			if (waitTime == 0xffffffff)
				return Wait(), true;

			std::unique_lock selfLock(lock);
			if (waitTime == 0)
				return !shouldWait;
			return condition.wait_for(selfLock, std::chrono::milliseconds(waitTime), [this] { return !shouldWait; });
			#endif
		}

		void Notify()
		{
			#if PLATFORM_WINDOWS || PLATFORM_LINUX
			if (state.exchange(1, std::memory_order_release) == 0)
			{
				#if PLATFORM_WINDOWS
				WakeByAddressSingle((void*)&state);
				#elif PLATFORM_LINUX
				syscall(SYS_futex, &state, FUTEX_WAKE_PRIVATE, INT_MAX, nullptr, nullptr, 0);
				#else
				#error
				#endif
			}
			#else
			// The lock must be held by Notify() until it is finished accessing its members, otherwise a waiting thread
			// may destroy the event after its wait but before Notify() has finished.
			std::unique_lock selfLock(lock);
			shouldWait = false;
			condition.notify_one();
			#endif
		}

	private:
		#if PLATFORM_WINDOWS || PLATFORM_LINUX
		UBA_NOINLINE void WaitSlow()
		{
			for (;;)
			{
				u32 localState = state.load(std::memory_order_acquire);
				if (localState)
					return;

				#if PLATFORM_WINDOWS
				WaitOnAddress(&state, &localState, sizeof(u32), INFINITE);
				#elif PLATFORM_LINUX
				syscall(SYS_futex, &state, FUTEX_WAIT_PRIVATE, 0, nullptr, nullptr, 0);
				#else
				#error
				#endif
			}
		}

		UBA_NOINLINE bool WaitForSlow(u32 waitTimeMs)
		{
			u32 localState = state.load(std::memory_order_acquire);
			if (localState || waitTimeMs == 0)
				return !!localState;

			#if PLATFORM_WINDOWS
			const bool timedOut = !WaitOnAddress(&state, &localState, sizeof(u32), waitTimeMs) && GetLastError() == ERROR_TIMEOUT;
			#elif PLATFORM_LINUX
			// FUTEX_WAIT takes a relative wait time.
			struct timespec waitTimeSpec;
			const struct timespec* waitTimeSpecPtr = MsToTimeSpec(waitTimeMs, waitTimeSpec);
			const bool timedOut = syscall(SYS_futex, &state, FUTEX_WAIT_PRIVATE, 0, waitTimeSpecPtr, nullptr, 0) == -1 && errno == ETIMEDOUT;
			#else
			#error
			#endif

			localState = state.load(std::memory_order_acquire);
			if (localState || timedOut)
				return !!localState;

			// Handle a spurious wake by waiting until the wait time has elapsed one more time because WaitForSlower
			// handles spurious wakes in a loop and avoids exceeding the originally requested wake time by more than
			// the typical variation due to scheduling imprecision.
			if (waitTimeMs == 0xffffffff)
				return WaitSlow(), true;
			return WaitForSlower(waitTimeMs, localState);
		}

		UBA_NOINLINE bool WaitForSlower(u32 waitTimeMs, u32 localState)
		{
			#if PLATFORM_WINDOWS
			const u64 endTimeMs = TimeToMs(GetTime()) + waitTimeMs;

			for (;;)
			{
				const bool timedOut = !WaitOnAddress(&state, &localState, sizeof(u32), waitTimeMs) && GetLastError() == ERROR_TIMEOUT;

				localState = state.load(std::memory_order_acquire);
				if (localState || timedOut)
					return !!localState;

				const u64 curTimeMs = TimeToMs(GetTime());
				if (endTimeMs <= curTimeMs)
					return false;

				waitTimeMs = u32(endTimeMs - curTimeMs);
			}
			#elif PLATFORM_LINUX
			struct timespec waitTimeSpec;
			if (clock_gettime(CLOCK_MONOTONIC, &waitTimeSpec) == -1)
				FatalError(1401, TC("clock_gettime(CLOCK_MONOTONIC) failed"));
			MsToTimeSpec(TimeSpecToMs(waitTimeSpec) + waitTimeMs, waitTimeSpec);

			for (;;)
			{
				if (state.load(std::memory_order_acquire))
					return true;

				// FUTEX_WAIT_BITSET takes an absolute wait time.
				if (syscall(SYS_futex, &state, FUTEX_WAIT_BITSET_PRIVATE, 0, &waitTimeSpec, nullptr, FUTEX_BITSET_MATCH_ANY) == -1 && errno == ETIMEDOUT)
					return false;
			}
			#else
			#error
			#endif
		}
		#endif

		#if PLATFORM_WINDOWS || PLATFORM_LINUX
		// 0 = Reset, 1 = Notified
		std::atomic<u32> state = 0;
		#else
		std::mutex lock;
		std::condition_variable condition;
		bool shouldWait = true;
		#endif
	};
} // uba::ParkingLot::Private

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace uba::ParkingLot
{
	struct WordMutexQueueNode
	{
		// Points to the next node in the tail-to-head direction. Only null for the current tail.
		WordMutexQueueNode* prev = nullptr;
		// Points to the next node in the head-to-tail direction. The tail points to the head.
		// Null until UnlockSlow() has traversed from the tail to fill in next pointers.
		WordMutexQueueNode* next = nullptr;

		#if PLATFORM_MAC
		// Reference a thread-local event to avoid creating and destroying an event on every contended lock.
		Private::PlatformManualResetEvent& event = []() -> Private::PlatformManualResetEvent&
		{
			thread_local Private::PlatformManualResetEvent tlsEvent;
			return tlsEvent;
		}();
		#else
		Private::PlatformManualResetEvent event;
		#endif
	};

	bool WordMutex::TryEnter()
	{
		u64 expected = 0;
		return state.compare_exchange_strong(expected, isLockedFlag, std::memory_order_acquire, std::memory_order_relaxed);
	}

	void WordMutex::Enter()
	{
		u64 expected = 0;
		if (state.compare_exchange_weak(expected, isLockedFlag, std::memory_order_acquire, std::memory_order_relaxed))
		{
			return;
		}

		EnterSlow();
	}

	void WordMutex::Leave()
	{
		// Unlock immediately to allow other threads to acquire the lock while this thread looks for a thread to wake.
		u64 currentState = state.fetch_sub(isLockedFlag, std::memory_order_release);
		UBA_ASSERT(currentState & isLockedFlag);

		// An empty queue indicates that there are no threads to wake.
		const bool isQueueEmpty = !(currentState & queueMask);
		// A locked queue indicates that another thread is looking for a thread to wake.
		const bool isQueueLocked = (currentState & isQueueLockedFlag);

		if (isQueueEmpty || isQueueLocked)
		{
			return;
		}

		LeaveSlow(currentState);
	}

	UBA_NOINLINE void WordMutex::EnterSlow()
	{
		static_assert((alignof(WordMutexQueueNode) & queueMask) == alignof(WordMutexQueueNode),
			"Alignment of WordMutexQueueNode is insufficient to pack flags into the lower bits.");

		constexpr u32 spinLimit = 40;
		u32 spinCount = 0;
		for (;;)
		{
			u64 currentState = state.load(std::memory_order_relaxed);

			// Try to acquire the lock if it was unlocked, even if there is a queue.
			// Acquiring the lock despite the queue means that this lock is not FIFO and thus not fair.
			if (!(currentState & isLockedFlag))
			{
				if (state.compare_exchange_weak(currentState, currentState | isLockedFlag, std::memory_order_acquire, std::memory_order_relaxed))
				{
					return;
				}
				continue;
			}

			// Spin up to the spin limit while there is no queue.
			if (!(currentState & queueMask) && spinCount < spinLimit)
			{
				YieldInSpinWait();
				++spinCount;
				continue;
			}

			// Create the node that will be used to add this thread to the queue.
			WordMutexQueueNode self;

			self.event.Reset();

			// The state points to the tail of the queue, and each node points to the previous node.
			if (WordMutexQueueNode* tail = (WordMutexQueueNode*)(currentState & queueMask))
			{
				self.prev = tail;
			}
			else
			{
				self.next = &self;
			}

			// Swap this thread in as the tail, which makes it visible to any other thread that acquires the queue lock.
			if (!state.compare_exchange_weak(currentState, (currentState & ~queueMask) | u64(&self), std::memory_order_acq_rel, std::memory_order_relaxed))
			{
				continue;
			}

			// Wait until another thread wakes this thread, which can happen as soon as the preceding store completes.
			self.event.Wait();

			// Loop back and try to acquire the lock.
			spinCount = 0;
		}
	}

	UBA_NOINLINE void WordMutex::LeaveSlow(u64 currentState)
	{
		// IsLockedFlag was cleared by Unlock().
		currentState &= ~isLockedFlag;

		for (;;)
		{
			// Try to lock the queue.
			if (state.compare_exchange_weak(currentState, currentState | isQueueLockedFlag, std::memory_order_acquire, std::memory_order_relaxed))
			{
				currentState |= isQueueLockedFlag;
				break;
			}

			// A locked queue indicates that another thread is looking for a thread to wake.
			if ((currentState & isQueueLockedFlag) || !(currentState & queueMask))
			{
				return;
			}
		}

		for (;;)
		{
			// This thread now holds the queue lock. Neither the queue nor State will change while the queue is locked.
			// The state points to the tail of the queue, and each node points to the previous node.
			WordMutexQueueNode* tail = (WordMutexQueueNode*)(currentState & queueMask);

			// Traverse from the tail to find the head and set next pointers for any nodes added since the last unlock.
			for (WordMutexQueueNode* node = tail; !tail->next;)
			{
				WordMutexQueueNode* prev = node->prev;
				UBA_ASSERT(prev);
				tail->next = prev->next;
				prev->next = node;
				node = prev;
			}

			// Another thread may acquire the lock while this thread has been finding a thread to unlock.
			// That case will not be detected on the first iteration of the loop, but only when this
			// thread has failed to unlock the queue at least once. Attempt to unlock the queue here
			// and allow the next unlock to find a thread to wake.
			if (currentState & isLockedFlag)
			{
				if (state.compare_exchange_weak(currentState, currentState & ~isQueueLockedFlag, std::memory_order_release, std::memory_order_acquire))
				{
					return;
				}
				continue;
			}

			// The next node from the tail is the head.
			WordMutexQueueNode* head = tail->next;

			// Remove the head from the queue and unlock the queue.
			if (WordMutexQueueNode* newHead = head->next; newHead == head)
			{
				// Unlock and clear the queue. Failure needs to restart the loop, because newly-added
				// nodes will have a pointer to the node being removed.
				if (!state.compare_exchange_strong(currentState, currentState & isLockedFlag, std::memory_order_release, std::memory_order_acquire))
				{
					continue;
				}
			}
			else
			{
				// Clear pointers to the head node being removed.
				UBA_ASSERT(newHead);
				newHead->prev = nullptr;
				tail->next = newHead;

				// Unlock the queue regardless of whether new nodes have been added in the meantime.
				state.fetch_and(~isQueueLockedFlag, std::memory_order_release);
			}

			// Wake the thread that was at the head of the queue.
			head->event.Notify();
			break;
		}
	}

} // uba::ParkingLot

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace uba::ParkingLot::Private
{
	inline u32 CountLeadingZerosNonZero(u32 value)
	{
	#ifdef __clang__
		return __builtin_clz(value);
	#else
		unsigned long bitIndex;
		_BitScanReverse(&bitIndex, value);
		return 31 - bitIndex;
	#endif
	}

	inline u32 RoundUpToPowerOfTwo(u32 value)
	{
		if (value <= 1)
			return 1;
		return u32(1) << (32 - CountLeadingZerosNonZero(value - 1));
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	template <typename RefType, typename AssignedType = RefType>
	struct GuardValue
	{
		GuardValue(const GuardValue&) = delete;
		GuardValue& operator=(const GuardValue&) = delete;

		[[nodiscard]] GuardValue(RefType& value, const AssignedType& newValue)
			: valueRef(value)
			, oldValue(value)
		{
			valueRef = newValue;
		}

		~GuardValue()
		{
			valueRef = oldValue;
		}

	private:
		RefType& valueRef;
		AssignedType oldValue;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	template <typename ReferencedType>
	class RefCountPtr
	{
		using ReferenceType = ReferencedType*;

	public:
		RefCountPtr() = default;

		inline RefCountPtr(ReferencedType* inReference)
		{
			reference = inReference;
			if (reference)
			{
				reference->AddRef();
			}
		}

		inline RefCountPtr(const RefCountPtr& other)
		{
			reference = other.reference;
			if (reference)
			{
				reference->AddRef();
			}
		}

		inline RefCountPtr(RefCountPtr&& other)
		{
			reference = other.reference;
			other.reference = nullptr;
		}

		inline ~RefCountPtr()
		{
			if (reference)
			{
				reference->Release();
			}
		}

		inline RefCountPtr& operator=(ReferencedType* newReference)
		{
			if (reference != newReference)
			{
				ReferencedType* oldReference = reference;
				reference = newReference;
				if (reference)
				{
					reference->AddRef();
				}
				if (oldReference)
				{
					oldReference->Release();
				}
			}
			return *this;
		}

		inline RefCountPtr& operator=(const RefCountPtr& other)
		{
			return *this = other.reference;
		}

		inline RefCountPtr& operator=(RefCountPtr&& other)
		{
			if (this != &other)
			{
				ReferencedType* oldReference = reference;
				reference = other.reference;
				other.reference = nullptr;
				if (oldReference)
				{
					oldReference->Release();
				}
			}
			return *this;
		}

		inline ReferencedType* operator->() const
		{
			return reference;
		}

		inline ReferencedType& operator*() const
		{
			return *reference;
		}

		inline explicit operator bool() const
		{
			return !!reference;
		}

	private:
		ReferencedType* reference = nullptr;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	template <typename LockType>
	class DynamicUniqueLock final
	{
	public:
		DynamicUniqueLock() = default;

		DynamicUniqueLock(const DynamicUniqueLock&) = delete;
		DynamicUniqueLock& operator=(const DynamicUniqueLock&) = delete;

		[[nodiscard]] inline explicit DynamicUniqueLock(LockType& lock)
			: mutex(&lock)
		{
			mutex->Enter();
			isLocked = true;
		}

		[[nodiscard]] inline DynamicUniqueLock(DynamicUniqueLock&& other)
			: mutex(other.mutex)
			, isLocked(other.isLocked)
		{
			other.mutex = nullptr;
			other.isLocked = false;
		}

		inline DynamicUniqueLock& operator=(DynamicUniqueLock&& other)
		{
			if (isLocked)
			{
				mutex->Leave();
			}
			mutex = other.mutex;
			isLocked = other.isLocked;
			other.mutex = nullptr;
			other.isLocked = false;
			return *this;
		}

		inline ~DynamicUniqueLock()
		{
			if (isLocked)
			{
				mutex->Leave();
			}
		}

		void Enter()
		{
			UBA_ASSERT(!isLocked);
			UBA_ASSERT(mutex);
			mutex->Enter();
			isLocked = true;
		}

		void Leave()
		{
			UBA_ASSERT(isLocked);
			isLocked = false;
			mutex->Leave();
		}

		inline bool OwnsLock() const
		{
			return isLocked;
		}

		inline explicit operator bool() const
		{
			return OwnsLock();
		}

	private:
		LockType* mutex = nullptr;
		bool isLocked = false;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	/**
	 * Identifies which callback function is currently executing.
	 *
	 * NOTE: ORDER MATTERS FOR COMPARISONS!
	 */
	enum class ExecutingFunction : u8
	{
		None = 0,
		BeforeWait = 1,
		CanWait = 2,
		OnWakeState = 3,
	};

	const tchar* LexToString(ExecutingFunction function)
	{
		switch (function)
		{
		default:
		case ExecutingFunction::None: return TC("None");
		case ExecutingFunction::BeforeWait: return TC("BeforeWait");
		case ExecutingFunction::CanWait: return TC("CanWait");
		case ExecutingFunction::OnWakeState: return TC("OnWakeState");
		}
	}

	static thread_local ExecutingFunction parkingLotExecutingFunction = ExecutingFunction::None;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	#if !UBA_STUB_BUILD
	#pragma warning(push)
	#pragma warning(disable:4324) // structure was padded due to alignment specifier
	#endif

	struct alignas(CacheLineSize) Thread final
	{
		Thread* next = nullptr;
		std::atomic<const void*> waitAddress = nullptr;
		u64 wakeToken = 0;
		PlatformManualResetEvent event;
		std::atomic<u32> referenceCount = 0;

		inline void AddRef()
		{
			referenceCount.fetch_add(1, std::memory_order_relaxed);
		}

		inline void Release()
		{
			if (referenceCount.fetch_sub(1, std::memory_order_acq_rel) == 1)
			{
				delete this;
			}
		}

		Thread() = default;
		~Thread() = default;

		Thread(const Thread&) = delete;
		Thread& operator=(const Thread&) = delete;
	};

	#if !UBA_STUB_BUILD
	#pragma warning(pop)
	#endif

	class ThreadLocalData
	{
	public:
		static RefCountPtr<Thread> Get()
		{
			static thread_local ThreadLocalData threadLocalData;
			ThreadLocalData& localThreadLocalData = threadLocalData;
			if (localThreadLocalData.thread)
			{
				return localThreadLocalData.thread;
			}
			if (localThreadLocalData.isDestroyed)
			{
				return new Thread;
			}
			localThreadLocalData.thread = new Thread;
			return localThreadLocalData.thread;
		}

	private:
		inline static std::atomic<u32> threadCount = 0;

		RefCountPtr<Thread> thread;
		bool isDestroyed = false;

		ThreadLocalData();
		~ThreadLocalData();

		ThreadLocalData(const ThreadLocalData&) = delete;
		ThreadLocalData& operator=(const ThreadLocalData&) = delete;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	enum class QueueAction
	{
		Stop,
		Continue,
		RemoveAndStop,
		RemoveAndContinue,
	};

	class alignas(CacheLineSize) Bucket final
	{
	public:
		[[nodiscard]] inline DynamicUniqueLock<WordMutex> LockDynamic() { return DynamicUniqueLock(mutex); }

		inline void Lock() { mutex.Enter(); }
		inline void Unlock() { mutex.Leave(); }

		inline bool IsEmpty() const { return !head; }

		void Enqueue(Thread* thread);
		Thread* Dequeue();

		template <typename VisitorType>
		void DequeueIf(VisitorType&& visitor);

		Bucket() = default;
		~Bucket() = default;

	private:
		Bucket(const Bucket&) = delete;
		Bucket& operator=(const Bucket&) = delete;

		WordMutex mutex;
		Thread* head = nullptr;
		Thread* tail = nullptr;
		[[maybe_unused]] u64 padding[5]{};
	};

	void Bucket::Enqueue(Thread* thread)
	{
		UBA_ASSERT(thread);
		UBA_ASSERT(!thread->next);
		if (tail)
		{
			tail->next = thread;
			tail = thread;
		}
		else
		{
			head = thread;
			tail = thread;
		}
	}

	Thread* Bucket::Dequeue()
	{
		Thread* thread = head;
		if (thread)
		{
			head = thread->next;
			thread->next = nullptr;
			if (tail == thread)
			{
				tail = nullptr;
			}
		}
		return thread;
	}

	template <typename VisitorType>
	void Bucket::DequeueIf(VisitorType&& visitor)
	{
		Thread** next = &head;
		Thread* prev = nullptr;
		while (Thread* thread = *next)
		{
			switch (visitor(thread))
			{
			case QueueAction::Stop:
				return;
			case QueueAction::Continue:
				prev = thread;
				next = &thread->next;
				break;
			case QueueAction::RemoveAndStop:
				if (tail == thread)
				{
					tail = prev;
				}
				*next = thread->next;
				thread->next = nullptr;
				return;
			case QueueAction::RemoveAndContinue:
				if (tail == thread)
				{
					tail = prev;
				}
				*next = thread->next;
				thread->next = nullptr;
				break;
			}
		}
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	#if !UBA_STUB_BUILD
	#pragma warning(push)
	#pragma warning(disable:4200) // nonstandard extension used: zero-length array in struct/union
	#endif

	class Table final
	{
		constexpr static u32 minSize = 32;

	public:
		static Bucket& FindOrCreateBucket(const void* address, DynamicUniqueLock<WordMutex>& outLock);
		static Bucket* FindBucket(const void* address, DynamicUniqueLock<WordMutex>& outLock);

		static void Reserve(u32 threadCount);

	private:
		static Table& CreateOrGet();
		static Table& Create(u32 size);
		static void Destroy(Table& table);

		static bool TryLock(Table& table, Vector<Bucket*>& outBuckets);
		static void Unlock(Vector<Bucket*>& lockedBuckets);

		static u32 HashAddress(const void* address);

		inline static std::atomic<Table*> globalTable;

		Table() = default;
		~Table() = default;

		Table(const Table&) = delete;
		Table& operator=(const Table&) = delete;

		template <typename AllocatorType>
		Bucket& FindOrCreateBucket(u32 index, AllocatorType&& bucketAllocator);

		u32 bucketCount = 0;
		std::atomic<Bucket*> buckets[0];
	};

	#if !UBA_STUB_BUILD
	#pragma warning(pop)
	#endif

	Bucket& Table::FindOrCreateBucket(const void* address, DynamicUniqueLock<WordMutex>& outLock)
	{
		const u32 hash = HashAddress(address);

		for (;;)
		{
			Table& table = CreateOrGet();
			const u32 index = hash % table.bucketCount;
			Bucket& bucket = table.FindOrCreateBucket(index, [] { return new Bucket; });
			outLock = bucket.LockDynamic();

			if (&table == globalTable.load(std::memory_order_acquire))
			{
				return bucket;
			}

			// Restart because the table was resized since it was loaded above.
			outLock.Leave();
		}
	}

	Bucket* Table::FindBucket(const void* address, DynamicUniqueLock<WordMutex>& outLock)
	{
		const u32 hash = HashAddress(address);

		for (;;)
		{
			if (Table* table = globalTable.load(std::memory_order_acquire))
			{
				const u32 index = hash % table->bucketCount;
				if (Bucket* bucket = table->buckets[index].load(std::memory_order_acquire))
				{
					outLock = bucket->LockDynamic();

					if (table == globalTable.load(std::memory_order_acquire))
					{
						return bucket;
					}

					// Restart because the table was resized since it was loaded above.
					outLock = {};
					continue;
				}
			}
			return nullptr;
		}
	}

	template <typename AllocatorType>
	Bucket& Table::FindOrCreateBucket(u32 index, AllocatorType&& bucketAllocator)
	{
		std::atomic<Bucket*>& bucketPtr = buckets[index];
		Bucket* bucket = bucketPtr.load(std::memory_order_acquire);
		if (!bucket)
		{
			Bucket* newBucket = bucketAllocator();
			if (bucketPtr.compare_exchange_strong(bucket, newBucket, std::memory_order_release, std::memory_order_acquire))
			{
				bucket = newBucket;
			}
			else
			{
				delete newBucket;
			}
			UBA_ASSERT(bucket);
		}
		return *bucket;
	}

	Table& Table::CreateOrGet()
	{
		Table* table = globalTable.load(std::memory_order_acquire);

		if (table)
		{
			return *table;
		}

		Table& newTable = Create(minSize);

		if (globalTable.compare_exchange_strong(table, &newTable, std::memory_order_release, std::memory_order_acquire))
		{
			return newTable;
		}

		Destroy(newTable);

		UBA_ASSERT(table);
		return *table;
	}

	Table& Table::Create(const u32 size)
	{
		const u64 memorySize = sizeof(Table) + sizeof(Bucket*) * u64(size);
		void* const memory = aligned_alloc(alignof(Table), memorySize);
		memset(memory, 0, memorySize);
		Table& table = *new(memory) Table;
		table.bucketCount = size;
		return table;
	}

	void Table::Destroy(Table& table)
	{
		table.~Table();
		aligned_free(&table);
	}

	void Table::Reserve(const u32 threadCount)
	{
		const u32 targetBucketCount = RoundUpToPowerOfTwo(threadCount);
		Vector<Bucket*> existingBuckets;

		for (;;)
		{
			Table& existingTable = CreateOrGet();

			if (existingTable.bucketCount >= targetBucketCount)
			{
				// Reserve is called every time a thread is created and has amortized constant time
				// because of its power-of-two table growth. Most calls return here without locking.
				return;
			}

			if (!TryLock(existingTable, existingBuckets))
			{
				continue;
			}

			// Gather waiting threads to be redistributed into the buckets of the new table.
			// Threads with the same address remain in the same relative order as they were queued.
			Vector<Thread*> threads;
			for (Bucket* bucket : existingBuckets)
			{
				while (Thread* thread = bucket->Dequeue())
				{
					threads.push_back(thread);
				}
			}

			Table& newTable = Create(targetBucketCount);

			// Reuse existing now-empty buckets when populating the new table.
			Vector<Bucket*> availableBuckets = existingBuckets;
			const auto allocateBucket = [&availableBuckets]() -> Bucket*
			{
				if (!availableBuckets.empty())
				{
					Bucket* bucket = availableBuckets.back();
					availableBuckets.pop_back();
					return bucket;
				}
				return new Bucket;
			};

			// Add waiting threads to the new table.
			UBA_ANALYZER_SUPPRESS
			for (Thread* thread : threads)
			{
				const u32 hash = HashAddress(thread->waitAddress.load(std::memory_order_relaxed));
				const u32 index = hash % newTable.bucketCount;
				Bucket& bucket = newTable.FindOrCreateBucket(index, allocateBucket);
				bucket.Enqueue(thread);
			}

			// Assign any available buckets to the table to avoid having to free them.
			UBA_ANALYZER_SUPPRESS
			for (u32 index = 0; !availableBuckets.empty() && index < newTable.bucketCount; ++index)
			{
				newTable.FindOrCreateBucket(index, allocateBucket);
			}
			UBA_ASSERT(availableBuckets.empty());

			// Make the new table visible to other threads.
			[[maybe_unused]] Table* compareTable = globalTable.exchange(&newTable, std::memory_order_release);
			UBA_ASSERT(compareTable == &existingTable);

			// Unlock buckets that came from the existing table now that the new table is visible.
			Unlock(existingBuckets);
			return;
		}
	}

	bool Table::TryLock(Table& table, Vector<Bucket*>& outBuckets)
	{
		outBuckets.clear();
		outBuckets.reserve(table.bucketCount);

		// Gather buckets from the table, creating them as needed because the lock is on the bucket.
		for (u32 index = 0; index < table.bucketCount; ++index)
		{
			outBuckets.push_back(&table.FindOrCreateBucket(index, [] { return new Bucket; }));
		}

		// Lock the buckets in order by address to ensure consistent ordering regardless of the table being locked.
		uba::Sort(outBuckets.begin(), outBuckets.end());
		for (Bucket* bucket : outBuckets)
		{
			bucket->Lock();
		}

		// Table is locked if the global table pointer still points to it, otherwise it has grown.
		if (&table == globalTable)
		{
			return true;
		}

		// Unlock and return that the table could not be locked.
		Unlock(outBuckets);
		outBuckets.clear();
		return false;
	}

	void Table::Unlock(Vector<Bucket*>& lockedBuckets)
	{
		for (Bucket* bucket : lockedBuckets)
		{
			bucket->Unlock();
		}
	}

	u32 Table::HashAddress(const void* address)
	{
		constexpr u64 A = 0xdc2b17dc9d2fbc29;
		constexpr u64 B = 0xcb1014192cb2c5fc;
		constexpr u64 C = 0x5b12db9242bd7ce7;
		const u64 value = u64(address);
		return u32(((A * (value >> 32)) + (B * (value & 0xffffffff)) + C) >> 32);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	ThreadLocalData::ThreadLocalData()
	{
		Table::Reserve(threadCount.fetch_add(1, std::memory_order_relaxed) + 1);
	}

	ThreadLocalData::~ThreadLocalData()
	{
		threadCount.fetch_sub(1, std::memory_order_relaxed);
		thread = nullptr;
		isDestroyed = true;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	WaitState Wait(const void* address, bool (*canWait)(void*), void* canWaitContext, void (*beforeWait)(void*), void* beforeWaitContext)
	{
		using namespace Private;

		// A failure of this assertion is either an error in a callback, because callbacks may not call into ParkingLot,
		// or a dependency on ParkingLot from within the memory allocator used by ParkingLot.
		UBA_ASSERTF(parkingLotExecutingFunction == ExecutingFunction::None,
			TC("Detected a re-entrant call into ParkingLot from within a %s callback."), LexToString(parkingLotExecutingFunction));

		RefCountPtr<Thread> self = ThreadLocalData::Get();

		UBA_ASSERTF(!self->waitAddress.load(std::memory_order_relaxed), TC("waitAddress must be null. This can happen if Wait is called by BeforeWait."));
		UBA_ASSERTF(self->wakeToken == 0, TC("wakeToken must be 0. This is an error in ParkingLot."));

		WaitState state;

		// Enqueue the thread if CanWait returns true while the bucket is locked.
		{
			DynamicUniqueLock<WordMutex> bucketLock;
			Bucket& bucket = Table::FindOrCreateBucket(address, bucketLock);
			if (canWait)
			{
				GuardValue guardActive(parkingLotExecutingFunction, ExecutingFunction::CanWait);
				if (!canWait(canWaitContext))
				{
					return state;
				}
			}
			state.didWait = true;
			self->waitAddress.store(address, std::memory_order_relaxed);
			self->event.Reset();
			bucket.Enqueue(&*self);
		}

		// BeforeWait must be invoked after the bucket is unlocked.
		if (beforeWait)
		{
			GuardValue guardActive(parkingLotExecutingFunction, ExecutingFunction::BeforeWait);
			beforeWait(beforeWaitContext);
		}

		// Wait until the thread has been dequeued.
		self->event.Wait();

		// waitAddress is reset when the thread is dequeued.
		UBA_ASSERT(!self->waitAddress.load(std::memory_order_relaxed));
		state.didWake = true;
		state.wakeToken = self->wakeToken;
		self->wakeToken = 0;
		return state;
	}

	static WaitState TimedWait(
		const void* address,
		bool (*canWait)(void*),
		void* canWaitContext,
		void (*beforeWait)(void*),
		void* beforeWaitContext,
		void (*waitOnEvent)(void*, PlatformManualResetEvent&),
		void* waitOnEventContext)
	{
		using namespace Private;

		// A failure of this assertion is either an error in a callback, because callbacks may not call into ParkingLot,
		// or a dependency on ParkingLot from within the memory allocator used by ParkingLot.
		UBA_ASSERTF(parkingLotExecutingFunction == ExecutingFunction::None,
			TC("Detected a re-entrant call into ParkingLot from within a %s callback."), LexToString(parkingLotExecutingFunction));

		RefCountPtr<Thread> self = ThreadLocalData::Get();

		UBA_ASSERTF(!self->waitAddress.load(std::memory_order_relaxed), TC("waitAddress must be null. This can happen if Wait is called by BeforeWait."));
		UBA_ASSERTF(self->wakeToken == 0, TC("wakeToken must be 0. This is an error in ParkingLot."));

		WaitState state;

		// Enqueue the thread if CanWait returns true while the bucket is locked.
		{
			DynamicUniqueLock<WordMutex> bucketLock;
			Bucket& bucket = Table::FindOrCreateBucket(address, bucketLock);
			if (canWait)
			{
				GuardValue guardActive(parkingLotExecutingFunction, ExecutingFunction::CanWait);
				if (!canWait(canWaitContext))
				{
					return state;
				}
			}
			state.didWait = true;
			self->waitAddress.store(address, std::memory_order_relaxed);
			self->event.Reset();
			bucket.Enqueue(&*self);
		}

		// BeforeWait must be invoked after the bucket is unlocked.
		if (beforeWait)
		{
			GuardValue guardActive(parkingLotExecutingFunction, ExecutingFunction::BeforeWait);
			beforeWait(beforeWaitContext);
		}

		// Wait until the timeout or until the thread has been dequeued.
		waitOnEvent(waitOnEventContext, self->event);

		// waitAddress is reset when the thread is dequeued.
		if (!self->waitAddress.load(std::memory_order_relaxed))
		{
			state.didWake = true;
			state.wakeToken = self->wakeToken;
			self->wakeToken = 0;
			return state;
		}

		// The timeout was reached and the thread needs to dequeue itself.
		// This can race with a call to wake a thread, which means Self is unsafe to access outside of the lock.
		bool didDequeue = false;
		if (DynamicUniqueLock<WordMutex> bucketLock; Bucket* bucket = Table::FindBucket(address, bucketLock))
		{
			bucket->DequeueIf([self = &*self, &didDequeue](Thread* thread)
			{
				if (thread == self)
				{
					didDequeue = true;
					thread->waitAddress.store(nullptr, std::memory_order_relaxed);
					return QueueAction::RemoveAndStop;
				}
				return QueueAction::Continue;
			});
		}

		// The thread did not dequeue itself, which means that we need to wait until the other thread
		// has finished waking this thread by setting its wait address to null.
		if (!didDequeue)
		{
			self->event.Wait();
			state.didWake = true;
			state.wakeToken = self->wakeToken;
			self->wakeToken = 0;
		}

		return state;
	}

	static void WaitForTime(void* waitTime, PlatformManualResetEvent& event)
	{
		event.WaitFor(*(const u32*)waitTime);
	}

	WaitState WaitFor(const void* address, bool (*canWait)(void*), void* canWaitContext, void (*beforeWait)(void*), void* beforeWaitContext, u32 waitTimeMs)
	{
		return TimedWait(address, canWait, canWaitContext, beforeWait, beforeWaitContext, WaitForTime, &waitTimeMs);
	}

	void WakeOne(const void* address, u64 (*onWakeState)(void*, WakeState), void* onWakeStateContext)
	{
		using namespace Private;

		// A failure of this assertion is either an error in a callback, because callbacks may not call into ParkingLot,
		// or a dependency on ParkingLot from within the memory allocator used by ParkingLot.
		UBA_ASSERTF(parkingLotExecutingFunction <= ExecutingFunction::BeforeWait,
			TC("Detected a re-entrant call into ParkingLot from within a %s callback."), LexToString(parkingLotExecutingFunction));

		RefCountPtr<Thread> wakeThread;
		u64 wakeToken = 0;

		{
			DynamicUniqueLock<WordMutex> bucketLock;
			Bucket& bucket = Table::FindOrCreateBucket(address, bucketLock);
			bucket.DequeueIf([address, &wakeThread](Thread* thread)
			{
				if (thread->waitAddress.load(std::memory_order_relaxed) == address)
				{
					wakeThread = thread;
					return QueueAction::RemoveAndStop;
				}
				return QueueAction::Continue;
			});
			GuardValue guardActive(parkingLotExecutingFunction, ExecutingFunction::OnWakeState);
			WakeState wakeState;
			wakeState.didWake = !!wakeThread;
			wakeState.hasWaitingThreads = !bucket.IsEmpty();
			wakeToken = onWakeState ? onWakeState(onWakeStateContext, wakeState) : 0;
		}

		if (wakeThread)
		{
			wakeThread->wakeToken = wakeToken;
			wakeThread->waitAddress.store(nullptr, std::memory_order_relaxed);
			wakeThread->event.Notify();
		}
	}

	static u64 CopyWakeState(void* outState, WakeState state)
	{
		*(WakeState*)outState = state;
		return 0;
	}
} // uba::ParkingLot::Private

namespace uba::ParkingLot
{
	WakeState WakeOne(const void* address)
	{
		WakeState outState;
		Private::WakeOne(address, Private::CopyWakeState, &outState);
		return outState;
	}

	u32 WakeMultiple(const void* const address, const u32 wakeCount)
	{
		using namespace Private;

		// A failure of this assertion is either an error in a callback, because callbacks may not call into ParkingLot,
		// or a dependency on ParkingLot from within the memory allocator used by ParkingLot.
		UBA_ASSERTF(parkingLotExecutingFunction <= ExecutingFunction::BeforeWait,
			TC("Detected a re-entrant call into ParkingLot from within a %s callback."), LexToString(parkingLotExecutingFunction));

		// TODO: TInlineAllocator?
		//Vector<RefCountPtr<FThread>, TInlineAllocator<128>> wakeThreads;
		Vector<RefCountPtr<Thread>> wakeThreads;

		if (DynamicUniqueLock<WordMutex> bucketLock; Bucket* bucket = Table::FindBucket(address, bucketLock))
		{
			bucket->DequeueIf([address, &wakeThreads, wakeCount](Thread* thread)
			{
				if (thread->waitAddress.load(std::memory_order_relaxed) == address)
				{
					wakeThreads.emplace_back(thread);
					return wakeThreads.size() == wakeCount ? QueueAction::RemoveAndStop : QueueAction::RemoveAndContinue;
				}
				return QueueAction::Continue;
			});
		}

		for (RefCountPtr<Thread>& wakeThread : wakeThreads)
		{
			wakeThread->waitAddress.store(nullptr, std::memory_order_relaxed);
			wakeThread->event.Notify();
		}

		return u32(wakeThreads.size());
	}

	u32 WakeAll(const void* address)
	{
		return WakeMultiple(address, 0xffffffff);
	}
} // uba::ParkingLot

#endif // UBA_USE_PARKINGLOT