// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaPlatform.h"
#include "UbaDefinitions.h"
#include "UbaProcessStats.h"
#include <oodle2.h>

#if PLATFORM_WINDOWS
#include <conio.h>
#include <psapi.h>
#else
#include <execinfo.h>
#include <limits.h>
#include <dlfcn.h>
#include <signal.h>
#include <sys/syscall.h>
#include <unistd.h>
#if PLATFORM_LINUX
#include <ucontext.h>
#include <link.h>
#include <sys/uio.h>
#else
#include <sys/sysctl.h>
#include <mach-o/dyld.h>
#endif
#endif


#define UBA_SUPPORT_CALLSTACKS 1 // TODO: should this be 0 for static linked linux target

#if UBA_SUPPORT_CALLSTACKS
#include "UbaAlgorithm.h"
#include "UbaVector.h"
#include "UbaString.h"
#endif


namespace uba
{
	extern KernelStats& g_kernelStats;
	thread_local KernelStats* t_kernelStats;

	KernelStats& KernelStats::GetCurrent()
	{
		KernelStats* stats = t_kernelStats;
		return stats ? *stats : g_kernelStats;
	}

	KernelStats& KernelStats::GetGlobal()
	{
		return g_kernelStats;
	}

	KernelStatsScope::KernelStatsScope(KernelStats& s) : stats(s)
	{
		t_kernelStats = &stats;
	}
	KernelStatsScope::~KernelStatsScope()
	{
		t_kernelStats = nullptr;
	}

	bool CreateGuid(Guid& out)
	{
		#if PLATFORM_WINDOWS
		return ::CoCreateGuid((GUID*)&out) == S_OK;
		#elif PLATFORM_MAC
		arc4random_buf(&out, 16);
		return true;
		#else
		const int f = open("/dev/urandom", O_RDONLY | O_CLOEXEC);
		if (f == -1)
			return false;
		size_t bytesRead = read(f, &out, 16);
		close(f);
		return bytesRead == 16;
		//syscall(SYS_getrandom, &out, sizeof(Guid), 0x0001);
		#endif
	}

	const char* GetWineVersion()
	{
		#if PLATFORM_WINDOWS
		static const char* wineVersion = []() -> const char*
			{
				if (HMODULE ntDllModule = GetModuleHandleW(L"ntdll.dll"))
				{
					using wine_get_version_func = const char*();
					if (auto wine_get_version = (wine_get_version_func*)(void*)GetProcAddress(ntDllModule, "wine_get_version"))
						return wine_get_version();
				}
				return nullptr;
			}();
		return wineVersion;
		#else
		return nullptr;
		#endif
	}

	bool IsRunningWine()
	{
		return GetWineVersion() != nullptr;
	}

	bool IsRunningDarling()
	{
		#if PLATFORM_MAC
		static bool isDarling = []()
		{
			struct stat attr;
			return stat("/Library/Developer/DarlingCLT", &attr) == 0 && S_ISDIR(attr.st_mode);
		}();
		return isDarling;
		#else
		return false;
		#endif
	}

	bool IsRunningArm()
	{
		#if PLATFORM_WINDOWS
		static bool isArm = []()
		{
			USHORT processMachine, nativeMachine;
			if (!IsWow64Process2(GetCurrentProcess(), &processMachine, &nativeMachine))
				return false;
			return nativeMachine == IMAGE_FILE_MACHINE_ARM64;
		}();
		return isArm;
		#else
		return IsArmBinary;
		#endif
	}

	void Sleep(u32 milliseconds)
	{
		#if PLATFORM_WINDOWS
		::Sleep(milliseconds);
		#elif defined(_POSIX_C_SOURCE) && _POSIX_C_SOURCE >= 199309L
		struct timespec ts;
		ts.tv_sec = milliseconds / 1000;
		ts.tv_nsec = (milliseconds % 1000) * 1000000;
		int res;
		do { res = nanosleep(&ts, &ts); } while (res && errno == EINTR);
		#else
		if (milliseconds >= 1000)
			sleep(milliseconds / 1000);
		usleep((milliseconds % 1000) * 1000);
		#endif
	}

	u32 GetUserDefaultUILanguage()
	{
		#if PLATFORM_WINDOWS
		return ::GetUserDefaultUILanguage();
		#else
		return 1;
		#endif
	}

	thread_local u32 t_lastError;

	u32 GetLastError()
	{
		#if PLATFORM_WINDOWS
		return ::GetLastError();
		#else
		return t_lastError;
		#endif
	}

	void SetLastError(u32 error)
	{
		#if PLATFORM_WINDOWS
		::SetLastError(error);
		#else
		t_lastError = error;
		#endif
	}

	bool GetComputerNameW(StringBufferBase& out)
	{
		#if PLATFORM_WINDOWS
		DWORD size = out.capacity;
		if (!::GetComputerNameExW(ComputerNameDnsHostname, out.data, &size))
			if (!::GetComputerNameExW(ComputerNameNetBIOS, out.data, &size))
				return false;
		out.count = size;
		#else
		if (gethostname(out.data, out.capacity) == -1)
			return false;
		out.count = TStrlen(out.data);
		#endif
		return true;
	}

	bool GetOsVersion(StringBufferBase& outPretty, u32& outValue)
	{
		outValue = 0;

#if PLATFORM_WINDOWS
		if (IsRunningWine())
			outPretty.Append(TC("Linux/Wine-")).Append(GetWineVersion());
		else
			outPretty.Append(TC("Windows"));
#elif PLATFORM_MAC
		FILE* osRelease = popen("sw_vers -productVersion", "r");
		if (!osRelease)
			return false;
		char buffer[256];
		bool success = fgets(buffer, sizeof_array(buffer), osRelease);
		pclose(osRelease);
		if (!success)
			return false;
		u32 bufferLen = TStrlen(buffer) - 1;
		buffer[bufferLen] = 0;
		if (IsRunningDarling())
			outPretty.Append(TCV("Linux/Darling-"));
		else
			outPretty.Append(TCV("MacOS-"));
		outPretty.Append(buffer, bufferLen);

		u32 majorVersion = 0;
		u32 minorVersion = 0;
		u32 patchVersion = 0;

		if (char* endOfMajor = TStrchr(buffer, '.'))
		{
			*endOfMajor = 0;
			char* minor = endOfMajor + 1;
			if (char* endOfMinor = TStrchr(minor, '.'))
			{
				*endOfMinor = 0;
				char* patch = endOfMinor + 1;
				if (*patch)
					patchVersion = strtoul(patch, nullptr, 10);
			}
			if (*minor)
				minorVersion = strtoul(minor, nullptr, 10);
		}
		majorVersion = strtoul(buffer, nullptr, 10);

		outValue = (majorVersion << 16) + (minorVersion << 8) + patchVersion;
#elif PLATFORM_LINUX
		FILE* osRelease = fopen("/etc/os-release", "r");
		if (!osRelease)
			return false;
		char buffer[256];
		bool success = fgets(buffer, sizeof_array(buffer), osRelease);
		fclose(osRelease);
		if (!success)
			return false;
		if (!StartsWith(buffer, "PRETTY_NAME=\""))
			return false;
		outPretty.Append(buffer + 13, TStrlen(buffer) - 14);
#endif
		return true;
	}

	void WriteAssertInfo(StringBufferBase& out, const tchar* text, const char* file, u32 line, const char* expr, void* context)
	{
		int signal = 0;
		#if !PLATFORM_WINDOWS
		if (context)
			signal = *(int*)context;
		#endif

		if (!context)
			out.Append(TCV("ASSERT: "));
		out.Append(*text ? text : TC("Unknown"));
		if (!*text && expr && *expr && strcmp(expr, "false") != 0)
			out.Appendf(TC("\n EXPR: ")).Append(expr);
		if (file && *file)
			out.Append(TCV("\n LOCATION: ")).Append(file).Append(':').AppendValue(line);
		if (signal)
			out.Append(TCV("\n SIGNAL: ")).AppendValue(signal);
	}

	#if defined(_M_ARM64)
		#define GET_PC(context) context.Pc
		#define GET_SP(context) context.Sp
	#else
		#define GET_PC(context) context.Rip
		#define GET_SP(context) context.Rsp
	#endif

	UBA_NOINLINE u32 GetCallstack(void** outCallstack, u32 callstackCapacity, u32 skipCallstack, void* contextPtr)
	{
#if PLATFORM_WINDOWS
		constexpr int MaxDepth = 24;// 62;
		void* callers[MaxDepth] = {0};
		u32 count = 0;//RtlCaptureStackBackTrace(0, MaxDepth, callers, NULL); // For some reason this does not work on wine
		if (!count)
		{
			CONTEXT context;
			if (contextPtr)
			{
				context = *(CONTEXT*)contextPtr;
				skipCallstack = 0;
			}
			else
				RtlCaptureContext(&context);

			if (GET_PC(context) && GET_SP(context))
			{
				UNWIND_HISTORY_TABLE unwindHistoryTable;
				RtlZeroMemory(&unwindHistoryTable, sizeof(UNWIND_HISTORY_TABLE));
				while (count < MaxDepth && GET_PC(context))
				{
					callers[count++] = (void*)GET_PC(context);
					ULONG64 ImageBase = 0;
					if (PRUNTIME_FUNCTION RuntimeFunction = RtlLookupFunctionEntry(GET_PC(context), &ImageBase, &unwindHistoryTable); RuntimeFunction)
					{
						KNONVOLATILE_CONTEXT_POINTERS nvcontext;
						RtlZeroMemory(&nvcontext, sizeof(KNONVOLATILE_CONTEXT_POINTERS));
						PVOID HandlerData = nullptr;
						ULONG64 EstablisherFrame = 0;
						RtlVirtualUnwind(0, ImageBase, GET_PC(context), RuntimeFunction, &context, &HandlerData, &EstablisherFrame, &nvcontext);
					}
					else
					{
						u64* spPtr = (u64*)GET_SP(context);
						if (!spPtr)
							break;
						u64 sp = *spPtr;
						if (!sp)
							break;
						GET_PC(context) = sp;
						GET_SP(context) += 8;
					}
				}
			}
		}
#else
		constexpr u32 maxCallers = 64;
		void* callers[maxCallers];
		u32 count = 0;
#if PLATFORM_LINUX
		// Walk the frame-pointer chain manually using process_vm_readv to dereference each
		// saved RBP / return-address slot. This replaces glibc's backtrace() (which calls
		// libgcc's _Unwind_Backtrace) because the latter SEGVs when:
		//   - a frame's CFI is missing/corrupt (JIT'd code, hand-written asm without .cfi_*)
		//   - another thread is mid-dlopen and the dl_iterate_phdr lock state is in flux
		//   - the unwinder is invoked from inside an assert handler that itself trips the
		//     unwinder again, deadlocking on the assert mutex (the failure mode we just hit
		//     on AL2025: thread #2 stuck in nested SIGSEGV → futex wait → entire NetClnt
		//     pool blocked → all detoured children hang → build stalls).
		// process_vm_readv returns -EFAULT for unmapped pages instead of segfaulting, so a
		// bad RBP just terminates the walk cleanly. Async-signal-safe (raw syscall, no malloc,
		// no locks) — we can call this from segfault_sigaction without re-entering libgcc.
		// Frame pointers are emitted in our Debug/Development builds (we don't pass
		// -fomit-frame-pointer), so SysV ABI x86_64 layout holds: [RBP] = saved RBP,
		// [RBP+8] = return address.
		{
			u64 fp = 0;
			if (contextPtr)
			{
				ucontext_t* uc = (ucontext_t*)contextPtr;
				#if PLATFORM_CPU_ARM_FAMILY
				fp = (u64)uc->uc_mcontext.regs[29]; // X29 is FP on AArch64
				if (count < maxCallers) callers[count++] = (void*)uc->uc_mcontext.pc;
				#else
				fp = (u64)uc->uc_mcontext.gregs[REG_RBP];
				if (count < maxCallers) callers[count++] = (void*)uc->uc_mcontext.gregs[REG_RIP];
				#endif
			}
			else
			{
				fp = (u64)__builtin_frame_address(0);
			}

			pid_t self = getpid();
			u64 prevFp = 0;
			while (count < maxCallers && fp && fp != prevFp && (fp & 7u) == 0)
			{
				// Bound the walk to a sane stack range. Saved RBP must be at a higher
				// address than the current one (stack grows down, saved RBP points up the
				// chain). Cap each step at 16 MiB to reject obviously-garbage values.
				if (prevFp && (fp <= prevFp || fp - prevFp > (16ull << 20)))
					break;

				u64 frame[2] = { 0, 0 };
				iovec local{ frame, sizeof(frame) };
				iovec remote{ (void*)fp, sizeof(frame) };
				ssize_t r = process_vm_readv(self, &local, 1, &remote, 1, 0);
				if (r != (ssize_t)sizeof(frame))
					break;

				u64 ret = frame[1];
				if (!ret)
					break;
				callers[count++] = (void*)ret;
				prevFp = fp;
				fp = frame[0];
			}
		}
		// Skip frames are accounted for by the caller (segfault_sigaction passes contextPtr
		// and bumps skipCallstack to elide its own frame). Our walker already injected the
		// trapping PC from contextPtr at index 0, so the +2 bump that backtrace() needed is
		// no longer correct.
		(void)contextPtr;
#else
		count = backtrace(callers, maxCallers);
		if (contextPtr) // Use this to know if we come from a signal
			skipCallstack += 2;
#endif
#endif
		if (count < skipCallstack)
			skipCallstack = count;
		u32 returnCount = count - skipCallstack;
		if (callstackCapacity < returnCount)
		{
			skipCallstack += returnCount - callstackCapacity;
			returnCount = count - skipCallstack;
		}

		memcpy(outCallstack, callers + skipCallstack, returnCount*sizeof(void*));
		return returnCount;
	}

	UBA_NOINLINE bool WriteCallstackInfo(BinaryWriter& out, u32 skipCallstack, void* contextPtr)
	{
		void* callstack[100];
		u32 callstackCount = GetCallstack(callstack, 100, skipCallstack, contextPtr);
		return WriteCallstackInfo(out, callstack, callstackCount);
	}

	bool WriteCallstackInfo(BinaryWriter& out, void** callstack, u32 callstackCount)
	{
#if UBA_SUPPORT_CALLSTACKS
#if PLATFORM_WINDOWS
		out.WriteBool(IsRunningWine());

		// Sorted by `end`; LowerBound(addr) finds first module whose end > addr.
		struct ModuleRec { u64 end; u64 start; u64 size; HMODULE handle; u32 index; };
		Vector<ModuleRec> moduleEndAddresses;

		Vector<HMODULE> loadedModules(128);
		DWORD needed = 0;
		bool success = EnumProcessModules(GetCurrentProcess(), loadedModules.data(), u32(loadedModules.size()*sizeof(HMODULE)), &needed);
		if (success && needed > loadedModules.size()*sizeof(HMODULE))
		{
			loadedModules.resize(needed/sizeof(HMODULE));
			success = EnumProcessModules(GetCurrentProcess(), loadedModules.data(), u32(loadedModules.size()*sizeof(HMODULE)), &needed);
			if (needed > loadedModules.size()*sizeof(HMODULE))
				success = false;
		}
		if (success)
		{
			u32 modCount = needed / sizeof(HMODULE);
			moduleEndAddresses.reserve(modCount);
			for (u32 i = 0; i != modCount; ++i)
			{
				MODULEINFO mi;
				GetModuleInformation(GetCurrentProcess(), loadedModules[i], &mi, sizeof(mi));
				ModuleRec rec;
				rec.end    = u64(mi.lpBaseOfDll) + mi.SizeOfImage;
				rec.start  = u64(mi.lpBaseOfDll);
				rec.size   = mi.SizeOfImage;
				rec.handle = loadedModules[i];
				rec.index  = ~0u;
				moduleEndAddresses.push_back(rec);
			}
			uba::Sort(moduleEndAddresses.begin(), moduleEndAddresses.end(),
				[](const ModuleRec& a, const ModuleRec& b) { return a.end < b.end; });
		}

		Vector<ModuleRec*> usedModules;

		out.Write7BitEncoded(callstackCount);

		for (u32 i = 0; i < callstackCount; i++)
		{
			auto addr = u64(callstack[i]);

			auto findIt = uba::LowerBound(moduleEndAddresses.begin(), moduleEndAddresses.end(), addr,
				[](const ModuleRec& m, u64 a) { return m.end < a; });
			if (findIt != moduleEndAddresses.end() && addr >= findIt->start)
			{
				ModuleRec& rec = *findIt;
				if (rec.index == ~0u)
				{
					rec.index = u32(usedModules.size());
					usedModules.push_back(&rec);
				}
				out.Write7BitEncoded(rec.index);
				out.Write7BitEncoded(addr - rec.start);
			}
			else
			{
				out.Write7BitEncoded(~0u);
				out.Write7BitEncoded(addr);
			}
		}

		out.Write7BitEncoded(usedModules.size());
		for (auto* rec : usedModules)
		{
			out.Write7BitEncoded(rec->start);
			out.Write7BitEncoded(rec->size);
			tchar str[1024];
			if (GetModuleFileNameW(rec->handle, str, sizeof_array(str)))
			{
				const tchar* moduleName = str;
				if (const tchar* lastSeparator = TStrrchr(str, PathSeparator))
					moduleName = lastSeparator + 1;
				out.WriteString(moduleName);
			}
			else
			{
				out.WriteString(TC(""));
			}
		}
#else

		// Sorted by `end`; LowerBound(addr) finds the first module whose end > addr.
		struct ModuleRec { u64 end; u64 start; u64 size; TString name; u32 index; };
		Vector<ModuleRec> moduleEndAddresses;

		#if PLATFORM_LINUX
		dl_iterate_phdr([](struct dl_phdr_info* info, size_t, void* data)
			{
				auto& moduleEndAddresses = *(Vector<ModuleRec>*)data;
				if (!info->dlpi_name)
					return 0;
				u64 base = info->dlpi_addr;
				u64 end = base;
				for (int i = 0; i < info->dlpi_phnum; i++)
				{
					if (info->dlpi_phdr[i].p_type != PT_LOAD)
						continue;
					u64 start = base + info->dlpi_phdr[i].p_vaddr;
					end = Max(end, start + info->dlpi_phdr[i].p_memsz);
				}
				ModuleRec rec;
				rec.end   = end;
				rec.start = base;
				rec.size  = end - base;
				rec.name  = info->dlpi_name;
				rec.index = ~0u;
				moduleEndAddresses.push_back(std::move(rec));
				return 0;
			}, &moduleEndAddresses);
		#else
		{
			// dyld reports by start address; derive end via sort-by-start + next-image's-start.
			Vector<ModuleRec> modules;
			for (u32 i = 0, e = _dyld_image_count(); i < e; i++)
			{
				const char* name = _dyld_get_image_name(i);
				if (!name)
					continue;
				auto hdr = _dyld_get_image_header(i);
				ModuleRec rec;
				rec.end   = 0; // filled in below
				rec.start = u64(hdr); // + _dyld_get_image_vmaddr_slide(i);
				rec.size  = 0;
				rec.name  = name;
				rec.index = ~0u;
				modules.push_back(std::move(rec));
			}
			uba::Sort(modules.begin(), modules.end(),
				[](const ModuleRec& a, const ModuleRec& b) { return a.start < b.start; });
			for (u64 i = 1, e = modules.size(); i < e; ++i)
			{
				ModuleRec rec = modules[i - 1];
				rec.end  = modules[i].start;
				rec.size = rec.end - rec.start;
				moduleEndAddresses.push_back(std::move(rec));
			}
		}
		#endif

		uba::Sort(moduleEndAddresses.begin(), moduleEndAddresses.end(),
			[](const ModuleRec& a, const ModuleRec& b) { return a.end < b.end; });

		out.Write7BitEncoded(callstackCount);

		Vector<ModuleRec*> usedModules;

		for (int i = 0; i < callstackCount; ++i)
		{
			u64 addr = u64(callstack[i]);
			auto findIt = uba::LowerBound(moduleEndAddresses.begin(), moduleEndAddresses.end(), addr,
				[](const ModuleRec& m, u64 a) { return m.end < a; });
			if (findIt != moduleEndAddresses.end() && addr >= findIt->start)
			{
				ModuleRec& rec = *findIt;
				if (rec.index == ~0u)
				{
					rec.index = u32(usedModules.size());
					usedModules.push_back(&rec);
				}
				out.Write7BitEncoded(rec.index);
				out.Write7BitEncoded(addr - rec.start);
			}
			else
			{
				out.Write7BitEncoded(~0u);
				out.Write7BitEncoded(addr);
			}
		}

		out.Write7BitEncoded(usedModules.size());
		for (auto* rec : usedModules)
		{
			out.Write7BitEncoded(rec->start);
			out.Write7BitEncoded(rec->size);
			out.WriteString(rec->name);
		}
#endif
#endif // UBA_SUPPORT_CALLSTACKS
		return true;
	}

	bool IsEscapePressed()
	{
		#if PLATFORM_WINDOWS
		return _kbhit() && _getch() == 27;
		#else
		return false;
		#endif
	}

	u32 GetCurrentProcessId()
	{
		#if PLATFORM_WINDOWS
		return ::GetCurrentProcessId();
		#else
		return getpid();
		#endif
	}

	void PrefetchVirtualMemory(const void* mem, u64 size)
	{
		#if PLATFORM_WINDOWS
		WIN32_MEMORY_RANGE_ENTRY entry;
		entry.VirtualAddress = const_cast<void*>(mem);
		entry.NumberOfBytes = size;
		PrefetchVirtualMemory(GetCurrentProcess(), 1, &entry, 0);
		#endif
	}

#if PLATFORM_WINDOWS
	void* g_startOfCurrentLibrary;
	void* g_endOfCurrentLibrary;
	bool g_reportAllExceptions;
	#if UBA_DEBUG
	thread_local bool g_expectingAccessViolation;
	#endif
	
	LONG UbaExceptionHandler(_EXCEPTION_POINTERS* ExceptionInfo)
	{
		auto& record = *ExceptionInfo->ExceptionRecord;
		u32 exceptionCode = record.ExceptionCode;

		#if UBA_DEBUG // A frame __except is expecting this AV; don't print a crash dump, let the unwind reach it.
		if (g_expectingAccessViolation && exceptionCode == EXCEPTION_ACCESS_VIOLATION)
			return EXCEPTION_CONTINUE_SEARCH;
		#endif

		if (!g_reportAllExceptions)
		{
			if (exceptionCode != EXCEPTION_STACK_OVERFLOW && exceptionCode != EXCEPTION_ACCESS_VIOLATION)
				return EXCEPTION_CONTINUE_SEARCH;

			if (!g_startOfCurrentLibrary)
			{
				HMODULE currentModule;
				GetModuleHandleExW(GET_MODULE_HANDLE_EX_FLAG_FROM_ADDRESS, (LPCWSTR)&g_startOfCurrentLibrary, &currentModule);
				MODULEINFO mi;
				GetModuleInformation(GetCurrentProcess(), currentModule, &mi, sizeof(mi));
				g_startOfCurrentLibrary = mi.lpBaseOfDll;
				g_endOfCurrentLibrary = (u8*)mi.lpBaseOfDll + mi.SizeOfImage;
			}

			if (record.ExceptionAddress < g_startOfCurrentLibrary || record.ExceptionAddress >= g_endOfCurrentLibrary)
				return EXCEPTION_CONTINUE_SEARCH;
		}
		CONTEXT* contextPtr = ExceptionInfo->ContextRecord;

		StringBuffer<> text;
		if (exceptionCode == EXCEPTION_ACCESS_VIOLATION)
			text.Appendf(L"ERROR: Access violation %s at address: 0x%p", (record.ExceptionInformation[0] == 1 ? L"writing" : L"reading"), (void*)record.ExceptionInformation[1]);
		else
			text.Appendf(L"ERROR: Unhandled Exception (Code: 0x%x)", exceptionCode);

		UbaAssert(text.data, nullptr, 0, nullptr, false, exceptionCode, contextPtr, 0);
		return EXCEPTION_CONTINUE_SEARCH;
	}

	DWORD UbaExceptionHandlerAllowAccessViolation(DWORD code, _EXCEPTION_POINTERS* info, const tchar* fileName)
	{
		if (code != EXCEPTION_ACCESS_VIOLATION)
			return EXCEPTION_CONTINUE_SEARCH;
		StringBuffer<> buf;
		buf.Appendf(L"ERROR: Access violation reading %s", fileName);
		UbaAssert(buf.data, __FILE__, __LINE__, "", true, code, info->ContextRecord, 0);
		return EXCEPTION_EXECUTE_HANDLER;
	}

#else
	static const int g_allSignals[] = {
		//#ifdef SIGHUP
		//SIGHUP,
		//#endif
		//#ifdef SIGQUIT
		//SIGQUIT,
		//#endif
		//#ifdef SIGTRAP
		//SIGTRAP,
		//#endif
		//#ifdef SIGIO
		//SIGIO,
		//#endif
		//SIGABRT,
		//SIGFPE,
		SIGBUS,
		SIGILL,
		//SIGINT,
		SIGSEGV,
		//SIGTERM 
	};
	static void segfault_sigaction(int signal, siginfo_t* info, void* ucontext_v)
	{
		// Recursion guard: the first-level path eventually calls UbaAssert, which acquires
		// the assert mutex and then calls __backtrace → libgcc _Unwind_Backtrace. The unwinder
		// can itself SEGV on a corrupt frame, re-entering this handler. The second UbaAssert
		// then blocks forever on the assert mutex held by the same thread, hanging the entire
		// agent (every NetClnt worker waiting on RPC blocks downstream). On nested entry, dump
		// a minimal record using async-signal-safe primitives only and _exit so the parent gets
		// a clean disconnect instead of a deadlock.

		#if PLATFORM_LINUX
		int tid = (int)syscall(SYS_gettid);
		#else
		int tid = 0;//gettid();
		#endif

		static thread_local int s_inHandler = 0;
		if (s_inHandler)
		{
			char buf[256];
			int n = snprintf(buf, sizeof(buf),
				"[NESTED SIGNAL %d] tid=%d — recursive fault inside crash handler (likely __backtrace); aborting without callstack to avoid mutex deadlock\n",
				signal, tid);
			(void)!write(2, buf, n);
			_exit(128 + signal);
		}
		s_inHandler = 1;

		const char* desc = signal == SIGSEGV ? "ERROR: Segmentation fault" : "ERROR: Unhandled signal";

		#if 0//PLATFORM_LINUX
		ucontext_t* uc = (ucontext_t*)ucontext_v;
		void* rip = uc ? (void*)uc->uc_mcontext.gregs[REG_RIP] : nullptr;
		void* fault_addr = info ? info->si_addr : nullptr;
		int pid = (int)getpid();
		int sicode = info ? info->si_code : 0;

		// JVM NPE-style faults look the same as real NULL-derefs from si_info; only post-mortem can distinguish.
		bool skipLog = false;

		// Per-pid file: detoured children may have stderr redirected away by the parent shell.
		char path[64];
		int n = snprintf(path, sizeof(path), "/tmp/uba_crash.%d.log", pid);
		(void)n;
		int fd = skipLog ? -1 : open(path, O_WRONLY | O_CREAT | O_APPEND, 0644);
		if (fd >= 0)
		{
			char buf[512];
			int blen = snprintf(buf, sizeof(buf),
				"[SIGNAL %d] pid=%d tid=%d rip=%p fault_addr=%p si_code=%d\n",
				signal, pid, tid, rip, fault_addr, sicode);
			(void)!write(fd, buf, blen);
			void* frames[40];
			int n2 = backtrace(frames, 40);
			for (int i = 0; i < n2; ++i)
			{
				blen = snprintf(buf, sizeof(buf), "  #%d %p\n", i, frames[i]);
				(void)!write(fd, buf, blen);
			}
			// Dump /proc/self/maps so frame addresses can be resolved post-mortem (ASLR makes
			// raw addresses meaningless). 8 KB cap covers typical layouts.
			(void)!write(fd, "[maps]\n", 7);
			int mapsFd = open("/proc/self/maps", O_RDONLY);
			if (mapsFd >= 0)
			{
				char mbuf[1024];
				ssize_t r;
				int written = 0;
				while ((r = read(mapsFd, mbuf, sizeof(mbuf))) > 0 && written < 8192)
				{
					(void)!write(fd, mbuf, (size_t)r);
					written += (int)r;
				}
				close(mapsFd);
			}
			close(fd);
		}
		
		fprintf(stderr, "\n[SIGNAL %d] tid=%d rip=%p fault_addr=%p si_code=%d\n", signal, tid, rip, fault_addr, sicode);
		fflush(stderr);
		#endif

		// Pass the kernel-provided ucontext_t to UbaAssert. GetCallstack's RBP walker reads
		// REG_RIP/REG_RBP out of it to seed the chain; the old backtrace() path treated this
		// as just a "from signal" marker and worked with any non-null pointer (e.g. &signal),
		// but the walker actually dereferences it.
		uba::UbaAssert(desc, "", 0, "", true, -1, ucontext_v, 0);
	}
#endif

	void AddExceptionHandler()
	{
#if PLATFORM_WINDOWS
		::AddVectoredExceptionHandler(TRUE, UbaExceptionHandler);
#else
		struct sigaction sa;
		memset(&sa, 0, sizeof(struct sigaction));
		sigemptyset(&sa.sa_mask);
		sa.sa_sigaction = segfault_sigaction;
		sa.sa_flags = SA_SIGINFO;
		for (auto signal : g_allSignals)
			sigaction(signal, &sa, NULL);

#endif
	}

	#if UBA_USE_MIMALLOC
	void* Oodle_MallocAligned(OO_SINTa bytes, OO_S32 alignment)
	{
		void* mem = mi_malloc_aligned(bytes, alignment);
		if (!mem)
			FatalError(9884, TC("Failed to allocate %llu bytes for oodle"), u64(bytes));
		return mem;
	}

	void Oodle_Free(void* ptr)
	{
		mi_free(ptr);
	}
	#endif

	#if UBA_USE_MIMALLOC && !PLATFORM_WINDOWS
	static void uba_mimalloc_error(int err, void* arg)
	{
		(void)arg;
		// Dump backtrace when mimalloc detects an error (double free, OOM, corruption).
		fprintf(stderr, "[mi-error err=%d] backtrace (resolve with: addr2line -e <bin> <addr>):\n", err);
		void* frames[32];
		int n = backtrace(frames, 32);
		// Async-signal-safe: avoid backtrace_symbols (malloc) — would deadlock during heap corruption.
		for (int i = 0; i < n; ++i)
			fprintf(stderr, "  #%d %p\n", i, frames[i]);
		fflush(stderr);
	}
	#endif

	void InitMemory()
	{
		#if UBA_USE_MIMALLOC

	#if MI_MALLOC_VERSION < 30000
		// 2.x: original behavior, purge_delay only.
		mi_option_set(mi_option_purge_delay, 100);
	#else
		// 3.x rewrote the allocator (segment → arena). Arenas reserve 1 GiB chunks
		// and only release on process exit (see arena.c:1347 — destroy_on_exit only).
		// On a 50K-process chromium build with mixed long-lived state + transient
		// per-edge allocs the defaults pile up to 25+ GB peak; the overrides below
		// bring it back to ~10 GB (matching 2.1.2).
		mi_option_set(mi_option_purge_delay, 100);              // 3.x default 1000 ms
		mi_option_set(mi_option_purge_decommits, 1);            // explicit decommit (default already 1)
		mi_option_set(mi_option_disallow_arena_alloc, 1);       // bypass arenas — go straight to OS
		mi_option_set(mi_option_page_full_retain, 0);           // default 2: free pages kept per size class
		mi_option_set(mi_option_page_max_reclaim, 0);           // default -1 (unlimited): per-theap retention
		mi_option_set(mi_option_page_cross_thread_max_reclaim, 0); // default 16/32: cross-thread retention
	#endif

		OodleCore_Plugins_SetAllocators(Oodle_MallocAligned, Oodle_Free);
		#if !PLATFORM_WINDOWS
		mi_register_error(uba_mimalloc_error, nullptr);
		#endif
		#endif
	}

	bool SetMemoryWorkingSet(u64 minWorkingSet)
	{
		#if PLATFORM_WINDOWS
		SIZE_T minWS, maxWS;
		DWORD flags;
		if (!GetProcessWorkingSetSizeEx(GetCurrentProcess(), &minWS, &maxWS, &flags))
			return false;
		if (minWorkingSet < minWS)
			return true;
		if (!SetProcessWorkingSetSizeEx(GetCurrentProcess(), minWorkingSet, SIZE_T(-1), flags))
			return false;
		#endif
		return true;
	}

	bool UseSemaphoreForLocalEvents()
	{
		#if PLATFORM_WINDOWS && UBA_TEST_WINDOWS_SEMAPHORES
		return true;
		#else
		return IsRunningDarling();
 		#endif
	}

	bool UseSemaphoreForSharedEvents()
	{
		return IsRunningDarling();
	}

#if !PLATFORM_WINDOWS
	void strcpy_s(tchar* dest, u64 destCapacity, const tchar* source)
	{
		u64 toCopy = Min(destCapacity-1, u64(strlen(source)));
		memcpy(dest, source, toCopy);
		dest[toCopy] = 0;
	}

	void strcat_s(tchar* dest, u64 destCapacity, const tchar* source)
	{
		u64 len = strlen(dest);
		strcpy_s(dest + len, destCapacity - len, source);
	}

	void GetMappingHandleName(StringBufferBase& out, u64 uid)
	{
		#if PLATFORM_MAC
		if (!IsRunningDarling())
			out.Append(TCV("/tmp"));
		#endif
		out.Append(TCV("/uba_")).AppendHex(uid);
	}

	bool GetPhysicallyInstalledSystemMemory(u64& outKb)
	{
	#if PLATFORM_MAC
		static long mem = []() -> int64_t
		{
			int64_t mem;
			size_t len = sizeof(mem);
			int mib[] = {CTL_HW, HW_MEMSIZE};
			if (sysctl(mib, 2, &mem, &len, NULL, 0) == 0)
				return mem;
			return 0;
		}();
	#else
		static long mem = sysconf(_SC_PHYS_PAGES) * sysconf(_SC_PAGE_SIZE);
	#endif
		outKb = mem/(1024*1024);
		return true;
	}

#endif

	void BinaryWriter::WriteLongString(const StringView& str, u32 threshold)
	{
		if (str.count < threshold)
		{
			WriteByte(0);
			WriteString(str);
			return;
		}

		WriteByte(sizeof(tchar));
		u32 uncompressedSize = str.count * sizeof(tchar);
		u64 needed = (u64)OodleLZ_GetCompressedBufferSizeNeeded(OodleLZ_Compressor_Kraken, uncompressedSize);
		Vector<u8> mem;
		mem.resize(needed);
		u64 compressedSize = OodleLZ_Compress(OodleLZ_Compressor_Kraken, str.data, (OO_SINTa)uncompressedSize, mem.data(), OodleLZ_CompressionLevel_Normal);
		UBA_ASSERT(compressedSize != OODLELZ_FAILED);
		Write7BitEncoded(str.count);
		Write7BitEncoded(compressedSize);
		WriteBytes(mem.data(), compressedSize);
	}

	BinaryWriter::PreparedLongString::PreparedLongString(const StringView& s, u32 threshold)
	:	str(s)
	{
		if (s.count < threshold)
			return;
		u32 uncompressedSize = s.count * sizeof(tchar);
		u64 needed = (u64)OodleLZ_GetCompressedBufferSizeNeeded(OodleLZ_Compressor_Kraken, uncompressedSize);
		compressedBuffer.resize(needed);
		u64 compressedSize = OodleLZ_Compress(OodleLZ_Compressor_Kraken, s.data, (OO_SINTa)uncompressedSize, compressedBuffer.data(), OodleLZ_CompressionLevel_Normal);
		UBA_ASSERT(compressedSize != OODLELZ_FAILED);
		compressedBuffer.resize(compressedSize);
	}

	u64 BinaryWriter::PreparedLongString::GetWriteSize() const
	{
		if (compressedBuffer.empty())
			return 1 + GetStringWriteSize(str.data, str.count);
		return 1 + Get7BitEncodedCount(str.count) + Get7BitEncodedCount(compressedBuffer.size()) + compressedBuffer.size();
	}

	void BinaryWriter::WriteLongString(const PreparedLongString& prepared)
	{
		if (prepared.compressedBuffer.empty())
		{
			WriteByte(0);
			WriteString(prepared.str);
			return;
		}
		WriteByte(sizeof(tchar));
		Write7BitEncoded(prepared.str.count);
		Write7BitEncoded(prepared.compressedBuffer.size());
		WriteBytes(prepared.compressedBuffer.data(), prepared.compressedBuffer.size());
	}

	TString BinaryReader::ReadLongString()
	{
		u8 s = ReadByte();
		if (!s)
			return ReadString();

		u64 stringLength = Read7BitEncoded();
		u64 uncompressedSize = stringLength * s;
		u64 compressedSize = Read7BitEncoded();

		const u8* data = GetPositionData();
		Skip(compressedSize);

		if (s == sizeof(tchar))
		{
			TString str;
			str.resize(stringLength);
			OO_SINTa decompLen = OodleLZ_Decompress(data, (OO_SINTa)compressedSize, str.data(), (OO_SINTa)uncompressedSize);
			UBA_ASSERT(decompLen == (OO_SINTa)uncompressedSize);(void)decompLen;
			return str;
		}
		else
		{
			// Tolerate desync: return empty rather than assert, so a malformed message fails one
			// process instead of taking down UbaNinja.
			if (s != 1)
				return TString();
			char* temp = new char[stringLength+1];
			auto g = MakeGuard([temp] { delete[] temp; });
			temp[stringLength] = 0;
			OO_SINTa decompLen = OodleLZ_Decompress(data, (OO_SINTa)compressedSize, temp, (OO_SINTa)uncompressedSize);
			if (decompLen != (OO_SINTa)uncompressedSize)
				return TString();
			return TString(temp, temp + stringLength);
		}
	}

	bool BinaryWriter::WriteBytesCompressed(const void* data, u64 size, bool assertOnFail)
	{
		UBA_ASSERT(data);
		UBA_ASSERT(size);
		u64 needed = (u64)OodleLZ_GetCompressedBufferSizeNeeded(OodleLZ_Compressor_Kraken, size);
		Vector<u8> mem;
		mem.resize(needed);
		u64 compressedSize = OodleLZ_Compress(OodleLZ_Compressor_Kraken, data, (OO_SINTa)size, mem.data(), OodleLZ_CompressionLevel_Normal);
		UBA_ASSERTF(!assertOnFail || compressedSize != OODLELZ_FAILED, TC("Failed to compress data of size %llu"), size);
		if (compressedSize == OODLELZ_FAILED)
			return false;
		Write7BitEncoded(compressedSize);
		WriteBytes(mem.data(), compressedSize);
		return true;
	}

	void BinaryReader::ReadBytesCompressed(void* outData, u64 outSize)
	{
		Vector<u8> compressed;
		compressed.resize(Read7BitEncoded());
		ReadBytes(compressed.data(), compressed.size());
		OO_SINTa decompLen = OodleLZ_Decompress(compressed.data(), (OO_SINTa)compressed.size(), outData, (OO_SINTa)outSize);
		UBA_ASSERT(decompLen == (OO_SINTa)outSize);(void)decompLen;
	}
}
