// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaSharedMemoryView.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaFileMapping.h"
#include "UbaHash.h"
#include "UbaMemory.h"
#include "UbaProcessStats.h"
#include "UbaTimer.h"

#if PLATFORM_WINDOWS
#include <windows.h>
#endif

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	SharedMemoryView::SharedMemoryView(FileMappingBackend& backend)
	:	m_fileMappingBackend(backend)
	{
	}

	bool SharedMemoryView::Init(SharedMemoryAllocatorHandle handle, u64 capacity, void* baseAddress, const tchar* hint)
	{
		return InternalInit(handle, capacity, baseAddress, hint);
	}

	bool SharedMemoryView::Init(SharedMemoryAllocatorHandle handle, u8* sharedMemory, BinaryReader& reader, u64 capacity, const tchar* hint, SharedMemoryMapType type)
	{
		u32 sliceCount = reader.ReadU32();
		if (sliceCount == 1 && sharedMemory)
		{
			u64 sliceOffset = reader.Read7BitEncoded();
			m_mappedSize = capacity;
			m_capacity = ~0ull;
			m_memory = sharedMemory + sliceOffset;
			return true;
		}
		if (!InternalInit(handle, capacity, nullptr, hint))
			return false;
		return AddRequestedMemory(handle, reader, sliceCount, type, 0, hint, 0);
	}

	bool SharedMemoryView::AddRequestedMemory(SharedMemoryAllocatorHandle handle, BinaryReader& reader, SharedMemoryMapType mapType, u64 offset, const tchar* hint, u8 init)
	{
		u32 sliceCount = reader.ReadU32();
		return AddRequestedMemory(handle, reader, sliceCount, mapType, offset, hint, init);
	}

	void SharedMemoryView::Reset()
	{
		if (!m_memory)
			return;

		if (m_capacity == ~0ull)
		{
			m_memory = nullptr;
			m_capacity = 0;
			m_mappedSize = 0;
			return;
		}

		m_fileMappingBackend.UnmapPlaceholder(m_memory, m_capacity, m_mappedSize, m_additionalMappings.data(), m_additionalMappings.size());
		m_additionalMappings.clear();
		m_mappedSize = 0;
		m_memory = nullptr;
		m_capacity = 0;
	}

	void SharedMemoryView::SetMemory(u8* memory, u64 capacity)
	{
		m_memory = memory;
		m_capacity = capacity;
	}

	u8* SharedMemoryView::DetachMemory()
	{
		u8* mem = m_memory;
		m_memory = nullptr;
		m_capacity = 0;
		m_mappedSize = 0;
		m_additionalMappings.clear();
		return mem;
	}

	SharedMemoryView::~SharedMemoryView()
	{
		Reset();
	}

	bool SharedMemoryView::InternalInit(SharedMemoryAllocatorHandle handle, u64 capacity, void* baseAddress, const tchar* hint)
	{
		if (m_memory)
			return true;
		m_capacity = AlignUp(capacity, PageSize);
		TimerScope ts(KernelStats::GetCurrent().virtualAlloc2);
		m_memory = (u8*)m_fileMappingBackend.ReservePlaceholder(baseAddress, m_capacity);
		return m_memory != nullptr;
	}

	bool SharedMemoryView::AddRequestedMemory(SharedMemoryAllocatorHandle handle, BinaryReader& reader, u32 sliceCount, SharedMemoryMapType mapType, u64 offset, const tchar* hint, u8 init)
	{
		while (sliceCount--)
		{
			u64 sliceOffset = reader.Read7BitEncoded();
			u64 sliceSize = reader.Read7BitEncoded();

			if (offset)
			{
				if (offset >= sliceSize)
				{
					offset -= sliceSize;
					continue;
				}
				else
				{
					sliceOffset += offset;
					sliceSize -= offset;
					offset = 0;
				}
			}

			if (!ExtendMemory(handle, sliceOffset, sliceSize, mapType, hint, init))
				return false;
		}
		return true;
	}

	void SharedMemoryView::ReferenceExternalMemory(u8* memory, u64 mappedSize)
	{
		m_memory = memory;
		m_mappedSize = mappedSize;
		m_capacity = ~0ull;
	}

	bool SharedMemoryView::ExtendMemory(SharedMemoryAllocatorHandle handle, u64 offset, u64 size, SharedMemoryMapType mapType, const tchar* hint, u8 init)
	{
		UBA_ASSERTF(handle.IsValid(), TC("Allocator handle not initialized (%s)"), hint);
		UBA_ASSERTF(m_memory, TC("Init failed?"));
		UBA_ASSERTF(m_capacity != ~0ull, TC("Can't extend memory on a view directly mapped on to allocator mapping (%s)"), hint);
		UBA_ASSERTF(AlignUp(offset, 4*1024) == offset, TC("ExtendMemory got offset %llu parameter which is not aligned (%s)"), offset, hint);

		u64 left = m_capacity - m_mappedSize;
		UBA_ASSERTF(left >= size, TC("Want to extend memory more than capacity (Capacity: %llu, Committed: %llu, Requested: %llu) (%s)"), m_capacity, m_mappedSize, size, hint);
		if (left < size)
			return false;

		auto& kernelStats = KernelStats::GetCurrent();

		TimerScope ts(kernelStats.mapViewOfFile3);
		void* viewA = m_fileMappingBackend.MapPlaceholder(handle, (MemoryMapType)mapType, m_memory, m_mappedSize, m_capacity, offset, size);
		UBA_ASSERTF(viewA == m_memory + m_mappedSize, TC("MapViewOfFile3 failed extending memory with %llu bytes at offset %llu - %s (%s)"), size, offset, LastErrorToText().data, hint);
		if (!viewA)
			return false;

		if (init)
			memset(viewA, init, size);

		if (m_mappedSize)
			m_additionalMappings.push_back(m_mappedSize);

		m_mappedSize += size;

		return true;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool IsMemoryHandle(const tchar* name)
	{
		UBA_ASSERT(name && *name);
		return *name == MemoryHandleChar || *name == FreeableMemoryHandleChar;
	}

	bool IsFreeableMemoryHandle(const tchar* name)
	{
		UBA_ASSERT(name && *name);
		return *name == FreeableMemoryHandleChar;
	}

	void GetMappingHandleAndOffset(const tchar* str, u64& outHandle, u64& outOffset)
	{
		const tchar* handleStr = str + 1;
		const tchar* handleStrEnd = TStrchr(handleStr, '-');
		outOffset = 0;
		if (!handleStrEnd)
		{
			handleStrEnd = handleStr + TStrlen(handleStr);
		}
		else
		{
			const tchar* mappingOffsetStr = handleStrEnd + 1;
			outOffset = StringToValueBase62(mappingOffsetStr, TStrlen(mappingOffsetStr));
		}
		outHandle = StringToValueBase62(handleStr, handleStrEnd - handleStr);
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
