// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaStringBuffer.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaPlatform.h"

namespace uba
{
	bool StartsWith(const tchar* data, const tchar* str, bool ignoreCase)
	{
		if (ignoreCase)
		{
			for (const tchar* ia = data, *ib = str;; ++ia, ++ib)
			{
				if (!*ib)
					return true;
				if (!*ia || ToLower(*ia) != ToLower(*ib))
					return false;
			}
		}
		else
		{
			for (const tchar* ia = data, *ib = str;; ++ia, ++ib)
			{
				if (!*ib)
					return true;
				if (!*ia || *ia != *ib)
					return false;
			}
		}
	}

	bool EndsWith(const tchar* str, u64 strLen, const tchar* value, u64 valueLen, bool ignoreCase)
	{
		if (strLen < valueLen)
			return false;
		if (!ignoreCase)
			return memcmp(str + strLen - valueLen, value, valueLen * sizeof(tchar)) == 0;
		for (const tchar* ia = str + strLen - valueLen, *ib = value; *ia; ++ia, ++ib)
			if (ToLower(*ia) != ToLower(*ib))
				return false;
		return true;
	}

	bool Contains(const tchar* str, const tchar* sub, bool ignoreCase, const tchar** pos)
	{
		if (!ignoreCase)
		{
			auto res = TStrstr(str, sub);
			if (pos)
				*pos = res;
			return res != nullptr;
		}
		for (const tchar* a = str; *a; ++a)
		{
			bool contains = true;
			for (const tchar* b = sub, *a2 = a; contains && *b; ++b, ++a2)
				contains = ToLower(*a2) == ToLower(*b);
			if (!contains)
				continue;
			if (pos)
				*pos = a;
			return true;
		}
		if (pos)
			*pos = nullptr;
		return false;
	}

	bool Equals(const tchar* str1, const tchar* str2, bool ignoreCase)
	{
		if (ignoreCase)
		{
			for (const tchar* ia = str1, *ib = str2;; ++ia, ++ib)
			{
				if (ToLower(*ia) != ToLower(*ib))
					return false;
				if (!*ia || !*ib)
					return *ia == *ib;
			}
		}
		else
			return TStrcmp(str1, str2) == 0;
	}

	bool Equals(const tchar* str1, const tchar* str2, u64 count, bool ignoreCase)
	{
		if (!count)
			return true;

		if (ignoreCase)
		{
			for (const tchar* ia = str1, *ib = str2;; ++ia, ++ib)
			{
				if (ToLower(*ia) != ToLower(*ib))
					return false;
				if (!--count)
					return true;
				if (!*ia || !*ib)
					return false;
			}
		}
		else
		{
			for (const tchar* ia = str1, *ib = str2;; ++ia, ++ib)
			{
				if (*ia != *ib)
					return false;
				if (!--count)
					return true;
				if (!*ia || !*ib)
					return false;
			}
		}
	}

	void Replace(tchar* str, tchar from, tchar to)
	{
		if (from == to)
			return;

		while (*str)
		{
			if (*str == from)
				*str = to;
			++str;
		}
	}

	void FixPathSeparators(tchar* str)
	{
		Replace(str, NonPathSeparator, PathSeparator);
	}

	bool Parse(u64& out, const tchar* str, u64 strLen)
	{
		if (!strLen)
			return false;

	#if UBA_STUB_BUILD
		// Freestanding stub has no strtoull. Detour core never parses numbers.
		(void)str; (void)strLen; out = 0; return false;
	#else
		#if PLATFORM_WINDOWS
		out = wcstoull(str, nullptr, 10);
		#else
		out = strtoull(str, nullptr, 10);
		#endif
		return out != 0 || Equals(str, TC("0"));
	#endif
	}

	const tchar* GetFileName(const tchar* path)
	{
		if (const tchar* lps = TStrrchr(path, PathSeparator))
			return lps + 1;
		return path;
	}

	StringBufferBase& StringBufferBase::Append(const tchar* str)
	{
		return Append(str, u32(TStrlen(str)));
	}

	StringBufferBase& StringBufferBase::Append(const tchar* str, u64 charCount)
	{
		UBA_ASSERTF(count + charCount < capacity, TC("Trying to append %llu character string to buffer which is %u long and has %u capacity left"), charCount, count, capacity - count);
		memcpy(data + count, str, charCount * sizeof(tchar));
		count += u32(charCount);
		data[count] = 0;
		return *this;
	}

	StringBufferBase& StringBufferBase::Append(StringView view)
	{
		return Append(view.data, view.count);
	}

	StringBufferBase& StringBufferBase::Appendf(const tchar* format, ...)
	{
		if (*format)
		{
			va_list arg;
			va_start(arg, format);
			Append(format, arg);
			va_end(arg);
		}
		return *this;
	}

	StringBufferBase& StringBufferBase::AppendDir(const StringBufferBase& str)
	{
		if (const tchar* last = str.Last(PathSeparator))
			return Append(str.data, u64(last - str.data));
		return *this;
	}

	StringBufferBase& StringBufferBase::AppendDir(const tchar* dir)
	{
		if (const tchar* last = TStrrchr(dir, PathSeparator))
			return Append(dir, u64(last - dir));
		return *this;
	}


	StringBufferBase& StringBufferBase::AppendFileName(const tchar* str)
	{
		const tchar* last = TStrrchr(str, PathSeparator);
		if (!last)
			last = TStrrchr(str, '/');
		else if (const tchar* last2 = TStrrchr(str, '/'))
			last = last2;
		if (last)
			return Append(last + 1);
		return Append(str);
	}

	StringBufferBase& StringBufferBase::AppendHex(u64 v)
	{
		tchar buf[256];
		ValueToString(buf, sizeof_array(buf), v);
		return Append(buf);
	}

	StringBufferBase& StringBufferBase::AppendBase62(u64 v)
	{
		tchar temp[16];
		tchar* it = temp + 16;
		if (v == 0)
			*--it = '0';
		while (v > 0)
		{
			*--it = g_base64Chars[v % 62];
			v /= 62;
		}
		return Append(it, 16 - u32(it - temp));
	}

	StringBufferBase& StringBufferBase::AppendValue(u64 v)
	{
		tchar inverse[32];
		int i = 0;
		do
		{
			inverse[i++] = '0' + (v % 10);
			v /= 10;
		}
		while (v);

		for (int j = 0; j < i; ++j)
			data[count++] = inverse[i - j - 1];
		data[count] = 0;

		return *this;
	}

	StringBufferBase& ReplaceEnd(StringBufferBase& sb, const tchar* str)
	{
		auto len = TStrlen(str);
		if (sb.count > sb.capacity - len - 1)
			sb.count = sb.capacity - len - 1;
		return sb.Append(str, len);
	}

	StringBufferBase& StringBufferBase::Append(const tchar* format, va_list& args)
	{
		#if PLATFORM_WINDOWS
		int len = _vscwprintf(format, args);
		va_list& args2 = args;
		#else
		va_list args2;
		va_copy(args2, args);
		auto g = MakeGuard([&]() { va_end(args2); });
		int len = vsnprintf(0, 0, format, args);
		#endif

		if (len < 0)
			return ReplaceEnd(*this, TC("PRINTF ERROR!"));

		if (len >= int(capacity - count) - 1)
			return ReplaceEnd(*this, TC("BUFFEROVERFLOW!"));

		int res = Tvsprintf_s(data + count, capacity - count, format, args2);

		if (res > 0)
			count += u32(res);
		else
			return ReplaceEnd(*this, TC("SPRINTF_ERROR!"));

		return *this;
	}

	StringBufferBase& StringBufferBase::Prepend(StringView view, u32 overwriteCount)
	{
		UBA_ASSERT(count + view.count - overwriteCount < capacity);
		memmove(data+view.count - overwriteCount, data, (count+1)*sizeof(tchar));
		memcpy(data, view.data, view.count*sizeof(tchar));
		count += view.count - overwriteCount;
		return *this;
	}

	#if PLATFORM_WINDOWS
	StringBufferBase& StringBufferBase::Append(const char* str)
	{
		u32 capacityEnd = capacity - 1;
		for (const char* i = str; *i; ++i)
			if (count < capacityEnd)
				data[count++] = *i;
		data[count] = 0;
		return *this;
	}
	
	StringBufferBase& StringBufferBase::Append(const char* str, u32 charCount)
	{
		u32 capacityEnd = capacity - 1;
		for (const char* i = str; charCount && *i; ++i, --charCount)
			if (count < capacityEnd)
				data[count++] = *i;
		data[count] = 0;
		return *this;
	}
	#endif

	StringBufferBase& StringBufferBase::Resize(u64 newSize)
	{
		UBA_ASSERT(newSize < capacity);
		data[newSize] = 0;
		count = u32(newSize);
		return *this;
	}

	StringBufferBase& StringBufferBase::Clear()
	{
		data[0] = 0;
		count = 0;
		return *this;
	}

	StringBufferBase& StringBufferBase::ReadString(BinaryReader& reader)
	{
		reader.ReadString(*this);
		return *this;
	}

	bool StringBufferBase::Contains(tchar c) const
	{
		return TStrchr(data, c) != nullptr;
	}

	bool StringBufferBase::StartsWith(StringView str, bool ignoreCase) const
	{
		return count >= str.count && uba::StartsWith(data, str.data, ignoreCase);
	}

	bool StringBufferBase::EndsWith(const tchar* value, bool ignoreCase) const
	{
		return uba::EndsWith(data, count, value, TStrlen(value), ignoreCase);
	}

	bool StringBufferBase::EndsWith(StringView value, bool ignoreCase) const
	{
		return uba::EndsWith(data, count, value.data, value.count, ignoreCase);
	}

	bool StringBufferBase::Equals(StringView str, bool ignoreCase) const
	{
		return count == str.count && uba::Equals(data, str.data, count, ignoreCase);
	}

	const tchar* StringBufferBase::First(tchar c, u64 offset) const
	{
		return TStrchr(data + offset, c);
	}

	const tchar* StringBufferBase::Last(tchar c, u64 offset) const
	{
		return TStrrchr(data + offset, c);
	}

	StringBufferBase& StringBufferBase::EnsureEndsWithSlash()
	{
		UBA_ASSERT(count);
		if (data[count - 1] == PathSeparator)
			return *this;
		UBA_ASSERT(count < capacity - 1);
		data[count++] = PathSeparator;
		data[count] = 0;
		return *this;
	}

	StringBufferBase& StringBufferBase::FixPathSeparators()
	{
		uba::FixPathSeparators(data);
		return *this;
	}

	StringBufferBase& StringBufferBase::MakeLower()
	{
		for (tchar* it = data; *it; ++it)
			*it = ToLower(*it);
		return *this;
	}

	bool StringBufferBase::Parse(u64& out) const
	{
		return ToView(*this).Parse(out);
	}

	bool StringBufferBase::Parse(u32& out, u64 offset) const
	{
		return ToView(*this).Parse(out, offset);
	}

	bool StringBufferBase::Parse(u16& out, u64 offset) const
	{
		return ToView(*this).Parse(out, offset);
	}

	bool StringBufferBase::Parse(bool& out) const
	{
		return ToView(*this).Parse(out);
	}

	bool StringBufferBase::Parse(float& out) const
	{
		return ToView(*this).Parse(out);
	}

	bool StringBufferBase::Parse(TString& out, u64 offset) const
	{
		return ToView(*this).Parse(out, offset);
	}

	bool StringBufferBase::Parse(StringBufferBase& out, u64 offset) const
	{
		return ToView(*this).Parse(out, offset);
	}

	u32 StringBufferBase::Parse(char* out, u64 outCapacity) const
	{
		return ToView(*this).Parse(out, outCapacity);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool StringView::Contains(tchar c, u32* outPos) const
	{
		for (const tchar* i = data, *e = data + count; i!=e; ++i)
		{
			if (*i != c)
				continue;
			if (outPos)
				*outPos = u32(i - data);
			return true;
		}
		return false;
	}

	bool StringView::StartsWith(StringView str, bool ignoreCase) const
	{
		return count >= str.count && uba::StartsWith(data, str.data, ignoreCase);
	}

	bool StringView::EndsWith(const tchar* value, bool ignoreCase) const
	{
		return uba::EndsWith(data, count, value, TStrlen(value), ignoreCase);
	}

	StringView StringView::GetFileName() const
	{
		if (count)
			for (const tchar* i=data+count-1, *e=data; i!=e; --i)
				if (*i == PathSeparator)
					return StringView(i+1, u32(data+count-i-1));
		return *this;
	}

	StringView StringView::GetPath(bool keepLastSlash) const
	{
		if (count)
			for (const tchar* i=data+count-1, *e=data; i!=e; --i)
				if (*i == PathSeparator)
					return StringView(data, u32(i-data) + u32(keepLastSlash));
		return {};
	}

	StringView StringView::Trim()
	{
		StringView sv(*this);
		while (*sv.data == ' ' || *sv.data == '\t')
		{
			++sv.data;
			--sv.count;
		}
		while (sv.count && (sv.data[sv.count-1] == ' ' || sv.data[sv.count-1] == '\t'))
			--count;
		return sv;
	}

	const tchar* StringView::CheckTerminated() const
	{
		UBA_ASSERTF(IsTerminated(), TC("StringView is not terminated (%.*s)"), count, data);
		return data;
	}

	bool StringView::Parse(u64& out) const
	{
		return uba::Parse(out, data, count);
	}

	bool StringView::Parse(u32& out, u64 offset) const
	{
		if (count <= offset)
			return false;
	#if UBA_STUB_BUILD
		(void)offset; out = 0; return false;
	#else
		#if PLATFORM_WINDOWS
		out = wcstoul(data + offset, nullptr, 10);
		#else
		out = strtoul(data + offset, nullptr, 10);
		#endif
		return out != 0 || Equals(TCV("0"));
	#endif
	}

	bool StringView::Parse(u16& out, u64 offset) const
	{
		u32 temp;
		if (!Parse(temp, offset))
			return false;
		if (temp > 65535)
			return false;
		out = u16(temp);
		return true;
	}

	bool StringView::Parse(bool& out) const
	{
		if (Equals(TCV("true")) || Equals(TCV("1")))
		{
			out = true;
			return true;
		}
		if (Equals(TCV("false")) || Equals(TCV("0")))
		{
			out = false;
			return true;
		}
		return false;
	}

	bool StringView::Parse(float& out) const
	{
		if (!count)
			return false;
	#if UBA_STUB_BUILD
		out = 0; return false;
	#else
		#if PLATFORM_WINDOWS
		out = wcstof(data, nullptr);
		#else
		out = strtof(data, nullptr);
		#endif
		return out != 0 || Equals(TCV("0"));
	#endif
	}

	bool StringView::Parse(TString& out, u64 offset) const
	{
		out.append(data + offset, count - offset);
		return true;
	}

	bool StringView::Parse(StringBufferBase& out, u64 offset) const
	{
		out.Append(data + offset, count - offset);
		return true;
	}

	u32 StringView::Parse(char* out, u64 outCapacity) const
	{
		#if PLATFORM_WINDOWS
		return WideCharToMultiByte(CP_ACP, 0, data, count+1, out, int(outCapacity), nullptr, nullptr);
		#else
		if (outCapacity == 0)
			return 0;
		u32 toCopy = Min(u32(outCapacity - 1), count);
		memcpy(out, data, toCopy);
		out[toCopy] = 0;
		return toCopy;
		#endif
	}


	StringView ToView(const tchar* s)
	{
		return s ? StringView(s, TStrlen(s)) : StringView();;
	}

	StringView ToView(StringView sv)
	{
		return sv;
	}

	StringView ToView(StringBufferBase& sb)
	{
		return StringView(sb);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

#if !UBA_STUB_BUILD
	// Stub has no GetLastError (Win32) / errno-to-text plumbing. The LD_PRELOAD
	// detour is the only non-test consumer of this ctor; gating it out keeps
	// GetLastError + LastErrorToText(u32) off the stub's unresolved-symbol list.
	LastErrorToText::LastErrorToText() : LastErrorToText(GetLastError())
	{
	}
#endif

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
