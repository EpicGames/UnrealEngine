// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaSynchronization.h"
#if UBA_USE_PARKINGLOT
#include "UbaParkingLot.h"
#endif
#include "UbaPlatform.h"
#include "UbaStringBuffer.h"

namespace uba
{
	#if UBA_USE_PARKINGLOT

	inline static u32 getCurrentThreadId()
	{
		#if PLATFORM_WINDOWS
		return GetCurrentThreadId();
		#elif PLATFORM_LINUX
		static thread_local u32 threadId = []
		{
			static_assert(sizeof(pid_t) <= sizeof(u32));
			u32 id = (u32)static_cast<pid_t>(syscall(SYS_gettid));
			UBA_ASSERT(id != 0);
			return id;
		}();
		return threadId;
		#elif PLATFORM_MAC
		return (u32)pthread_mach_thread_np(pthread_self());
		#endif
	}

	bool CriticalSection::TryEnter()
	{
		const u32 currentThreadId = getCurrentThreadId();
		u32 currentState = state.load(std::memory_order_relaxed);

		// Try to acquire the lock if it was unlocked, even if there are waiting threads.
		// Acquiring the lock despite the waiting threads means that this lock is not FIFO and thus not fair.
		if (!(currentState & lockCountMask))
		{
			if (state.compare_exchange_strong(currentState, currentState | (1 << lockCountShift), std::memory_order_acquire, std::memory_order_relaxed))
			{
				UBA_ASSERT(threadId.load(std::memory_order_relaxed) == 0);
				threadId.store(currentThreadId, std::memory_order_relaxed);
				return true;
			}
		}

		// Lock recursively if this is the thread that holds the lock.
		if (threadId.load(std::memory_order_relaxed) == currentThreadId)
		{
			state.fetch_add(1 << lockCountShift, std::memory_order_relaxed);
			return true;
		}

		return false;
	}

	void CriticalSection::Enter()
	{
		const u32 currentThreadId = getCurrentThreadId();
		u32 currentState = state.load(std::memory_order_relaxed);

		// Try to acquire the lock if it was unlocked, even if there are waiting threads.
		// Acquiring the lock despite the waiting threads means that this lock is not FIFO and thus not fair.
		if (!(currentState & lockCountMask))
		{
			if (state.compare_exchange_weak(currentState, currentState | (1 << lockCountShift), std::memory_order_acquire, std::memory_order_relaxed))
			{
				UBA_ASSERT(threadId.load(std::memory_order_relaxed) == 0);
				threadId.store(currentThreadId, std::memory_order_relaxed);
				return;
			}
		}

		// Lock recursively if this is the thread that holds the lock.
		if (threadId.load(std::memory_order_relaxed) == currentThreadId)
		{
			state.fetch_add(1 << lockCountShift, std::memory_order_relaxed);
			return;
		}

		EnterSlow(currentState, currentThreadId);
	}

	void CriticalSection::EnterSlow(u32 currentState, u32 currentThreadId)
	{
		constexpr u32 spinLimit = 40;
		u32 spinCount = 0;
		for (;;)
		{
			// Try to acquire the lock if it was unlocked, even if there are waiting threads.
			// Acquiring the lock despite the waiting threads means that this lock is not FIFO and thus not fair.
			if (!(currentState & lockCountMask))
			{
				if (state.compare_exchange_weak(currentState, currentState | (1 << lockCountShift), std::memory_order_acquire, std::memory_order_relaxed))
				{
					UBA_ASSERT(threadId.load(std::memory_order_relaxed) == 0);
					threadId.store(currentThreadId, std::memory_order_relaxed);
					return;
				}
				continue;
			}

			// Spin up to the spin limit while there are no waiting threads.
			if (!(currentState & mayHaveWaitingLockFlag) && spinCount < spinLimit)
			{
				YieldInSpinWait();
				++spinCount;
				currentState = state.load(std::memory_order_relaxed);
				continue;
			}

			// Store that there are waiting threads. Restart if the state has changed since it was loaded.
			if (!(currentState & mayHaveWaitingLockFlag))
			{
				if (!state.compare_exchange_weak(currentState, currentState | mayHaveWaitingLockFlag, std::memory_order_relaxed))
				{
					continue;
				}
				currentState |= mayHaveWaitingLockFlag;
			}

			// Wait if the state has not changed. Either way, loop back and try to acquire the lock after trying to wait.
			ParkingLot::Wait(&state, [this, currentState] { return state.load(std::memory_order_relaxed) == currentState; }, nullptr);
			currentState = state.load(std::memory_order_relaxed);
		}
	}

	void CriticalSection::Leave()
	{
		u32 currentState = state.load(std::memory_order_relaxed);
		UBA_ASSERT(currentState & lockCountMask);
		UBA_ASSERT(threadId.load(std::memory_order_relaxed) == getCurrentThreadId());

		if ((currentState & lockCountMask) == (1 << lockCountShift))
		{
			// Remove the association with this thread before unlocking.
			threadId.store(0, std::memory_order_relaxed);

			// Unlock immediately to allow other threads to acquire the lock while this thread looks for a thread to wake.
			const u32 lastState = state.fetch_sub(1 << lockCountShift, std::memory_order_release);

			// Wake one exclusive waiter if there are waiting threads.
			if (lastState & mayHaveWaitingLockFlag)
			{
				WakeWaitingThread();
			}
		}
		else
		{
			// This is recursively locked. Decrement the lock count.
			state.fetch_sub(1 << lockCountShift, std::memory_order_relaxed);
		}
	}

	void CriticalSection::WakeWaitingThread()
	{
		ParkingLot::WakeOne(&state, [this](ParkingLot::WakeState wakeState) -> u64
		{
			if (!wakeState.hasWaitingThreads)
			{
				state.fetch_and(~mayHaveWaitingLockFlag, std::memory_order_relaxed);
			}
			return 0;
		});
	}

	inline const void* ReaderWriterLock::GetLockAddress() const
	{
		return &state;
	}

	inline const void* ReaderWriterLock::GetReadLockAddress() const
	{
		// Read locks need a distinct address from exclusive locks to allow threads waiting for exclusive ownership
		// to be woken up without waking any threads waiting for read ownership.
		return (const u8*)&state + 1;
	}

	UBA_NOINLINE void ReaderWriterLock::EnterSlow()
	{
		constexpr u32 spinLimit = 40;
		u32 spinCount = 0;
		for (u32 currentState = state.load(std::memory_order_relaxed);;)
		{
			// Try to acquire the lock if it was unlocked, even if there are waiting threads.
			// Acquiring the lock despite the waiting threads means that this lock is not FIFO and thus not fair.
			if (!(currentState & (isLockedFlag | readLockCountMask)))
			{
				if (state.compare_exchange_weak(currentState, currentState | isLockedFlag, std::memory_order_acquire, std::memory_order_relaxed))
				{
					return;
				}
				continue;
			}

			// Spin up to the spin limit while there are no waiting threads.
			if (!(currentState & mayHaveWaitingLockFlag) && spinCount < spinLimit)
			{
				YieldInSpinWait();
				++spinCount;
				currentState = state.load(std::memory_order_relaxed);
				continue;
			}

			// Store that there are waiting threads. Restart if the state has changed since it was loaded.
			if (!(currentState & mayHaveWaitingLockFlag))
			{
				if (!state.compare_exchange_weak(currentState, currentState | mayHaveWaitingLockFlag, std::memory_order_relaxed))
				{
					continue;
				}
				currentState |= mayHaveWaitingLockFlag;
			}

			// Wait if the state has not changed. Either way, loop back and try to acquire the lock after trying to wait.
			ParkingLot::Wait(GetLockAddress(), [this]
			{
				const u32 newState = state.load(std::memory_order_relaxed);
				return (newState & (isLockedFlag | readLockCountMask)) && (newState & mayHaveWaitingLockFlag);
			}, nullptr);
			currentState = state.load(std::memory_order_relaxed);
		}
	}

	UBA_NOINLINE void ReaderWriterLock::EnterReadSlow() const
	{
		constexpr u32 spinLimit = 40;
		u32 spinCount = 0;
		for (u32 currentState = state.load(std::memory_order_relaxed);;)
		{
			// Try to acquire the lock if it is unlocked and there are no waiting threads.
			if (!(currentState & (isLockedFlag | mayHaveWaitingLockFlag)))
			{
				if (state.compare_exchange_weak(currentState, currentState + (1 << readLockCountShift), std::memory_order_acquire, std::memory_order_relaxed))
				{
					return;
				}
				continue;
			}

			// Spin up to the spin limit while there are no waiting threads.
			if (!(currentState & mayHaveWaitingLockFlag) && spinCount < spinLimit)
			{
				YieldInSpinWait();
				++spinCount;
				currentState = state.load(std::memory_order_relaxed);
				continue;
			}

			// Store that there are waiting threads. Restart if the state has changed since it was loaded.
			if (!(currentState & mayHaveWaitingReadLockFlag))
			{
				if (!state.compare_exchange_weak(currentState, currentState | mayHaveWaitingReadLockFlag, std::memory_order_relaxed))
				{
					continue;
				}
				currentState |= mayHaveWaitingReadLockFlag;
			}

			// Wait if the state has not changed. Either way, loop back and try to acquire the lock after trying to wait.
			ParkingLot::Wait(GetReadLockAddress(), [this, currentState]
			{
				return state.load(std::memory_order_relaxed) == currentState;
			}, nullptr);
			currentState = state.load(std::memory_order_relaxed);
		}
	}

	UBA_NOINLINE void ReaderWriterLock::WakeWaitingThread() const
	{
		ParkingLot::WakeOne(GetLockAddress(), [this](ParkingLot::WakeState wakeState) -> u64
		{
			if (!wakeState.didWake)
			{
				// Keep the flag until no thread wakes, otherwise read locks may win before
				// an exclusive lock has a chance.
				state.fetch_and(~mayHaveWaitingLockFlag, std::memory_order_relaxed);
			}
			return 0;
		});
	}

	UBA_NOINLINE void ReaderWriterLock::WakeWaitingThreads(u32 lastState)
	{
		if (lastState & mayHaveWaitingLockFlag)
		{
			// Wake one thread that is waiting to acquire an exclusive lock.
			bool didWake = false;
			ParkingLot::WakeOne(GetLockAddress(), [this, &didWake](ParkingLot::WakeState wakeState) -> u64
			{
				if (!wakeState.didWake)
				{
					// Keep the flag until no thread wakes, otherwise read locks may win before
					// an exclusive lock has a chance.
					state.fetch_and(~mayHaveWaitingLockFlag, std::memory_order_relaxed);
				}
				didWake = wakeState.didWake;
				return 0;
			});
			if (didWake)
			{
				return;
			}

			// Reload the state if there were no read waiters because new
			// ones may have registered themselves since lastState was read.
			if (!(lastState & mayHaveWaitingReadLockFlag))
			{
				lastState = state.load(std::memory_order_relaxed);
			}
		}

		if (lastState & mayHaveWaitingReadLockFlag)
		{
			// Wake every thread that is waiting to acquire a read lock.
			// The awoken threads might race against other exclusive locks.
			if (state.fetch_and(~mayHaveWaitingReadLockFlag, std::memory_order_relaxed) & mayHaveWaitingReadLockFlag)
			{
				ParkingLot::WakeAll(GetReadLockAddress());
			}
		}
	}

	#else // #if UBA_USE_PARKINGLOT

	CriticalSection::CriticalSection(bool recursive)
	{
		#if PLATFORM_WINDOWS
		static_assert(alignof(CRITICAL_SECTION) == alignof(CriticalSection));
		static_assert(sizeof(data) >= sizeof(CRITICAL_SECTION));
		InitializeCriticalSection((CRITICAL_SECTION*)&data);
		#else
		static_assert(alignof(pthread_mutex_t) == alignof(CriticalSection));
		static_assert(sizeof(data) >= sizeof(pthread_mutex_t));
		int res;
		if (recursive)
		{
			pthread_mutexattr_t attr;
			pthread_mutexattr_init(&attr);
			pthread_mutexattr_settype(&attr, PTHREAD_MUTEX_RECURSIVE);
			res = pthread_mutex_init((pthread_mutex_t*)data, &attr);
		}
		else
			res = pthread_mutex_init((pthread_mutex_t*)data, nullptr);

		UBA_ASSERTF(res == 0, TC("pthread_mutex_init failed: %i"), res);(void)res;
		#endif
	}

	CriticalSection::~CriticalSection()
	{
		#if PLATFORM_WINDOWS
		#if UBA_DEBUG
		if (TryEnterCriticalSection((CRITICAL_SECTION*)&data))
			LeaveCriticalSection((CRITICAL_SECTION*)&data);
		else
			UBA_ASSERT(false);
		#endif
		DeleteCriticalSection((CRITICAL_SECTION*)&data);
		#else
		int res = pthread_mutex_destroy((pthread_mutex_t*)data);(void)res;
		UBA_ASSERTF(res == 0, TC("pthread_mutex_destroy failed: %i"), res);
		#endif
	}

	void CriticalSection::Enter()
	{
		#if PLATFORM_WINDOWS
		EnterCriticalSection((CRITICAL_SECTION*)&data);
		#else
		int res = pthread_mutex_lock((pthread_mutex_t*)data);(void)res;
		UBA_ASSERTF(res == 0, TC("pthread_mutex_lock failed: %i"), res);
		#endif
	}

	void CriticalSection::Leave()
	{
		#if PLATFORM_WINDOWS
		LeaveCriticalSection((CRITICAL_SECTION*)&data);
		#else
		int res = pthread_mutex_unlock((pthread_mutex_t*)data);(void)res;
		UBA_ASSERTF(res == 0, TC("pthread_mutex_unlock failed: %i"), res);
		#endif
	}

	bool CriticalSection::TryEnter() const
	{
		#if PLATFORM_WINDOWS
		return TryEnterCriticalSection((CRITICAL_SECTION*)&data);
		#else
		int res = pthread_mutex_trylock((pthread_mutex_t*)data);(void)res;
		if (res == 0)
			return true;
		UBA_ASSERTF(res == EBUSY, TC("pthread_mutex_lock failed: %i"), res);
		return false;
		#endif
	}

	ReaderWriterLock::ReaderWriterLock()
	{
		#if PLATFORM_WINDOWS
		static_assert(sizeof(data) >= sizeof(SRWLOCK));
		static_assert(alignof(SRWLOCK) == alignof(ReaderWriterLock));
		*(SRWLOCK*)&data = SRWLOCK_INIT;
		#else
		static_assert(alignof(pthread_rwlock_t) == alignof(ReaderWriterLock));
		static_assert(sizeof(data) >= sizeof(pthread_rwlock_t));
		int res = pthread_rwlock_init((pthread_rwlock_t*)data, NULL);(void)res;
		UBA_ASSERTF(res == 0, TC("pthread_rwlock_init failed: %i"), res);
		#endif
	}

	ReaderWriterLock::~ReaderWriterLock()
	{
		#if PLATFORM_WINDOWS
		#if UBA_DEBUG
		if (TryAcquireSRWLockExclusive((SRWLOCK*)&data))
			ReleaseSRWLockExclusive((SRWLOCK*)&data);
		else
			UBA_ASSERT(false);
		#endif
		#else
		int res = pthread_rwlock_destroy((pthread_rwlock_t*)data);(void)res;
		UBA_ASSERTF(res == 0, TC("pthread_rwlock_destroy failed: %i"), res);
		#endif
	}

	void ReaderWriterLock::EnterRead() const
	{
		#if PLATFORM_WINDOWS
		AcquireSRWLockShared((SRWLOCK*)&data);
		#else
		int res = pthread_rwlock_rdlock((pthread_rwlock_t*)data);(void)res;
		UBA_ASSERTF(res == 0, TC("pthread_rwlock_rdlock failed: %i"), res);
		#endif
	}

	void ReaderWriterLock::LeaveRead() const
	{
		#if PLATFORM_WINDOWS
		ReleaseSRWLockShared((SRWLOCK*)&data);
		#else
		int res = pthread_rwlock_unlock((pthread_rwlock_t*)data);(void)res;
		UBA_ASSERTF(res == 0, TC("pthread_rwlock_unlock failed: %i"), res);
		#endif
	}

	void ReaderWriterLock::Enter()
	{
		#if PLATFORM_WINDOWS
		AcquireSRWLockExclusive((SRWLOCK*)&data);
		#else
		int res = pthread_rwlock_wrlock((pthread_rwlock_t*)data);(void)res;
		UBA_ASSERTF(res == 0, TC("pthread_rwlock_wrlock failed: %i"), res);
		#endif
	}

	bool ReaderWriterLock::TryEnter()
	{
		#if PLATFORM_WINDOWS
		return TryAcquireSRWLockExclusive((SRWLOCK*)&data) != 0;
		#else
		return pthread_rwlock_trywrlock((pthread_rwlock_t*)data) == 0;
		#endif
	}

	void ReaderWriterLock::Leave()
	{
		#if PLATFORM_WINDOWS
		ReleaseSRWLockExclusive((SRWLOCK*)&data);
		#else
		int res = pthread_rwlock_unlock((pthread_rwlock_t*)data);(void)res;
		UBA_ASSERTF(res == 0, TC("pthread_rwlock_unlock failed: %i"), res);
		#endif
	}

	#endif // #if UBA_USE_PARKINGLOT / #else

	#if UBA_TRACK_CONTENTION

	List<ContentionTracker>& GetContentionTrackerList()
	{
		static List<ContentionTracker> trackers;
		return trackers;
	}


	ContentionTracker& GetContentionTracker(const char* file, u64 line)
	{
		static Futex rwl;
		ScopedFutex l(rwl);
		ContentionTracker& ct = GetContentionTrackerList().emplace_back();
		ct.file = file;
		ct.line = line;
		return ct;
	}
	#endif
}
