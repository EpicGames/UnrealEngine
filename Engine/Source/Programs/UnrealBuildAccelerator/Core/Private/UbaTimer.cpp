// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaTimer.h"
#include "UbaPlatform.h"

#include <wchar.h>
#if !PLATFORM_WINDOWS
#include <sys/time.h>
#endif

#define UBA_USE_GETTIMEOFDAY 0

namespace uba
{
	#if !PLATFORM_WINDOWS
	u64 GetRawTimeNs()
	{
		#if PLATFORM_MAC
		return clock_gettime_nsec_np(CLOCK_UPTIME_RAW);
		#else
		// CLOCK_REALTIME, not CLOCK_MONOTONIC: under WSL2/Hyper-V the kernel's
		// TSC calibration can be off by several percent, but CLOCK_REALTIME is
		// step-corrected to track the host wall-clock via host-time-integration.
		// Same vDSO cost (~17 ns/call), accurate units, agrees with the visualizer
		// running on the host.
		struct timespec ts;
		if (clock_gettime(CLOCK_REALTIME, &ts) == -1)
			FatalError(1401, TC("clock_gettime(CLOCK_REALTIME) failed"));
		return u64(ts.tv_sec * 1'000'000'000LL + ts.tv_nsec);
		#endif
	}
	#endif

	u64 GetTime()
	{
	#if PLATFORM_WINDOWS
		LARGE_INTEGER li;
		QueryPerformanceCounter(&li);
		return li.QuadPart;
	#elif UBA_USE_GETTIMEOFDAY
		timeval tv;
		gettimeofday(&tv, NULL); // Returns time in microseconds since 1 Jan 1970
		return u64(tv.tv_sec) * 1'000'000ull + u64(tv.tv_usec);
	#else
		return GetRawTimeNs();
	#endif
	}

	u64 GetFrequency()
	{
	#if PLATFORM_WINDOWS
		static u64 frequency = []() { LARGE_INTEGER li; QueryPerformanceFrequency(&li); return li.QuadPart; }();
		return frequency;
	#elif UBA_USE_GETTIMEOFDAY
		return 1000000LL;
	#else
		return 1'000'000'000LL;
	#endif
	}

	u64 GetSystemTimeUs()
	{
		#if PLATFORM_WINDOWS
		constexpr u64 EPOCH_DIFF = 11'644'473'600ull; // Seconds from 1 Jan. 1601 to 1970 (windows to linux)
		FILETIME st;
		GetSystemTimeAsFileTime(&st);
		return *(u64*)&st / 10 - (EPOCH_DIFF*1'000'000ull);
		#else
		timeval tv;
		gettimeofday(&tv, NULL); // Returns time in microseconds since 1 Jan 1970
		return u64(tv.tv_sec) * 1'000'000ull + u64(tv.tv_usec);
		#endif
	}
}