// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBase.h"
#include <utility> // std::move

namespace uba
{
	// Minimal std::lower_bound replacement. Returns the first `it` in [first, last)
	// for which `cmp(*it, value)` is false. Requires random-access iterators.
	template<typename It, typename T, typename Cmp>
	inline It LowerBound(It first, It last, const T& value, Cmp cmp)
	{
		while (first < last)
		{
			It mid = first + (last - first) / 2;
			if (cmp(*mid, value))
				first = mid + 1;
			else
				last = mid;
		}
		return first;
	}

	template<typename It, typename T>
	inline It LowerBound(It first, It last, const T& value)
	{
		return LowerBound(first, last, value, [](const auto& a, const auto& b) { return a < b; });
	}

	// Minimal std::binary_search replacement. Returns true if `value` is present.
	template<typename It, typename T, typename Cmp>
	inline bool BinarySearch(It first, It last, const T& value, Cmp cmp)
	{
		It it = LowerBound(first, last, value, cmp);
		return it != last && !cmp(value, *it);
	}

	template<typename It, typename T>
	inline bool BinarySearch(It first, It last, const T& value)
	{
		return BinarySearch(first, last, value, [](const auto& a, const auto& b) { return a < b; });
	}

	// Minimal introsort-style sort. Quicksort with median-of-three pivot; falls
	// back to insertion sort on small partitions. Requires random-access iterators.
	namespace detail
	{
		template<typename It, typename Cmp>
		inline void InsertionSort(It first, It last, Cmp cmp)
		{
			for (It it = first + 1; it < last; ++it)
			{
				auto tmp = std::move(*it);
				It hole = it;
				while (hole > first && cmp(tmp, *(hole - 1)))
				{
					*hole = std::move(*(hole - 1));
					--hole;
				}
				*hole = std::move(tmp);
			}
		}

		template<typename It, typename Cmp>
		inline void QuickSort(It first, It last, Cmp cmp)
		{
			while (last - first > 16)
			{
				// Median-of-three pivot: first, mid, last-1.
				It mid = first + (last - first) / 2;
				It hi  = last - 1;
				if (cmp(*mid, *first)) { auto t = std::move(*mid); *mid = std::move(*first); *first = std::move(t); }
				if (cmp(*hi, *first))  { auto t = std::move(*hi);  *hi  = std::move(*first); *first = std::move(t); }
				if (cmp(*hi, *mid))    { auto t = std::move(*hi);  *hi  = std::move(*mid);   *mid   = std::move(t); }
				// Pivot now at mid. Move pivot to hi-1 as sentinel, partition [first+1, hi-1).
				{ auto t = std::move(*mid); *mid = std::move(*(hi - 1)); *(hi - 1) = std::move(t); }
				auto pivot = std::move(*(hi - 1));
				It i = first, j = hi - 1;
				for (;;)
				{
					while (cmp(*(++i), pivot)) {}
					while (cmp(pivot, *(--j))) {}
					if (i >= j) break;
					auto t = std::move(*i); *i = std::move(*j); *j = std::move(t);
				}
				*(hi - 1) = std::move(*i); *i = std::move(pivot);
				// Recurse smaller side, iterate larger.
				if (i - first < last - (i + 1))
				{
					QuickSort(first, i, cmp);
					first = i + 1;
				}
				else
				{
					QuickSort(i + 1, last, cmp);
					last = i;
				}
			}
			InsertionSort(first, last, cmp);
		}
	}

	template<typename It, typename Cmp>
	inline void Sort(It first, It last, Cmp cmp)
	{
		detail::QuickSort(first, last, cmp);
	}

	template<typename It>
	inline void Sort(It first, It last)
	{
		Sort(first, last, [](const auto& a, const auto& b) { return a < b; });
	}
}
