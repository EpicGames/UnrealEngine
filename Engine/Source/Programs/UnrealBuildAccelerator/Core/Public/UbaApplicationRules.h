// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaStringBuffer.h"

namespace uba
{
	class GlobalRules
	{
	public:
		bool FileCanBeCompressed(const StringView& fileName);
		u8 GetUsedCountBeforeFree(const StringView& fileName);
	};

	extern GlobalRules g_globalRules;


	enum DependencyCrawlerType
	{
		DependencyCrawlerType_None,
		DependencyCrawlerType_ClangCompiler,
		DependencyCrawlerType_MsvcCompiler,
		DependencyCrawlerType_ClangLinker,
		DependencyCrawlerType_MsvcLinker,
	};


	class ApplicationRules
	{
	public:
		// This means that process can run entirely without console. (win32 flag DETACHED_PROCESS)
		// Uba stub out console interaction. This is an optimization that is entirely optional
		virtual bool AllowDetach() const
		{
			return false;
		}

		// Control if certain files should not be detoured
		virtual bool CanDetour(StringView file, bool isRunningRemote) const
		{
			return true;
		}

		// Throw-away means that the file is temporary and will not be used after process exists. (By default these are kept in memory and never touch disk)
		virtual bool IsThrowAway(StringView fileName, bool isRunningRemote) const
		{
			return false;
		}

		// Keep file in memory
		// If this returns true it means that file will be kept in memory and never touch disk.
		virtual bool KeepInMemory(StringView fileName, StringView systemTemp, bool isRunningRemote, bool isWrite) const
		{
			return IsThrowAway(fileName, isRunningRemote);
		}

		// Max file size if using memory files. Setting this value to a proper size is important because of Wine on linux use this
		// .. on windows this size can be very big since it is just virtual address space and nothing is committed...
		// On wine (as of version 9.10) a ftruncate is done on the wine server causing large max sizes to take up actual disk space on the tmp partition
		virtual u64 FileTypeMaxSize(const StringView& file, bool isSystemOrTempFile) const
		{
			return 8ull * 1024 * 1024 * 1024;
		}

		// When writing this type of file, how big commits should shared memory system take. We want as few as possible without wasting memory
		virtual u32 FileTypeCommitSize(StringView file) const
		{
			return 1 * 1024 * 1024;
		}

		// Outputfile means that it is kept in memory and then sent back to session process which can decide to write it to disk or send it over network
		virtual bool IsOutputFile(StringView fileName, StringView systemTemp) const
		{
			return false;
		}

		// In general, output files in temp are dropped and never gets written to disk or networked to host.
		virtual bool KeepTempOutputFile(const StringView& fileName) const
		{
			return false;
		}

		// Useful when sending files back that is never read by any processes spawned by uba (keep internal tables smaller)
		virtual bool IsInvisible(const StringView& fileName) const
		{
			return false;
		}

		// If returns false this means that all GetFileAttribute etc will return file-not-found on this file
		virtual bool CanExist(StringView file) const
		{
			return true;
		}

		// Return true if the file is only read by this process or very rarely read more than once
		// This is an optimization to not store the file in the mapping table since it will not be read again and would just take up space
		virtual bool IsRarelyRead(const StringView& file) const
		{
			return true;
		}

		// Return true if the file is never/rarely read after it was written.
		// This is an optimization where the written file is not kept in file mappings after written
		virtual bool IsMaybeReadAfterWritten(const StringView& fileName) const
		{
			return false;
		}

		virtual bool AllowStorageProxy(const StringView& file) const
		{
			return !IsRarelyRead(file);
		}

		// Process is allowed to detour crt allocator and use mimalloc
		virtual bool AllowMiMalloc() const
		{
			return true;// !IsRunningWine();
		}

		// Can be used to disallow libraries we don't want loaded
		virtual bool AllowLoadLibrary(const tchar* libraryName) const
		{
			if (Contains(libraryName, TC("nvinject.dll")) || Contains(libraryName, TC("nviewH64.dll")))
				return false;
			return true;
		}

		// Silently suppress log lines.
		// Use this to prevent log lines from be marshalled over from detoured process to session and 
		virtual bool SuppressLogLine(const tchar* logLine, u32 logLineLen) const
		{
			return false;
		}

		// Return true if exit code is a success for this application
		virtual bool IsExitCodeSuccess(u32 exitCode) const
		{
			return true;
		}

		// Attempt to work around bugs in pvs studio producing malformed lib paths
		virtual void RepairMalformedLibPath(const tchar* path) const
		{
			// Do nothing
		}

		// Is used by uba scheduler to decide if application can be cached.
		virtual bool IsCacheable() const
		{
			return true;
		}

		virtual bool CanDependOnCompressedFiles()
		{
			return false;
		}

		// Return false if we for example know file sent from client is already compressed (no need to compress again)
		virtual bool SendFileCompressedFromClient(const StringView& fileName) const
		{
			return true;
		}

		// Return true if uba should attempt to extract symbols from produced file
		// (Used when compiling and extracing symbols from obj files)
		virtual bool ShouldExtractSymbols(const StringView& fileName) const
		{
			return false;
		}

		// Preload if libraries. This is used to prevent deadlocks in combination with mimalloc also loading modules
		virtual const tchar* const* LibrariesToPreload() const // Array should be null terminated
		{
			return nullptr;
		}

		// Returns true if produced file should be devirtualized (paths are turned into local paths)
		virtual bool ShouldDevirtualizeFile(const StringView& fileName, bool& outEscapeSpaces) const
		{
			return false;
		}

		virtual DependencyCrawlerType GetDependencyCrawlerType() const
		{
			return DependencyCrawlerType_None;
		}

		virtual bool ReportAllExceptions() const
		{
			return false;
		}

		// Return true if process should be allowed to close file mappings of files after used.
		// For example, when linker is done with obj files they are likely never used again so we can unmap them.
		// .. but if we have both .lib and .dll creation we might end up unmap just to remap obj files (and if they are compressed that means decompressing twice)
		virtual bool CloseFileMappingAfterUse(StringView fileName) const
		{
			return false;
		}

		// Files that can be written as placeholders if WritePlaceholders is enabled
		virtual bool SupportsPlaceholder(StringView fileName) const
		{
			return false;
		}

		virtual bool SupportsRacing() const
		{
			return false;
		}

		// Return true if we allow file writes to happen before process has been exited by the kernel
		// This is useful for processes that has tons of file mappings open and take a long time to exit
		virtual bool ShouldWriteFilesBeforeWaitOnExit() const
		{
			return false;
		}

		virtual bool Time64AlwaysReturnZero() const
		{
			return false;
		}

		u32 index = ~0u;
	};


	struct RulesRec { u32 hash; ApplicationRules* rules; };

	const RulesRec* GetApplicationRules();
	u32 GetApplicationHash(const StringView& str);

	enum SpecialRulesIndex
	{
		SpecialRulesIndex_ClExe = 2,
		SpecialRulesIndex_LinkExe = 3,
		SpecialRulesIndex_LdExePlatform1 = 15,
	};
}
