// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// UBA_STUB_BUILD: set to 1 when compiling the freestanding static-detour
// stub under Engine/Source/Programs/UnrealBuildAccelerator/DetoursStatic.
// In that mode UBA's Core headers skip libc / libstdc++ dependencies. All
// normal builds leave it at 0.
#ifndef UBA_STUB_BUILD
#define UBA_STUB_BUILD 0
#endif

#if UBA_STUB_BUILD
// Freestanding — pull size_t/ptrdiff_t/nullptr_t into the global namespace so
// shared UBA headers can use them before UbaPlatform.h is reached.
#include <stddef.h>
#endif

namespace uba
{
	using u8 = unsigned char;
	using u16 = unsigned short;
	using u32 = unsigned int;
	using u64 = unsigned long long;
	using s64 = long long;

	struct Guid
	{
		u32 data1 = 0; u16 data2 = 0; u16 data3 = 0; u8 data4[8] = { 0 };
		bool operator==(const Guid& o) const { return *(u64*)&data1 == *(u64*)&o.data1 && *(u64*)data4 == *(u64*)o.data4; }
	};

	template<class T> const T& Min(const T& a, const T& b) { return (b < a) ? b : a; }
	template<class T> const T& Max(const T& a, const T& b) { return (b > a) ? b : a; }

	#if PLATFORM_WINDOWS
	inline constexpr bool IsWindows = true;
	inline constexpr bool IsLinux = false;
	inline constexpr bool IsMac = false;
	using tchar = wchar_t;
	#define TC(x) L##x
	#define PERCENT_HS L"%hs"
	#else
	inline constexpr bool IsWindows = false;
	inline constexpr bool IsLinux = PLATFORM_LINUX != 0;
	inline constexpr bool IsMac = !IsLinux;
	using tchar = char;
	#define TC(x) x
	#define PERCENT_HS "%s"
	#endif

	#if (defined(_M_IX86) || defined(__i386__) || defined(_M_X64) || defined(__amd64__) || defined(__x86_64__)) && !defined(_M_ARM64EC)
	#define PLATFORM_CPU_X86_FAMILY	1
	#else
	#define PLATFORM_CPU_X86_FAMILY	0
	#endif

	#if (defined(__arm__) || defined(_M_ARM) || defined(__aarch64__) || defined(_M_ARM64) || defined(_M_ARM64EC))
	#define PLATFORM_CPU_ARM_FAMILY	1
	#else
	#define PLATFORM_CPU_ARM_FAMILY	0
	#endif

	#if PLATFORM_WINDOWS && defined(_M_ARM64) // For now we only care about windows arm..
	inline constexpr bool IsArmBinary = true;
	#else
	inline constexpr bool IsArmBinary = false;
	#endif

	#if PLATFORM_WINDOWS && defined(UBA_BUILD)
	#pragma warning(disable:4100) // This is needed because of single header compiles where AdditionalCompilerArguments is not included
	#endif

	inline constexpr u32 CacheLineSize = 64;
}

#if !defined(UBA_API)
	#if PLATFORM_WINDOWS
		#define UBA_API __declspec(dllimport)
	#elif PLATFORM_LINUX
		#define UBA_API __attribute__((weak))
	#else
		#define UBA_API 
	#endif
#endif

#if defined(__clang__)
	#define UBA_LIFETIMEBOUND [[clang::lifetimebound]]
#elif defined(_MSC_VER)
	#define UBA_LIFETIMEBOUND [[msvc::lifetimebound]]
#else
	#define UBA_LIFETIMEBOUND
#endif

#if defined(__clang__)
	#define UBA_ASSUME(expr) __builtin_assume(expr)
#elif defined(_MSC_VER)
	#define UBA_ASSUME(expr) __assume(expr)
#else
	#define UBA_ASSUME(expr) ((void)0)
#endif

#if defined(__clang__)
	#define UBA_ANALYZER_SUPPRESS [[clang::suppress]]
#else
	#define UBA_ANALYZER_SUPPRESS
#endif

#define UBA_USE_PARKINGLOT !PLATFORM_MAC