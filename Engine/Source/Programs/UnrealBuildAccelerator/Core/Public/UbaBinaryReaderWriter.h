// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaHash.h"
#include "UbaStringBuffer.h"
#include "UbaVector.h"

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	enum ProcessCommunication { ProcessCommunication_Write };

	template<typename CharType>
	inline u64 GetUtf8WrittenBytes(const CharType* str, u64 strlen);
	inline u8 Get7BitEncodedCount(u64 value);
	inline u64 GetStringWriteSize(const tchar* str, u64 len);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct BinaryWriter
	{
		inline void WriteByte(u8 value);
		inline void WriteBytes(const void* data, u64 size);

		inline void WriteU16(u16 value);
		inline void WriteU32(u32 value);
		inline void WriteU64(u64 value);
		inline void WriteString(const tchar* str);
		inline void WriteString(const tchar* str, u64 len);
		inline void WriteString(const StringView& str);
		inline void WriteStringf(const tchar* format, ...);
		inline void WriteString(const TString& str);
		inline void WriteStringKey(const StringKey& g);
		inline void WriteGuid(const Guid& g);
		inline void WriteCasKey(const CasKey& casKey);
		inline void Write7BitEncoded(u64 value);
		inline void WriteBool(bool value) { WriteByte(value ? 1 : 0); }

		template<typename CharType>
		inline void WriteUtf8String(const CharType* str, u64 len);

		void WriteLongString(const StringView& str, u32 threshold = 10000);

		// Pre-encoded form for callers that need to know the on-wire size before writing
		// (e.g. slot reservation in lock-free ring buffers). The constructor compresses
		// when needed; GetWriteSize then returns the exact byte count.
		struct PreparedLongString
		{
			PreparedLongString(const StringView& s, u32 threshold = 10000);
			u64 GetWriteSize() const;

			StringView str;
			Vector<u8> compressedBuffer; // Empty -> short-string path (length below threshold).
		};
		void WriteLongString(const PreparedLongString& prepared);

		bool WriteBytesCompressed(const void* data, u64 size, bool assertOnFail = true);

		inline void WriteFileTime(u64 fileTime) { WriteU64(fileTime); }
		inline void WriteFileAttributes(u32 attributes) { Write7BitEncoded(attributes); }
		inline void WriteVolumeSerial(u32 volumeSerial) { Write7BitEncoded(volumeSerial); }
		inline void WriteFileIndex(u64 fileIndex) { WriteU64(fileIndex); }
		inline void WriteFileSize(u64 fileSize) { Write7BitEncoded(fileSize); }

		inline u8* AllocWrite(u64 bytes);
		inline u64 GetPosition() const { return u64(m_pos - m_begin); }
		inline u64 GetCapacityLeft() const { return u64(m_end - m_pos); }
		inline u8* GetData() const { return m_begin; }
		inline void ChangeData(u8* newData, u64 newCapacity = InvalidValue) { u64 offset = u64(m_pos - m_begin); m_begin = newData; m_pos = newData + offset; m_end = newData + newCapacity; }
		inline void Reset(u8* data, u64 offset = 0, u64 capacity = InvalidValue) { m_begin = data; m_pos = data + offset; m_end = data + capacity; }
		inline BinaryWriter(u8* data, u64 offset = 0, u64 capacity = InvalidValue) { Reset(data, offset, capacity); }

		BinaryReader Flush(bool waitOnResponse = true);
		BinaryWriter(ProcessCommunication);

	protected:
		u8* m_begin;
		u8* m_pos;
		u8* m_end;
	};

	template<u32 Capacity>
	struct StackBinaryWriter : BinaryWriter
	{
		StackBinaryWriter() : BinaryWriter(buffer, 0, Capacity) {}
		StackBinaryWriter& Reset() { m_pos = m_begin; m_end = buffer + Capacity; return *this; }
		u8 buffer[Capacity];
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct BinaryReader
	{
		inline void ReadBytes(void* data, u64 size);
		inline u8 ReadByte();
		inline u16 ReadU16();
		inline u32 ReadU32();
		inline u64 ReadU64();
		inline u64 ReadString(tchar* str, u64 strCapacity);
		inline void ReadString(StringBufferBase& out);
		inline TString ReadString();
		inline void SkipString();
		inline StringKey ReadStringKey();
		inline Guid ReadGuid();
		inline CasKey ReadCasKey();
		inline bool ReadBool() { return ReadByte() != 0; }
		inline u64 Read7BitEncoded();
		template<typename CharType> inline CharType ReadUtf8Char();
		TString ReadLongString();
		void ReadBytesCompressed(void* outData, u64 outCapacity);

		inline u64 ReadFileTime() { return ReadU64(); }
		inline u32 ReadFileAttributes() { return u32(Read7BitEncoded()); }
		inline u32 ReadVolumeSerial() { return u32(Read7BitEncoded()); }
		inline u64 ReadFileIndex() { return ReadU64(); }
		inline u64 ReadFileSize() { return Read7BitEncoded(); }

		inline bool TryRead7BitEncoded(u64& outValue);
		inline bool TryReadString(StringBufferBase& out);
		inline u32 PeekU32();
		inline u64 PeekU64();
		inline void Skip(u64 size) { m_pos += size; }
		inline u64 GetPosition() const { return u64(m_pos - m_begin); }
		inline u64 GetLeft() const { return u64(m_end - m_pos); }
		inline void SetPosition(u64 pos) { m_pos = m_begin + pos; }
		inline void SetSize(u64 size) { m_end = m_begin + size; }
		inline const u8* GetPositionData() { return m_pos; }
		inline BinaryReader(const u8* data, u64 offset = 0, u64 size = InvalidValue) { m_begin = data; m_pos = data + offset; m_end = data + size; }
		inline BinaryReader(const Vector<u8>& data, u64 offset = 0) { m_begin = data.data(); m_pos = m_begin + offset; m_end = m_begin + data.size(); }

	protected:
		inline u64 InternalReadString(tchar* str, u64 charLen);
		inline void InternalSkipString(u64 charLen);
		const u8* m_begin;
		const u8* m_pos;
		const u8* m_end;
	private:
		friend struct BinaryWriter;
		BinaryReader() = delete;
	};

	template<u32 Capacity>
	struct StackBinaryReader : BinaryReader
	{
		StackBinaryReader() : BinaryReader(buffer, 0, Capacity) { *buffer = 0; }
		StackBinaryReader& Reset() { m_pos = m_begin; m_end = buffer + Capacity; return *this; }
		u8 buffer[Capacity];
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	#define UBA_ASSERT_WRITE(size) UBA_ASSERTF(m_pos + size <= m_end, TC("BinaryWriter overflow. Written: %llu, Capacity: %llu, Trying to write: %zu"), u64(m_pos - m_begin), u64(m_end - m_begin), size_t(size))
	#define UBA_ASSERT_READ(size) UBA_ASSERTF(m_pos + size <= m_end, TC("BinaryReader overflow. Addr: 0x%llx Offset: %llu, Size: %llu, Trying to read: %zu"), u64(m_begin), u64(m_pos - m_begin), u64(m_end - m_begin), size_t(size))

	template<typename CharType>
	inline u64 GetUtf8WrittenBytes(const CharType* str, u64 strlen)
	{
		u64 actualBytes = 0;
		for (const CharType* i = str, *e = str + strlen; i != e; ++i)
		{
			u16 c = u16(*i);
			if (c < 128)
				actualBytes += 1;
			else if (c <= 2047)
				actualBytes += 2;
			else
				actualBytes += 3;
		}
		return actualBytes;
	}

	inline u8 Get7BitEncodedCount(u64 value)
	{
		u8 count = 0;
		do
		{
			++count;
			value >>= 7;
		} while (value > 0);
		return count;
	}

	inline u64 GetStringWriteSize(const tchar* str, u64 len)
	{
		#if PLATFORM_WINDOWS
		u64 actualBytes = GetUtf8WrittenBytes(str, len);
		#else
		u64 actualBytes = len;
		#endif
		return Get7BitEncodedCount(len) + actualBytes;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void BinaryWriter::WriteByte(u8 value)
	{
		UBA_ASSERT_WRITE(1);
		*m_pos = value;
		++m_pos;
	}

	void BinaryWriter::WriteBytes(const void* data, u64 size)
	{
		UBA_ASSERT_WRITE(size);
		memcpy(m_pos, data, size);
		m_pos += size;
	}

	void BinaryWriter::WriteString(const tchar* str)
	{
		UBA_ASSERT(str);
		WriteString(str, u64(TStrlen(str)));
	}

	template<typename CharType>
	void BinaryWriter::WriteUtf8String(const CharType* str, u64 len)
	{
		UBA_ASSERT(str);

		UBA_ASSERT_WRITE(GetUtf8WrittenBytes(str, len));

		for (const CharType* i = str, *e = str + len; i != e; ++i)
		{
			int c = *i;
			if (c < 128)
				*m_pos++ = u8(c);
			else if (c <= 2047)
			{
				*m_pos++ = u8(c / 64 + 192);
				*m_pos++ = u8(c % 64 + 128);
			}
			else
			{
				*m_pos++ = u8(c / 4096 + 224);
				*m_pos++ = u8((c / 64) % 64 + 128);
				*m_pos++ = u8((c % 64) + 128);
			}
		}
	}

	void BinaryWriter::WriteString(const tchar* str, u64 len)
	{
		UBA_ASSERT(str);

		Write7BitEncoded(len);

		#if PLATFORM_WINDOWS
		UBA_ASSERT_WRITE(GetUtf8WrittenBytes(str, len));
		WriteUtf8String(str, len);
		#else
		WriteBytes(str, len);
		#endif
	}

	void BinaryWriter::WriteString(const StringView& str)
	{
		WriteString(str.data, str.count);
	}

	void BinaryWriter::WriteStringf(const tchar* format, ...)
	{
		va_list arg;
		va_start (arg, format);
		tchar buffer[1024];
		int done = Tvsprintf_s(buffer, 1024, format, arg); (void)done;
		UBA_ASSERT(done >= 0);
		va_end (arg);
		WriteString(buffer);
	}

	void BinaryWriter::WriteString(const TString& str)
	{
		WriteString(str.c_str(), str.size());
	}

	void BinaryWriter::WriteGuid(const Guid& g)
	{
		UBA_ASSERT_WRITE(16);
		u64* v = (u64*)&g;
		((u64*)m_pos)[0] = v[0];
		((u64*)m_pos)[1] = v[1];
		m_pos += sizeof(u64) * 2;
	}

	void BinaryWriter::WriteStringKey(const StringKey& key)
	{
		UBA_ASSERT_WRITE(16);
		((u64*)m_pos)[0] = key.a;
		((u64*)m_pos)[1] = key.b;
		m_pos += sizeof(u64) * 2;
	}

	void BinaryWriter::WriteCasKey(const CasKey& key)
	{
		UBA_ASSERT_WRITE(20);
		((u64*)m_pos)[0] = key.a;
		((u64*)m_pos)[1] = key.b;
		((u32*)m_pos)[4] = key.c;
		m_pos += sizeof(u64) * 2 + sizeof(u32);
	}

	void BinaryWriter::WriteU32(u32 value)
	{
		UBA_ASSERT_WRITE(sizeof(u32));
		*(u32*)m_pos = value;
		m_pos += sizeof(u32);
	}

	void BinaryWriter::WriteU16(u16 value)
	{
		UBA_ASSERT_WRITE(sizeof(u16));
		*(u16*)m_pos = value;
		m_pos += sizeof(u16);
	}

	void BinaryWriter::WriteU64(u64 value)
	{
		UBA_ASSERT_WRITE(sizeof(u64));
		*(u64*)m_pos = value;
		m_pos += sizeof(u64);
	}

	void BinaryWriter::Write7BitEncoded(u64 value)
	{
		UBA_ASSERT_WRITE(Get7BitEncodedCount(value));
		do
		{
			u8 HasMoreBytes = (u8)((value > u64(0x7F)) << 7);
			*(m_pos++) = (u8)(value & 0x7F) | HasMoreBytes;
			value >>= 7;
		} while (value > 0);
	}

	u8* BinaryWriter::AllocWrite(u64 bytes)
	{
		UBA_ASSERT_WRITE(bytes);
		u8* data = m_pos;
		m_pos += bytes;
		return data;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void BinaryReader::ReadBytes(void* data, u64 size)
	{
		UBA_ASSERT_READ(size);
		memcpy(data, m_pos, size);
		m_pos += size;
	}


	u8 BinaryReader::ReadByte()
	{
		UBA_ASSERT_READ(sizeof(u8));
		return *m_pos++;
	}

	u16 BinaryReader::ReadU16()
	{
		UBA_ASSERT_READ(sizeof(u16));
		u16 value = *(u16*)m_pos;
		m_pos += sizeof(u16);
		return value;
	}

	u32 BinaryReader::ReadU32()
	{
		UBA_ASSERT_READ(sizeof(u32));
		u32 value = *(u32*)m_pos;
		m_pos += sizeof(u32);
		return value;
	}

	u64 BinaryReader::ReadU64()
	{
		UBA_ASSERT_READ(sizeof(u64));
		u64 value = *(u64*)m_pos;
		m_pos += sizeof(u64);
		return value;
	}

	u64 BinaryReader::ReadString(tchar* str, u64 strCapacity)
	{
		u64 charLen = Read7BitEncoded();
		if (charLen < strCapacity - 1)
			return InternalReadString(str, charLen);

		u64 newCharLen = strCapacity - 1;
		u64 res = InternalReadString(str, newCharLen);
		UBA_ASSERTF(false, TC("Strlen: %llu, Capacity: %llu Read: %s"), charLen, strCapacity, str);
		InternalSkipString(charLen - newCharLen);
		return res;
	}

	u64 BinaryReader::InternalReadString(tchar* str, u64 charLen)
	{
		#if PLATFORM_WINDOWS
		tchar* it = str;
		const u8* pos = m_pos;
		u64 left = charLen;
		while (left--)
		{
			UBA_ASSERT_READ(1);
			u8 a = *pos++;
			if (a <= 127)
			{
				*it++ = a;
				continue;
			}
			UBA_ASSERT_READ(1);
			u8 b = *pos++;
			if (a >= 192 && a <= 223)
			{
				*it++ = (a-192)*64 + (b-128);
				continue;
			}
			UBA_ASSERT_READ(1);
			u8 c = *pos++;
			if (a >= 224 && a <= 239)
			{
				*it++ = (a-224)*4096 + (b-128)*64 + (c-128);
				continue;
			}
			if (a >= 240 && a <= 253)
			{
				// Wide chars cannot exceed 16 bits
				UBA_ASSERTF(false, TC("This should not happen, data is corrupt (Reader Start:0x%llx Pos:%llu Length:%llu Data:%.*s)"), u64(m_begin), u64(pos - m_begin), u64(m_end - m_begin), u32(it - str), str); // Wide chars cannot exceed 16 bits
				*it++ = tchar(~0);
				continue;
			}
			UBA_ASSERTF(false, TC("This should not happen, data is corrupt (Reader Start:0x%llx Pos:%llu Length:%llu Data:%.*s)"), u64(m_begin), u64(pos - m_begin), u64(m_end - m_begin), u32(it - str), str); // Wide chars cannot exceed 16 bits
			*it++ = tchar(~0);
		}
		*it = 0;
		m_pos = pos;
		return u64(it - str);
		#else
		ReadBytes(str, charLen);
		str[charLen] = 0;
		return charLen;
		#endif
	}

	void BinaryReader::InternalSkipString(u64 charLen)
	{
		#if PLATFORM_WINDOWS
		const u8* pos = m_pos;
		u64 left = charLen;
		while (left--)
		{
			UBA_ASSERT_READ(1);
			u8 a = *pos++;
			if (a <= 127)
				continue;
			UBA_ASSERT_READ(1);
			pos++;
			if (a >= 192 && a <= 223)
				continue;
			UBA_ASSERT_READ(1);
			pos++;
			if (a >= 224 && a <= 239)
				continue;
			if (a >= 240 && a <= 253)
			{
				UBA_ASSERT(false); // Wide chars cannot exceed 16 bits
				continue;
			}
			UBA_ASSERT(false); // Wide chars cannot exceed 16 bits
		}
		m_pos = pos;
		#else
		Skip(charLen);
		#endif
	}

	void BinaryReader::ReadString(StringBufferBase& out)
	{
		out.count += u32(ReadString(out.data + out.count, (out.capacity - out.count)));
	}

	TString BinaryReader::ReadString()
	{
		TString res;
		u64 len = Read7BitEncoded();
		res.resize(len);
		InternalReadString((tchar*)res.data(), len);
		return res;
	}

	void BinaryReader::SkipString()
	{
		u64 charLen = Read7BitEncoded();
		InternalSkipString(charLen);
	}

	Guid BinaryReader::ReadGuid()
	{
		UBA_ASSERT_READ(sizeof(Guid));
		u64 g[2];
		g[0] = *(u64*)m_pos;
		g[1] = ((u64*)m_pos)[1];
		m_pos += sizeof(u64) * 2;
		return *(Guid*)g;
	}

	StringKey BinaryReader::ReadStringKey()
	{
		UBA_ASSERT_READ(sizeof(StringKey));
		StringKey k;
		k.a = *(u64*)m_pos;
		k.b = ((u64*)m_pos)[1];
		m_pos += sizeof(u64) * 2;
		return k;
	}

	CasKey BinaryReader::ReadCasKey()
	{
		UBA_ASSERT_READ(sizeof(CasKey));
		CasKey k;
		k.a = *(u64*)m_pos;
		k.b = ((u64*)m_pos)[1];
		k.c = ((u32*)m_pos)[4];
		m_pos += sizeof(u64) * 2 + sizeof(u32);
		return k;
	}

	u64 BinaryReader::Read7BitEncoded()
	{
		const u8* pos = m_pos;
		u64 result = 0;
		u64 byteIndex = 0;
		bool hasMoreBytes;
		do
		{
			UBA_ASSERT_READ(1);
			u8 value = *pos++;
			hasMoreBytes = value & 0x80;
			result |= u64(value & 0x7f) << (byteIndex * 7);
			++byteIndex;
		} while (hasMoreBytes);
		m_pos = pos;
		return result;
	}

	bool BinaryReader::TryRead7BitEncoded(u64& outValue)
	{
		const u8* pos = m_pos;
		u64 result = 0;
		u64 byteIndex = 0;
		bool hasMoreBytes;
		do
		{
			if (m_pos >= m_end)
				return false;
			u8 value = *pos++;
			hasMoreBytes = value & 0x80;
			result |= u64(value & 0x7f) << (byteIndex * 7);
			++byteIndex;
		} while (hasMoreBytes);
		outValue = result;
		m_pos = pos;
		return true;
	}

	bool BinaryReader::TryReadString(StringBufferBase& out)
	{
		#if PLATFORM_WINDOWS
		u64 charLen;
		if (!TryRead7BitEncoded(charLen))
			return false;
		tchar* it = out.data + out.count;
		u64 left = charLen;
		while (left--)
		{
			if (m_pos >= m_end)
				return false;
			u8 a = *m_pos++;
			if (a <= 127)
			{
				*it++ = a;
				continue;
			}
			if (m_pos >= m_end)
				return false;
			u8 b = *m_pos++;
			if (a >= 192 && a <= 223)
			{
				*it++ = (a-192)*64 + (b-128);
				continue;
			}
			if (m_pos >= m_end)
				return false;
			u8 c = *m_pos++;
			if (a >= 224 && a <= 239)
			{
				*it++ = (a-224)*4096 + (b-128)*64 + (c-128);
				continue;
			}
			return false;
		}
		*it = 0;
		out.count = u32(it - out.data);
		#else
		u64 charLen;
		if (!TryRead7BitEncoded(charLen))
			return false;
		u32 strCapacity = out.capacity - out.count;
		if (charLen >= strCapacity)
			return false;
		ReadBytes(out.data + out.count, charLen);
		out.count += charLen;
		out.data[out.count] = 0;
		#endif
		return true;
	}

	template<typename CharType>
	CharType BinaryReader::ReadUtf8Char()
	{
		UBA_ASSERT_READ(1);
		u8 a = *m_pos++;
		if (a <= 127)
			return CharType(a);

		UBA_ASSERT_READ(1);
		u8 b = *m_pos++;
		if (a >= 192 && a <= 223)
			return CharType((a-192)*64 + (b-128));

		UBA_ASSERT_READ(1);
		u8 c = *m_pos++;
		if (a >= 224 && a <= 239)
			return CharType((a-224)*4096 + (b-128)*64 + (c-128));

		if (a >= 240 && a <= 253)
		{
			UBA_ASSERT(false); // Wide chars cannot exceed 16 bits
			return CharType(~0);
		}

		UBA_ASSERT(false); // Wide chars cannot exceed 16 bits
		return CharType(~0);
	}

	u32 BinaryReader::PeekU32()
	{
		return *(u32*)m_pos;
	}

	u64 BinaryReader::PeekU64()
	{
		return *(u64*)m_pos;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
