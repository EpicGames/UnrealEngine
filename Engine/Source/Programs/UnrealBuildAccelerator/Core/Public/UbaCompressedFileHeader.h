// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaHash.h"

namespace uba
{
	struct CompressedFileHeader
	{
		inline static constexpr u8 Magic[] = { 'U', 'B', 'A', 0x01 };

		CompressedFileHeader(const CasKey& key) : casKey(key)
		{
			memcpy(&magic, Magic, sizeof(magic));
		}

		bool IsValid() const { return memcmp(&magic, Magic, sizeof(Magic)) == 0; }

		u8 magic[sizeof(Magic)];
		CasKey casKey;
	};
}
