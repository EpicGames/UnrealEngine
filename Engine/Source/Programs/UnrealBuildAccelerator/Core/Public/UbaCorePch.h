// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UBA_USE_MIMALLOC
#include <mimalloc.h>
#include <mimalloc-override.h>
#if PLATFORM_LINUX
namespace std { inline void* mi_realloc(void* p, size_t newsize) { return ::mi_realloc(p, newsize); } }
#endif
#endif

#include "UbaPlatform.h"
