// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Use native mach semaphores for IPC.
#define UBA_USE_NATIVE_MAC_SEMAPHORES 0//PLATFORM_MAC

// Disable pthread cancellation to reduce work when waking up from event.
// No uba code should rely on cancelling threads and shutdown is gracefully so this can be disabled
#define UBA_DISABLE_PTHREAD_CANCELLATIONS 1

// Enable this to test semaphore implementation for local events
#define UBA_TEST_WINDOWS_SEMAPHORES 0

// Enable to add logging of file mappings
#define UBA_DEBUG_FILE_MAPPING 0

#define UBA_PROTOCOL_GUARD 0

// Need special binary patching to be able to hook Go applications on Linux. Only works for non-static applications.
#define UBA_SUPPORTS_GO PLATFORM_LINUX && !PLATFORM_CPU_ARM_FAMILY

#define UBA_SUPPORT_SYMLINKS !PLATFORM_WINDOWS

#define UBA_CATCH_NTREAD_EXCEPTIONS 0

namespace uba
{
	inline constexpr bool EventIsNative = bool(PLATFORM_WINDOWS && !UBA_TEST_WINDOWS_SEMAPHORES);
}