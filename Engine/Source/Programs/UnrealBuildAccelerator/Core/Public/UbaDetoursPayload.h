// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaStringBuffer.h"
#include "UbaPlatform.h"

#if PLATFORM_WINDOWS

namespace uba
{
	struct __declspec(uuid("{a1041c70-bf7c-4e94-8f8b-bacf0f31ba9a}")) DetoursPayload
	{
		HANDLE hostProcess = 0;
		HANDLE cancelEvent = 0;
		HANDLE writeEvent = 0;
		HANDLE readEvent = 0;
		u64 communicationHandle = 0;
		u64 communicationOffset = 0;
		u32 version = 0;
		u32 rulesIndex = 0;
		bool runningRemote = false;
		bool useCustomAllocator = true;
		bool isRunningWine = false;
		bool isChild = false;
		bool allowLazyCopyOnWrite = false;
		bool allowKeepFilesInMemory = IsWindows;
		bool allowOutputFiles = IsWindows;
		bool allowDiscardVirtualMemory = false;
		bool suppressLogging = false;
		bool readIntermediateFilesCompressed = false;
		bool reportAllExceptions = false;
		int uiLanguage = 0;
		StringBuffer<512> logFile;
	};

	static const _GUID DetoursPayloadGuid = __uuidof(DetoursPayload);
}

#endif