// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBinaryReaderWriter.h"
#include "UbaDefinitions.h"
#include "UbaPathUtils.h"
#include "UbaUnorderedMap.h"

#if 1
#define WRITE_DEBUG(format, ...) \
{ \
	static int debugFile = open("/dev/tty", O_WRONLY); \
	StringBuffer<> str; \
	str.Appendf(format "\n", __VA_ARGS__); \
	write(debugFile, str.data, str.count); \
}
#endif


namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// DirectoryTable is a journal that can be parsed into a table of directories and files
	// When a directory changes it will create a new entry in the journal pointing back to the previous
	// Files are attached to the directory entries and can be populated on demand into the files lookup.
	//
	// Directory:
	//   StringKey dir (16 bytes)
	//   enc7 OffsetToPreviousEntry  (this is offset to previous entry after StringKey)
	//   if OffsetToPreviousEntry == InvalidTableOffset
	//      enc7 FileAttributes
	//      enc7 VolumeSerialIndex
	//      u64 FileIndex
	//   enc7 ItemCount
	//      Items
	// 
	// Item (in directory):
	//   String name
	//   enc7 FileAttributes
	//   enc7 VolumeSerialIndex
	//   u64 FileIndex
	//   if not directory (FileAttributes provide info)
	//      u64 FileTime
	//      enc7 FileSize
	// 
	////////////////////////////////////////////////////////////////////////////////////////////////////

	enum : u64 { InvalidTableOffset = 0 }; // Important that it is zero to compress enc7 as good as possible
	enum : u32 { OverlayTableFlag = 0x10000000 };
	enum : u32 { DirectoryEntryFlag = 0x80000000 };

	struct DirTableSize { u32 main = 0; u32 overlay = 0; };
	inline DirTableSize FromU64(u64 v) { return {u32(v), u32(v >> 32u)}; }
	inline u64 ToU64(DirTableSize size) { return size.main + (u64(size.overlay) << 32llu); }

	struct DirTableOffset { u32 internal = InvalidTableOffset; };
	inline bool IsDirectoryEntry(DirTableOffset d) { return (d.internal & DirectoryEntryFlag) != 0; }
	inline bool IsValidEntry(DirTableOffset d) { return d.internal != InvalidTableOffset; }
	inline bool IsOverlayOffset(DirTableOffset d) { return (d.internal & OverlayTableFlag) != 0; }

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class DirectoryTable
	{
	public:

		// m_block==nullptr -> system heap (used by snapshot builder so per-build rehashes free
		// their old bucket arrays); m_block set -> bump-allocate from the mapping arena.
		struct BucketAllocator : Allocator
		{
			void* Allocate(u64 size, u64 align)
			{
				if (!m_block)
					return Allocator::Allocate(size, align);
				u64 aligned = AlignUp(m_block->writtenSize, align);
				if (aligned + size > m_block->reserveSize)
					return Allocator::Allocate(size, align);
				return m_block->Allocate(size, align, TC("Buckets"));
			}
			void Deallocate(void* p, u64 size, u64 align)
			{
				if (!m_block || p < m_block->memory || p >= m_block->memory + m_block->reserveSize)
					Allocator::Deallocate(p, size, align);
			}
			BucketAllocator() : m_block(nullptr) {}
			BucketAllocator(MemoryBlock& b) : m_block(&b) {}
			MemoryBlock* m_block;
		};

		using EntryLookup = UnorderedMap<StringKey, DirTableOffset, std::hash<StringKey>, std::equal_to<StringKey>, GrowingAllocator, BucketAllocator, u32>;

		struct Directory
		{
			Directory(MemoryBlock& block) : files(block, block) {}
			u32 latestOffset = InvalidTableOffset;
			u32 populatedOffset = InvalidTableOffset;
			u32 latestOverlayOffset = InvalidTableOffset;
			u32 populatedOverlayOffset = InvalidTableOffset;
			EntryLookup files;
			ReaderWriterLock lock;
		};

		using DirLookup = UnorderedMap<StringKey, Directory, std::hash<StringKey>, std::equal_to<StringKey>, GrowingAllocator, BucketAllocator, u32>;

		inline void Init(const u8* mem, u32 tableCount, u32 tableSize);
		inline void InitOverlay(const u8* mem, u32 tableSize);
		inline void DeinitOverlay();
		inline void ParseDirectoryTable(DirTableSize size);
		inline void ParseDirectoryTableNoLock(DirTableSize size);
		inline void ParseDirectoryTableNoLock(const u8* memory, u32 from, u32 to, u32 isOverlayFlag);
		inline void PopulateDirectory(const StringKeyHasher& hasher, Directory& dir);
		inline bool IsDirectoryUpToDate(Directory& dir);
		inline void PopulateDirectoryNoLock(const StringKeyHasher& hasher, Directory& dir);
		inline BinaryReader GetReader(u32 tableOffset);
		inline void PopulateDirectoryRecursive(const StringKeyHasher& hasher, u32 latestOffset, u32 populatedOffset, EntryLookup& files, u32 isOverlayFlag);
		inline void PopulateDirectoryWithFiles(BinaryReader& reader, const StringKeyHasher& hasher, EntryLookup& files, u32 isOverlayFlag);

		enum Exists
		{
			Exists_Yes,
			Exists_No,
			Exists_Maybe,
		};

		// When checkIfDir is true these functions will search for the offset where a directory stores its files.
		// It is only useful when the intention is to traverse files inside the dir.
		// NOTE, even if checkIfDir is true you can still get back a normal "file entry" representing the dir. "IsDirectoryEntry(tableOffset)" can be used to check which one it is
		inline Exists EntryExists(StringKey entryKey, StringView entryName, bool checkIfDir = false, DirTableOffset* outTableOffset = nullptr);
		inline Exists EntryExistsNoLock(StringKey entryKey, StringView entryName, bool checkIfDir = false, DirTableOffset* outTableOffset = nullptr);
		inline Exists EntryExists(StringView str, bool checkIfDir = false, DirTableOffset* outTableOffset = nullptr);

		struct EntryInformation
		{
			u32 attributes = 0;
			u32 volumeSerial = 0;
			u64 fileIndex = 0;
			u64 size = 0;
			u64 lastWrite = 0;
		};

		inline u32 GetAttributes(DirTableOffset tableOffset);
		inline u32 GetEntryInformation(EntryInformation& outInfo, DirTableOffset tableOffset, tchar* outFileName = nullptr, u32 fileNameCapacity = 0);
		inline bool GetEntrySymlink(StringBufferBase& out, DirTableOffset tableOffset);
		inline void GetFinalPath(StringBufferBase& out, const tchar* path);
		inline bool GetRealPath(StringBufferBase& out, const tchar* path, u32 maxLinkDepth = 40);
		inline bool GetLatestOffset(DirTableOffset& out, const Directory& dir);
		inline bool ContainsFile(const StringKeyHasher& hasher, Directory& dir, StringKey fileNameKey);

		// This function only traverses the entries that was in the original populate and does not populate the lookup table.
		// Can be used for scenarios where we know we only want the files that existed before the session started and not files that were added while active
		template<typename Func> inline void TraverseOriginalFilesNoPopulate(DirTableOffset tableOffset, const Func& func);

		// Func is (const DirectoryTable::EntryInformation& info, StringView fileName, DirTableOffset fileOffset)
		template<typename Func> inline void TraverseFilesRecursiveNoLock(StringView path, const Func& func);
		template<typename Func> inline void TraverseAllFilesNoLock(const Func& func);

		inline DirectoryTable(MemoryBlock& block);
		DirectoryTable(const DirectoryTable&) = delete;
		void operator=(const DirectoryTable&) = delete;
		inline bool IsOverlayOffset(u32 offset) { return (offset & OverlayTableFlag) != 0; }

		MemoryBlock& m_memoryBlock;
		ReaderWriterLock m_lookupLock;
		DirLookup m_lookup;

		ReaderWriterLock m_memoryLock;
		const u8* m_memory = nullptr;
		u32 m_memorySize = 0;

		const u8* m_overlayMemory = nullptr;
		u32 m_overlaySize = 0;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void DirectoryTable::Init(const u8* mem, u32 tableCount, u32 tableSize)
	{
		m_memory = mem;
		m_lookup.reserve(tableCount + 100);
		m_memoryBlock.CommitNoLock(tableCount*(sizeof(DirLookup::value_type)+16), TC(""));
		ParseDirectoryTable({tableSize, 0u});
	}

	void DirectoryTable::InitOverlay(const u8* mem, u32 tableSize)
	{
		m_overlayMemory = mem;
		ParseDirectoryTableNoLock(mem, 0, tableSize, OverlayTableFlag);
		m_overlaySize = tableSize;
	}

	void DirectoryTable::DeinitOverlay()
	{
		if (!m_overlaySize)
			return;

		for (auto i=m_lookup.begin(), e=m_lookup.end(); i!=e;)
		{
			auto& dir = i->second;

			// Directory has no overlay
			if (dir.latestOverlayOffset == InvalidTableOffset)
			{
				++i;
				continue;
			}

			// Directory was only overlay, just remove it
			if (dir.latestOffset == InvalidTableOffset)
			{
				i = m_lookup.erase(i);
				e = m_lookup.end();
				continue;
			}

			dir.latestOverlayOffset = InvalidTableOffset;
			dir.populatedOverlayOffset = InvalidTableOffset;

			// Let's force a repopulate 
			dir.files.clear();
			dir.populatedOffset = InvalidTableOffset;

			++i;
		}
		m_overlayMemory = nullptr;
		m_overlaySize = 0;
	}

	void DirectoryTable::ParseDirectoryTable(DirTableSize size)
	{
		SCOPED_WRITE_LOCK(m_lookupLock, lock);
		ParseDirectoryTableNoLock(size);
	}

	void DirectoryTable::ParseDirectoryTableNoLock(DirTableSize size)
	{
		if (size.main > m_memorySize)
		{
			ParseDirectoryTableNoLock(m_memory, m_memorySize, size.main, 0);
			m_memorySize = size.main;
		}
		if (size.overlay > m_overlaySize)
		{
			ParseDirectoryTableNoLock(m_overlayMemory, m_overlaySize, size.overlay, OverlayTableFlag);
			m_overlaySize = size.overlay;
		}
	}

	void DirectoryTable::ParseDirectoryTableNoLock(const u8* memory, u32 from, u32 to, u32 isOverlayFlag)
	{
		BinaryReader reader(memory, from, to);
		while (true)
		{
			u64 pos = reader.GetPosition();
			if (pos == to)
				break;
			UBA_ASSERTF(pos < to, TC("Should never read past size (pos: %llu, size: %u)"), pos, to);
			u64 storageSize = reader.Read7BitEncoded();
			StringKey dirKey = reader.ReadStringKey();
			Directory& dir = m_lookup.try_emplace(dirKey, m_memoryBlock).first->second; // Note that this is allowed to overwrite
			u32& tableOffset = isOverlayFlag ? dir.latestOverlayOffset : dir.latestOffset;
			UBA_ASSERTF(!IsOverlayOffset(tableOffset) || IsOverlayOffset(isOverlayFlag), TC("Can't add non-overlay directory items after overlay items"));
			tableOffset = u32(reader.GetPosition()) | isOverlayFlag;
			reader.Skip(storageSize - sizeof(dirKey));
		}
	}

	void DirectoryTable::PopulateDirectory(const StringKeyHasher& hasher, Directory& dir)
	{
		SCOPED_WRITE_LOCK(dir.lock, lock);
		PopulateDirectoryNoLock(hasher, dir);
	}

	bool DirectoryTable::IsDirectoryUpToDate(Directory& dir)
	{
		return dir.populatedOffset == dir.latestOffset && dir.populatedOverlayOffset == dir.latestOverlayOffset;
	}

	void DirectoryTable::PopulateDirectoryNoLock(const StringKeyHasher& hasher, Directory& dir)
	{
		if (dir.populatedOffset != dir.latestOffset)
		{
			PopulateDirectoryRecursive(hasher, dir.latestOffset, dir.populatedOffset, dir.files, 0);
			dir.populatedOffset = dir.latestOffset;
		}
		if (dir.populatedOverlayOffset != dir.latestOverlayOffset)
		{
			PopulateDirectoryRecursive(hasher, dir.latestOverlayOffset, dir.populatedOverlayOffset, dir.files, OverlayTableFlag);
			dir.populatedOverlayOffset = dir.latestOverlayOffset;
		}
	}

	BinaryReader DirectoryTable::GetReader(u32 tableOffset)
	{
		if (IsOverlayOffset(tableOffset))
			return BinaryReader(m_overlayMemory, tableOffset - OverlayTableFlag);
		else
			return BinaryReader(m_memory, tableOffset);
	}

	void DirectoryTable::PopulateDirectoryRecursive(const StringKeyHasher& hasher, u32 latestOffset, u32 populatedOffset, EntryLookup& files, u32 isOverlayFlag)
	{
		BinaryReader reader(isOverlayFlag?m_overlayMemory:m_memory, latestOffset&~isOverlayFlag);

		u32 prevTableOffset = u32(reader.Read7BitEncoded());

		// Inline cache. The 192 KB (48*1024) frame is fine on the host / LD_PRELOAD side where the C stack is multi-MB, but it overflows
		// the static-stub's 256 KB alt-stack when combined with the bridge save area + Rpc machinery + BLAKE3 frames, scribbling into adjacent
		// mmap regions and producing "stack split at bad time" panics on Go binaries. Stub uses a 4 KB inline buffer; the heap fallback below
		// already handles long chains in either build.
		#if UBA_STUB_BUILD
		u32 buffer[1024];
		#else
		u32 buffer[48*1024];
		#endif
		u32 count = 0;
		u32* readerOffsets = buffer;
		readerOffsets[count++] = u32(reader.GetPosition());
		bool firstIsRoot = true;
		while (true)
		{
			if (prevTableOffset == InvalidTableOffset || prevTableOffset == populatedOffset)
			{
				firstIsRoot = prevTableOffset == InvalidTableOffset;
				break;
			}
			reader.SetPosition(prevTableOffset);
			prevTableOffset = u32(reader.Read7BitEncoded());
			readerOffsets[count++] = u32(reader.GetPosition());
			if (count == sizeof_array(buffer))
			{
				readerOffsets = new u32[1024*1024]; // This sucks, but sometimes the directory is huuuge. Ideally these files should be spread out over multiple directories
				memcpy(readerOffsets, buffer, sizeof(buffer));
			}
		}

		for (u32 i=count; i>0; --i)
		{
			reader.SetPosition(readerOffsets[i-1]);
			if (firstIsRoot)
			{
				firstIsRoot = false;
				u32 attr = reader.ReadFileAttributes();
				if (!attr)
					continue;
				reader.ReadVolumeSerial(); // Directory volume serial
				reader.ReadFileIndex(); // Directory file index
			}

			PopulateDirectoryWithFiles(reader, hasher, files, isOverlayFlag);
		}

		if (readerOffsets != buffer)
			delete[] readerOffsets;
	}

	void DirectoryTable::PopulateDirectoryWithFiles(BinaryReader& reader, const StringKeyHasher& hasher, EntryLookup& files, u32 isOverlayFlag)
	{
		u64 itemCount = reader.Read7BitEncoded();

		files.reserve(u32(files.size() + itemCount));

		StringBuffer<> filename;
		filename.Append(PathSeparator);

		while (itemCount--)
		{
			u32 offset = u32(reader.GetPosition());
			filename.Resize(1);
			reader.ReadString(filename);
			if (CaseInsensitiveFs)
				filename.MakeLower();
			u32 attr = reader.ReadFileAttributes();
			reader.ReadVolumeSerial();
			reader.ReadFileIndex();

			#if UBA_SUPPORT_SYMLINKS
			if (IsSymlink(attr))
				reader.SkipString();
			#endif

			if (!IsDirectory(attr))
			{
				reader.ReadFileTime();
				reader.ReadFileSize();
			}

			StringKey filenameKey = ToStringKey(hasher, filename.data, filename.count);
			
			DirTableOffset& fileOffset = files[filenameKey];
			// Overlay-vs-main-table conflict resolution: the overlay represents the live process-tree's view (output sizes written via Rpc_UpdateCloseHandle → RegisterOverlayCreateOrWrite).
			// The main table is a disk-scan snapshot from WriteDirectoryEntriesInternal, which may have observed the file mid-write.
			// If we already have an overlay-flagged entry and the new one is from the main table, the main-table size is stale — keep the overlay. Overlay entries always overwrite
			if (!isOverlayFlag && fileOffset.internal != InvalidTableOffset && IsOverlayOffset(fileOffset.internal))
				continue;
			fileOffset.internal = offset | isOverlayFlag; // Always write, since same file might have been added with new info
		}
	}

	DirectoryTable::Exists DirectoryTable::EntryExists(StringKey entryKey, StringView entryName, bool checkIfDir, DirTableOffset* outTableOffset)
	{
		SCOPED_READ_LOCK(m_lookupLock, lock);
		return EntryExistsNoLock(entryKey, entryName, checkIfDir, outTableOffset);
	}

	DirectoryTable::Exists DirectoryTable::EntryExistsNoLock(StringKey entryKey, StringView entryName, bool checkIfDir, DirTableOffset* outTableOffset)
	{
		u32 startSkip = 2;
		if (checkIfDir)
		{
			auto findIt = m_lookup.find(entryKey);
			if (findIt != m_lookup.end())
			{
				if (outTableOffset)
				{
					if (findIt->second.latestOverlayOffset)
						outTableOffset->internal = u32(findIt->second.latestOverlayOffset) | DirectoryEntryFlag; // Use significant bit to say that this is a dir
					else
						outTableOffset->internal = u32(findIt->second.latestOffset) | DirectoryEntryFlag; // Use significant bit to say that this is a dir
				}
				return Exists_Yes;
			}
			startSkip = 1;
		}

		// Scan backwards
		const tchar* rend = entryName.data;
		const tchar* rit = rend + entryName.count - startSkip;

		bool inAncestor = false;
		while (rit > rend)
		{
			if (*rit != PathSeparator)
			{
				--rit;

#if !PLATFORM_WINDOWS
				if (rit != rend) // We want to test empty for non-windows
#endif
					continue;
			}

			u64 sublen = u64(rit - rend);

			StringKeyHasher ancestorHasher;
			ancestorHasher.Update(rend, sublen);
			StringKey ancestorKey = ToStringKey(ancestorHasher);
			auto dirIt = m_lookup.find(ancestorKey);
			if (dirIt != m_lookup.end())
			{
				DirectoryTable::Directory& parentDir = dirIt->second;
				if (parentDir.populatedOffset != parentDir.latestOffset || parentDir.populatedOverlayOffset != parentDir.latestOverlayOffset)
				{
					SCOPED_WRITE_LOCK(parentDir.lock, lock);
					if (parentDir.populatedOffset != parentDir.latestOffset)
					{
						PopulateDirectoryRecursive(ancestorHasher, parentDir.latestOffset, parentDir.populatedOffset, parentDir.files, 0);
						parentDir.populatedOffset = parentDir.latestOffset;
					}
					if (parentDir.populatedOverlayOffset != parentDir.latestOverlayOffset)
					{
						PopulateDirectoryRecursive(ancestorHasher, parentDir.latestOverlayOffset, parentDir.populatedOverlayOffset, parentDir.files, OverlayTableFlag);
						parentDir.populatedOverlayOffset = parentDir.latestOverlayOffset;
					}
				}

				SCOPED_READ_LOCK(parentDir.lock, lock);
				auto entryIt = parentDir.files.find(entryKey);
				if (entryIt == parentDir.files.end())
					return Exists_No;
				if (inAncestor)
				{
					BinaryReader reader(GetReader(entryIt->second.internal));
					reader.SkipString();
					if (!IsDirectory(reader.ReadFileAttributes()))
						return Exists_No;
					return Exists_Maybe;
				}
				if (outTableOffset)
					outTableOffset->internal = entryIt->second.internal;
				return Exists_Yes;
			}

			entryKey = ancestorKey;
			--rit;
			inAncestor = true;
		}
		return Exists_Maybe;
	}

	DirectoryTable::Exists DirectoryTable::EntryExists(StringView str, bool checkIfDir, DirTableOffset* outTableOffset)
	{
		StringBuffer<> str2(str);
		if (str2[str2.count-1] == PathSeparator)
			str2.Resize(str2.count-1);
		if (CaseInsensitiveFs)
			str2.MakeLower();
		return EntryExists(ToStringKey(str2), str2, checkIfDir, outTableOffset);
	}

	u32 DirectoryTable::GetAttributes(DirTableOffset tableOffset)
	{
		if (IsDirectoryEntry(tableOffset))
		{
			BinaryReader reader(GetReader(tableOffset.internal & ~DirectoryEntryFlag));
			u64 prevTableOffset = reader.Read7BitEncoded();
			while (prevTableOffset != InvalidTableOffset)
			{
				reader.SetPosition(prevTableOffset);
				prevTableOffset = reader.Read7BitEncoded();
			}
			return reader.ReadFileAttributes();
		}
		BinaryReader reader(GetReader(tableOffset.internal));
		reader.SkipString();
		return reader.ReadFileAttributes();
	}

	u32 DirectoryTable::GetEntryInformation(EntryInformation& outInfo, DirTableOffset tableOffset, tchar* outFileName, u32 fileNameCapacity)
	{
		if (IsDirectoryEntry(tableOffset))
		{
			BinaryReader reader(GetReader(tableOffset.internal & ~DirectoryEntryFlag));
			u64 prevTableOffset = reader.Read7BitEncoded();
			while (prevTableOffset != InvalidTableOffset)
			{
				reader.SetPosition(prevTableOffset);
				prevTableOffset = reader.Read7BitEncoded();
			}
			outInfo.attributes = reader.ReadFileAttributes();
			if (outInfo.attributes)
			{
				#if UBA_SUPPORT_SYMLINKS
				UBA_ASSERT(!IsSymlink(outInfo.attributes));
				#endif
				
				outInfo.volumeSerial = reader.ReadVolumeSerial();
				outInfo.fileIndex = reader.ReadFileIndex();
			}
			outInfo.size = 0;
			outInfo.lastWrite = 0;
			UBA_ASSERT(!outFileName);
			return ~u32(0);
		}

		BinaryReader reader(GetReader(tableOffset.internal));
		if (outFileName)
			reader.ReadString(outFileName, fileNameCapacity);
		else
			reader.SkipString();
		outInfo.attributes = reader.ReadFileAttributes();
		outInfo.volumeSerial = reader.ReadVolumeSerial();
		outInfo.fileIndex = reader.ReadFileIndex();

		#if UBA_SUPPORT_SYMLINKS
		if (IsSymlink(outInfo.attributes))
			reader.SkipString();
		#endif

		if (IsDirectory(outInfo.attributes))
		{
			outInfo.size = 0;
			outInfo.lastWrite = 0;
		}
		else
		{
			outInfo.lastWrite = reader.ReadFileTime();
			outInfo.size = reader.ReadFileSize();
		}
		return u32(reader.GetPosition());
	}

	bool DirectoryTable::GetEntrySymlink(StringBufferBase& out, DirTableOffset tableOffset)
	{
#if UBA_SUPPORT_SYMLINKS
		if (IsDirectoryEntry(tableOffset))
			return false;
		BinaryReader reader(GetReader(tableOffset.internal));
		reader.SkipString();
		u32 attributes = reader.ReadFileAttributes();
		if (!IsSymlink(attributes))
			return false;
		reader.ReadVolumeSerial();
		reader.ReadFileIndex();
		reader.ReadString(out);
		return true;
#else
		return false;
#endif
	}

	void DirectoryTable::GetFinalPath(StringBufferBase& out, const tchar* path)
	{
		UBA_ASSERT(IsAbsolutePath(path));

		Directory* directory = nullptr;
		const tchar* prevSlash = TStrchr(path+3, PathSeparator);
		if (!prevSlash)
		{
			// return root directory as-is
			out.Append(path);
			return;
		}
		out.Append(path, u64(prevSlash - path));
		const tchar* end = path + TStrlen(path);

		StringBuffer<> forHash;
		forHash.Append(path, u64(prevSlash - path));
		if (CaseInsensitiveFs)
			forHash.MakeLower();

		StringKeyHasher hasher;
		hasher.Update(forHash.data, forHash.count);

		SCOPED_READ_LOCK(m_lookupLock, lock);
		while (true)
		{
			const tchar* slash = TStrchr(prevSlash + 1, PathSeparator);
			if (!slash)
				slash = end;

			forHash.Clear().Append(prevSlash, u64(slash - prevSlash));
			if (CaseInsensitiveFs)
				forHash.MakeLower();
			hasher.Update(forHash.data, forHash.count);
			StringKey fileNameKey = ToStringKey(hasher);

			if (directory)
			{
				SCOPED_READ_LOCK(directory->lock, lock2);
				auto fileIt = directory->files.find(fileNameKey);
				if (fileIt != directory->files.end())
				{
					UBA_ASSERT(IsValidEntry(fileIt->second));
					BinaryReader reader(GetReader(fileIt->second.internal));
					StringBuffer<> fileName;
					reader.ReadString(fileName);
					out.Append(PathSeparator).Append(fileName);
				}
				else
					out.Append(prevSlash, u64(slash - prevSlash));
			}
			else
				out.Append(prevSlash, u64(slash - prevSlash));

			if (slash == end)
				return;

			prevSlash = slash;

			auto findIt = m_lookup.find(fileNameKey);
			if (findIt == m_lookup.end())
			{
				directory = nullptr;
				continue;
			}
			directory = &findIt->second;
			PopulateDirectory(hasher, *directory); // This is needed to make sure files lookup is populated for query above
		}
	}

	bool DirectoryTable::GetRealPath(StringBufferBase& out, const tchar* path, u32 maxLinkDepth)
	{
#if UBA_SUPPORT_SYMLINKS
		UBA_ASSERT(IsAbsolutePath(path));

		// Working buffer for the path being walked. When a symlink is hit
		// we splice the target into this and restart from the top.
		StringBuffer<MaxPath> work;
		work.Append(path);

	restart:

		if (maxLinkDepth-- == 0)
			return false; // ELOOP — too many levels of symbolic links

		out.Clear();

		const tchar* end = work.data + work.count;
		const tchar* p = work.data;

		StringBuffer<> forHash;
		StringKeyHasher hasher;

		// Set up the root and skip past it. After this, `p` points at the
		// first byte of the first real component, and `parent` is the root
		// Directory if it's tracked. Each loop iteration then processes one
		// component, looking it up in its parent's `files` — including the
		// FIRST component (so a top-level symlink like `/lib` is detected).
		//   Linux:   root path is "" (UbaDirectoryTableHolder treats dirlen==0
		//            as root). Skip leading "/" without adding it to hasher
		//            or out — components prepend their own '/'.
		//   Windows: root is "X:\" — copy first 3 chars into out + hasher.
		Directory* parent = nullptr;
		if constexpr (IsWindows)
		{
			out.Append(p, 3);
			forHash.Append(p, 3);
			if (CaseInsensitiveFs)
				forHash.MakeLower();
			hasher.Update(forHash.data, forHash.count);
			p += 3;
		}
		else
		{
			if (*p == '/')
				++p;
		}

		SCOPED_READ_LOCK(m_lookupLock, lock);

		// Seed `parent` with the root Directory (if tracked). This is what
		// allows iter 1 to find a top-level symlink like `/lib` in root's
		// files map.
		{
			StringKey rootKey = ToStringKey(hasher);
			auto findIt = m_lookup.find(rootKey);
			if (findIt != m_lookup.end())
			{
				parent = &findIt->second;
				PopulateDirectory(hasher, *parent);
			}
		}

		while (p < end)
		{
			const tchar* slash = TStrchr(p, PathSeparator);
			if (!slash)
				slash = end;

			// Component is [p, slash). Hash key includes the leading '/' so
			// the cumulative hash matches "/foo", "/foo/bar", etc.
			forHash.Clear().Append(PathSeparator).Append(p, u64(slash - p));
			if (CaseInsensitiveFs)
				forHash.MakeLower();
			hasher.Update(forHash.data, forHash.count);
			StringKey fileNameKey = ToStringKey(hasher);

			bool emitted = false;

			if (parent)
			{
				SCOPED_READ_LOCK(parent->lock, lock2);
				auto fileIt = parent->files.find(fileNameKey);
				if (fileIt != parent->files.end())
				{
					UBA_ASSERT(IsValidEntry(fileIt->second));
					BinaryReader reader(GetReader(fileIt->second.internal));
					StringBuffer<> fileName;
					reader.ReadString(fileName);
					u32 attributes = reader.ReadFileAttributes();

					if (IsSymlink(attributes))
					{
						reader.ReadVolumeSerial();
						reader.ReadFileIndex();
						StringBuffer<> linkTarget;
						reader.ReadString(linkTarget);

						// `out` at this point is the resolved prefix UP TO
						// (but not including) the symlink component —
						// exactly the parent directory POSIX uses for
						// relative-link resolution. `slash..end` (if any)
						// is the unresolved tail to keep walking.
						StringBuffer<MaxPath> next;
						if (IsAbsolutePath(linkTarget.data))
							next.Append(linkTarget);
						else
						{
							next.Append(out);
							next.Append(PathSeparator).Append(linkTarget);
						}
						if (slash != end)
							next.Append(slash); // slash starts with '/'

						// Normalize `..` segments before restarting so chained prefix-symlinks
						// like Soong/AOSP `gen/cmd/src/gen -> ../../src` then `gen/src/include
						// -> ../../include` resolve in subsequent passes. Without this the
						// walker can't promote `parent` past the `..` and skips later symlinks.
						// `next` is absolute (starts at root via `out` or with leading '/'), so
						// workingDir is unused.
						StringBuffer<MaxPath> fixed;
						FixPath(fixed, ToView(next), StringView());
						work.Clear();
						work.Append(fixed);

						// Unwinds both SCOPED_READ_LOCKs (destructors run on
						// scope exit), then restarts the walk from the new
						// `work` path with depth--.
						goto restart;
					}
					out.Append(PathSeparator).Append(fileName);
					emitted = true;
				}
			}

			if (!emitted)
				out.Append(PathSeparator).Append(p, u64(slash - p));

			if (slash == end)
				return true;

			p = slash + 1;

			// Promote: if the path-so-far is a tracked directory, descend
			// into it for next iteration. Otherwise leave parent null and
			// remaining components emit verbatim.
			auto findIt = m_lookup.find(fileNameKey);
			if (findIt != m_lookup.end())
			{
				parent = &findIt->second;
				PopulateDirectory(hasher, *parent);
			}
			else
			{
				parent = nullptr;
			}
		}
		return true;
#else
		return false;
#endif

	}

	bool DirectoryTable::GetLatestOffset(DirTableOffset& out, const Directory& dir)
	{
		if (dir.latestOverlayOffset != InvalidTableOffset)
		{
			out.internal = dir.latestOverlayOffset | DirectoryEntryFlag | OverlayTableFlag;
			return true;
		}
		if (dir.latestOffset != InvalidTableOffset)
		{
			out.internal = dir.latestOffset | DirectoryEntryFlag;
			return true;
		}
		return false;
	}

	bool DirectoryTable::ContainsFile(const StringKeyHasher& hasher, Directory& dir, StringKey fileNameKey)
	{
		SCOPED_WRITE_LOCK(dir.lock, lock);
		PopulateDirectoryNoLock(hasher, dir);
		auto findIt = dir.files.find(fileNameKey);
		if (findIt == dir.files.end())
			return false;
		EntryInformation info;
		GetEntryInformation(info, findIt->second);
		return info.attributes != 0;
	}

	template<typename Func>
	void DirectoryTable::TraverseOriginalFilesNoPopulate(DirTableOffset tableOffset, const Func& func)
	{
		UBA_ASSERT(IsDirectoryEntry(tableOffset));
		tableOffset.internal &= ~DirectoryEntryFlag;

		// Go to first offset
		BinaryReader reader(m_memory, tableOffset.internal, m_memorySize);
		while (true)
		{
			u32 prevTableOffset = u32(reader.Read7BitEncoded());
			if (!prevTableOffset)
				break;
			reader.SetPosition(prevTableOffset);
		}

		u32 dirAttr = reader.ReadFileAttributes();
		if (!dirAttr)
			return;
		reader.ReadVolumeSerial();
		reader.ReadFileIndex();
		u64 itemCount = reader.Read7BitEncoded();
		while (itemCount--)
		{
			StringBuffer<> fileName;
			reader.ReadString(fileName);
			if (CaseInsensitiveFs)
				fileName.MakeLower();
			u32 attr = reader.ReadFileAttributes();
			bool isDir = IsDirectory(attr);
			func(fileName, isDir);
			reader.ReadVolumeSerial();
			reader.ReadFileIndex();

			#if UBA_SUPPORT_SYMLINKS
			if (IsSymlink(attr))
				reader.SkipString();
			#endif

			if (isDir)
				continue;
			reader.ReadFileTime();
			reader.ReadFileSize();
		}
	}

	template<typename Func>
	void DirectoryTable::TraverseFilesRecursiveNoLock(StringView path, const Func& func)
	{
		auto findIt = m_lookup.find(ToStringKey(path));
		if (findIt == m_lookup.end())
			return;
		StringKeyHasher hasher;
		hasher.Update(path.data, path.count);
		PopulateDirectory(hasher, findIt->second);
		for (auto& fileKv : findIt->second.files)
		{
			DirectoryTable::EntryInformation info;
			StringBuffer<> fileName(path);
			fileName.Append(PathSeparator);
			DirTableOffset fileOffset = fileKv.second;
			GetEntryInformation(info, fileOffset, fileName.data + fileName.count, fileName.capacity - fileName.count);
			fileName.count = TStrlen(fileName.data);
			if (CaseInsensitiveFs)
				fileName.MakeLower();
			func(info, fileName, fileOffset);
			TraverseFilesRecursiveNoLock(fileName, func);
		}
	}

	template<typename Func>
	void DirectoryTable::TraverseAllFilesNoLock(const Func& func)
	{
		#if PLATFORM_WINDOWS
		for (tchar l='a';l!='z'; ++l)
		{
			StringBuffer<4> drive;
			drive.Append(l).Append(':');
			TraverseFilesRecursiveNoLock(drive, func);
		}
		#else
		TraverseFilesRecursiveNoLock(TCV("/"), func);
		#endif
	}

	DirectoryTable::DirectoryTable(MemoryBlock& block)
	:	m_memoryBlock(block)
	,	m_lookup(block, block)
	{
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
