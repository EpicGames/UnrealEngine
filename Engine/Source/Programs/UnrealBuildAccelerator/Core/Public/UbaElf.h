// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBase.h"

namespace uba
{
	struct Elf64FileHeader
	{
		unsigned char	e_ident[16];	// Magic number and other info
		u16	e_type;			// Object file type
		u16	e_machine;		// Architecture
		u32	e_version;		// Object file version
		u64	e_entry;		// Entry point virtual address
		u64	e_phoff;		// Program header table file offset
		u64	e_shoff;		// Section header table file offset
		u32	e_flags;		// Processor-specific flags
		u16	e_ehsize;		// ELF header size in bytes
		u16	e_phentsize;	// Program header table entry size
		u16	e_phnum;		// Program header table entry count
		u16	e_shentsize;	// Section header table entry size
		u16	e_shnum;		// Section header table entry count
		u16	e_shstrndx;		// Section header string table index
	};

	inline const constexpr char* ELFMAG	= "\177ELF";
	inline constexpr int SELFMAG	= 4;
	inline constexpr int EI_CLASS	= 4;
	inline constexpr int ELFCLASS32	= 1;
	inline constexpr int ELFCLASS64	= 2;

	struct Elf64ProgramHeader
	{
		u32 p_type;    // Segment type
		u32 p_flags;   // Segment flags
		u64 p_offset;  // Offset in file
		u64 p_vaddr;   // Virtual address in memory
		u64 p_paddr;   // Physical address (ignored)
		u64 p_filesz;  // Size of segment in file
		u64 p_memsz;   // Size of segment in memory
		u64 p_align;   // Alignment
	};

	struct Elf64SectionHeader
	{
		u32	sh_name;		// Section name (string tbl index)
		u32	sh_type;		// Section type
		u64	sh_flags;		// Section flags
		u64	sh_addr;		// Section virtual addr at execution
		u64	sh_offset;		// Section file offset
		u64	sh_size;		// Section size in bytes
		u32	sh_link;		// Link to another section
		u32	sh_info;		// Additional section information
		u64	sh_addralign;	// Section alignment
		u64	sh_entsize;		// Entry size if section holds table
	};

	inline constexpr int SHT_PROGBITS		= 1;	// Program data
	inline constexpr int SHT_SYMTAB			= 2;	// Symbol table
	inline constexpr int SHT_STRTAB			= 3;
	inline constexpr int SHT_RELA			= 4;	// Relocation entries with addends
	inline constexpr int SHT_DYNAMIC		= 6;	// Dynamic linking information
	inline constexpr int SHT_REL			= 9;	// Relocation entries, no addends
	inline constexpr int SHT_DYNSYM			= 11;	// Dynamic linker symbol table
	inline constexpr int SHT_SYMTAB_SHNDX	= 18;	// Extended section indeces

	struct Elf64Dyn
	{
		s64 d_tag;		// Dynamic entry type
		union
		{
			u64 d_val;  /* Integer value */
			u64  d_ptr;  /* Address value */
		} d_un;
	};

	using Elf64Addr = u64;
	using Elf64Off = u64;
}