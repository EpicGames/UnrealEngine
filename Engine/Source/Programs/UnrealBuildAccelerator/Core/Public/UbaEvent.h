// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBase.h"

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	class StringBufferBase;
	struct StringView;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	enum EventResetType : u8
	{
		EventResetType_Auto,
		EventResetType_Manual,
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class Event
	{
	public:
		Event() = default;
		Event(EventResetType resetType);
		~Event();
		bool Create(EventResetType resetType);
		void Destroy();
		void Set();
		void Reset();
		bool IsCreated();
		bool IsSet(u32 timeOutMs = ~0u);
		void* GetHandle();
		static bool IsNative();
		static void ClearCache();
	private:
		void* m_impl = nullptr;
		Event(const Event&) = delete;
		void operator=(const Event&) = delete;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class SharedEvent
	{
	public:
		SharedEvent();
		~SharedEvent();
		bool Create(EventResetType resetType);
		bool Create(StringView name, bool isOwner);
		void Destroy();
		void Set();
		bool IsSet(u32 timeOutMs = ~0u);
		void ToString(StringBufferBase& out);
		void* GetHandle();
	private:
		bool IsSemaphore();
		#if PLATFORM_MAC || defined(__aarch64__)
		u64 m_data[16];
		#elif !PLATFORM_WINDOWS
		u64 m_data[13];
		#else
		u64 m_data[2];
		#endif
		SharedEvent(const SharedEvent&) = delete;
		void operator=(const SharedEvent&) = delete;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

}

#include "UbaEventWaitOnAddress.inl"

namespace uba
{
	#if PLATFORM_WINDOWS
	using EventSlim = EventWaitOnAddress;
	#else
	using EventSlim = Event;
	#endif
}