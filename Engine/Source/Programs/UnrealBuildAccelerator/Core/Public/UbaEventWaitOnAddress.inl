// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaTimer.h"

#if PLATFORM_WINDOWS

#include "UbaPlatform.h"

namespace uba
{
	struct alignas(4) EventWaitOnAddress
	{
		Atomic<u32> m_state;
		EventResetType m_resetType;

		EventWaitOnAddress() = default;
		EventWaitOnAddress(EventResetType rt) { Create(rt); }

		bool Create(EventResetType rt)
		{
			static_assert(sizeof(Atomic<u32>) == 4);
			m_resetType = rt;
			m_state.store(0, std::memory_order_relaxed);
			return true;
		}

		void Set()
		{
			m_state.store(1, std::memory_order_release);
			if (m_resetType == EventResetType_Manual)
				WakeByAddressAll((void*)&m_state);
			else
				WakeByAddressSingle((void*)&m_state);
		}

		void Reset()
		{
			m_state.store(0, std::memory_order_release);
		}

		bool IsSet(u32 timeoutMs = ~0u)
		{
			if (m_resetType == EventResetType_Manual)
			{
				if (m_state.load(std::memory_order_acquire) == 1)
					return true;

				u64 start = GetTime();
				while (true)
				{
					u32 expected = 0;
					if (!WaitOnAddress((void*)&m_state, &expected, sizeof(expected), timeoutMs))
						if (GetLastError() == ERROR_TIMEOUT)
							return false;

					if (m_state.load(std::memory_order_acquire) == 1)
						return true;

					if (timeoutMs == ~0u)
						continue;
					u64 now = GetTime();
					u32 timePassedMs = u32(TimeToMs(now - start));
					if (timePassedMs >= timeoutMs)
						return false;
					start = now;
					timeoutMs -= timePassedMs;
				}
			}
			else
			{
				u32 expected = 1;
				if (m_state.compare_exchange_strong(expected, 0, std::memory_order_acquire, std::memory_order_relaxed))
					return true;

				u64 start = GetTime();
				while (true)
				{
					expected = 0;
					if (!WaitOnAddress((void*)&m_state, &expected, sizeof(expected), timeoutMs))
						if (GetLastError() == ERROR_TIMEOUT)
							return false;

					expected = 1;
					if (m_state.compare_exchange_strong(expected, 0, std::memory_order_acquire, std::memory_order_relaxed))
						return true;

					if (timeoutMs == ~0u)
						continue;
					u64 now = GetTime();
					u32 timePassedMs = u32(TimeToMs(now - start));
					if (timePassedMs >= timeoutMs)
						return false;
					start = now;
					timeoutMs -= timePassedMs;
				}
			}
		}
	};
}

#endif
