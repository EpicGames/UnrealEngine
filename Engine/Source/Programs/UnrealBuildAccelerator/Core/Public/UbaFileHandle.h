// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBase.h"

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	enum FileHandle : u64 {};
	inline constexpr FileHandle InvalidFileHandle = (FileHandle)-1;

	inline constexpr u64 FileHandleFlagMask = 0x0000'0000'ffff'ffff;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	#if PLATFORM_WINDOWS
	typedef void *HANDLE;
	inline HANDLE AsHANDLE(FileHandle fh) { return (HANDLE)(fh == InvalidFileHandle ? InvalidFileHandle : (fh & FileHandleFlagMask)); }
	#else
	inline int AsFileDescriptor(FileHandle fh) { return fh == InvalidFileHandle ? (int)fh : (int)(fh & FileHandleFlagMask); }
	#endif

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
