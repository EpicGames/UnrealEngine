// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaFileMappingHandle.h"

namespace uba
{
	enum FileHandle : u64;
	class Logger;

#ifdef UBA_FM_STRING_TYPE
	using fmchar = UBA_FM_STRING_TYPE;
#else
	using fmchar = tchar;
#endif

	////////////////////////////////////////////////////////////////////////////////////////////////////

#if !PLATFORM_WINDOWS
	inline constexpr u32 FILE_MAP_WRITE = 0x0002;
	inline constexpr u32 FILE_MAP_READ = 0x0004;
	inline constexpr u32 FILE_MAP_ALL_ACCESS = FILE_MAP_WRITE | FILE_MAP_READ;
	inline constexpr u32 SEC_RESERVE = 0x04000000;
	inline constexpr u32 PAGE_READONLY = PROT_READ;
	inline constexpr u32 PAGE_READWRITE = PROT_READ | PROT_WRITE;

#if PLATFORM_MAC
	inline constexpr u32 SHM_MAX_FILENAME = PSHMNAMLEN;
#else
	inline constexpr u32 SHM_MAX_FILENAME = 38;
#endif
#endif

	// LazyCopyOnWrite: like CopyOnWrite but on Windows the view is mapped read-only so it does
	// not eagerly charge commit for the whole view. Pages are flipped to copy-on-write on the
	// first write fault by a vectored exception handler (see RegisterLazyCowRange in the detour),
	// so commit tracks only the pages the tool actually dirties. On POSIX it is identical to
	// CopyOnWrite (MAP_PRIVATE already commits lazily on write fault).
	enum MemoryMapType { MemoryMapType_ReadOnly, MemoryMapType_ReadWrite, MemoryMapType_CopyOnWrite, MemoryMapType_LazyCopyOnWrite };

	FileMappingHandle FileMapping_Create(Logger& logger, u32 flProtect, u64 maxSize, const fmchar* name, const fmchar* hint);
	FileMappingHandle FileMapping_CreateFromFile(Logger& logger, FileHandle file, u32 flProtect, u64 maxSize, const fmchar* hint);
	u8* FileMapping_Map(Logger& logger, FileMappingHandle fileMappingObject, u32 dwDesiredAccess, u64 offset, u64 dwNumberOfBytesToMap, bool allowDiscard = false);
	bool FileMapping_Commit(Logger& logger, void* address, u64 size, bool allowDiscard = false);
	bool FileMapping_Unmap(Logger& logger, const void* lpBaseAddress, u64 bytesToUnmap, const fmchar* hint, bool allowDiscard = false);
	bool FileMapping_Close(Logger& logger, FileMappingHandle h, const fmchar* hint);
	void* FileMapping_ReservePlaceholder(void* baseAddress, u64 capacity);
	void* FileMapping_MapPlaceholder(FileMappingHandle handle, MemoryMapType mapType, void* targetAddress, u64 targetOffset, u64 targetCapacity, u64 handleOffset, u64 size);
	bool FileMapping_UnmapPlaceholder(void* memory, u64 capacity, u64 mappedSize, u64* subMappings, u64 subMappingsCount);
	void FileMapping_CopyMem(void* dest, const void* source, u64 size);

	FileMappingHandle FileMapping_DuplicateFromHost(FileMappingHandle handle, const char* hint);
	u8* FileMapping_MapFromHost(FileMappingHandle handle, u64 size, u64 offset, bool writable, const char* hint);
	u8* FileMapping_MapFromHost2(FileMappingHandle handle, u64 size, u64 offset, MemoryMapType mapType, void* targetAddress, const char* hint);

	// Pick memfd/shm/disk backing once before any mapping is created (Linux). outEnv gets the env var to inject into children.
	bool FileMapping_ResolveBacking(Logger& logger, const fmchar* preferredDir, u64 shmFloorBytes, char* outEnv = nullptr, int outEnvCap = 0);
	extern char g_fileMappingBackingEnv[512]; // ResolveBacking's choice; the session injects it into children


	////////////////////////////////////////////////////////////////////////////////////////////////////

	#define UBA_FM_FUNCTIONS \
		UBA_FM_FUNC(Create) \
		UBA_FM_FUNC(CreateFromFile) \
		UBA_FM_FUNC(Map) \
		UBA_FM_FUNC(Commit) \
		UBA_FM_FUNC(Unmap) \
		UBA_FM_FUNC(Close) \
		UBA_FM_FUNC(ReservePlaceholder) \
		UBA_FM_FUNC(MapPlaceholder) \
		UBA_FM_FUNC(UnmapPlaceholder) \
		UBA_FM_FUNC(DuplicateFromHost) \
		UBA_FM_FUNC(MapFromHost) \
		UBA_FM_FUNC(ResolveBacking) \

		//UBA_FM_FUNC(CopyMem)

	struct FileMappingBackend
	{
		#define UBA_FM_FUNC(func) decltype(FileMapping_##func)* func = FileMapping_##func;
		UBA_FM_FUNCTIONS
		#undef UBA_FM_FUNC

		#if PLATFORM_WINDOWS
		bool linuxBackend = false;
		#endif
	};

	#if PLATFORM_WINDOWS
	bool FileMapping_GetWineBackend(FileMappingBackend& out, Logger& logger, HMODULE wineDll);
	#endif

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
