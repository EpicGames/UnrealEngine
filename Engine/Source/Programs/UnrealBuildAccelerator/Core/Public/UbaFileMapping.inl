// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// No includes in this file!
#ifndef UBA_USE_MEMFD
#define UBA_USE_MEMFD 0 // PLATFORM_LINUX — This does not work cross linux namespaces
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////

#if 0
#define WRITE_DEBUG(format, ...) \
do { \
    char str[1024]; \
    int len = snprintf(str, sizeof(str), format "\n", ##__VA_ARGS__); \
	ssize_t ignored_result = write(STDERR_FILENO, str, (size_t)len); \
} while (0)
#endif

#if PLATFORM_WINDOWS
Bottleneck& g_createFileHandleBottleneck = *new Bottleneck(8); // Allocated and leaked just to prevent shutdown asserts in debug

HANDLE InternalCreateFileMappingW(Logger& logger, HANDLE hFile, DWORD flProtect, DWORD dwMaximumSizeHigh, DWORD dwMaximumSizeLow, LPCWSTR lpName, const tchar* hint)
{
	HANDLE res;
	if (flProtect == PAGE_READWRITE)
	{
		// Experiment to try to prevent lock happening on AWS servers when lots of helpers are sending back obj files.
		Timer timer;
		BottleneckScope bs(g_createFileHandleBottleneck, timer);
		res = ::CreateFileMappingW(hFile, NULL, flProtect, dwMaximumSizeHigh, dwMaximumSizeLow, lpName);
	}
	else
		res = ::CreateFileMappingW(hFile, NULL, flProtect, dwMaximumSizeHigh, dwMaximumSizeLow, lpName);

#if UBA_DEBUG_FILE_MAPPING
	if (res)
	{
		SCOPED_FUTEX(g_fileMappingsLock, lock);
		UBA_FOR_ASSERT(auto insres =) g_fileMappings.try_emplace(res, hint);
		UBA_ASSERT(insres.second);
		//logger.Info(TC("FILEMAPPING CREAT: %s %llu (%llu)"), hint, u64(res), g_fileMappings.size());
	}
#endif

	return res;
}
static void InternalDiscard(Logger& logger, u8* mem, u64 size, const tchar* from, const tchar* hint)
{
#if PLATFORM_WINDOWS
	u64 stepSize = 16ull*1024*1024*1024; // Extremely large.. here to test if it is faster to do multiple calls
	u64 left = size;
	while (left)
	{
		u64 toDiscard = Min(left, stepSize);
		DWORD res = DiscardVirtualMemory(mem, AlignUp(toDiscard, 4*1024));
		if (res != ERROR_SUCCESS)
			logger.Warning(TC("%s - Failed to discard memory (%s) (%u)"), from, hint, res);
		mem += toDiscard;
		left -= toDiscard;
	}
#endif
}
#else  // !PLATFORM_WINDOWS

#if PLATFORM_LINUX
	// Linux syscall UAPI constants, lifted so the !UBA_USE_MEMFD shm_open fallback can use raw
	// syscalls too. Without this the stub blob links unresolved shm_open/shm_unlink/fcntl/close
	// stubs (no libc, no librt) and SIGSEGVs the first time the !UBA_USE_MEMFD branch executes.
	// macOS keeps using libc (no raw-syscall path exists for darwin).
	#if defined(__x86_64__)
		#define UBA_NR_OPEN         2
		#define UBA_NR_CLOSE        3
		#define UBA_NR_MMAP         9
		#define UBA_NR_MUNMAP       11
		#define UBA_NR_GETPID       39
		#define UBA_NR_FCNTL        72
		#define UBA_NR_FTRUNCATE    77
		#define UBA_NR_OPENAT       257
		#define UBA_NR_UNLINKAT     263
		#define UBA_NR_MKDIRAT      258
		#define UBA_NR_STATFS       137
		#define UBA_NR_FORK         57
		#define UBA_NR_WAIT4        61
		#define UBA_NR_EXIT_GROUP   231
		#define UBA_NR_PRCTL        157
		#define UBA_NR_MEMFD_CREATE 319
		#define UBA_NR_PIDFD_OPEN   434
		#define UBA_NR_PIDFD_GETFD  438
	#elif defined(__aarch64__)
		#define UBA_NR_UNLINKAT     35
		#define UBA_NR_FCNTL        25
		#define UBA_NR_FTRUNCATE    46
		#define UBA_NR_OPENAT       56
		#define UBA_NR_CLOSE        57
		#define UBA_NR_MKDIRAT      34
		#define UBA_NR_STATFS       43
		#define UBA_NR_CLONE        220
		#define UBA_NR_WAIT4        260
		#define UBA_NR_EXIT_GROUP   94
		#define UBA_NR_PRCTL        167
		#define UBA_NR_GETPID       172
		#define UBA_NR_MUNMAP       215
		#define UBA_NR_MMAP         222
		#define UBA_NR_MEMFD_CREATE 279
		#define UBA_NR_PIDFD_OPEN   434
		#define UBA_NR_PIDFD_GETFD  438
	#else
		#error "UbaFileMapping.inl: unsupported Linux architecture"
	#endif
	#define UBA_MFD_CLOEXEC          0x0001U
	#define UBA_PROT_NONE            0x0
	#define UBA_PROT_READ            0x1
	#define UBA_PROT_WRITE           0x2
	#define UBA_MAP_SHARED           0x01
	#define UBA_MAP_PRIVATE          0x02
	#define UBA_MAP_FIXED            0x10
	#define UBA_MAP_ANONYMOUS        0x20
	#define UBA_MAP_FIXED_NOREPLACE  0x100000
	#define UBA_AT_FDCWD             (-100)
	#define UBA_O_RDONLY             0x0
	#define UBA_O_RDWR               0x2
	#define UBA_O_CLOEXEC            0x80000
	#define UBA_F_SETFD              2
	#define UBA_FD_CLOEXEC           1
	// mmap-style return: r1 in [-4095, -1] is -errno, anything else is the mapped address.
	inline bool UbaSyscallIsErr(long r) { return (unsigned long)r > (unsigned long)-4096L; }

	// UBA_FILE_MAPPING_DIR => disk files under it; else /dev/shm. Uncached: ResolveBacking's setenv may run after an early mapping.
	inline const char* FmBackingDir()
	{
#if UBA_STUB_BUILD
		extern const char* g_ubaFileMappingDir; // stub getenv() is a nullptr stub; set by its envp scan
		return (g_ubaFileMappingDir && *g_ubaFileMappingDir) ? g_ubaFileMappingDir : nullptr;
#else
		const char* d = getenv("UBA_FILE_MAPPING_DIR");
		return (d && *d) ? d : nullptr;
#endif
	}
	// memfd mode for CREATION only. Attach side dispatches on FileMappingHandle::MemfdFlag — the
	// creator's mode travels with the handle, so mixed-mode trees still attach correctly.
	inline bool FmUseMemfd()
	{
#if UBA_STUB_BUILD
		extern bool g_ubaUseMemfd; // set by the stub's envp scan
		return g_ubaUseMemfd;
#else
		const char* m = getenv("UBA_FILE_MAPPING_MEMFD");
		return m && *m == '1';
#endif
	}
#if !UBA_STUB_BUILD
	inline long FmRawFork() // arm64 has no fork; clone(SIGCHLD) is the equivalent
	{
	#if defined(__x86_64__)
		return RawSyscallDirect(UBA_NR_FORK, 0, 0, 0).r1;
	#else
		return RawSyscallDirect(UBA_NR_CLONE, 17 /*SIGCHLD*/, 0, 0, 0, 0).r1;
	#endif
	}
	// memfd works only if a child can pidfd_getfd back to us — fork a child that tries it. PR_SET_PTRACER_ANY
	// lifts yama scope=1 (some LSMs still reject). outWhy gets the failing step+errno on false.
	inline bool FmMemfdProbe(char* outWhy, int whyCap)
	{
		// Config can force memfd off (the host's detoured children can't attach memfd mappings).
		if (const char* m = getenv("UBA_FILE_MAPPING_MEMFD"); m && *m == '0') { snprintf(outWhy, whyCap, "not allowed by config"); return false; }

		RawSyscallDirect(UBA_NR_PRCTL, 0x59616d61 /*PR_SET_PTRACER*/, -1L /*PR_SET_PTRACER_ANY*/, 0, 0, 0);
		long probeFd = RawSyscallDirect(UBA_NR_MEMFD_CREATE, (long)"ubaprobe", UBA_MFD_CLOEXEC, 0).r1;
		if (UbaSyscallIsErr(probeFd)) { snprintf(outWhy, whyCap, "memfd_create errno=%ld", -probeFd); return false; }
		long myPid = RawSyscallDirect(UBA_NR_GETPID, 0, 0, 0).r1;
		long child = FmRawFork();
		if (child == 0)
		{ // child: raw syscalls only — glibc/wine state is unsafe after a raw fork. Exit code carries the errno.
			long pidfd = RawSyscallDirect(UBA_NR_PIDFD_OPEN, myPid, 0, 0).r1;
			if (UbaSyscallIsErr(pidfd)) RawSyscallDirect(UBA_NR_EXIT_GROUP, 50 + (-pidfd & 0x3f), 0, 0);
			long got = RawSyscallDirect(UBA_NR_PIDFD_GETFD, pidfd, probeFd, 0).r1;
			RawSyscallDirect(UBA_NR_EXIT_GROUP, UbaSyscallIsErr(got) ? 128 + (-got & 0x7f) : 0, 0, 0);
			__builtin_unreachable();
		}
		if (child < 0) { RawSyscallDirect(UBA_NR_CLOSE, probeFd, 0, 0); snprintf(outWhy, whyCap, "fork errno=%ld", -child); return false; }
		long status = 0;
		long w = RawSyscallDirect(UBA_NR_WAIT4, child, (long)&status, 0, 0).r1;
		RawSyscallDirect(UBA_NR_CLOSE, probeFd, 0, 0); // only after the child has grabbed it — closing earlier races pidfd_getfd -> EBADF
		if (UbaSyscallIsErr(w)) { snprintf(outWhy, whyCap, "wait4 errno=%ld", -w); return false; }
		if ((status & 0x7f) != 0) { snprintf(outWhy, whyCap, "killed sig %ld", status & 0x7f); return false; }
		int code = (int)((status >> 8) & 0xff);
		if (code >= 128) { snprintf(outWhy, whyCap, "pidfd_getfd errno=%d", code - 128); return false; }
		if (code >= 50) { snprintf(outWhy, whyCap, "pidfd_open errno=%d", code - 50); return false; }
		return true;
	}
#endif
	// "<dir>/uba_NNNN" (disk) or "/dev/shm/uba_NNNN" (shm). Returns true if disk-backed.
	inline bool FmBuildShmPath(const char* name, char* out, int outSize)
	{
		const char* prefix = FmBackingDir();
		bool disk = prefix != nullptr;
		if (!disk) prefix = "/dev/shm";
		int i = 0;
		for (; prefix[i] && i < outSize - 1; ++i) out[i] = prefix[i];
		while (*name && i < outSize - 1) out[i++] = *name++;
		out[i] = 0;
		return disk;
	}
	inline int FmShmOpen(const char* name, int flags, int mode)
	{
		char path[512];
		FmBuildShmPath(name, path, sizeof(path));
		auto r = RawSyscallDirect(UBA_NR_OPENAT, UBA_AT_FDCWD, (long)path, flags | UBA_O_CLOEXEC, mode);
		if (UbaSyscallIsErr(r.r1)) { SetLastError((int)-r.r1); return -1; }
		return (int)r.r1;
	}
	inline int FmShmUnlink(const char* name)
	{
		char path[512];
		FmBuildShmPath(name, path, sizeof(path));
		auto r = RawSyscallDirect(UBA_NR_UNLINKAT, UBA_AT_FDCWD, (long)path, 0);
		if (UbaSyscallIsErr(r.r1)) { SetLastError((int)-r.r1); return -1; }
		return 0;
	}
	inline int FmFcntl(int fd, int cmd, int arg)
	{
		auto r = RawSyscallDirect(UBA_NR_FCNTL, fd, cmd, arg);
		if (UbaSyscallIsErr(r.r1)) { SetLastError((int)-r.r1); return -1; }
		return (int)r.r1;
	}
	inline int FmClose(int fd)
	{
		auto r = RawSyscallDirect(UBA_NR_CLOSE, fd, 0, 0);
		if (UbaSyscallIsErr(r.r1)) { SetLastError((int)-r.r1); return -1; }
		return 0;
	}
	inline void* FmMmap(void* addr, u64 size, int prot, int flags, int fd, u64 offset)
	{
		auto r = RawSyscallDirect(UBA_NR_MMAP, (long)addr, (long)size, prot, flags, fd, (long)offset);
		if (UbaSyscallIsErr(r.r1)) { SetLastError((int)-r.r1); return (void*)-1L; } // MAP_FAILED
		return (void*)r.r1;
	}
	inline int FmMunmap(void* addr, u64 size)
	{
		auto r = RawSyscallDirect(UBA_NR_MUNMAP, (long)addr, (long)size, 0);
		if (UbaSyscallIsErr(r.r1)) { SetLastError((int)-r.r1); return -1; }
		return 0;
	}
#else // PLATFORM_MAC — libSystem is always linked; just delegate to glibc.
	inline int FmShmOpen(const char* name, int flags, int mode) { return shm_open(name, flags, mode); }
	inline int FmShmUnlink(const char* name) { return shm_unlink(name); }
	inline int FmFcntl(int fd, int cmd, int arg) { return fcntl(fd, cmd, arg); }
	inline int FmClose(int fd) { return close(fd); }
	inline void* FmMmap(void* addr, u64 size, int prot, int flags, int fd, u64 offset) { return mmap(addr, size, prot, flags, fd, (off_t)offset); }
	inline int FmMunmap(void* addr, u64 size) { return munmap(addr, size); }
#endif

#if !UBA_USE_MEMFD

void AppendPrefixAndValue(char* out, const char* prefix, u64 value, bool addSeparatorAfterPrefix = false)
{
	// Hand-rolled copy: stub blob links without libc, so strcpy/strlen are unresolved
	// (clang -O2 does NOT always inline these for non-trivial buffer pairs).
	while (*prefix) *out++ = *prefix++;
	if (addSeparatorAfterPrefix)
		*out++ = '/';
	constexpr char hexChars[] = "0123456789abcdef";
	for (int i=0;i!=8;++i)
	{
		*out++ = hexChars[(value >> 4) & 0xf];
		*out++ = hexChars[value & 0xf];
		value = value >> 8;
		if (!value)
			break;
	}
	*out = 0;
}
void GetMappingHandleName(char* out, u64 uid)
{
	#if PLATFORM_MAC
	if (!IsRunningDarling())
	{
		memcpy(out, "/tmp", 4);
		out += 4;
	}
	#endif
	AppendPrefixAndValue(out, "/uba_", uid);
}
u64 ParseMappingUid(const char* str)
{
	u64 result = 0;
	int shift = 0;
	while (str[0] && str[1] && shift < 64)
	{
		auto hexVal = [](char c) -> u8 { return c >= 'a' ? c - 'a' + 10 : c - '0'; };
		result |= (u64)((hexVal(str[0]) << 4) | hexVal(str[1])) << shift;
		shift += 8;
		str += 2;
	}
	return result;
}

#if !UBA_STUB_BUILD
// glibc create/unlink (errno-setting, unlike the raw FmShm* above) that honor UBA_FILE_MAPPING_DIR.
inline int FmHostShmOpen(const char* leafName, int flags, int mode)
{
	const char* dir = getenv("UBA_FILE_MAPPING_DIR");
	if (dir && *dir)
	{
		char p[512];
		snprintf(p, sizeof(p), "%s%s", dir, leafName);
		return open(p, flags, mode);
	}
	return shm_open(leafName, flags, mode);
}
inline int FmHostShmUnlink(const char* leafName)
{
	const char* dir = getenv("UBA_FILE_MAPPING_DIR");
	if (dir && *dir)
	{
		char p[512];
		snprintf(p, sizeof(p), "%s%s", dir, leafName);
		return unlink(p);
	}
	return shm_unlink(leafName);
}

// Host-only: shm uid counter + lock-dir GC. The stub never creates shms or
// runs GC — it just maps existing shm names produced by the host.
Atomic<u64> g_mappingUidCounter = 0;
static pthread_mutex_t g_mappingUidCounterLock = PTHREAD_MUTEX_INITIALIZER;

// /tmp preferred — shared so user B's GC reclaims user A's leaked shms after a crash.
// XDG_RUNTIME_DIR per-user fallback when /tmp isn't usable (PrivateTmp=yes, sandbox, ro fs).
// pthread_once over function-local static so this links without libstdc++ (winegcc).
char g_lockDirBuf[256];
bool g_lockDirShared = false;
pthread_once_t g_lockDirOnce = PTHREAD_ONCE_INIT;
void InitLockDir()
{
	auto isUsable = [](const char* p)
	{
		struct stat st;
		return ::stat(p, &st) == 0 && S_ISDIR(st.st_mode) && access(p, R_OK|W_OK|X_OK) == 0;
	};

	const char* tmp = getenv("TMPDIR");
	if (!tmp || !*tmp)
		tmp = "/tmp";
	if (isUsable(tmp))
	{
		snprintf(g_lockDirBuf, sizeof(g_lockDirBuf), "%s/uba_shm_locks", tmp);
		g_lockDirShared = true;
		return;
	}
	const char* xdg = getenv("XDG_RUNTIME_DIR");
	if (xdg && *xdg && isUsable(xdg))
	{
		snprintf(g_lockDirBuf, sizeof(g_lockDirBuf), "%s/uba_shm_locks", xdg);
		return;
	}
	snprintf(g_lockDirBuf, sizeof(g_lockDirBuf), "uba_shm_locks"); // last resort — cwd
}
const char* GetLockDir()
{
	pthread_once(&g_lockDirOnce, InitLockDir);
	return g_lockDirBuf;
}

bool GarbageCollectFileMappings(Logger& logger)
{
	// Since we need to not leak file mappings we use files as a trick to know which ones are used and not
	char filePath[512];
	strcpy(filePath, GetLockDir());

	// Failure is fatal: without a usable lock dir we'd hand out invalid handles
	// and the next FileMapping_Create caller would deadlock on g_mappingUidCounterLock
	// (locked by our caller and not unlocked on the false-return path).
	mode_t dirMode = g_lockDirShared ? 0777 : 0700;
	if (mkdir(filePath, dirMode) == 0)
	{
		if (g_lockDirShared)
			chmod(filePath, 0777); // override umask so other users can write into the shared dir
	}
	else if (errno != EEXIST)
		FatalError(9886, TC("Failed to create %s for memory mapping (%s)"), filePath, strerror(errno));

	// Clear out all orphaned shm_open
	if (DIR* dir = opendir(filePath))
	{
		auto pathLen = strlen(filePath);
		filePath[pathLen++] = '/';
		while (true)
		{
			errno = 0;
			struct dirent* pDirent = readdir(dir);
			if (!pDirent)
			{
				if (errno == 0)
					break;

				// This is actually an error.. should return false?
				break;
			}

			char* fileName = pDirent->d_name;
			if (fileName[0] == '.' && (fileName[1] == 0 || (fileName[1] == '.' && fileName[2] == 0))) 
				continue;

			u32 uid = ParseMappingUid(fileName);

			strcpy(filePath+pathLen, fileName);
			int lockFd = open(filePath, O_RDWR | O_CLOEXEC, S_IRUSR | S_IWUSR);
			if (lockFd == -1)
			{
				if (errno == EPERM)
				{
					g_mappingUidCounter = uid;
					return true;
				}
				logger.Warning("Failed to open %s for memory mapping (%s)", filePath, strerror(errno));
				UBA_ASSERTF(false, "Failed to open %s (%s)", filePath, strerror(errno));
				return false;
			}

			if (flock(lockFd, LOCK_EX | LOCK_NB) == 0)
			{
				char uidName[64];
				GetMappingHandleName(uidName, uid);
				if (FmHostShmUnlink(uidName) == 0)
					;// logger.Info("Removed old shared memory %s", uidName.data);
				remove(filePath);
			}
			else
			{
				g_mappingUidCounter = uid;
			}
			close(lockFd);
		}
		closedir(dir);
	}
	if (g_mappingUidCounter)
		logger.Info("Starting shared memory files at %u", g_mappingUidCounter.load());
	return true;
}
#endif // !UBA_STUB_BUILD
#endif // !UBA_USE_MEMFD
#endif // !PLATFORM_WINDOWS

// Host-side functions — gated out of the static-stub build. The stub only needs
// FileMapping_MapFromHost / FileMapping_MapFromHost2 (memfd path uses RawSyscallDirect,
// no glibc, no Logger). Gating these out avoids unresolved Logger:: symbols at stub link.
#if !UBA_STUB_BUILD

// outEnv gets the env var the caller injects into children ("UBA_FILE_MAPPING_MEMFD=1"/"..._DIR=<dir>"/"").
bool FileMapping_ResolveBackingUnix(Logger& logger, const char* preferredDirUnix, u64 shmFloorBytes, char* outEnv, int outEnvCap)
{
	if (outEnv && outEnvCap) outEnv[0] = 0;
#if PLATFORM_LINUX
	char why[48] = ""; // memfd-unavailable reason, folded into the shm/disk decision line

	// Inherited memfd decision from a parent? Verify it still works from this process before honoring —
	// a stale UBA_FILE_MAPPING_MEMFD=1 (baked into a container image or CI env) would otherwise make
	// every mapping created here unattachable ("Failed to attach comm mapping").
	if (const char* m = getenv("UBA_FILE_MAPPING_MEMFD"); m && *m == '1')
	{
		if (FmMemfdProbe(why, sizeof(why)))
		{
			if (outEnv) snprintf(outEnv, outEnvCap, "UBA_FILE_MAPPING_MEMFD=1");
			logger.Info("FileMapping backing: memfd (inherited)");
			return true;
		}
		setenv("UBA_FILE_MAPPING_MEMFD", "0", 1); // sanitize environ so children don't inherit the stale choice
		logger.Warning("FileMapping backing: ignoring inherited UBA_FILE_MAPPING_MEMFD=1 - memfd does not work in this process (%s)", why);
	}
	if (const char* cur = getenv("UBA_FILE_MAPPING_DIR"); cur && *cur)
	{
		if (outEnv) snprintf(outEnv, outEnvCap, "UBA_FILE_MAPPING_DIR=%s", cur);
		logger.Info("FileMapping backing: disk %s (UBA_FILE_MAPPING_DIR override)", cur);
		return true; // operator (or a parent) already chose disk-backed
	}

	if (!why[0] && FmMemfdProbe(why, sizeof(why))) // memfd first — anonymous RAM, no /dev/shm or disk limits
	{
		setenv("UBA_FILE_MAPPING_MEMFD", "1", 1);
		if (outEnv) snprintf(outEnv, outEnvCap, "UBA_FILE_MAPPING_MEMFD=1");
		logger.Info("FileMapping backing: memfd");
		return true;
	}
	// pidfd_getfd EPERM without a blocking config is the container signature (default docker/podman
	// seccomp gates it behind CAP_SYS_PTRACE) — tell the container owner how to get fast backing back.
	const char* containerHint = strstr(why, "pidfd_getfd errno=1")
		? " Container? memfd needs CAP_SYS_PTRACE (or a seccomp profile allowing pidfd_getfd); alternatively raise --shm-size to use /dev/shm."
		: "";

	// raw statfs avoids needing <sys/vfs.h> across the many .inl includers
	struct { u64 w[16]; } sf; // kernel struct statfs: all fields 64-bit on x86_64/aarch64
	auto r = RawSyscallDirect(UBA_NR_STATFS, (long)"/dev/shm", (long)&sf, 0);
	u64 total = UbaSyscallIsErr(r.r1) ? 0 : sf.w[1] /*f_bsize*/ * sf.w[2] /*f_blocks*/;
	if (total >= shmFloorBytes)
	{
		logger.Info("FileMapping backing: /dev/shm (%lluMB; memfd: %s)", total >> 20, why);
		return false;
	}

	if (!preferredDirUnix || !*preferredDirUnix)
	{
		logger.Warning("FileMapping backing: /dev/shm (%lluMB < floor %lluMB, no fallback dir; memfd: %s) - mappings may fail with ENOSPC under load.%s", total >> 20, shmFloorBytes >> 20, why, containerHint);
		return false;
	}

	char dir[1024]; // "<preferredDir>/uba_filemap"
	int i = 0;
	for (; preferredDirUnix[i] && i < (int)sizeof(dir) - 16; ++i) dir[i] = preferredDirUnix[i];
	if (i && dir[i-1] != '/') dir[i++] = '/';
	for (const char* leaf = "uba_filemap"; *leaf; ++leaf) dir[i++] = *leaf;
	dir[i] = 0;

	RawSyscallDirect(UBA_NR_MKDIRAT, UBA_AT_FDCWD, (long)dir, 0777); // best-effort; EEXIST is fine
	setenv("UBA_FILE_MAPPING_DIR", dir, 1);
	if (outEnv) snprintf(outEnv, outEnvCap, "UBA_FILE_MAPPING_DIR=%s", dir);
	logger.Info("FileMapping backing: disk %s (/dev/shm %lluMB < floor %lluMB; memfd: %s) - this works but is slower.%s", dir, total >> 20, shmFloorBytes >> 20, why, containerHint);
	return true;
#else
	(void)logger; (void)preferredDirUnix; (void)shmFloorBytes;
	return false; // mac: not the constrained target — keep shm
#endif
}

bool FileMapping_ResolveBacking(Logger& logger, const fmchar* preferredDir, u64 shmFloorBytes, char* outEnv, int outEnvCap)
{
	if (outEnv && outEnvCap) outEnv[0] = 0;
#if PLATFORM_WINDOWS
	(void)logger; (void)preferredDir; (void)shmFloorBytes;
	return false; // native Windows host: SEC_RESERVE + pagefile commit, no tmpfs limit
#elif defined(UBA_FM_STRING_TYPE)
	// winelib: the real entry is the WINAPI wrapper (translates the wchar Windows path); this native
	// symbol only satisfies the default function pointer and is never dispatched.
	(void)logger; (void)preferredDir; (void)shmFloorBytes;
	return false;
#else
	return FileMapping_ResolveBackingUnix(logger, preferredDir, shmFloorBytes, outEnv, outEnvCap);
#endif
}

FileMappingHandle FileMapping_Create(Logger& logger, u32 flProtect, u64 maxSize, const fmchar* name, const fmchar* hint)
{
#if PLATFORM_WINDOWS
	ExtendedTimerScope ts(KernelStats::GetCurrent().createFileMapping);
	return u64(InternalCreateFileMappingW(logger, INVALID_HANDLE_VALUE, flProtect, (DWORD)ToHigh(maxSize), ToLow(maxSize), name, hint));
#else
	UBA_ASSERT(!name);
	UBA_ASSERT((flProtect & (~u32(PAGE_READWRITE | SEC_RESERVE))) == 0);

#if PLATFORM_LINUX
	if (FmUseMemfd())
	{
	alignas(16) static const char s_memfdName[] = "uba"; // memfd_create requires a non-null name; not user-visible.
	auto rc = RawSyscallDirect(UBA_NR_MEMFD_CREATE, (long)s_memfdName, UBA_MFD_CLOEXEC, 0);
	int shmFd = (int)rc.r1;
	if (shmFd < 0)
	{
		SetLastError(-shmFd);
		UBA_ASSERTF(false, "memfd_create failed for %s (errno %d)", hint, -shmFd);
		return {};
	}
	if (maxSize != 0)
	{
		auto rt = RawSyscallDirect(UBA_NR_FTRUNCATE, shmFd, (long)maxSize, 0);
		if (rt.r1 < 0)
		{
			SetLastError(-(int)rt.r1);
			RawSyscallDirect(UBA_NR_CLOSE, shmFd, 0, 0);
			return {};
		}
	}
	SetLastError(0);
	int myPid = (int)RawSyscallDirect(UBA_NR_GETPID, 0, 0, 0).r1;
	FileMappingHandle h;
	h.internal = FileMappingMemfdFlag | (u64(u32(myPid)) << 32) | u64(u32(shmFd)); // tagged (creatorPid, creatorFd) — recipient does pidfd_open+pidfd_getfd
	h.shmFd = shmFd;
	h.lockFd = -1;
	return h;
	}
#endif // PLATFORM_LINUX
	pthread_mutex_lock(&g_mappingUidCounterLock);
	if (!g_mappingUidCounter)
		if (!GarbageCollectFileMappings(logger))
		{
			pthread_mutex_unlock(&g_mappingUidCounterLock);
			return {};
		}
	pthread_mutex_unlock(&g_mappingUidCounterLock);

	// Let's find a free shm name
	u64 uid;
	int shmFd;
	int lockFd;
	char lockFilePath[256];
	char uidName[64];

	int retryCount = 4;
	while (true)
	{
		uid = ++g_mappingUidCounter;

		AppendPrefixAndValue(lockFilePath, GetLockDir(), uid, true);

		mode_t lockFileMode = g_lockDirShared ? 0666 : (S_IRUSR | S_IWUSR); // 0666 in shared mode so any user can flock/remove
		lockFd = open(lockFilePath, O_CREAT | O_RDWR | O_NOFOLLOW | O_EXCL | O_CLOEXEC, lockFileMode);
		if (lockFd == -1)
		{
			if (errno == EEXIST)
				continue;
			logger.Warning(TC("Failed to open/create %s (%s)"), lockFilePath, strerror(errno));
			UBA_ASSERTF(false, "Failed to open/create %s (%s)", lockFilePath, strerror(errno));
			continue;
		}

		if (flock(lockFd, LOCK_EX | LOCK_NB) != 0) // Some other process is using this one
		{
			close(lockFd);
			continue;
		}

		GetMappingHandleName(uidName, uid);

		// Linux shm_open is implemented via open("/dev/shm/<name>", …) so O_NOFOLLOW
		// is meaningful (and a useful guard). Darwin's shm_open is a kernel-namespace
		// call that validates oflag against {O_RDONLY,O_WRONLY,O_RDWR,O_CREAT,O_EXCL,
		// O_TRUNC} and returns EINVAL for any other bit, so O_NOFOLLOW must be omitted.
		#if PLATFORM_MAC
		int oflags = O_CREAT | O_RDWR | O_EXCL;
		#else
		int oflags = O_CREAT | O_RDWR | O_NOFOLLOW | O_EXCL;
		#endif
		shmFd = FmHostShmOpen(uidName, oflags, S_IRUSR | S_IWUSR);
		if (shmFd != -1)
			break;

		int err = errno;
		if (err == EEXIST)
		{
			// We hold the exclusive flock, so no other process owns this uid.
			// The shm is an orphan from a previous run that lost its lock file
			// (e.g. ftruncate failure). Unlink it and retry with a fresh shm.
			FmHostShmUnlink(uidName);
			shmFd = FmHostShmOpen(uidName, oflags, S_IRUSR | S_IWUSR);
			if (shmFd != -1)
				break;
			err = errno;
		}

		bool retry = retryCount-- > 0;

		auto logType = retry ? LogEntryType_Warning : LogEntryType_Error;
		logger.Logf(logType, TC("Failed to create shm %s after getting lock-file %s (%s)"), uidName, lockFilePath, strerror(err));

		remove(lockFilePath);
		close(lockFd);

		if (retry)
			continue;

		SetLastError(err);
		return {};
	}

	if (maxSize != 0)
	{
		if (ftruncate(shmFd, (s64)maxSize) == -1)
		{
			SetLastError(errno);
			close(shmFd);
			FmHostShmUnlink(uidName);
			remove(lockFilePath);
			close(lockFd);
			//UBA_ASSERTF(false, "Failed to truncate file mapping '%s' to size %llu (%s)", name, maxSize, strerror(errno));
			return {};
		}
	}
	SetLastError(0);
	FileMappingHandle h(uid);
	h.shmFd = shmFd;
	h.lockFd = lockFd;
	return h;
#endif // !PLATFORM_WINDOWS
}

FileMappingHandle FileMapping_CreateFromFile(Logger& logger, FileHandle file, u32 protect, u64 maxSize, const fmchar* hint)
{
#if PLATFORM_WINDOWS
	ExtendedTimerScope ts(KernelStats::GetCurrent().createFileMapping);
	return u64(InternalCreateFileMappingW(logger, AsHANDLE(file), protect, (DWORD)ToHigh(maxSize), ToLow(maxSize), NULL, hint));
#else
	FileMappingHandle h;
	int fd = AsFileDescriptor(file);
	if (maxSize && (protect & (~PAGE_READONLY)) != 0)
	{
#if PLATFORM_MAC // For some reason lseek+write does not work on apple silicon platform
		if (ftruncate(fd, maxSize) == -1)
		{
			logger.Error("ftruncate to %llu on fd %i failed for %s: %s\n", maxSize, fd, hint, strerror(errno));
			return h;
		}
#else
		if (lseek(fd, maxSize - 1, SEEK_SET) != maxSize - 1)
		{
			logger.Error("lseek to %llu failed for %s: %s\n", maxSize - 1, hint, strerror(errno));
			return h;
		}

		errno = 0;
		int res = write(fd, "", 1);
		if (res != 1)
		{
			logger.Error("write one byte at %llu on fd %i (%s) failed (res: %i): %s\n", maxSize - 1, fd, hint, res, strerror(errno));
			return h;
		}
#endif
	}

	h.shmFd = fd;
	h.internal = ~0ull;
	return h;
#endif
}

u8* FileMapping_Map(Logger& logger, FileMappingHandle fileMappingObject, u32 desiredAccess, u64 offset, u64 bytesToMap, bool allowDiscard)
{
#if PLATFORM_WINDOWS
	ExtendedTimerScope ts(KernelStats::GetCurrent().mapViewOfFile);
	u8* res = (u8*)::MapViewOfFile(AsHANDLE(fileMappingObject), desiredAccess, (DWORD)ToHigh(offset), ToLow(offset), bytesToMap);
	if (!res)
		return nullptr;
	if (allowDiscard)
		InternalDiscard(logger, res, bytesToMap, TC("MapViewOfFile"), TC(""));
#elif UBA_USE_MEMFD
	int prot = 0;
	if (desiredAccess & FILE_MAP_READ)
		prot |= UBA_PROT_READ;
	if (desiredAccess & FILE_MAP_WRITE)
		prot |= UBA_PROT_WRITE;
	UBA_ASSERT(fileMappingObject.IsValid());
	int shmFd = fileMappingObject.shmFd;
	auto rm = RawSyscallDirect(UBA_NR_MMAP, 0, (long)bytesToMap, prot, UBA_MAP_SHARED, shmFd, (long)offset);
	if (UbaSyscallIsErr(rm.r1))
	{
		SetLastError(-(int)rm.r1);
		return nullptr;
	}
	u8* res = (u8*)rm.r1;
#else
	int prot = 0;
	if (desiredAccess & FILE_MAP_READ)
		prot |= PROT_READ;
	if (desiredAccess & FILE_MAP_WRITE)
		prot |= PROT_WRITE;
	UBA_ASSERT(fileMappingObject.IsValid());
	int shmFd = fileMappingObject.shmFd;
	void* rptr = mmap(NULL, bytesToMap, prot, MAP_SHARED, shmFd, s64(offset));
	if (rptr == MAP_FAILED)
	{
		SetLastError(errno);
		return nullptr;
	}
	u8* res = (u8*)rptr;
	//UBA_ASSERTF(false, "Failed to map file with fd %i, desiredAccess %u offset %llu, bytesToMap %llu (%s)", fd, desiredAccess, offset, bytesToMap, strerror(errno));
#endif

#if UBA_DEBUG_FILE_MAPPING
	{
		SCOPED_FUTEX(g_fileMappingsLock, lock);
		auto findIt = g_fileMappings.find(fileMappingObject.mh);
		UBA_ASSERTF(findIt != g_fileMappings.end(), TC("Mapping nonexisting handle: %llu"), u64(fileMappingObject.mh));
		auto insres = g_viewMappings.try_emplace(res, fileMappingObject.mh);
		UBA_ASSERT(insres.second);
		++findIt->second.viewCount;
	}
#endif

	return res;
}

bool FileMapping_Commit(Logger& logger, void* address, u64 size, bool allowDiscard)
{
#if PLATFORM_WINDOWS
	ExtendedTimerScope ts(KernelStats::GetCurrent().virtualAlloc);
	if (!::VirtualAlloc(address, size, MEM_COMMIT, PAGE_READWRITE))
		return false;
	if (allowDiscard)
		InternalDiscard(logger, (u8*)address, size, TC("MapViewCommit"), TC(""));
	return true;
#else
	return true; // no real commit on Linux — pages fault in lazily
#endif
}

bool FileMapping_Unmap(Logger& logger, const void* address, u64 bytesToUnmap, const fmchar* hint, bool allowDiscard)
{
	if (!address)
		return true;

#if UBA_DEBUG_FILE_MAPPING
	{
		SCOPED_FUTEX(g_fileMappingsLock, lock);
		auto findIt = g_viewMappings.find(address);
		UBA_ASSERT(findIt != g_viewMappings.end());
		auto findIt2 = g_fileMappings.find(findIt->second);
		g_viewMappings.erase(findIt);
		UBA_ASSERTF(findIt2 != g_fileMappings.end(), TC("Unmap nonexisting handle: %llu"), u64(findIt->second));
		--findIt2->second.viewCount;
	}
#endif

#if PLATFORM_WINDOWS
	ExtendedTimerScope ts(KernelStats::GetCurrent().unmapViewOfFile);
	if (allowDiscard)
		InternalDiscard(logger, (u8*)address, bytesToUnmap, TC("UnmapViewOfFile"), hint);
	return ::UnmapViewOfFile(address);
#elif UBA_USE_MEMFD
	UBA_ASSERTF(bytesToUnmap, TC("bytesToUnmap is zero unmapping %p (%s)"), address, hint);
	auto ru = RawSyscallDirect(UBA_NR_MUNMAP, (long)address, (long)bytesToUnmap, 0);
	if (ru.r1 == 0)
		return true;
	UBA_ASSERT(false);
	return false;
#else
	UBA_ASSERTF(bytesToUnmap, TC("bytesToUnmap is zero unmapping %p (%s)"), address, hint);
	if (munmap((void*)address, bytesToUnmap) == 0)
		return true;
	UBA_ASSERT(false);
	return false;
#endif
}

bool FileMapping_Close(Logger& logger, FileMappingHandle h, const fmchar* hint)
{
	if (!h.IsValid())
		return true;

#if UBA_DEBUG_FILE_MAPPING
	SCOPED_FUTEX(g_fileMappingsLock, lock);
	auto findIt = g_fileMappings.find(h.mh);
	UBA_ASSERTF(findIt != g_fileMappings.end(), TC("Handle: %llu"), u64(h.mh));
	UBA_ASSERTF(!findIt->second.viewCount.load(), TC("Closing file mapping with %llu open views (%s)"), findIt->second.viewCount.load(), findIt->second.hint.c_str());
	logger.Info(TC("FILEMAPPING CLOSE: %s %llu (%llu)"), findIt->second.hint.c_str(), u64(h.mh), g_fileMappings.size() - 1);
	g_fileMappings.erase(findIt);
#endif

#if PLATFORM_WINDOWS
	return ::CloseHandle(AsHANDLE(h));
#else
	#if PLATFORM_LINUX
	if (IsMemfdHandle(h))
	{
		// memfd: kernel reclaims when last fd ref drops across all processes. No name to unlink.
		// internal == ~0ull marks a FileMapping_CreateFromFile handle whose fd is caller-owned.
		if (h.shmFd != -1 && h.internal != ~0ull)
		{
			auto rc = RawSyscallDirect(UBA_NR_CLOSE, h.shmFd, 0, 0);
			if (rc.r1 < 0)
				UBA_ASSERTF(false, "Failed to close memfd (errno %ld)", -rc.r1);
		}
		return true;
	}
	#endif
	if (h.shmFd != -1 && h.internal != ~0ull)
		if (close(h.shmFd) != 0)
			UBA_ASSERTF(false, "Failed to close file mapping (%s)", strerror(errno));

	if (h.internal != 0 && h.internal != ~0ull)
	{
		char uidName[128];
		GetMappingHandleName(uidName, h.internal);
		if (FmHostShmUnlink(uidName) != 0)
		{
			SetLastError(errno);
			UBA_ASSERTF(false, "Failed to unlink %s (%s)", uidName, strerror(errno));
			return false;
		}

		char lockFile[256];
		AppendPrefixAndValue(lockFile, GetLockDir(), h.internal, true);
		remove(lockFile);
	}
	if (h.lockFd != -1)
		close(h.lockFd);
	return true;
#endif
}

#endif // !UBA_STUB_BUILD — host-side Logger/FileHandle-using functions end here.

void* FileMapping_ReservePlaceholder(void* baseAddress, u64 capacity)
{
#if PLATFORM_WINDOWS
	void* memory = VirtualAlloc2(GetCurrentProcess(), baseAddress, capacity, MEM_RESERVE | MEM_RESERVE_PLACEHOLDER, PAGE_NOACCESS, nullptr, 0);
#else
#if PLATFORM_LINUX
	int flags = UBA_MAP_PRIVATE | UBA_MAP_ANONYMOUS;
	if (baseAddress)
		flags |= UBA_MAP_FIXED;
	void* memory = FmMmap(baseAddress, capacity, UBA_PROT_NONE, flags, -1, 0);
#else
	int flags = MAP_PRIVATE | MAP_ANON;
	if (baseAddress)
		flags |= MAP_FIXED;
	void* memory = FmMmap(baseAddress, capacity, PROT_NONE, flags, -1, 0);
#endif
	if (memory == (void*)-1L)
		memory = nullptr;
#endif
	UBA_ASSERTF(memory, TC("VirtualAlloc2 failed (%s)"), LastErrorToText().data);
	return memory;
}

void* FileMapping_MapPlaceholder(FileMappingHandle handle, MemoryMapType mapType, void* targetAddress, u64 targetOffset, u64 targetCapacity, u64 handleOffset, u64 size)
{
#if PLATFORM_WINDOWS
	u32 protect = PAGE_READWRITE;
	if (mapType == MemoryMapType_ReadOnly)
		protect = PAGE_READONLY;
	else if (mapType == MemoryMapType_CopyOnWrite)
		protect = PAGE_WRITECOPY;
	else if (mapType == MemoryMapType_LazyCopyOnWrite)
		protect = PAGE_READONLY; // Caller flips pages to PAGE_WRITECOPY on first write fault to avoid eager full-view commit

	size = AlignUp(size, 4*1024); // This is needed for VirtualFree to align properly

	if (u64 toRelease = targetCapacity - targetOffset - size)
	{
		//TimerScope ts(kernelStats.virtualFree);
		bool res = VirtualFree((u8*)targetAddress + targetOffset + size, toRelease, MEM_RELEASE | MEM_PRESERVE_PLACEHOLDER);
		//UBA_ASSERTF(res, TC("VirtualFree for placeholder failed releasing %llu at %llu (Capacity: %llu MappedSize: %llu) - %s (%s)"), toRelease, size, m_capacity, m_mappedSize, LastErrorToText().data, hint);
		if (!res)
			return nullptr;//false;
	}

	//kernelStats.mapViewOfFile3.bytes += size;
	//TimerScope ts(kernelStats.mapViewOfFile3);
	void* viewA = MapViewOfFile3(AsHANDLE(handle), GetCurrentProcess(), (u8*)targetAddress+targetOffset, handleOffset, size, MEM_REPLACE_PLACEHOLDER, protect, nullptr, 0);
#else
#if PLATFORM_LINUX
	u32 protect = UBA_PROT_READ;
	u32 mapFlags = UBA_MAP_FIXED;
	if (mapType == MemoryMapType_CopyOnWrite || mapType == MemoryMapType_LazyCopyOnWrite) // POSIX MAP_PRIVATE already commits lazily on write fault
	{
		mapFlags |= UBA_MAP_PRIVATE;
		protect |= UBA_PROT_WRITE;
	}
	else
	{
		mapFlags |= UBA_MAP_SHARED;
		if (mapType == MemoryMapType_ReadWrite)
			protect |= UBA_PROT_WRITE;
	}
#else
	u32 protect = PROT_READ;
	u32 mapFlags = MAP_FIXED;
	if (mapType == MemoryMapType_CopyOnWrite || mapType == MemoryMapType_LazyCopyOnWrite) // POSIX MAP_PRIVATE already commits lazily on write fault
	{
		mapFlags |= MAP_PRIVATE;
		protect |= PROT_WRITE;
	}
	else
	{
		mapFlags |= MAP_SHARED;
		if (mapType == MemoryMapType_ReadWrite)
			protect |= PROT_WRITE;
	}
#endif
	void* viewA = FmMmap((u8*)targetAddress+targetOffset, size, protect, mapFlags, handle.shmFd, handleOffset);
	if (viewA == (void*)-1L)
		viewA = nullptr;
#endif
	return viewA;
}

bool FileMapping_UnmapPlaceholder(void* memory, u64 capacity, u64 mappedSize, u64* subMappings, u64 subMappingsCount)
{
#if PLATFORM_WINDOWS
	if (mappedSize)
	{
		auto& kernelStats = KernelStats::GetCurrent();
		{
			TimerScope ts(kernelStats.unmapViewOfFile2);
			if (!UnmapViewOfFile2(GetCurrentProcess(), memory, MEM_PRESERVE_PLACEHOLDER))
				UBA_ASSERTF(false, TC("UnmapViewOfFile2 failed (%u)"), GetLastError());
		}

		for (u64 i=0;i!=subMappingsCount; ++i)
		{
			TimerScope ts(kernelStats.unmapViewOfFile2);
			if (!UnmapViewOfFile2(GetCurrentProcess(), (u8*)memory + subMappings[i], MEM_PRESERVE_PLACEHOLDER))
				UBA_ASSERTF(false, TC("UnmapViewOfFile2 failed (%u)"), GetLastError());
		}

		if (mappedSize < capacity || subMappingsCount)
		{
			TimerScope ts(kernelStats.virtualFree);
			if (!VirtualFree(memory, capacity, MEM_RELEASE | MEM_COALESCE_PLACEHOLDERS))
				UBA_ASSERTF(false, TC("VirtualFree failed (%u)"), GetLastError());
		}
	}
	if (!VirtualFree(memory, 0, MEM_RELEASE))
		UBA_ASSERTF(false, TC("VirtualFree failed (%u)"), GetLastError());
#else
	int res = FmMunmap(memory, capacity);
	UBA_ASSERT(res == 0); (void)res;
#endif
	return true;
}

FileMappingHandle FileMapping_DuplicateFromHost(FileMappingHandle handle, const char* hint)
{
#if PLATFORM_WINDOWS
	HANDLE realHandle;
	if (!DuplicateHandle(g_hostProcess, AsHANDLE(handle), GetCurrentProcess(), &realHandle, 0, FALSE, DUPLICATE_SAME_ACCESS))
	{
		UBA_ASSERTF(false, L"Failed to duplicate %S handle (%u)", hint, GetLastError());
		return {};
	}
	return FileMappingHandle(u64(realHandle));
#else
	#if PLATFORM_LINUX
	if (IsMemfdHandle(handle)) // per-handle, not FmUseMemfd(): creator's mode travels with the handle
	{
	u32 creatorPid = u32((handle.internal & ~FileMappingMemfdFlag) >> 32);
	u32 creatorFd  = u32(handle.internal & 0xffffffffu);
	auto rp = RawSyscallDirect(UBA_NR_PIDFD_OPEN, (long)creatorPid, 0, 0);
	int pidfd = (int)rp.r1;
	if (pidfd < 0)
	{
		SetLastError(-pidfd);
		UBA_ASSERTF(false, "pidfd_open(%u) failed for %s (errno %d)", creatorPid, hint, -pidfd);
		return {};
	}
	auto rg = RawSyscallDirect(UBA_NR_PIDFD_GETFD, pidfd, (long)creatorFd, 0);
	int shmFd = (int)rg.r1;
	RawSyscallDirect(UBA_NR_CLOSE, pidfd, 0, 0);
	if (shmFd < 0)
	{
		SetLastError(-shmFd);
		UBA_ASSERTF(false, "pidfd_getfd(pid=%u fd=%u) failed for %s (errno %d)", creatorPid, creatorFd, hint, -shmFd);
		return {};
	}
	FileMappingHandle newHandle = handle;
	newHandle.shmFd = shmFd;
	return newHandle;
	}
	#endif // PLATFORM_LINUX
	char uidName[128];
	GetMappingHandleName(uidName, handle.internal);
#if PLATFORM_LINUX
	int shmFd = FmShmOpen(uidName, UBA_O_RDWR, 0600);
#else
	int shmFd = FmShmOpen(uidName, O_RDWR, S_IRUSR | S_IWUSR);
#endif
	UBA_ASSERTF(shmFd != -1, "shm_open failed for %s and handle %s", hint, uidName);
	if (shmFd == -1)
		return {};
#if PLATFORM_LINUX
	FmFcntl(shmFd, UBA_F_SETFD, UBA_FD_CLOEXEC);
#else
	FmFcntl(shmFd, F_SETFD, FD_CLOEXEC);
#endif
	FileMappingHandle newHandle = handle;
	newHandle.shmFd = shmFd;
	return newHandle;
#endif
}

u8* FileMapping_MapFromHost(FileMappingHandle handle, u64 size, u64 offset, bool writable, const char* hint)
{
	MemoryMapType mapType = writable ? MemoryMapType_ReadWrite : MemoryMapType_ReadOnly;
	return FileMapping_MapFromHost2(handle, size, offset, mapType, nullptr, hint);
}

u8* FileMapping_MapFromHost2(FileMappingHandle handle, u64 size, u64 offset, MemoryMapType mapType, void* targetAddress, const char* hint)
{
#if PLATFORM_WINDOWS
	DWORD dwDesiredAccess = FILE_MAP_READ;
	if (mapType == MemoryMapType_ReadWrite)
		dwDesiredAccess |= FILE_MAP_WRITE;
	else if (mapType == MemoryMapType_CopyOnWrite)
		dwDesiredAccess = FILE_MAP_COPY;
	LARGE_INTEGER li;
	li.QuadPart = (LONGLONG)offset;
	//TimerScope ts(g_kernelStats.mapViewOfFile);
	u8* mem = (u8*)::MapViewOfFileEx(AsHANDLE(handle), dwDesiredAccess, li.HighPart, li.LowPart, size, targetAddress);
	::CloseHandle(AsHANDLE(handle));
	return mem;
#else
	#if PLATFORM_LINUX
	if (IsMemfdHandle(handle)) // per-handle, not FmUseMemfd(): creator's mode travels with the handle
	{
	u32 creatorPid = u32((handle.internal & ~FileMappingMemfdFlag) >> 32);
	u32 creatorFd  = u32(handle.internal & 0xffffffffu);
	auto rp = RawSyscallDirect(UBA_NR_PIDFD_OPEN, (long)creatorPid, 0, 0);
	int pidfd = (int)rp.r1;
	if (pidfd < 0)
	{
		SetLastError(-pidfd);
		UBA_ASSERTF(false, "pidfd_open(%u) failed for %s (errno %d)", creatorPid, hint, -pidfd);
		return nullptr;
	}
	auto rg = RawSyscallDirect(UBA_NR_PIDFD_GETFD, pidfd, (long)creatorFd, 0);
	int shmFd = (int)rg.r1;
	RawSyscallDirect(UBA_NR_CLOSE, pidfd, 0, 0);
	if (shmFd < 0)
	{
		SetLastError(-shmFd);
		UBA_ASSERTF(false, "pidfd_getfd(pid=%u fd=%u) failed for %s (errno %d)", creatorPid, creatorFd, hint, -shmFd);
		return nullptr;
	}
	int prot = UBA_PROT_READ;
	if (mapType == MemoryMapType_ReadWrite || mapType == MemoryMapType_CopyOnWrite)
		prot |= UBA_PROT_WRITE;
	int mmapFlags = (mapType == MemoryMapType_CopyOnWrite) ? UBA_MAP_PRIVATE : UBA_MAP_SHARED;
	if (targetAddress)
		mmapFlags |= UBA_MAP_FIXED_NOREPLACE;
	auto rm = RawSyscallDirect(UBA_NR_MMAP, (long)targetAddress, (long)size, prot, mmapFlags, shmFd, (long)offset);
	RawSyscallDirect(UBA_NR_CLOSE, shmFd, 0, 0);
	if (!UbaSyscallIsErr(rm.r1))
		return (u8*)rm.r1;
	SetLastError(-(int)rm.r1);
	if (targetAddress)
		return nullptr;
	UBA_ASSERTF(false, "mmap failed for %s (errno %ld)", hint, -rm.r1);
	return nullptr;
	}
	#endif // PLATFORM_LINUX
	char uidName[128];
	GetMappingHandleName(uidName, handle.internal);
#if PLATFORM_LINUX
	int openFlags = (mapType == MemoryMapType_ReadWrite) ? UBA_O_RDWR : UBA_O_RDONLY;
	int shmFd = FmShmOpen(uidName, openFlags, 0600);
	UBA_ASSERTF(shmFd != -1, "shm_open failed for %s and handle %s", hint, uidName);
	if (shmFd == -1)
		return nullptr; // GetLastError() = openat errno (ENOENT typically means creator used a different backing/dir)
	int prot = UBA_PROT_READ;
	if (mapType == MemoryMapType_ReadWrite || mapType == MemoryMapType_CopyOnWrite)
		prot |= UBA_PROT_WRITE;
	int mmapFlags = (mapType == MemoryMapType_CopyOnWrite) ? UBA_MAP_PRIVATE : UBA_MAP_SHARED;
	if (targetAddress)
		mmapFlags |= UBA_MAP_FIXED_NOREPLACE;
#else
	int openFlags = (mapType == MemoryMapType_ReadWrite) ? O_RDWR : O_RDONLY;
	int shmFd = FmShmOpen(uidName, openFlags, S_IRUSR | S_IWUSR);
	UBA_ASSERTF(shmFd != -1, "shm_open failed for %s and handle %s", hint, uidName);
	if (shmFd == -1)
		return nullptr;
	int prot = PROT_READ;
	if (mapType == MemoryMapType_ReadWrite || mapType == MemoryMapType_CopyOnWrite)
		prot |= PROT_WRITE;
	int mmapFlags = (mapType == MemoryMapType_CopyOnWrite) ? MAP_PRIVATE : MAP_SHARED;
	if (targetAddress)
		mmapFlags |= MAP_FIXED;
#endif
	void* mem = FmMmap(targetAddress, size, prot, mmapFlags, shmFd, offset);
	if (mem != (void*)-1L)
	{
		FmClose(shmFd);
		return (u8*)mem;
	}
	if (targetAddress)
	{
		FmClose(shmFd);
		return nullptr;
	}
	[[maybe_unused]] int mmapErr = (int)GetLastError(); // capture before FmClose overwrites
	FmClose(shmFd);
	UBA_ASSERTF(false, "mmap failed for %s (errno=%d size=%llu offset=%llu prot=0x%x flags=0x%x targetAddress=%p shmFd=%d uidName=%s)", hint, mmapErr, (u64)size, (u64)offset, prot, mmapFlags, targetAddress, shmFd, uidName);
	return nullptr;
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////
