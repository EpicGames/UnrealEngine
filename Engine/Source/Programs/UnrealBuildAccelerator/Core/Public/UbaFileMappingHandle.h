// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBase.h"
#if PLATFORM_WINDOWS
#include "UbaPlatform.h" // HANDLE / uintptr_t for AsHANDLE
#endif

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct FileMappingHandle
	{
		FileMappingHandle() = default;
		bool operator==(const FileMappingHandle& o) const { return internal == o.internal; }
		bool IsValid() const { return internal != 0; }
		u64 ToU64() const { return internal; }
		static FileMappingHandle FromU64(u64 v) { return { v }; }

		FileMappingHandle(u64 v) { internal = v; }
		u64 internal = 0; // HANDLE on windows, uid or tagged (pid,fd) on posix

		int shmFd = -1; // Unused on windows
		int lockFd = -1; // Unused on windows
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	#if PLATFORM_WINDOWS
	inline HANDLE AsHANDLE(FileMappingHandle h) { return (HANDLE)h.internal; }
	#else
	// Linux memfd handles are (creatorPid<<32)|creatorFd plus this tag, so attach-side code can pick
	// the pidfd path per handle instead of trusting process-wide env (which can disagree with the
	// creator's mode, e.g. stale UBA_FILE_MAPPING_MEMFD in a container image).
	// Bit 63 is taken: SharedMemoryAllocator's external-mapping flag is added on top of ToU64().
	inline constexpr u64 FileMappingMemfdFlag = 1ull << 62;
	inline bool IsMemfdHandle(FileMappingHandle h) { return h.internal != ~0ull && (h.internal & FileMappingMemfdFlag) != 0; } // ~0ull = caller-owned-fd sentinel
	#endif

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
