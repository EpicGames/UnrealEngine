// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	HANDLE g_hostProcess;
	#include "UbaFileMapping.inl"

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool FileMapping_GetWineBackend(FileMappingBackend& out, Logger& logger, HMODULE wineDll)
	{
		#define UBA_FM_FUNC(func) \
			auto func##Func = (decltype(FileMapping_##func)*)(void*)GetProcAddress(wineDll, "FileMapping_" #func); \
			if (!func##Func) return logger.Warning(TC("FileMapping_" #func " is not exported from wine dll"));
		UBA_FM_FUNCTIONS
		#undef UBA_FM_FUNC

		#define UBA_FM_FUNC(func) out.func = func##Func;
		UBA_FM_FUNCTIONS
		#undef UBA_FM_FUNC
		out.linuxBackend = true;
		return true;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
