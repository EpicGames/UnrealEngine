// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBase.h"
#include <utility>

namespace uba
{
	// Minimal std::function replacement. Type-erased callable, heap-allocated
	// impl (no SBO yet). Copyable (target must be copyable) and movable.
	template<typename Sig> class Function;

	template<typename R, typename... Args>
	class Function<R (Args...)>
	{
		struct CallableBase
		{
			virtual ~CallableBase() = default;
			virtual R Invoke(Args... args) = 0;
			virtual CallableBase* Clone() const = 0;
		};

		template<typename F>
		struct Callable : CallableBase
		{
			F fn;
			template<typename G>
			explicit Callable(G&& g) : fn(std::forward<G>(g)) {}
			R Invoke(Args... args) override
			{
				if constexpr (std::is_void_v<R>)
					fn(static_cast<Args&&>(args)...);
				else
					return fn(static_cast<Args&&>(args)...);
			}
			CallableBase* Clone() const override { return new Callable(*this); }
		};

	public:
		Function() = default;
		Function(decltype(nullptr)) {}

		// Restrict to actually-callable targets so overload resolution doesn't
		// accept non-invocable types (which would create spurious conversion
		// paths and make otherwise-unambiguous overloads ambiguous).
		template<typename F,
		         typename = std::enable_if_t<
		             !std::is_same_v<std::decay_t<F>, Function> &&
		             std::is_invocable_r_v<R, std::decay_t<F>&, Args...>>>
		Function(F&& f) : m_impl(new Callable<std::decay_t<F>>(std::forward<F>(f))) {}

		Function(const Function& o) : m_impl(o.m_impl ? o.m_impl->Clone() : nullptr) {}

		Function(Function&& o) noexcept : m_impl(o.m_impl) { o.m_impl = nullptr; }

		~Function() { delete m_impl; }

		Function& operator=(const Function& o)
		{
			if (this != &o)
			{
				delete m_impl;
				m_impl = o.m_impl ? o.m_impl->Clone() : nullptr;
			}
			return *this;
		}

		Function& operator=(Function&& o) noexcept
		{
			if (this != &o)
			{
				delete m_impl;
				m_impl = o.m_impl;
				o.m_impl = nullptr;
			}
			return *this;
		}

		Function& operator=(decltype(nullptr))
		{
			delete m_impl;
			m_impl = nullptr;
			return *this;
		}

		template<typename F,
		         typename = std::enable_if_t<
		             !std::is_same_v<std::decay_t<F>, Function> &&
		             std::is_invocable_r_v<R, std::decay_t<F>&, Args...>>>
		Function& operator=(F&& f)
		{
			delete m_impl;
			m_impl = new Callable<std::decay_t<F>>(std::forward<F>(f));
			return *this;
		}

		explicit operator bool() const { return m_impl != nullptr; }

		R operator()(Args... args) const
		{
			return m_impl->Invoke(static_cast<Args&&>(args)...);
		}

	private:
		CallableBase* m_impl = nullptr;
	};


	// Preserved from the original UbaFunctional.h — lightweight non-owning
	// callable with explicit context pointer. No allocation.
	template <typename FunctionType>
	class FunctionWithContext;

	template <typename ReturnType, typename... ArgTypes>
	class FunctionWithContext<ReturnType (ArgTypes...)>
	{
	public:
		using FunctionType = ReturnType (void*, ArgTypes...);

		template <typename CallableType>
		inline FunctionWithContext(CallableType&& callable UBA_LIFETIMEBOUND)
			requires (!std::is_same_v<std::decay_t<CallableType>, FunctionWithContext>) &&
		std::is_invocable_r_v<ReturnType, std::decay_t<CallableType>, ArgTypes...>
			: function(&Call<std::decay_t<CallableType>>)
			, context(&callable)
		{
		}

		template <typename CallableType>
		inline FunctionWithContext& operator=(CallableType&& callable UBA_LIFETIMEBOUND)
			requires (!std::is_same_v<std::decay_t<CallableType>, FunctionWithContext>) &&
		std::is_invocable_r_v<ReturnType, std::decay_t<CallableType>, ArgTypes...>
		{
			function = &Call<std::decay_t<CallableType>>;
			context = &callable;
			return *this;
		}

		inline explicit FunctionWithContext(FunctionType* inFunction, void* inContext)
			: function(inFunction)
			, context(inContext)
		{
		}

		inline constexpr FunctionWithContext(decltype(nullptr))
		{
		}

		inline explicit operator bool() const
		{
			return !!function;
		}

		inline ReturnType operator()(ArgTypes... args) const
		{
			return function(context, (ArgTypes&&)args...);
		}

		inline FunctionType* GetFunction() const
		{
			return function;
		}

		inline void* GetContext() const
		{
			return context;
		}

	private:
		template <typename CallableType>
		static ReturnType Call(void* callable, ArgTypes... args)
		{
			if constexpr (std::is_void_v<ReturnType>)
				(*(CallableType*)callable)((ArgTypes&&)args...);
			else
				return (*(CallableType*)callable)((ArgTypes&&)args...);
		}

		FunctionType* function = nullptr;
		void* context = nullptr;
	};
}
