// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaStringBuffer.h"
#include "UbaPlatform.h"
#if !UBA_STUB_BUILD
#include <algorithm>
#endif

namespace uba
{

	struct StringKey
	{
		constexpr StringKey(u64 a_ = 0, u64 b_ = 0) : a(a_), b(b_) {}
		u64 a;
		u64 b;
		bool operator==(const StringKey& o) const { return a == o.a && b == o.b; }
		bool operator!=(const StringKey& o) const { return a != o.a || b != o.b; }
		bool operator<(const StringKey& o) const { if (a != o.a) return a < o.a; return b < o.b; }
	};
	constexpr StringKey StringKeyZero;

#pragma pack(push)
#pragma pack(4)
	struct CasKey
	{
		constexpr CasKey() : a(0), b(0), c(0) {}
		constexpr CasKey(u64 a_, u64 b_, u32 c_) : a(a_), b(b_), c(c_) {}
		u64 a;
		u64 b;
		u32 c;
		bool operator==(const CasKey& o) const { return a == o.a && b == o.b && c == o.c; }
		bool operator!=(const CasKey& o) const { return a != o.a || b != o.b || c != o.c; }
		bool operator<(const CasKey& o) const { if (a != o.a) return a < o.a; if (b != o.b) return b < o.b; return c < o.c; }
	};
#pragma pack(pop)

	constexpr CasKey CasKeyZero;
	constexpr CasKey CasKeyInvalid(~0ull, ~0ull, ~0u);

	// Use 36+1 characters
	inline void GuidToStr(tchar* out, u32 capacity, const Guid& g)
	{
		TSprintf_s(out, capacity, TC("%08x-%04hx-%04hx-%02hhx%02hhx-%02hhx%02hhx%02hhx%02hhx%02hhx%02hhx"), g.data1, g.data2, g.data3, g.data4[0], g.data4[1], g.data4[2], g.data4[3], g.data4[4], g.data4[5], g.data4[6], g.data4[7]);
	}

	struct GuidToString : StringView
	{
		GuidToString(const Guid& g) : StringView(str, 36) { GuidToStr(str, 37, g); }
		tchar str[37];
	};

	struct KeyToString : StringBuffer<33>
	{
		KeyToString(const StringKey& key) { AppendHex(key.a); AppendHex(key.b); }
	};

	// REMOVE!!
	struct HashString
	{
		size_t operator()(const tchar* s) const
		{
			size_t h = 5381;
			while (tchar c = *s++)
				h = ((h << 5) + h) + size_t(c);
			return h;
		}
	};

	struct EqualString
	{
		bool operator()(const tchar* a, const tchar* b) const
		{
			return TStrcmp(a, b) == 0;
		}
	};

	struct HashStringNoCase
	{
		size_t operator()(const tchar* s) const
		{
			size_t h = 5381;
			while (tchar c = *s++)
				h = ((h << 5) + h) + size_t(ToLower(c));
			return h;
		}
	};

	struct EqualStringNoCase
	{
		bool operator()(const tchar* a, const tchar* b) const
		{
			return Equals(a, b, true);
		}
	};

	struct StringKeyHasher
	{
		StringKeyHasher();
		void Update(const StringView& str);
		void Update(const tchar* str, u64 strLen);
		void UpdateNoCheck(const StringView& str);
		u64 hasher[1912 / sizeof(u64)];
	};

	StringKey ToStringKey(const tchar* str, u64 strLen);
	StringKey ToStringKeyLower(const tchar* str, u64 strLen);
	StringKey ToStringKey(const StringView& b);
	StringKey ToStringKeyLower(const StringView& b);
	StringKey ToStringKey(const StringKeyHasher& hasher, const tchar* str, u64 strLen);
	StringKey ToStringKey(const StringKeyHasher& hasher);
	StringKey ToStringKeyNoCheck(const tchar* str, u64 strLen);
	StringKey ToStringKeyRaw(const void* data, u64 dataLen);

	constexpr inline u64 AlignUp(u64 v, u64 a) { return ((v + a - 1)/a) * a; }
	constexpr u64 InvalidValue = 0x7ffffffff; // ~34gb - the largest number that can 7bit encode to 5 bytes

	constexpr u8 EmptyMask = 0;
	constexpr u8 IsCompressedMask = 1;
	constexpr u8 IsNormalizedMask = 2;
	constexpr u8 IsProxyMask = 4;
	constexpr u8 IsExecutableMask = 8;

	inline bool IsCompressed(const CasKey& key)
	{
		UBA_ASSERT(key != CasKeyZero);
		return (((u8*)&key)[19] & IsCompressedMask) == IsCompressedMask;
	}

	inline CasKey AsCompressed(const CasKey& key, bool compressed)
	{
		UBA_ASSERT(key != CasKeyZero);
		CasKey newKey = key;
		#ifndef __clang_analyzer__
		u8 flagField = ((u8*)&key)[19];
		((u8*)&newKey)[19] = compressed ? (flagField | IsCompressedMask) : (flagField & ~IsCompressedMask);
		#endif
		return newKey;
	}

	// For posix we need to set +x on file so we track this on the cas key because it is easier in terms of downloading from cache server etc
	inline bool IsExecutable(const CasKey& key)
	{
		UBA_ASSERT(key != CasKeyZero);
		return (((u8*)&key)[19] & IsExecutableMask) == IsExecutableMask;
	}

	inline CasKey AsExecutable(const CasKey& key, bool executable)
	{
		UBA_ASSERT(key != CasKeyZero);
		CasKey newKey = key;
		#ifndef __clang_analyzer__
		u8 flagField = ((u8*)&key)[19];
		((u8*)&newKey)[19] = executable ? (flagField | IsExecutableMask) : (flagField & ~IsExecutableMask);
		#endif
		return newKey;
	}

	inline bool IsNormalized(const CasKey& key)
	{
		UBA_ASSERT(key != CasKeyZero);
		return (((u8*)&key)[19] & IsNormalizedMask) == IsNormalizedMask;
	}

	inline CasKey AsNormalized(const CasKey& key, bool normalized)
	{
		UBA_ASSERT(key != CasKeyZero);
		CasKey newKey = key;
		u8 flagField = ((u8*)&key)[19];
		((u8*)&newKey)[19] = normalized ? (flagField | IsNormalizedMask) : (flagField & ~IsNormalizedMask);
		return newKey;
	}

	struct CasKeyHasher
	{
		CasKeyHasher();
		CasKeyHasher& Update(const void* data, u64 bytes);
		u64 hasher[1912/sizeof(u64)];
	};

	CasKey ToCasKey(const CasKeyHasher& hasher, bool compressed);

	inline CasKey CasKeyFromString(const tchar* str)
	{
		UBA_ASSERT(TStrlen(str) == 40);
		CasKey key;
		u8* data = (u8*)&key;
		const tchar* pos = str;
		while (*pos)
		{
			u8 a = HexToByte(*pos++);
			u8 b = HexToByte(*pos++);
			*data++ = u8(a << 4) | b;
		}
		return key;
	}

	inline void ToString(tchar* out, int capacity, const CasKey& key)
	{
		tchar* pos = out;
		#ifndef __clang_analyzer__
		(void)capacity;
		u8* data = (u8*)&key;
		for (int i=0; i!=sizeof(key); ++i)
		{
			u8 first = (data[i] >> 4);
			u8 second = (data[i] & 0xf);
			*pos++ = g_hexChars[first];
			*pos++ = g_hexChars[second];
		}
		#endif
		*pos = 0;
	}

	struct CasKeyString : StringView
	{
		CasKeyString(const CasKey& g) : StringView(str, 40) { uba::ToString(str, 41, g); }
		tchar str[41];
	};

	#if UBA_DEBUG
	inline void CheckPath(const StringView& fileName)
	{
		tchar history[3] = { 0 };
		for (auto i = fileName.data, e = i + fileName.count; i != e; ++i)
		{
			tchar c = *i;
			UBA_ASSERTF(!CaseInsensitiveFs || c < 'A' || c > 'Z', TC("Path is not valid (%.*s)"), fileName.count, fileName.data); // No upper case characters if case insensitive
			UBA_ASSERTF(!(c == PathSeparator && history[0] == '.' && history[1] == PathSeparator) || (IsWindows && fileName.Contains(TC("\\\\.\\pipe"))), TC("Path is not valid (%.*s)"), fileName.count, fileName.data);  //  /./ not allowed
			UBA_ASSERTF(!(c == PathSeparator && history[0] == '.' && history[1] == '.' && history[2] == PathSeparator), TC("Path is not valid (%.*s)"), fileName.count, fileName.data); // /../ not allowed
			UBA_ASSERTF(!IsWindows || c != NonPathSeparator, TC("Path is not valid (%.*s)"), fileName.count, fileName.data);
			//UBA_ASSERTF(c != PathSeparator || history[0] != PathSeparator || (IsWindows && fileName.Contains(TC("\\\\.\\pipe"))), TC("Path is not valid (%.*s)"), fileName.count, fileName.data); // Double backslash
			history[2] = history[1];
			history[1] = history[0];
			history[0] = c;
		}
		//UBA_ASSERTF(history[0] != ' ' && (history[0] != PathSeparator || (fileName.data[1] == ':' && fileName.data[3] == 0)), TC("Path is not valid (%.*s)"), fileName.count, fileName.data);
		// Commented out because this asserts on dlls in subfolders on remote machines (example: 2057\clui.dll). Likely related to helper not having some language id as host
		// UBA_ASSERTF(!hasSlash || fileName[1] == ':', TC("Invalid path: %s"), fileName);
	}
	#define CHECK_PATH(path) CheckPath(path);
	#else
	#define CHECK_PATH(path) while (false) {}
	#endif

	struct BloomFilter
	{
		u8 bytes[128] = { 0 };

		void Add(const StringKey& key);
		bool IsGuaranteedMiss(const StringKey& key) const;
		bool IsEmpty() const;
	};
}

//template<> struct std::hash<uba::Guid> { size_t operator()(const uba::Guid& g) const { std::hash<uba::u64> hash; return hash(((uba::u64*)&g)[0]) ^ hash(((uba::u64*)&g)[1]); } };
template<> struct std::hash<uba::StringKey> { size_t operator()(const uba::StringKey& g) const { return g.a; } };
template<> struct std::hash<uba::CasKey> { size_t operator()(const uba::CasKey& g) const { return g.a; } };

template <class Map> 
void PrintMapInfo(const char* name, const Map& map)
{
    double l = map.size() / double(map.bucket_count());
    double c = 0.0;
    for (auto& kv : map)
        c += map.bucket_size(map.bucket(kv.first));
    c /= map.size();

    double quality = 1.0 - uba::Max(0.0, c / (1 + l) - 1);
	printf("%s Size: %llu Buckets: %llu Quality: %f\r\n", name, map.size(), map.bucket_count(), quality);
}