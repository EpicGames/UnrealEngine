// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaElf.h"
#include "UbaFunctional.h"
#include "UbaList.h"
#include "UbaPlatform.h"
#include "UbaStringBuffer.h"

#if PLATFORM_MAC
#include "UbaMacBinDependencyParser.h"
#endif

#if !defined(UBA_IS_DETOURED_INCLUDE)
#define TRUE_WRAPPER(func) func
#endif

#define UBA_FINDIMPORTS_DEBUG 0

namespace uba
{
	inline constexpr int PT_LOAD	= 1;	// Elf64_Phdr	Load segment into memory
	inline constexpr int PT_DYNAMIC = 2;	// Elf64_Phdr	Dynamic linker info
	inline constexpr int DT_NULL	= 0;	// Elf64_Dyn	End marker
	inline constexpr int DT_NEEDED	= 1;	// Elf64_Dyn	Dependency
	inline constexpr int DT_STRTAB	= 5;	// Elf64_Dyn	String table address
	inline constexpr int DT_STRSZ	= 10;	// Elf64_Dyn	String table size
	inline constexpr int DT_RPATH	= 15;	// Elf64_Dyn	Legacy library path
	inline constexpr int DT_RUNPATH = 29;	// Elf64_Dyn	Modern library path

	#if !PLATFORM_LINUX
	namespace linux {
	#else
	struct BinaryInfo
	{
	};
	#endif

	// Libraries with hard kernel / glibc / host-daemon ABI coupling.
	// These MUST NOT be transferred from coordinator to helper — shipping them
	// across hosts will break when the two sides have different kernel, glibc,
	// or host-service versions (e.g. libutil from glibc 2.39 on a 2.34 helper).
	constexpr StringView g_kernelBoundFiles[] =
	{
		// glibc family (syscall ABI, vDSO, NPTL, NSS — all coupled to the running kernel/glibc)
		TCV("ld-linux-x86-64.so"),
		TCV("libc.so"),
		TCV("libpthread.so"),
		TCV("libdl.so"),
		TCV("libm.so"),
		TCV("libresolv.so"),
		TCV("librt.so"),
		TCV("libutil.so"),
		TCV("libanl.so"),
		TCV("libnsl.so"),
		TCV("libthread_db.so"),
		TCV("libmvec.so"),
		TCV("libBrokenLocale.so"),
		TCV("libnss_"),           // prefix: libnss_files.so.2, libnss_dns.so.2, libnss_systemd.so.2, ...
		TCV("libcrypt.so"),
		// Kernel-policy / syscall coupled. libselinux deliberately omitted — AOSP sbox ships its own copy (and libpcre2), helper system copy may ABI-mismatch.
		TCV("libcap.so"),
		TCV("libacl.so"),
		TCV("libattr.so"),
		// Host device / daemon coupled
		TCV("libudev.so"),
		TCV("libdbus-1.so"),
		TCV("libsystemd.so"),
	};

	// Libraries that are expected to exist on the helper and are skipped for
	// efficiency, but would not be inherently broken by transfer. If you hit an
	// ABI mismatch on one of these, consider either shipping it bundled with
	// your binary or moving it to g_kernelBoundFiles.
	constexpr StringView g_commonSystemFiles[] =
	{
		TCV("libstdc++.so"),
		TCV("libgcc_s.so"),
		TCV("libz.so"),
		TCV("libzstd.so"),
		TCV("libexpat.so"),
		TCV("libtinfo.so"),
		TCV("libdxil.so"),
		//TCV("libX11.so"),
		//TCV("libXext.so"),
		//TCV("libXcursor.so"),
		//TCV("libXi.so"),
		//TCV("libXfixes.so"),
		//TCV("libXrandr.so"),
		//TCV("libXss.so"),

		TCV("libglib-2.0.so"),
		TCV("libgobject-2.0.so"),
		TCV("libgio-2.0.so"),
		TCV("libgmodule-2.0.so"),
		TCV("libgthread-2.0.so"),
		TCV("libpcre.so"),
		// libpcre2- excluded — paired with libselinux above.

		TCV("libnspr4.so"),
		TCV("libplc4.so"),      // NSPR companion
		TCV("libplds4.so"),     // NSPR companion

		TCV("libnss3.so"),
		TCV("libsmime3.so"),
		TCV("libssl3.so"),
		TCV("libnssutil3.so"),
	};

	inline bool IsKnownSystemFile(const tchar* fileName)
	{
		StringView file = ToView(fileName);
		for (auto systemFile : g_kernelBoundFiles)
			if (file.StartsWith(systemFile.data))
				return true;
		for (auto systemFile : g_commonSystemFiles)
			if (file.StartsWith(systemFile.data))
				return true;
		return false;
	}

	inline bool ParseBinary(StringView filePath, StringView originalPath, BinaryInfo& outInfo, const Function<void(const tchar* import, bool isKnown, const tchar* const* loaderPaths)>& func, StringBufferBase& outError)
	{
		#if UBA_FINDIMPORTS_DEBUG
		printf("FINDIMPORTS: %s\n", filePath.data);
		#endif

		UBA_ASSERT(filePath.IsTerminated());

		#if !PLATFORM_WINDOWS
		int fd = TRUE_WRAPPER(open)(filePath.data, O_RDONLY | O_CLOEXEC);
		if (fd == -1)
			return outError.Appendf("Open failed for file %s", filePath.data).ToFalse();

		auto closeFileHandle = MakeGuard([&]() { TRUE_WRAPPER(close)(fd); });
		struct stat sb;
		if (TRUE_WRAPPER(fstat)(fd, &sb) == -1)
			return outError.Appendf("Stat failed for file %s", filePath.data).ToFalse();

		u32 fileSize = sb.st_size;

		auto mem = (u8*)mmap(nullptr, fileSize, PROT_READ, MAP_PRIVATE, fd, 0);
		if (mem == MAP_FAILED)
			return outError.Appendf("Mmap failed for file %s", filePath.data).ToFalse();
		auto unmap = MakeGuard([&]() { munmap(mem, fileSize); });
		#else
		HANDLE fileHandle = ::CreateFileW(filePath.data, GENERIC_READ, FILE_SHARE_READ, 0, OPEN_EXISTING, FILE_ATTRIBUTE_NORMAL | FILE_FLAG_SEQUENTIAL_SCAN, 0);
		if (fileHandle == INVALID_HANDLE_VALUE)
			return true;
		auto closeFileHandle = MakeGuard([&]() { ::CloseHandle(fileHandle); });
		HANDLE fileMapping = ::CreateFileMappingW(fileHandle, NULL, PAGE_READONLY, 0, 0, NULL);
		if (!fileMapping)
			return false;
		auto closeMappingHandle = MakeGuard([&]() { ::CloseHandle(fileMapping); });
		u8* mem = (u8*)::MapViewOfFile(fileMapping, FILE_MAP_READ, 0, 0, 0);
		if (!mem)
			return false;
		auto unmap = MakeGuard([&]() { ::UnmapViewOfFile(mem); });
		LARGE_INTEGER fileSize2;
		if (!::GetFileSizeEx(fileHandle, &fileSize2))
			return false;
		u32 fileSize = u32(fileSize2.QuadPart);
		#endif

		auto ehdr = (Elf64FileHeader*)mem;
		if (memcmp(ehdr->e_ident, ELFMAG, SELFMAG) != 0 || ehdr->e_ident[EI_CLASS] != ELFCLASS64)
			return outError.Append("Not a valid 64-bit ELF file").ToFalse();

		auto phdrs = (Elf64ProgramHeader*)(mem + ehdr->e_phoff);
		auto shdrs = (Elf64SectionHeader*)(mem + ehdr->e_shoff);
		Elf64Dyn* dynamic = NULL;
		size_t dyn_count = 0;

		auto vaddr_to_offset = [](Elf64Addr vaddr, Elf64ProgramHeader* phdrs, int phnum) -> Elf64Off {
			for (int i = 0; i < phnum; i++) {
				if (phdrs[i].p_type != PT_LOAD) continue;
				Elf64Addr start = phdrs[i].p_vaddr;
				Elf64Addr end = start + phdrs[i].p_memsz;
				if (vaddr >= start && vaddr < end) {
					return phdrs[i].p_offset + (vaddr - start);
				}
			}
			return 0;
		};

		for (int i = 0; i < ehdr->e_phnum; ++i)
		{
			if (phdrs[i].p_type != PT_DYNAMIC)
				continue;
			Elf64Off dyn_offset = vaddr_to_offset(phdrs[i].p_vaddr, phdrs, ehdr->e_phnum);
			if (dyn_offset >= fileSize || dyn_offset + sizeof(Elf64Dyn) > fileSize)
				return outError.Appendf(TC("Dynamic offset out of file bounds!")).ToFalse();
			dynamic = (Elf64Dyn*)(mem + dyn_offset);			
			dyn_count = phdrs[i].p_filesz / sizeof(Elf64Dyn);
			break;
		}

		if (!dynamic)
			return outError.Appendf(TC("No PT_DYNAMIC segment found.")).ToFalse();
		if ((u8*)dynamic < mem || (u8*)dynamic >= mem + fileSize)
			return outError.Appendf(TC("Dynamic segment is out of file bounds!")).ToFalse();

		Elf64Addr strtab_addr = 0;
		u64 strsz = 0;
		u64 needed[512];
		u32 needed_count = 0;

		for (size_t i = 0; i < dyn_count; ++i)
		{
			auto& dyn = dynamic[i];
			if (dyn.d_tag == DT_NULL)
				break;
			if (dyn.d_tag == DT_STRTAB)
				strtab_addr = dyn.d_un.d_ptr;
			else if (dyn.d_tag == DT_STRSZ)
				strsz = dyn.d_un.d_val;
			else if (dyn.d_tag == DT_NEEDED && needed_count < sizeof_array(needed))
				needed[needed_count++] = dyn.d_un.d_val;
		}

		char* strtab = NULL;

		// Step 3: If DT_STRTAB is missing, fallback to section header lookup
		if (!strtab_addr && ehdr->e_shoff != 0) {
			for (int i = 0; i < ehdr->e_shnum; i++) {
				if (shdrs[i].sh_type == SHT_STRTAB && shdrs[i].sh_flags == 0) { // Likely .dynstr
					strtab = (char *)mem + shdrs[i].sh_offset;
					strsz = shdrs[i].sh_size;
					break;
				}
			}
		}

		// If we have strtab_addr but not strtab, resolve via vaddr to file offset
		if (strtab_addr && !strtab)
		{
			Elf64Off str_off = vaddr_to_offset(strtab_addr, phdrs, ehdr->e_phnum);
			strtab = (char *)mem + str_off;
		}

		if (!strtab)
			return outError.Appendf(TC("Failed to find string table.")).ToFalse();

		const tchar* loaderPaths[256];
		u32 loaderPathsCount = 0;

		List<TString> fixedLoaderPaths;

		#if UBA_FINDIMPORTS_DEBUG
		printf("ELF: %s\n", filePath.data);
		#endif

		#if UBA_FINDIMPORTS_DEBUG
		printf("RPATH / RUNPATH:\n");
		#endif

		
		//StringView origin(fileName, strrchr(fileName, '/') - fileName);

		for (Elf64Dyn* dyn = dynamic; dyn->d_tag != DT_NULL; dyn++)
		{
			if (dyn->d_tag != DT_RPATH && dyn->d_tag != DT_RUNPATH)
				continue;

			#if UBA_FINDIMPORTS_DEBUG
			if (dyn->d_tag == DT_RPATH)
				printf("  DT_RPATH: %s\n", strtab + dyn->d_un.d_val);
			else if (dyn->d_tag == DT_RUNPATH)
				printf("  DT_RUNPATH: %s\n", strtab + dyn->d_un.d_val);
			#endif

			char* begin = strtab + dyn->d_un.d_val;
			char* end = begin + strlen(begin);
			char* pathBegin = begin;
			while (pathBegin < end)
			{
				char* pathEnd = strchr(pathBegin, ':');
				if (!pathEnd)
					pathEnd = end;
				#if PLATFORM_WINDOWS
				StringBuffer temp;
				temp.Append(pathBegin, u32(pathEnd - pathBegin));
				StringView loaderPath(temp);
				#else
				StringView loaderPath(pathBegin, pathEnd - pathBegin);
				#endif

				if (loaderPath.StartsWith(TCV("${ORIGIN}")) || loaderPath.StartsWith(TCV("$ORIGIN")))
				{
					// Expand $ORIGIN to lib's dir; absolute loaderPath bypasses HandleEnsureBinaryFile's applicationDir join (would otherwise anchor at launcher's bin/).
					u32 skip = loaderPath.StartsWith(TCV("${ORIGIN}")) ? 9 : 7;
					StringView rest = loaderPath.Skip(skip);
					if (rest.StartsWith(TCV("/")))
						rest = rest.Skip(1);
					TString resolved;
					resolved.append(originalPath.data, originalPath.count);
					if (!resolved.empty() && resolved.back() != '/')
						resolved.push_back('/');
					resolved.append(rest.data, rest.count);
					loaderPaths[loaderPathsCount++] = fixedLoaderPaths.emplace_back(std::move(resolved)).c_str();
				}
				else
					loaderPaths[loaderPathsCount++] = fixedLoaderPaths.emplace_back(loaderPath.ToString()).c_str();

				#if UBA_FINDIMPORTS_DEBUG
				printf("  SEARCH_PATH: %s\n", loaderPaths[loaderPathsCount-1]);
				#endif

				pathBegin = pathEnd + 1;
			}
		}
		loaderPaths[loaderPathsCount] = nullptr;

		#if UBA_FINDIMPORTS_DEBUG
		printf("DT_NEEDED:\n");
		#endif
		
		#if 0
		for (Elf64Dyn* dyn = dynamic; dyn->d_tag != DT_NULL; dyn++) {
			if (dyn->d_tag != DT_NEEDED)
				continue;
			printf("  %s\n", strtab + dyn->d_un.d_val);
			func(strtab + dyn->d_un.d_val, false, loaderPaths);
		}
		#endif

		for (u32 i = 0; i < needed_count; i++) {
			if (needed[i] < strsz || strsz == 0)
			{
#if PLATFORM_WINDOWS
				StringBuffer<> temp;
				temp.Append(strtab + needed[i]);
				StringView file(temp);
#else
				StringView file = ToView(strtab + needed[i]);
#endif
				if (IsKnownSystemFile(file.data))
					continue;

				#if UBA_FINDIMPORTS_DEBUG
				printf("  %s\n", strtab + needed[i]);
				#endif

				func(file.data, false, loaderPaths);
			}
			else
			{
				#if UBA_FINDIMPORTS_DEBUG
				printf("  [offset %llu out of bounds]\n", needed[i]);
				#endif
			}
		}		
		return true;
	}

	#if !PLATFORM_LINUX
	}
	#endif

}
