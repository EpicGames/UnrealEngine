// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if PLATFORM_LINUX

namespace uba
{
	// Raw Linux syscall with no glibc involvement (no errno TLS, no wrappers).
	// Safe to call from any thread, including goroutine OS threads that lack a
	// proper glibc TCB (the original motivation for this helper).
	// On error: r1 is -errno (negative), r2 is 0.
	// On success: r1 is the return value, r2 is rdx after syscall (usually 0).
	struct RawSyscallResult { long r1; long r2; };
	inline RawSyscallResult RawSyscallDirect(long nr, long a1, long a2, long a3, long a4 = 0, long a5 = 0, long a6 = 0)
	{
#if PLATFORM_CPU_ARM_FAMILY
		register long x8 __asm__("x8") = nr;
		register long x0 __asm__("x0") = a1;
		register long x1 __asm__("x1") = a2;
		register long x2 __asm__("x2") = a3;
		register long x3 __asm__("x3") = a4;
		register long x4 __asm__("x4") = a5;
		register long x5 __asm__("x5") = a6;
		__asm__ volatile (
			"svc #0"
			: "+r" (x0), "+r" (x1)
			: "r" (x8), "r" (x2), "r" (x3), "r" (x4), "r" (x5)
			: "memory"
			);
		return { x0, x1 };
#else
		register long rax __asm__("rax") = nr;
		register long rdx __asm__("rdx") = a3;
		register long r10 __asm__("r10") = a4;
		register long r8  __asm__("r8")  = a5;
		register long r9  __asm__("r9")  = a6;
		__asm__ volatile (
			"syscall"
			: "+a" (rax), "+d" (rdx)
			: "D" (a1), "S" (a2), "r" (r10), "r" (r8), "r" (r9)
			: "memory", "rcx", "r11"
			);
		return { rax, rdx };
#endif
	}

	// Stub-safe readlinkat. Both x86_64 and aarch64 have SYS_readlinkat; aarch64 dropped
	// the legacy SYS_readlink, so always use the *at variant with AT_FDCWD.
	inline long RawReadlinkat(const char* path, char* buf, unsigned long bufSize)
	{
#if PLATFORM_CPU_ARM_FAMILY
		constexpr long NR_READLINKAT = 78;
#else
		constexpr long NR_READLINKAT = 267;
#endif
		auto r = RawSyscallDirect(NR_READLINKAT, /*AT_FDCWD*/ -100, (long)path, (long)buf, (long)bufSize);
		return r.r1; // negative on error (-errno), nonnegative on success
	}

	// Arity-keyed wrappers around RawSyscallDirect — discard the rdx component (only mattering for pipe/fork/etc. which write through pointers anyway).
	// Compilers inline RawSyscallDirect into these so the generated code matches a hand-rolled syscall instruction.
	inline long RawSyscall0(long num)                                                       { return RawSyscallDirect(num, 0, 0, 0).r1; }
	inline long RawSyscall1(long num, long a)                                               { return RawSyscallDirect(num, a, 0, 0).r1; }
	inline long RawSyscall3(long num, long a, long b, long c)                               { return RawSyscallDirect(num, a, b, c).r1; }
	inline long RawSyscall6(long num, long a, long b, long c, long d, long e, long f)       { return RawSyscallDirect(num, a, b, c, d, e, f).r1; }
}

#endif
