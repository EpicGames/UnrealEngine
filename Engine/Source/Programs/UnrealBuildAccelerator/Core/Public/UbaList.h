// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaMemory.h"
#include <utility>

namespace uba
{
	// Minimal std::list replacement: doubly-linked, heap-allocated nodes.
	// Elements never move in memory — pointers/refs/iterators stay valid
	// across insert/erase of other elements.
	template<typename T, typename NodeAlloc = Allocator>
	class List
	{
		struct NodeBase
		{
			NodeBase* prev;
			NodeBase* next;
		};
		struct Node : NodeBase
		{
			T value;
			template<typename... Args>
			Node(Args&&... args) : value(std::forward<Args>(args)...) {}
		};

	public:
		using value_type = T;
		using size_type  = u64;
		using reference  = T&;
		using const_reference = const T&;

		class const_iterator;
		class iterator
		{
		public:
			iterator() = default;
			iterator(NodeBase* n) : m_node(n) {}
			T& operator*()  const { return static_cast<Node*>(m_node)->value; }
			T* operator->() const { return &static_cast<Node*>(m_node)->value; }
			iterator& operator++() { m_node = m_node->next; return *this; }
			iterator  operator++(int) { iterator t = *this; m_node = m_node->next; return t; }
			iterator& operator--() { m_node = m_node->prev; return *this; }
			iterator  operator--(int) { iterator t = *this; m_node = m_node->prev; return t; }
			bool operator==(const iterator& o) const { return m_node == o.m_node; }
			bool operator!=(const iterator& o) const { return m_node != o.m_node; }
			NodeBase* m_node = nullptr;
		};

		class const_iterator
		{
		public:
			const_iterator() = default;
			const_iterator(const NodeBase* n) : m_node(n) {}
			const_iterator(iterator it) : m_node(it.m_node) {}
			const T& operator*()  const { return static_cast<const Node*>(m_node)->value; }
			const T* operator->() const { return &static_cast<const Node*>(m_node)->value; }
			const_iterator& operator++() { m_node = m_node->next; return *this; }
			const_iterator  operator++(int) { const_iterator t = *this; m_node = m_node->next; return t; }
			const_iterator& operator--() { m_node = m_node->prev; return *this; }
			const_iterator  operator--(int) { const_iterator t = *this; m_node = m_node->prev; return t; }
			bool operator==(const const_iterator& o) const { return m_node == o.m_node; }
			bool operator!=(const const_iterator& o) const { return m_node != o.m_node; }
			const NodeBase* m_node = nullptr;
		};

		List() { init_sentinel(); }

		List(const List& o)
		{
			init_sentinel();
			for (auto it = o.begin(); it != o.end(); ++it) emplace_back(*it);
		}

		List(List&& o) noexcept
		{
			init_sentinel();
			swap(o);
		}

		~List() { clear(); }

		List& operator=(const List& o)
		{
			if (this != &o)
			{
				clear();
				for (auto it = o.begin(); it != o.end(); ++it) emplace_back(*it);
			}
			return *this;
		}

		List& operator=(List&& o) noexcept
		{
			if (this != &o) { clear(); swap(o); }
			return *this;
		}

		bool empty() const { return m_size == 0; }
		size_type size() const { return m_size; }

		T& front()             { return static_cast<Node*>(m_sentinel.next)->value; }
		const T& front() const { return static_cast<const Node*>(m_sentinel.next)->value; }
		T& back()              { return static_cast<Node*>(m_sentinel.prev)->value; }
		const T& back()  const { return static_cast<const Node*>(m_sentinel.prev)->value; }

		iterator       begin()        { return iterator(m_sentinel.next); }
		iterator       end()          { return iterator(&m_sentinel); }
		const_iterator begin()  const { return const_iterator(m_sentinel.next); }
		const_iterator end()    const { return const_iterator(&m_sentinel); }
		const_iterator cbegin() const { return const_iterator(m_sentinel.next); }
		const_iterator cend()   const { return const_iterator(&m_sentinel); }

		// Minimal reverse iterators (forward-decrementing).
		class reverse_iterator
		{
		public:
			reverse_iterator() = default;
			reverse_iterator(NodeBase* n) : m_node(n) {}
			T& operator*()  const { return static_cast<Node*>(m_node)->value; }
			T* operator->() const { return &static_cast<Node*>(m_node)->value; }
			reverse_iterator& operator++() { m_node = m_node->prev; return *this; }
			reverse_iterator  operator++(int) { reverse_iterator t = *this; m_node = m_node->prev; return t; }
			bool operator==(const reverse_iterator& o) const { return m_node == o.m_node; }
			bool operator!=(const reverse_iterator& o) const { return m_node != o.m_node; }
			NodeBase* m_node = nullptr;
		};
		reverse_iterator rbegin() { return reverse_iterator(m_sentinel.prev); }
		reverse_iterator rend()   { return reverse_iterator(&m_sentinel); }

		void clear()
		{
			NodeBase* n = m_sentinel.next;
			while (n != &m_sentinel)
			{
				NodeBase* nx = n->next;
				destroy_node(static_cast<Node*>(n));
				n = nx;
			}
			init_sentinel();
			m_size = 0;
		}

		template<typename... Args>
		T& emplace_back(Args&&... args)
		{
			Node* n = create_node(std::forward<Args>(args)...);
			link_before(&m_sentinel, n);
			++m_size;
			return n->value;
		}

		template<typename... Args>
		T& emplace_front(Args&&... args)
		{
			Node* n = create_node(std::forward<Args>(args)...);
			link_before(m_sentinel.next, n);
			++m_size;
			return n->value;
		}

		void push_back(const T& v)  { emplace_back(v); }
		void push_back(T&& v)       { emplace_back(std::move(v)); }
		void push_front(const T& v) { emplace_front(v); }
		void push_front(T&& v)      { emplace_front(std::move(v)); }

		void pop_back()
		{
			if (m_size == 0) return;
			Node* n = static_cast<Node*>(m_sentinel.prev);
			unlink(n);
			destroy_node(n);
			--m_size;
		}

		void pop_front()
		{
			if (m_size == 0) return;
			Node* n = static_cast<Node*>(m_sentinel.next);
			unlink(n);
			destroy_node(n);
			--m_size;
		}

		iterator insert(const_iterator pos, const T& v)
		{
			Node* n = create_node(v);
			link_before(const_cast<NodeBase*>(pos.m_node), n);
			++m_size;
			return iterator(n);
		}

		// emplace(pos, args...) — construct in place before `pos`, return iter to new element.
		template<typename... Args>
		iterator emplace(const_iterator pos, Args&&... args)
		{
			Node* n = create_node(std::forward<Args>(args)...);
			link_before(const_cast<NodeBase*>(pos.m_node), n);
			++m_size;
			return iterator(n);
		}

		iterator insert(const_iterator pos, T&& v)
		{
			Node* n = create_node(std::move(v));
			link_before(const_cast<NodeBase*>(pos.m_node), n);
			++m_size;
			return iterator(n);
		}

		iterator erase(const_iterator pos)
		{
			NodeBase* target = const_cast<NodeBase*>(pos.m_node);
			NodeBase* nx = target->next;
			unlink(target);
			destroy_node(static_cast<Node*>(target));
			--m_size;
			return iterator(nx);
		}

		void swap(List& o) noexcept
		{
			// Sentinels are embedded, so nodes' prev/next point AT the sentinel
			// address. Swap by re-routing the chain's head/tail links, then
			// swap sizes.
			NodeBase* thisFirst  = m_sentinel.next;
			NodeBase* thisLast   = m_sentinel.prev;
			NodeBase* otherFirst = o.m_sentinel.next;
			NodeBase* otherLast  = o.m_sentinel.prev;
			bool thisEmpty  = (thisFirst  == &m_sentinel);
			bool otherEmpty = (otherFirst == &o.m_sentinel);

			if (otherEmpty) init_sentinel();
			else
			{
				m_sentinel.next = otherFirst;
				m_sentinel.prev = otherLast;
				otherFirst->prev = &m_sentinel;
				otherLast->next  = &m_sentinel;
			}
			if (thisEmpty) o.init_sentinel();
			else
			{
				o.m_sentinel.next = thisFirst;
				o.m_sentinel.prev = thisLast;
				thisFirst->prev = &o.m_sentinel;
				thisLast->next  = &o.m_sentinel;
			}
			size_type ts = m_size; m_size = o.m_size; o.m_size = ts;
		}

	private:
		void init_sentinel() { m_sentinel.prev = &m_sentinel; m_sentinel.next = &m_sentinel; }

		template<typename... Args>
		Node* create_node(Args&&... args)
		{
			Node* n = (Node*)m_alloc.Allocate(sizeof(Node), alignof(Node));
			new (n) Node(std::forward<Args>(args)...);
			return n;
		}

		void destroy_node(Node* n)
		{
			n->~Node();
			m_alloc.Deallocate(n, sizeof(Node), alignof(Node));
		}

		void link_before(NodeBase* pos, NodeBase* n)
		{
			n->prev = pos->prev;
			n->next = pos;
			pos->prev->next = n;
			pos->prev = n;
		}

		void unlink(NodeBase* n)
		{
			n->prev->next = n->next;
			n->next->prev = n->prev;
		}

		NodeBase m_sentinel{};
		size_type m_size = 0;
		NodeAlloc m_alloc;
	};
}
