// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if PLATFORM_MAC

#include "UbaBinaryReaderWriter.h"
#include "UbaFunctional.h"
#include "UbaPlatform.h"
#include <mach-o/fat.h>

#if !defined(UBA_IS_DETOURED_INCLUDE)
#define TRUE_WRAPPER(func) func
#endif

namespace uba
{
	struct BinaryInfo
	{
		u32 minVersion = 0;
	};

	inline bool IsKnownSystemFile(const tchar* fileName)
	{
		return false;
	}

	inline bool ParseBinary(const StringView& filePath, const StringView& originalPath, BinaryInfo& outInfo, const Function<void(const tchar* import, bool isKnown, const char* const* loaderPaths)>& func, StringBufferBase& outError)
	{
		int fd = TRUE_WRAPPER(open)(filePath.data, O_RDONLY);
		if (fd == -1)
		{
			outError.Appendf("Open failed for file %s", filePath.data);
			return false;
		}
		auto closeFileHandle = MakeGuard([&]() { TRUE_WRAPPER(close)(fd); });
		struct stat sb;
		if (TRUE_WRAPPER(fstat)(fd, &sb) == -1)
		{
			outError.Appendf("Stat failed for file %s", filePath.data);
			return false;
		}
		u32 fileSize = Min(u32(sb.st_size), 1024u*1024u);

		auto mem = (u8*)TRUE_WRAPPER(mmap)(nullptr, fileSize, PROT_READ, MAP_PRIVATE, fd, 0);
		if (mem == MAP_FAILED)
		{
			outError.Appendf("Mmap failed for file %s", filePath.data);
			return false;
		}
		auto unmap = MakeGuard([&]() { TRUE_WRAPPER(munmap)(mem, fileSize); });

		StringView fileName = originalPath.GetFileName();

		const char* libs[1024];
		u32 libsCount = 0;

		const char* loaderPaths[256];
		u32 loaderPathsCount = 0;

		auto handleHeader = [&](u32 offset)
		{
			u8* memIt = mem + offset;
			u32 magic = *(const u32*)memIt;
			UBA_ASSERT(magic == MH_MAGIC_64);
			if (magic != MH_MAGIC_64)
				return true;
			auto& mh = *(mach_header_64*)memIt;
			memIt += sizeof(mach_header_64);
			for (u32 i = 0; i < mh.ncmds; i++)
			{
				auto& lc = *(load_command*)memIt;
				auto g = MakeGuard([&]() { memIt += lc.cmdsize; });

				if (lc.cmd ==0x24) // LC_VERSION_MIN_MAXOSX
				{
					struct version_min_command { u32 cmd; u32 cmdsize; u32 version; u32 sdk; };
					auto& version = *(version_min_command*)memIt;
					outInfo.minVersion = version.version;
					continue;
				}
				if (lc.cmd == LC_BUILD_VERSION)
				{
					auto& version = *(build_version_command*)memIt;
					outInfo.minVersion = version.minos;
					continue;
				}

				if (lc.cmd != LC_LOAD_DYLIB && lc.cmd != LC_RPATH) //  && lc.cmd != LC_LOAD_WEAK_DYLIB
					continue;
				auto& cmdData = *(dylib_command*)memIt;
				u32 nameOffset = cmdData.dylib.name.offset;
				if (nameOffset >= lc.cmdsize)
					continue;
				auto name = ToView((const char*)(memIt + nameOffset));
				if (*name.data != '@')
					continue;
				name = name.Skip(1);
				if (name.StartsWith("rpath/"))
				{
					auto importFile = name.Skip(6);
					if (!importFile.EndsWith(".dylib"))
					{
						outError.Appendf("Found @rpath in binary %s that did not end with .dylib (%s)", filePath.data, importFile.data);
						return false;
					}
					if (fileName.Equals(importFile))
						continue;
					libs[libsCount++] = importFile.data;
				}
				else if (name.StartsWith("executable_path/"))
				{
					auto executablePath = name.Skip(16).data;
					if (*executablePath != 0)
						loaderPaths[loaderPathsCount++] = executablePath;
				}
				else if (name.StartsWith("loader_path/"))
				{
					loaderPaths[loaderPathsCount++] = name.Skip(12).data;
				}
			}
			return true;
		};

		u32 magic = *(const u32*)mem;
		if (magic == FAT_CIGAM || magic == FAT_MAGIC)
		{
			u8* memIt = mem;
			auto& fh = *(fat_header*)memIt;
			memIt += sizeof(fat_header);
			for (u32 i = 0, e = ntohl(fh.nfat_arch); i<e; i++)
			{
				auto& arch = *(fat_arch_64*)memIt;
				memIt += sizeof(fat_arch_64);
				u32 offset = ntohl(arch.offset);
				u32 cputype = ntohl(arch.cputype);
				if (cputype == CPU_TYPE_X86_64 || cputype == CPU_TYPE_ARM64)
					if (!handleHeader(offset))
						return false;
			}
		}
		else if (!handleHeader(0))
			return false;

		loaderPaths[loaderPathsCount] = nullptr;
		for (u32 i=0; i!=libsCount; ++i)
			func(libs[i], false, loaderPaths);
		return true;
	}
}

#endif