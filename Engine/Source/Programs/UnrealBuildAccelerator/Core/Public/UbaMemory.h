// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaSynchronization.h"

#if UBA_USE_MIMALLOC
#include <mimalloc.h>
#else
#include <memory> // for std::allocator fallback
#endif

#if PLATFORM_WINDOWS && !defined(aligned_alloc)
#define aligned_alloc(a, s) _aligned_malloc(s, a)
#define aligned_free(p) _aligned_free(p)
#else
#define aligned_free(p) free(p)
#endif

namespace uba
{
	struct StringView;

	template<typename T>
	constexpr inline T AlignUp(T arg, uintptr_t alignment) { return T(((uintptr_t)arg+(alignment-1)) & ~(alignment-1)); }


	#if UBA_USE_MIMALLOC
	struct Allocator
	{
		Allocator() = default;
		Allocator(const Allocator& o) = default;
		Allocator(Allocator&& o) noexcept = default;
		Allocator& operator=(const Allocator& o) = default;
		Allocator& operator=(Allocator&& o) noexcept = default;
		void* Allocate(u64 size, u64 align) { return mi_malloc_aligned(size, align); }
		void Deallocate(void* p, u64 size, u64 align) { mi_free_size_aligned(p, size, align); }
		bool TryGrowInPlace(void* p, u64 /*oldSize*/, u64 newSize) { return mi_expand(p, newSize) != nullptr; }
	};
	#elif UBA_STUB_BUILD
	void* StubAllocatorAllocate(u64 bytes, u64 alignment);
	void StubAllocatorFree(void* p, u64 bytes, u64 alignment);
	struct Allocator
	{
		Allocator() = default;
		void* Allocate(u64 size, u64 align) { return StubAllocatorAllocate(size, align); }
		void Deallocate(void* p, u64 size, u64 align) { StubAllocatorFree(p, size, align); }
	};
	#else
	struct Allocator
	{
		Allocator() = default;
		Allocator(const Allocator& o) = default;
		Allocator(Allocator&& o) noexcept = default;
		Allocator& operator=(const Allocator& o) = default;
		Allocator& operator=(Allocator&& o) noexcept = default;
		void* Allocate(u64 size, u64 align)
		{
			// macOS/glibc aligned_alloc requires align >= sizeof(void*) and size to be a
			// multiple of align; malloc already returns max_align_t-aligned memory, so use it
			// for small alignments (e.g. alignof(char) = 1 from TString).
			if (align <= sizeof(void*))
				return malloc(size);
			return aligned_alloc(align, (size + align - 1) & ~(align - 1));
		}
		void Deallocate(void* p, u64 size, u64 align) { free(p); }
		bool TryGrowInPlace(void* p, u64 /*oldSize*/, u64 newSize) { return false; }
	};
	#endif

	struct MemoryBlock
	{
		MemoryBlock(const tchar* name_, u64 reserveSize_, void* baseAddress_ = nullptr);
		MemoryBlock(const tchar* name_, u8* baseAddress_ = nullptr);
		~MemoryBlock();
		bool Init(u64 reserveSize_, void* baseAddress_ = nullptr, bool useHugePages = false);
		void Deinit();
		void* Allocate(u64 bytes, u64 alignment, const tchar* hint);
		void* AllocateNoLock(u64 bytes, u64 alignment, const tchar* hint, bool lockMemory = false);
		void* CommitNoLock(u64 bytes, const tchar* hint);
		// Grow the allocation ending at (p + oldBytes) to newBytes in place. Only succeeds if
		// that allocation sits at the current bump tail; otherwise returns false and the caller
		// must allocate + copy. Used by Vector::reserve to avoid the copy when growing alone.
		bool TryGrowTail(void* p, u64 oldBytes, u64 newBytes);
		bool TryGrowTailNoLock(void* p, u64 oldBytes, u64 newBytes);
		void Free(void* p);
		StringView Strdup(const StringView& str);
		tchar* Strdup(const tchar* str);
		void Swap(MemoryBlock& other);
		
		Futex lock;
		u8* memory;
		u64 reserveSize = 0;
		u64 writtenSize = 0;
		u64 committedSize = 0;
		const tchar* name;
	};


	struct GrowingAllocator
	{
		GrowingAllocator(MemoryBlock& block) : m_block(&block) {}
		GrowingAllocator(const GrowingAllocator& o) : m_block(o.m_block) {}
		GrowingAllocator(GrowingAllocator&& o) noexcept : m_block(o.m_block) {}
		void* Allocate(u64 size, u64 align) { return m_block->Allocate(size, align, TC("GrowingAllocator")); }
		void Deallocate(void* p, u64 size, u64 align) { (void)size;(void)align; m_block->Free(p); }
		bool TryGrowInPlace(void* p, u64 oldSize, u64 newSize) { return m_block->TryGrowTail(p, oldSize, newSize); }
		MemoryBlock* m_block;
	};

	struct GrowingAllocatorNoLock
	{
		GrowingAllocatorNoLock(MemoryBlock* block) : m_block(block) {}
		GrowingAllocatorNoLock(const GrowingAllocatorNoLock& o) : m_block(o.m_block) {}
		GrowingAllocatorNoLock(GrowingAllocatorNoLock&& o) noexcept : m_block(o.m_block) {}
		void* Allocate(u64 size, u64 align) { return m_block->AllocateNoLock(size, align, TC("GrowingAllocator")); }
		void Deallocate(void* p, u64 size, u64 align) { (void)p;(void)size;(void)align; }
		bool TryGrowInPlace(void* p, u64 oldSize, u64 newSize) { return m_block->TryGrowTailNoLock(p, oldSize, newSize); }
		MemoryBlock* m_block;
	};

	template<typename Type>
	struct BlockAllocator
	{
	public:
		BlockAllocator(MemoryBlock& memory) : m_memory(memory)
		{
		}

		void* Allocate()
		{
			SCOPED_FUTEX(m_lock, lock);
			if (m_nextFree)
			{
				void* ptr = (void*)m_nextFree;
				m_nextFree = *(u64*)m_nextFree;
				return ptr;
			}
			void* mem = m_memory.Allocate(sizeof(Type), alignof(Type), TC("BlockAllocator"));
			return mem;
		}

		void Free(void* mem)
		{
			#if UBA_DEBUG
			memset(mem, 0xFE, sizeof(Type));
			#endif
			SCOPED_FUTEX(m_lock, lock);
			*(u64*)mem = m_nextFree;
			m_nextFree = u64(mem);
		}

		Futex m_lock;
		MemoryBlock& m_memory;
		u64 m_nextFree = 0;
	};

	inline u8 HexToByte(tchar c)
	{
		if (c >= '0' && c <= '9') return u8(c - '0');
		else if (c >= 'a' && c <= 'f') return u8(c - 'a' + 10);
		else if (c >= 'A' && c <= 'F') return u8(c - 'A' + 10);
		else return 0; // Invalid hex character
	}
	constexpr tchar g_hexChars[] = TC("0123456789abcdef");

	// TODO: These are backwards but changing would break cas storage
	inline u32 ValueToString(tchar* out, int capacity, u64 value)
	{
		(void)capacity;
		tchar* it = out;
		for (int i=0;i!=8;++i)
		{
			*it++ = g_hexChars[(value >> 4) & 0xf];
			*it++ = g_hexChars[value & 0xf];
			value = value >> 8;
			if (!value)
				break;
		}
		*it = 0;
		return u32(it - out);
	}

	// TODO: These are backwards but changing would break cas storage
	inline u64 StringToValue(const tchar* str, u64 len)
	{
		u64 v = 0;
		const tchar* pos = str + len;
		while (pos != str)
		{
			u8 b = HexToByte(*--pos);
			u8 a = HexToByte(*--pos);
			v = u64(v << 8) | u64(a << 4 | b);
		}

		return v;
	}

	inline u64 StringToValue2(const tchar* str, u64 len)
	{
		u64 v = 0;
		const tchar* pos = str;
		while (*pos)
		{
			u8 a = HexToByte(*pos++);
			u8 b = HexToByte(*pos++);
			v = u64(v << 8) | u64(a << 4 | b);
		}

		return v;
	}

	constexpr tchar g_base64Chars[] = TC("0123456789ABCDEFGHIJKLMNOPQRSTUVWXYZabcdefghijklmnopqrstuvwxyz");

	inline u64 StringToValueBase62(const tchar* str, u64 len)
	{
		u64 result = 0;
		const tchar* it = str;
		const tchar* end = it + len;
		while (it != end)
		{
			tchar c = *it++;
			result *= 62;
			if ('0' <= c && c <= '9')
				result += c - '0';
			else if ('A' <= c && c <= 'Z')
				result += c - 'A' + 10;
			else // if ('a' <= c && c <= 'z')
				result += c - 'a' + 36;
		}
		return result;
	}

	inline u32 RoundUpPow2(u64 value)
	{
		u32 v = u32(value);
		v--;
		v |= v >> 1;
		v |= v >> 2;
		v |= v >> 4;
		v |= v >> 8;
		v |= v >> 16;
		v++;
		return v;
	}

	bool SupportsHugePages();
	u64 GetHugePageCount();
}
