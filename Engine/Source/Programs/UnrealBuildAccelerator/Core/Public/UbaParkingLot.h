// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBase.h"
#include "UbaFunctional.h"

#if UBA_USE_PARKINGLOT

#include <atomic>
#include <type_traits>

namespace uba::ParkingLot
{
	struct WaitState final
	{
		/** Did this thread wait? True only if canWait returned true. */
		bool didWait = false;
		/** Did this wake from a wait? True only if a Wake call woke the thread, false for timeouts. */
		bool didWake = false;
		/** Optional value that was provided by the callback in WakeOne. */
		u64 wakeToken = 0;
	};

	struct WakeState final
	{
		/** Did a thread wake up? */
		bool didWake = false;
		/** Does the queue MAYBE have another thread waiting on this address? */
		bool hasWaitingThreads = false;
	};

	namespace Private
	{
		UBA_API WaitState Wait(const void* address, bool (*canWait)(void*), void* canWaitContext, void (*beforeWait)(void*), void* beforeWaitContext);
		UBA_API WaitState WaitFor(const void* address, bool (*canWait)(void*), void* canWaitContext, void (*beforeWait)(void*), void* beforeWaitContext, u32 waitTimeMs);
		UBA_API void WakeOne(const void* address, u64 (*onWakeState)(void*, WakeState), void* onWakeStateContext);
	} // Private

	/** Queue the calling thread to wait if canWait returns true. beforeWait is only called if canWait returns true. */
	inline WaitState Wait(const void* address, FunctionWithContext<bool ()> canWait, FunctionWithContext<void ()> beforeWait)
	{
		return Private::Wait(address, canWait.GetFunction(), canWait.GetContext(), beforeWait.GetFunction(), beforeWait.GetContext());
	}

	/** Queue the calling thread to wait if canWait returns true. beforeWait is only called if canWait returns true. */
	inline WaitState WaitFor(const void* address, FunctionWithContext<bool ()> canWait, FunctionWithContext<void ()> beforeWait, u32 waitTimeMs)
	{
		return Private::WaitFor(address, canWait.GetFunction(), canWait.GetContext(), beforeWait.GetFunction(), beforeWait.GetContext(), waitTimeMs);
	}

	/** Wake one thread from the queue of threads waiting on the address. */
	inline void WakeOne(const void* address, FunctionWithContext<u64 (WakeState)> onWakeState)
	{
		return Private::WakeOne(address, onWakeState.GetFunction(), onWakeState.GetContext());
	}

	/** Wake one thread from the queue of threads waiting on the address. */
	UBA_API WakeState WakeOne(const void* address);

	/** Wake up to wakeCount threads from the queue of threads waiting on the address. */
	UBA_API u32 WakeMultiple(const void* address, u32 wakeCount);

	/** Wake all threads from the queue of threads waiting on the address. */
	UBA_API u32 WakeAll(const void* address);

	/**
	 * A mutex that is the size of a pointer and does not depend on ParkingLot.
	 *
	 * Prefer CriticalSection or ReaderWriterLock to WordMutex whenever possible.
	 * This mutex is not fair and does not support recursive locking.
	 */
	class WordMutex final
	{
	public:
		constexpr WordMutex() = default;

		WordMutex(const WordMutex&) = delete;
		WordMutex& operator=(const WordMutex&) = delete;

		bool TryEnter();
		void Enter();
		void Leave();

	private:
		void EnterSlow();
		void LeaveSlow(u64 currentState);

		static constexpr u64 isLockedFlag = 1 << 0;
		static constexpr u64 isQueueLockedFlag = 1 << 1;
		static constexpr u64 queueMask = ~(isLockedFlag | isQueueLockedFlag);

		std::atomic<u64> state = 0;
	};
} // uba::ParkingLot

#endif // UBA_USE_PARKINGLOT