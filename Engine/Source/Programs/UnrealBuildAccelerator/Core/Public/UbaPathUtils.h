// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaPlatform.h"
#include "UbaStringBuffer.h"

namespace uba
{
	#if !defined(UBA_IS_DETOURED_INCLUDE)
	u32 GetLongPathNameW(const tchar* lpszShortPath, tchar* lpszLongPath, u32 cchBuffer);
	#define Local_GetLongPathNameW GetLongPathNameW
	#endif

	inline bool IsUncPath(const tchar* path) { return IsWindows ? (path[0] == '\\' && path[1] == '\\') : false; }
	inline bool IsUncPath(StringView path) { return IsUncPath(path.data); }

	inline bool FixPath2(const tchar* fileName, u32 fileNameLen, const tchar* workingDir, u64 workingDirCharLen, tchar* buffer, u64 bufferCharCapacity, u32* outBufferCharLen)
	{
		buffer[1] = 0;
		UBA_ASSERT(fileName);
		UBA_ASSERTF(workingDir == nullptr || !*workingDir || workingDir[workingDirCharLen-1] == PathSeparator, TC("WorkingDir needs to end with path separator"));

#if PLATFORM_WINDOWS

		if (fileNameLen == ~0u)
			fileNameLen = TStrlen(fileName);

		const tchar* read = fileName;
		tchar* write = buffer;

		tchar lastLastChar = 0;
		tchar lastChar = 0;
		bool hasDotBeforeSlash = false;
		bool hasDotDot = false;
		bool seenNonBackslash = false;

		if (fileName[0] == '\"')
		{
			++read;
			--fileNameLen;
		}

		StringBuffer<1024> fullName;
		while (true)
		{
			tchar temp[1024];
			if (read == fullName.data)
			{
				TStrcpy_s(temp, sizeof_array(temp), read);
				read = temp;
			}

			StringView sv(read, fileNameLen);
			u32 tildePos;
			if (!sv.Contains('~', &tildePos))
				break;
			
			// Since this might be a in memory file we can't use the actual full name, so let's find first slash after '~'
			StringView tilde = sv.Skip(tildePos);
			u32 backslashAfterTilde = tilde.count;
			bool hasBackslashAfterTilde = tilde.Contains('\\', &backslashAfterTilde);

			fullName.Clear().Append(sv.data, tildePos + backslashAfterTilde);
			u32 len = Local_GetLongPathNameW(fullName.data, fullName.data, sizeof_array(fullName));
			if (!len)
			{
				if (outBufferCharLen)
					*outBufferCharLen = 0;
				return false;
			}
			fullName.count = len;
			if (hasBackslashAfterTilde)
				fullName.Append(tilde.Skip(backslashAfterTilde));
			read = fullName.data;
			fileNameLen = fullName.count;
		}

		if (read[0] == '\\')
		{
			const tchar* prevRead = read;
			if (read[1] == '\\')
			{
				if (read[2] == '?' && read[3] == '\\')
					read += 4;
				else if (read[2] == '.' && read[3] == '\\')
				{
					if (read[5] != ':') // Non volume path.. keep prefix
					{
						UBA_ASSERT(false);
						*write++ = '\\';
						*write++ = '\\';
						*write++ = '.';
						*write++ = '\\';
					}
					read += 4;
				}
			}
			else if (read[1] == '?' && read[2] == '?' && read[3] == '\\')
			{
				read += 4;
				if (read[0] == 'U' && read[1] == 'N' && read[2] == 'C')
				{
					*write++ = '\\';
					read += 3;
				}
			}
			fileNameLen -= u32(read - prevRead);
		}

		bool isAbsolute = false;

		const tchar* readEnd = read + fileNameLen;
		while (read < readEnd)
		{
			tchar c = *read++;
			if (!c)
				break;

			if (c == '/')
				c = '\\';
			else if (c == '\"')
			{
				*write = 0;
				break;
			}
			else if (c == '.' && lastChar == '.')
				hasDotDot = true;
			else if (c == ':')
			{
				if ((write - buffer) != 1) // colon must be on position 1
					return false;
				isAbsolute = true;
			}
			if (c == '\\')
			{
				if (lastChar == '.')
					hasDotBeforeSlash = true;
				if (lastChar == '\\' && seenNonBackslash)
					continue;
			}
			else
			{
				seenNonBackslash = true;
			}
			*write++ = c;
			lastLastChar = lastChar;
			lastChar = c;
		}

		if (lastChar == '.' && lastLastChar == '\\') // Fix path <path>\.
			write -= 2;

		if (lastChar == '\\')
			--write;
		*write = 0;

		u64 charLen = u64(write - buffer + 1);
		bool startsWithDoubleBackslash = false;

		if (lastChar == '.' && lastLastChar == 0) // Sometimes path is '.'
		{
			UBA_ASSERTF(workingDir && *workingDir, TC("Working dir is null or empty"));
			UBA_ASSERTF(workingDirCharLen < bufferCharCapacity, TC("%llu < %llu"), workingDirCharLen, bufferCharCapacity);
			memcpy(buffer, workingDir, workingDirCharLen*sizeof(tchar));
			buffer[workingDirCharLen - 1] = 0;
			charLen = workingDirCharLen;
		}
		else if (IsUncPath(buffer)) // Network path, or pipe or something
		{
			startsWithDoubleBackslash = true;
		}
		else if (!isAbsolute) // If not absolute, add current dir
		{
			tchar* copyFrom = buffer;
			if (copyFrom[0] == '\\')
			{
				++copyFrom;
				--charLen;
			}

			bool hasWorkDir = workingDir && *workingDir;
			UBA_ASSERTF(hasWorkDir, TC("No working dir provided but path is relative (%s)"), buffer);
			if (!hasWorkDir)
				return false;
			tchar temp2[1024];
			UBA_ASSERTF(workingDirCharLen + charLen < sizeof_array(temp2), TC("%llu + %llu < %i"), workingDirCharLen, charLen, sizeof_array(temp2));
			memcpy(temp2, workingDir, workingDirCharLen*sizeof(tchar));
			memcpy(temp2 + workingDirCharLen, copyFrom, charLen*sizeof(tchar));
			charLen += workingDirCharLen;
			UBA_ASSERTF(charLen+1 <= bufferCharCapacity, TC("%llu+1 <= %llu"), charLen, bufferCharCapacity);
			memcpy(buffer, temp2, (charLen+1)*sizeof(tchar));
			startsWithDoubleBackslash = IsUncPath(buffer);
		}
		else if (lastChar == '.' && charLen == 4) // X:.  .. this expands to X:\ unless working dir matches drive, then it becomes working dir
		{
			UBA_ASSERT(workingDir && *workingDir);
			if (ToLower(fileName[0]) == ToLower(workingDir[0]))
			{
				memcpy(buffer, workingDir, workingDirCharLen * sizeof(tchar));
				buffer[workingDirCharLen - 1] = 0;
				charLen = workingDirCharLen;
			}
			else
			{
				--charLen; // Turn it to X:
			}
		}

		if (hasDotDot || hasDotBeforeSlash) // Clean up \..\ and such
		{
			write = buffer;
			if (startsWithDoubleBackslash)
				write += 2;
			read = write;

			tchar* folders[128];
			u32 folderCount = 0;

			tchar lastLastLastChar = 0;
			lastLastChar = 0;
			lastChar = 0;
			while (true)
			{
				tchar c = *read;

				if (c == '\\' || c == 0)
				{
					if (lastChar == '.' && lastLastChar == '.' && lastLastLastChar == '\\')
					{
						if (folderCount > 1)
							--folderCount;
						write = folders[folderCount - 1];
					}
					else if (lastChar == '.' && lastLastChar == '\\')
					{
						UBA_ASSERT(folderCount > 0);
						if (folderCount > 0)
							write = folders[folderCount - 1];
					}
					else if (lastChar == '\\')
					{
						--write;
					}
					else
						folders[folderCount++] = write;
					if (c == 0)
						break;
				}

				lastLastLastChar = lastLastChar;
				lastLastChar = lastChar;
				lastChar = c;

				*write = *read;
				read++;
				write++;
			}

			UBA_ASSERT(write);
			if (write)
				*write = 0;
			charLen = u32(write - buffer + 1);
		}
		
		if (buffer[charLen - 2] == '\\')
		{
			--charLen;
			buffer[charLen-1] = 0;
		}
		else if (charLen == 3) // If it is only <drive>:\ we re-add the last backslash
		{
			buffer[2] = '\\';
			buffer[3] = 0;
			++charLen;
		}

		UBA_ASSERTF(charLen <= bufferCharCapacity, TC("Buffer overflow (Capacity %llu) fixing path %s"), bufferCharCapacity, fileName);

		if (outBufferCharLen)
			*outBufferCharLen = u32(charLen - 1); // Remove terminator
		return true;
#else
		if (fileName[0] == '\"')
			++fileName;

		StringBuffer<MaxPath> tmp;
		if (fileName[0] == '~')
		{
			const char* homeDir = getenv("HOME");
			tmp.Append(homeDir).EnsureEndsWithSlash().Append(fileName + 1);
			fileName = tmp.data;
		}

		u64 memPos = 0;
		if (fileName[0] != '/')
		{
			UBA_ASSERTF(workingDir && *workingDir, "Need workingDir to fix path %s", fileName);
			UBA_ASSERTF(workingDirCharLen < bufferCharCapacity, TC("Buffer overflow (Capacity %llu, workingDirCharLen %llu) fixing path %s with workingDir %s"), bufferCharCapacity, workingDirCharLen, fileName, workingDir);
			memcpy(buffer, workingDir, workingDirCharLen);
			memPos = workingDirCharLen;
		}
		else
		{
			while (fileName[1] == '/')
				++fileName;
		}
		u64 len = strlen(fileName);
		UBA_ASSERTF(memPos + len + 1 <= bufferCharCapacity, TC("Buffer overflow (Capacity %llu, memPos %llu, len %llu) fixing path %s"), bufferCharCapacity, memPos, len, fileName);
		memcpy(buffer + memPos, fileName, len);
		memPos += len;
		if (buffer[memPos -1] == '\"')
			--memPos;
		buffer[memPos] = 0;
		//printf("PATH: %s\n", buffer);
		//fflush(stdout);

		{
			char* write = buffer;
			char* read = write;
			char* folders[128];
			u32 folderCount = 0;

			char lastLastLastChar = 0;
			char lastLastChar = 0;
			char lastChar = 0;
			while (true)
			{
				tchar c = *read;

				if (c == '/' || c == 0)
				{
					if (lastChar == '.' && lastLastChar == '.' && lastLastLastChar == '/')
					{
						if (folderCount > 1)
							--folderCount;
						write = folders[folderCount - 1];
					}
					else if (lastChar == '.' && lastLastChar == '/')
					{
						//folderCount -= 1;
						write = folders[folderCount - 1];
					}
					else if (lastChar == '/')
					{
						--write;
					}
					else
						folders[folderCount++] = write;
					if (c == 0)
						break;
				}

				lastLastLastChar = lastLastChar;
				lastLastChar = lastChar;
				lastChar = c;

				*write = *read;
				read++;
				write++;
			}

			*write = 0;
			memPos = u32(write - buffer);
		}
		
		if (memPos == 0) // If it is only drive '/' we re-add the last slash
		{
			buffer[0] = '/';
			buffer[1] = 0;
			memPos = 1;
		}
		UBA_ASSERTF(memPos + 1 <= bufferCharCapacity, TC("Buffer overflow (Capacity %llu, memPos %llu) fixing path %s"), bufferCharCapacity, memPos, fileName);
		buffer[memPos] = 0;

		if (outBufferCharLen)
			*outBufferCharLen = memPos;
		return true;
#endif
	}

	inline bool FixPath(const tchar* fileName, const tchar* workingDir, u64 workingDirCharLen, StringBufferBase& buffer)
	{
		u32 count = buffer.count;
		bool res = FixPath2(fileName, ~0u, workingDir, workingDirCharLen, buffer.data + count, buffer.capacity - count, &buffer.count);
		buffer.count += count;
		return res;
	}

	inline bool FixPath(StringBufferBase& out, StringView fileName, StringView workingDir)
	{
		u32 count = out.count;
		bool res = FixPath2(fileName.data, fileName.count, workingDir.data, workingDir.count, out.data + count, out.capacity - count, &out.count);
		out.count += count;
		return res;
	}

	inline bool IsAbsolutePath(const tchar* path)
	{
		if (!path)
			return false;
		if constexpr (IsWindows)
			return (path[0] && path[1] == ':') || IsUncPath(path);
		else
			return path[0] == '/';
	}

	inline bool IsAbsolutePath(StringView path)
	{
		if (!path.count)
			return false;
		if constexpr (IsWindows)
			return (path[0] && path[1] == ':') || IsUncPath(path);
		else
			return path[0] == '/';
	}
}
