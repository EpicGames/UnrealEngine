// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBase.h"

#if PLATFORM_WINDOWS
	#define WINDOWS_LEAN_AND_MEAN
	#define NOMINMAX
	#include <ntstatus.h>
	#define WIN32_NO_STATUS
	#define _WINSOCK_DEPRECATED_NO_WARNINGS
	#include <ws2tcpip.h>
	#include <windows.h>

	#if defined( __clang_analyzer__ )
	#define ANALYSIS_NORETURN __attribute__((analyzer_noreturn))
	#else
	#define ANALYSIS_NORETURN __analysis_noreturn
	#endif
#elif UBA_STUB_BUILD
	// Freestanding stub: pure C headers only — anything pulling in <cmath>/<memory>/pthread/libstdc++ must stay behind `#if !UBA_STUB_BUILD`.
	// libc isn't linked; stub provides bodies via raw-syscall wrappers.
	#include <stddef.h>
	#include <stdarg.h>
	#include <stdio.h>
	#include <stdlib.h>
	#include <string.h>
	#include <errno.h>
	#include <fcntl.h>
	#include <unistd.h>
	#include <limits.h>
	#include <sys/mman.h>
	#include <sys/stat.h>
	#include <sys/syscall.h>
	#include <time.h>
	#include <atomic>      // UbaSynchronization
	#include <functional>  // hash/equal_to (DirectoryTable)
	#include <utility>     // std::pair
	#define ANALYSIS_NORETURN
	// -ffreestanding strips these from stdlib; shared core references them on dead paths or via stub-local bodies — declare so .cpp compiles.
	extern "C" char* getenv(const char* name);
	extern "C" void* aligned_alloc(size_t alignment, size_t size);
	extern "C" void free(void* ptr);
#else
	#include <fcntl.h>
	#include <wchar.h>
	#include <stdio.h>
	#include <stdarg.h>
	#include <math.h>
	#include <string.h>
	#include <sys/mman.h>
	#include <sys/syscall.h>
	#include <sys/socket.h>
	#include <sys/stat.h>
	#include <sys/wait.h>
	#include <unistd.h>
	#include <spawn.h>
	#include <signal.h>

	#if PLATFORM_MAC
	#include <sys/random.h>
    #include <sys/posix_shm.h>
	#include  <mach-o/dyld.h>
	#else
	#include <linux/random.h>
	#include <time.h>
	#include <termios.h>
	#include <stropts.h>
	#include <sys/select.h>
	#include <sys/ioctl.h>
	#endif

	#define ANALYSIS_NORETURN // __attribute__((analyzer_noreturn))
#endif

#if PLATFORM_CPU_X86_FAMILY
	#include <immintrin.h>
#endif

#define UBA_EXPERIMENTAL 0

#if UBA_DEBUG && defined(_MSC_VER)
	#define UBA_FMT_CHECK(Format, ...) do { if (false) { wprintf(Format, ##__VA_ARGS__); } } while(false)
#else
	#define UBA_FMT_CHECK(Format, ...)
#endif

namespace uba
{
	class StringBufferBase;
	struct BinaryReader;
	struct BinaryWriter;

	#ifndef sizeof_array
	#define	sizeof_array(array) int(sizeof(array)/sizeof(array[0]))
	#endif

	#define UBA_SHIPPING_ASSERT(x) do { if (x) break; uba::UbaAssert(TC(""), __FILE__, __LINE__, #x, true, 543221, nullptr, 0); } while(false)
	#define UBA_SHIPPING_ASSERTF(x, fmt, ...) do { UBA_FMT_CHECK(fmt, ##__VA_ARGS__); if (x) break; uba::StringBuffer<> _buf; _buf.Appendf(fmt, ##__VA_ARGS__); uba::UbaAssert(_buf.data, __FILE__, __LINE__, #x, true, 543221, nullptr, 0); } while(false)

	#if UBA_DEBUG
	#define UBA_ASSERT(x) UBA_SHIPPING_ASSERT(x)
	#define UBA_ASSERTF(x, fmt, ...) UBA_SHIPPING_ASSERTF(x, fmt, ##__VA_ARGS__)
	#define UBA_ASSERT_MESSAGEBOX 0
	#define UBA_FOR_ASSERT(...) __VA_ARGS__
	inline constexpr bool IsDebug = true;
	#else
	#define UBA_ASSERT(x) do { } while(false)
	#define UBA_ASSERTF(x, ...) do { } while(false)
	#define UBA_ASSERT_MESSAGEBOX 0
	#define UBA_FOR_ASSERT(...)
	inline constexpr bool IsDebug = false;
	#endif

	#define UBA_NOT_IMPLEMENTED(x) UBA_ASSERTF(false, TC("%s not implemented!"), #x);

	u32 GetCallstack(void** outCallstack, u32 callstackCapacity, u32 skipCallstack, void* contextPtr);
	bool WriteCallstackInfo(BinaryWriter& out, u32 skipCallstack, void* context);
	bool WriteCallstackInfo(BinaryWriter& out, void** callstack, u32 callstackCount);
	void WriteAssertInfo(StringBufferBase& out, const tchar* text, const char* file, u32 line, const char* expr, void* context = nullptr);
	ANALYSIS_NORETURN void UbaAssert(const tchar* text, const char* file, u32 line, const char* expr, bool allowTerminate, u32 terminateCode, void* context, u32 skipCallstackCount);
	ANALYSIS_NORETURN void FatalError(u32 code, const tchar* format, ...);
	using CustomAssertHandler = void(const tchar* text, u32 textLen, void* userData);
	void SetCustomAssertHandler(CustomAssertHandler* handler, void* userData);
	bool CreateGuid(Guid& out);
	const char* GetWineVersion();
	bool IsRunningWine();
	bool IsRunningDarling();
	bool IsRunningArm();
	bool IsEscapePressed();

	void Sleep(u32 milliseconds);

	inline void YieldInSpinWait()
	{
	#if PLATFORM_CPU_X86_FAMILY
		_mm_pause();
	#elif PLATFORM_CPU_ARM_FAMILY
		#if !defined(__clang__)
		__yield();
		#else
		__builtin_arm_yield();
		#endif
	#else
		#error YieldInSpinWait() is not implemented on this architecture.
	#endif
	}

	u32 GetUserDefaultUILanguage();
	u32 GetLastError();
	void SetLastError(u32 error);
	bool GetComputerNameW(StringBufferBase& out);
	bool GetOsVersion(StringBufferBase& outPretty, u32& outValue);
	u32 GetCurrentProcessId();

	void AddExceptionHandler();
	void InitMemory();
	bool SetMemoryWorkingSet(u64 minWorkingSet);
	bool UseSemaphoreForLocalEvents();
	bool UseSemaphoreForSharedEvents();

	inline constexpr u64 PageSize = 64 * 1024;

#if PLATFORM_WINDOWS
	inline constexpr bool CaseInsensitiveFs = true;
	inline constexpr tchar PathSeparator = '\\';
	inline constexpr tchar NonPathSeparator = '/';
	inline constexpr u32 MaxPath = 1024;
	inline DWORD ToLow(u64 v) { LARGE_INTEGER li; li.QuadPart = (LONGLONG)v; return li.LowPart; }
	inline LONG ToHigh(u64 v) { LARGE_INTEGER li; li.QuadPart = (LONGLONG)v; return li.HighPart; }
	inline LARGE_INTEGER ToLargeInteger(u64 v) { LARGE_INTEGER li; li.QuadPart = (LONGLONG)v; return li; }
	inline LARGE_INTEGER ToLargeInteger(u32 high, u32 low) { LARGE_INTEGER li; li.HighPart = (LONG)high; li.LowPart = low; return li; }
	inline bool IsDirectory(u32 attributes) { return (attributes & FILE_ATTRIBUTE_DIRECTORY) != 0; }
	inline bool IsSymlink(u32 attributes) { return false; };
	inline bool IsExecutable(u32 attributes) { return false; }
	inline tchar* TStrdup(const tchar* s) { return _wcsdup(s); }
	#define TStrlen(s) u32(wcslen(s))
	#define TStrchr(a, b) wcschr(a, b)
	#define TStrrchr(a, b) wcsrchr(a, b)
	#define TStrstr(a, b) wcsstr(a, b)
	#define TStrcmp(a, b) wcscmp(a, b)
	#define TSprintf_s swprintf_s
	#define Tvsprintf_s(...) vswprintf_s(__VA_ARGS__)
	#define TStrcpy_s wcscpy_s
	#define TStrcat_s wcscat_s
	#define TFputs fputws
	#define UBA_NOINLINE __declspec(noinline)
	#if UBA_DEBUG
	extern thread_local bool g_expectingAccessViolation;
	#endif
#else
	inline constexpr tchar PathSeparator = '/';
	inline constexpr tchar NonPathSeparator = '\\';
	inline constexpr u32 MaxPath = IsLinux ? 2048 : 1024; // AOSP and similar deep-symlink builds need more headroom; we assert at this limit.
	#define TStrlen(s) u32(strlen(s))
	#define TStrchr(a, b) strchr(a, b)
	#define TStrrchr(a, b) strrchr(a, b)
	#define TStrstr(a, b) strstr(a, b)
	#define TStrcmp(a, b) strcmp(a, b)
	#define TSprintf_s(...) snprintf(__VA_ARGS__)
	#define Tvsprintf_s(a, b, c, d) vsnprintf(a, b, c, d)
	#define TStrcpy_s(a, b, c) strcpy_s(a, b, c)
	#define TStrcat_s(a, b, c) strcat_s(a, b, c)
	#define TFputs fputs
	#define localtime_s(a,b) localtime_r(b,a)
	void strcpy_s(tchar* dest, u64 destCapacity, const tchar* source);
	void strcat_s(tchar* dest, u64 destCapacity, const tchar* source);
	void GetMappingHandleName(StringBufferBase& out, u64 uid);
	bool GetPhysicallyInstalledSystemMemory(u64& outKb);
	inline u64 FromTimeSpec(const timespec& ts) { return u64(ts.tv_sec) * 10'000'000ull + u64(ts.tv_nsec/100); }
	inline timespec ToTimeSpec(u64 time) { timespec ts; ts.tv_sec = time / 10'000'000ull; ts.tv_nsec = (time - (u64(ts.tv_sec) * 10'000'000ull)) * 100; return ts; }
	inline bool IsDirectory(u32 attributes) { return S_ISDIR(attributes); }
	inline bool IsSymlink(u32 attributes) { return S_ISLNK(attributes); };
	inline bool IsExecutable(u32 attributes) { return (attributes & (S_IXUSR | S_IXGRP | S_IXOTH)) != 0; }
	inline tchar* TStrdup(const tchar* s) { return strdup(s); }
	#if PLATFORM_LINUX
	inline constexpr bool CaseInsensitiveFs = false;
	#define st_atimespec st_atim
	#define st_mtimespec st_mtim
	#define st_ctimespec st_ctim
	#else
	inline constexpr bool CaseInsensitiveFs = false; // This can be true since mac can allow insensitive file system.
	#endif
	#define UBA_NOINLINE __attribute__((noinline))
    #define ERROR_SUCCESS 0
	#define DUPLICATE_SAME_ACCESS 0
#endif // PLATFORM_WINDOWS
}
