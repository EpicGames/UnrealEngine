// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBinaryReaderWriter.h"
#include "UbaTimer.h"

namespace uba
{
	class Logger;

	inline void Write(BinaryWriter& writer, const Timer& timer) { writer.Write7BitEncoded(timer.time); writer.Write7BitEncoded(timer.count); }
	inline void Write(BinaryWriter& writer, const AtomicU64& v) { writer.Write7BitEncoded(v.load()); }
	inline bool IsEmpty(const Timer& timer) { return timer.count == 0; }
	inline bool IsEmpty(const AtomicU64& v) { return v.load() == 0; }

	struct ProcessStats
	{
		Timer waitOnResponse;

#define UBA_PROCESS_STATS \
		UBA_PROCESS_STAT(attach, 0) \
		UBA_PROCESS_STAT(detach, 0) \
		UBA_PROCESS_STAT(init, 0) \
		UBA_PROCESS_STAT(createFile, 0) \
		UBA_PROCESS_STAT(closeFile, 0) \
		UBA_PROCESS_STAT(getFullFileName, 0) \
		UBA_PROCESS_STAT(deleteFile, 0) \
		UBA_PROCESS_STAT(moveFile, 0) \
		UBA_PROCESS_STAT(chmod, 17) \
		UBA_PROCESS_STAT(copyFile, 0) \
		UBA_PROCESS_STAT(createProcess, 0) \
		UBA_PROCESS_STAT(updateTables, 0) \
		UBA_PROCESS_STAT(listDirectory, 0) \
		UBA_PROCESS_STAT(createTempFile, 0) \
		UBA_PROCESS_STAT(openTempFile, 0) \
		UBA_PROCESS_STAT(virtualAllocFailed, 0) \
		UBA_PROCESS_STAT(log, 0) \
		UBA_PROCESS_STAT(sendFiles, 0) \
		UBA_PROCESS_STAT(writeFiles, 19) \
		UBA_PROCESS_STAT(queryCache, 24) \
		UBA_PROCESS_STAT(waitDecompress, 30) \
		UBA_PROCESS_STAT(preparseObjFiles, 30) \
		UBA_PROCESS_STAT(fileTable, 30) \
		UBA_PROCESS_STAT(dirTable, 30) \
		UBA_PROCESS_STAT(longPathName, 31) \
		UBA_PROCESS_STAT(createSharedMemory, 50) \
		UBA_PROCESS_STAT(commitSharedMemory, 50) \
		UBA_PROCESS_STAT(getSharedMemory, 50) \
		UBA_PROCESS_STAT(sendInput, 53) \
		UBA_PROCESS_STAT(callstack, 53) \
		UBA_PROCESS_STAT(checkRemapping, 53) \
		UBA_PROCESS_STAT(registerCreateWrite, 53) \
		UBA_PROCESS_STAT(forkProcess, 53) \
		UBA_PROCESS_STAT(execProcess, 53) \
		UBA_PROCESS_STAT(startProcess, 53) \
		UBA_PROCESS_STAT(inheritFiles, 53) \
		UBA_PROCESS_STAT(symlink, 53) \

		#define UBA_PROCESS_STAT(T, ver) Timer T;
		UBA_PROCESS_STATS
		#undef UBA_PROCESS_STAT

		enum
		{
			#define UBA_PROCESS_STAT(var, ver) Bit_##var,
			UBA_PROCESS_STATS
			#undef UBA_PROCESS_STAT
		};

		AtomicU64 startupTime;
		AtomicU64 exitTime;

		// Don't add in GetTotalTime()
		AtomicU64 wallTime;

		// From job object (windows only)
		AtomicU64 cpuTime;
		AtomicU64 peakProcessMemory;
		AtomicU64 peakJobMemory;

		AtomicU64 detoursMemory;
		AtomicU64 peakMemory;

		AtomicU64 iopsRead;
		AtomicU64 iopsWrite;
		AtomicU64 iopsOther;

		AtomicU64 hostTotalTime;

		void Print(Logger& logger, u64 frequency = GetFrequency());

		u64 GetTotalTime();
		u32 GetTotalCount();
		void Read(BinaryReader& reader, u32 version);
		void Add(const ProcessStats& other);

		void Write(BinaryWriter& writer) const
		{
			uba::Write(writer, waitOnResponse);

			u64 bits = 0;
			#define UBA_PROCESS_STAT(var, ver) if (!uba::IsEmpty(var)) bits |= (1llu << Bit_##var);
			UBA_PROCESS_STATS
			#undef UBA_PROCESS_STAT

			writer.Write7BitEncoded(bits);

			#define UBA_PROCESS_STAT(var, ver) if (!uba::IsEmpty(var)) uba::Write(writer, var);
			UBA_PROCESS_STATS
			#undef UBA_PROCESS_STAT

			writer.Write7BitEncoded(startupTime);
			writer.Write7BitEncoded(exitTime);
			writer.Write7BitEncoded(wallTime);
			writer.Write7BitEncoded(cpuTime);
			writer.Write7BitEncoded(peakProcessMemory);
			writer.Write7BitEncoded(peakJobMemory);
			writer.Write7BitEncoded(detoursMemory);
			writer.Write7BitEncoded(peakMemory);
			writer.Write7BitEncoded(iopsRead);
			writer.Write7BitEncoded(iopsWrite);
			writer.Write7BitEncoded(iopsOther);
			writer.Write7BitEncoded(hostTotalTime);
		}
	};

	struct TimeAndBytes : ExtendedTimer
	{
		void operator+=(const TimeAndBytes& o) { time += o.time; count += o.count; bytes += o.bytes; }
		AtomicU64 bytes;
	};

	inline void Write(BinaryWriter& writer, const TimeAndBytes& timer) { writer.Write7BitEncoded(timer.time); writer.Write7BitEncoded(timer.count); writer.Write7BitEncoded(timer.bytes); }

	struct KernelStats
	{
		#define UBA_KERNEL_STATS \
			UBA_KERNEL_STAT(ExtendedTimer, createFile, 0) \
			UBA_KERNEL_STAT(ExtendedTimer, closeFile, 0) \
			UBA_KERNEL_STAT(TimeAndBytes, writeFile, 0) \
			UBA_KERNEL_STAT(TimeAndBytes, memoryCopy, 30) \
			UBA_KERNEL_STAT(TimeAndBytes, readFile, 0) \
			UBA_KERNEL_STAT(ExtendedTimer, setFileInfo, 0) \
			UBA_KERNEL_STAT(ExtendedTimer, getFileInfo, 29) \
			UBA_KERNEL_STAT(ExtendedTimer, createFileMapping, 0) \
			UBA_KERNEL_STAT(ExtendedTimer, mapViewOfFile, 0) \
			UBA_KERNEL_STAT(ExtendedTimer, unmapViewOfFile, 0) \
			UBA_KERNEL_STAT(ExtendedTimer, getFileTime, 0) \
			UBA_KERNEL_STAT(ExtendedTimer, closeHandle, 0) \
			UBA_KERNEL_STAT(ExtendedTimer, traverseDir, 27) \
			UBA_KERNEL_STAT(ExtendedTimer, virtualAlloc, 30) \
			UBA_KERNEL_STAT(TimeAndBytes, memoryCompress, 41) \
			UBA_KERNEL_STAT(ExtendedTimer, renameFile, 43) \
			UBA_KERNEL_STAT(ExtendedTimer, renameFileFallback, 43) \
			UBA_KERNEL_STAT(ExtendedTimer, copyFile, 43) \
			UBA_KERNEL_STAT(ExtendedTimer, moveFile, 43) \
			UBA_KERNEL_STAT(ExtendedTimer, deleteFile, 43) \
			UBA_KERNEL_STAT(ExtendedTimer, virtualFree, 50) \
			UBA_KERNEL_STAT(ExtendedTimer, virtualAlloc2, 50) \
			UBA_KERNEL_STAT(TimeAndBytes, mapViewOfFile3, 50) \
			UBA_KERNEL_STAT(ExtendedTimer, unmapViewOfFile2, 50) \

		#define UBA_KERNEL_STAT(type, var, ver) type var;
		UBA_KERNEL_STATS
		#undef UBA_KERNEL_STAT

		void Read(BinaryReader& reader, u32 version);
		void Print(Logger& logger, bool writeHeader, u64 frequency = GetFrequency());
		bool IsEmpty();
		void Add(const KernelStats& other);
		static KernelStats& GetCurrent();
		static KernelStats& GetGlobal();

		enum
		{
			#define UBA_KERNEL_STAT(type, var, ver) Bit_##var,
			UBA_KERNEL_STATS
			#undef UBA_KERNEL_STAT
		};

		void Write(BinaryWriter& writer)
		{
			u64 bits = 0;
			#define UBA_KERNEL_STAT(type, var, ver) if (var.count) bits |= (1 << Bit_##var);
			UBA_KERNEL_STATS
			#undef UBA_KERNEL_STAT

			writer.Write7BitEncoded(bits);

			#define UBA_KERNEL_STAT(type, var, ver) if (var.count) uba::Write(writer, var);
			UBA_KERNEL_STATS
			#undef UBA_KERNEL_STAT
		}
	};

	struct KernelStatsScope
	{
		KernelStatsScope(KernelStats& stats);
		~KernelStatsScope();

		KernelStats& stats;
	};
}
