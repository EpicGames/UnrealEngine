// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaPlatform.h"

namespace uba
{
	// func is void(const CharType* arg, u32 strArg)
	template<typename CharType, typename Func>
	bool ParseArguments(const CharType* arguments, u64 argumentsLen, const Func& argumentFunc)
	{
		if (argumentsLen == 0)
			return true;

		const CharType* argStart = arguments;
		bool isInArg = false;
		bool isInQuotes = false;
		bool isInSingleQuotes = false;
		bool isInBackslashQuote = false;
		bool isEnd = *arguments == 0;
		CharType currentChar = 0;
		CharType lastChar = 0;
		bool isBackslashOwned = false;

		
		CharType temp[4*1024];

		CharType* argWriteBuffer = temp;
		u64 argBufferSize = sizeof_array(temp);
		auto g = MakeGuard([&]() { if (argWriteBuffer != temp) delete[] argWriteBuffer; });

		for (const CharType* it = arguments; !isEnd; lastChar = currentChar, ++it)
		{
			bool pastEnd = u64(it - arguments) == argumentsLen;
			if (!pastEnd)
				currentChar = *it;
			isEnd = pastEnd || currentChar == 0;
			if (isEnd || currentChar == ' ' || currentChar == '\t' || currentChar == '\n')
			{
				if (isInQuotes || isInSingleQuotes || !isInArg)
					continue;

#if !PLATFORM_WINDOWS
				// Backslash-space in unquoted context (e.g. CARGO_PKG_DESCRIPTION=The\ Rust\ Core\ Library):
				// a lone backslash escapes the following space, keeping the argument alive.
				// A paired backslash (\\) does NOT escape the space -- isBackslashOwned tracks this.
				if (!isEnd && currentChar == TC(' ') && lastChar == TC('\\') && !isBackslashOwned)
					continue;
#endif

				const CharType* argIt = argStart;
				const CharType* argEnd = it;

				if ((pastEnd || *argEnd == '\n') && argStart != argEnd && *(argEnd-1) == '\r')
					--argEnd;

				// Check once if this argument might overflow the buffer
				u64 maxArgLen = argEnd - argStart;
				if (maxArgLen + 1 > argBufferSize)
				{
					// This argument is too large for our buffer - allocate exactly what we need
					if (argWriteBuffer != temp)
						delete[] argWriteBuffer;
					argBufferSize = maxArgLen + 16;
					argWriteBuffer = new CharType[argBufferSize];
				}

				CharType* argWriteIt = argWriteBuffer;
				CharType lastChar2 = 0;
				isBackslashOwned = false;
				bool inSingleQ = false;
				bool inDoubleQ = false;
				while (argIt != argEnd)
				{
					// Single quotes: strip markers; content inside is literal (no " or \ processing)
					if (*argIt == '\'' && !inDoubleQ)
					{
						inSingleQ = !inSingleQ;
						lastChar2 = 0;
						++argIt;
						continue;
					}

					// Double quotes: strip markers (with \" escape support); only outside single-quoted regions
					if (*argIt == '\"' && !inSingleQ)
					{
						if (lastChar2 == '\\' && !isBackslashOwned)
						{
							// Escaped quote: emit a literal " (already written the \, replace it)
							// and do NOT toggle inDoubleQ — we remain in the same quoting mode.
							argWriteIt[-1] = '\"';
						}
						else
						{
							inDoubleQ = !inDoubleQ;
						}
						lastChar2 = 0;
						++argIt;
						continue;
					}

					// Backslash pair tracking only outside single-quoted regions
					if (!inSingleQ)
					{
						if (*argIt == '\\' && lastChar2 == '\\')
							isBackslashOwned = !isBackslashOwned;
						else
							isBackslashOwned = false;
					}

#if !PLATFORM_WINDOWS
					// POSIX unquoted backslash: strip it and emit the next character literally.
					// Covers \%, \(, \), \\ → \, \ → space, etc.
					if (!inSingleQ && !inDoubleQ && *argIt == '\\' && argIt + 1 != argEnd)
					{
						++argIt; // skip the backslash
						*argWriteIt++ = *argIt;
						lastChar2 = *argIt;
						isBackslashOwned = false;
						++argIt;
						continue;
					}
#endif

					*argWriteIt++ = *argIt;

					lastChar2 = *argIt;
					++argIt;
				}

				if (argWriteBuffer != argWriteIt)
				{
					*argWriteIt = 0;
					argumentFunc(argWriteBuffer, u32(argWriteIt - argWriteBuffer));
				}
				isInArg = false;
				isBackslashOwned = false;
				continue;
			}

			if (!isInArg)
			{
				isInArg = true;
				argStart = it;
				if (currentChar == '\"')
					isInQuotes = true;
				else if (currentChar == '\'')
					isInSingleQuotes = true;
				continue;
			}

			// Toggle single-quote mode (only outside double-quoted regions)
			if (*it == '\'' && !isInQuotes)
			{
#if !PLATFORM_WINDOWS
				// POSIX: a backslash-escaped apostrophe outside all quotes is a literal
				// character (e.g. d\'Antras in a shell-escaped argument). It must not
				// toggle single-quote mode — otherwise the argument never terminates
				// and, at end-of-input, line 41 drops it entirely. The inner pass
				// below strips the backslash.
				if (!isInSingleQuotes && lastChar == '\\' && !isBackslashOwned)
				{
					// literal apostrophe; fall through without toggling
				}
				else
#endif
					isInSingleQuotes = !isInSingleQuotes;
			}

			// Toggle double-quote mode (only outside single-quoted regions)
			if (*it == '\"' && !isInSingleQuotes)
			{
				if (isInQuotes && lastChar == '\\' && (!isBackslashOwned && !isInBackslashQuote))
					continue;

				isInQuotes = !isInQuotes;
				isInBackslashQuote = isInQuotes && lastChar == '\\';
			}

			if (*it == '\\' && lastChar == '\\')
				isBackslashOwned = !isBackslashOwned;
			else
				isBackslashOwned = false;
		}
		return true;
	}

	template<typename Func>
	bool ParseArguments(const tchar* arguments, const Func& argumentFunc)
	{
		return ParseArguments(arguments, TStrlen(arguments), argumentFunc);
	}
}
