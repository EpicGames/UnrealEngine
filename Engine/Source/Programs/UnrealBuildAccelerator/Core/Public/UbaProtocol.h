// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBase.h"

namespace uba
{
	#define UBA_PROCESS_MESSAGES \
		UBA_PROCESS_MESSAGE(Init) \
		UBA_PROCESS_MESSAGE(CreateFile) \
		UBA_PROCESS_MESSAGE(RegisterFileForWrite) \
		UBA_PROCESS_MESSAGE(GetFullFileName) \
		UBA_PROCESS_MESSAGE(GetLongPathName) \
		UBA_PROCESS_MESSAGE(CloseFile) \
		UBA_PROCESS_MESSAGE(DeleteFile) \
		UBA_PROCESS_MESSAGE(CopyFile) \
		UBA_PROCESS_MESSAGE(MoveFile) \
		UBA_PROCESS_MESSAGE(Chmod) \
		UBA_PROCESS_MESSAGE(Symlink) \
		UBA_PROCESS_MESSAGE(CreateDirectory) \
		UBA_PROCESS_MESSAGE(RemoveDirectory) \
		UBA_PROCESS_MESSAGE(ListDirectory) \
		UBA_PROCESS_MESSAGE(UpdateTables) \
		UBA_PROCESS_MESSAGE(GetWrittenFiles) \
		UBA_PROCESS_MESSAGE(CreateProcess) \
		UBA_PROCESS_MESSAGE(StartProcess) \
		UBA_PROCESS_MESSAGE(ForkProcess) \
		UBA_PROCESS_MESSAGE(ExecProcess) \
		UBA_PROCESS_MESSAGE(ExitChildProcess) \
		UBA_PROCESS_MESSAGE(InheritFiles) \
		UBA_PROCESS_MESSAGE(VirtualAllocFailed) \
		UBA_PROCESS_MESSAGE(Log) \
		UBA_PROCESS_MESSAGE(InputDependencies) \
		UBA_PROCESS_MESSAGE(Exit) \
		UBA_PROCESS_MESSAGE(FlushWrittenFiles) \
		UBA_PROCESS_MESSAGE(UpdateEnvironment) \
		UBA_PROCESS_MESSAGE(GetNextProcess) \
		UBA_PROCESS_MESSAGE(Custom) \
		UBA_PROCESS_MESSAGE(SHGetKnownFolderPath) \
		UBA_PROCESS_MESSAGE(RpcCommunication) \
		UBA_PROCESS_MESSAGE(HostRun) \
		UBA_PROCESS_MESSAGE(ResolveCallstack) \
		UBA_PROCESS_MESSAGE(CheckRemapping) \
		UBA_PROCESS_MESSAGE(CheckRemapping2) \
		UBA_PROCESS_MESSAGE(RunSpecialProgram) \
		UBA_PROCESS_MESSAGE(CreateSharedMemory) \
		UBA_PROCESS_MESSAGE(CommitSharedMemory) \
		UBA_PROCESS_MESSAGE(GetSharedMemory) \
		UBA_PROCESS_MESSAGE(SnapshotDuplicated) \

	enum MessageType : u8
	{
		MessageType_None = 0,
		#define UBA_PROCESS_MESSAGE(type) MessageType_##type,
		UBA_PROCESS_MESSAGES
		#undef UBA_PROCESS_MESSAGE
	};

	inline constexpr u32 ProcessMessageVersion = 1352;

	inline constexpr u32 CommunicationMemSize = IsWindows ? 64*1024 : 64*1024*2; // Macos expands some commandlines to be crazy long

	inline constexpr u32 FileMappingTableMaxSize = 16 * 1024 * 1024;
	inline constexpr u32 DirTableMaxSize = 256 * 1024 * 1024;
	inline constexpr u32 OverlayTableMaxSize = 16 * 1024 * 1024; // Seen ~600kb max size for ltcg/thinlto linking of our biggest binaries (one temp file per .o file)
	inline constexpr u32 DirTableSnapshotMaxSize = 96 * 1024 * 1024;
	inline constexpr u32 EmergencyShutdownExitCode = 9886;
}

// Build can override (e.g. static-detour stub wants debug log without UBA_DEBUG asserts).
#ifndef UBA_DEBUG_LOG_ENABLED
	#if UBA_DEBUG
	#define UBA_DEBUG_LOG_ENABLED 1
	#else
	#define UBA_DEBUG_LOG_ENABLED 0
	#endif
#endif
#ifndef UBA_DEBUG_VALIDATE
#define UBA_DEBUG_VALIDATE 0
#endif
