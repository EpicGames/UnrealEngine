// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaFileMappingHandle.h"
#include "UbaMemory.h"
#include "UbaVector.h"

namespace uba
{
	struct BinaryReader;
	struct FileMappingBackend;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	using SharedMemoryAllocatorHandle = FileMappingHandle;
	
	struct SharedMemoryHandle
	{
		inline bool IsValid() const { return internalHandle != 0; }
		inline u64 ToU64() const { return internalHandle; }
		inline bool operator==(const SharedMemoryHandle& o) const { return internalHandle == o.internalHandle; }
		static SharedMemoryHandle FromU64(u64 v) { return { v }; }
		u64 internalHandle = 0;
	};

	// Must stay parallel to MemoryMapType (UbaFileMapping.h) - SharedMemoryView casts between them.
	enum SharedMemoryMapType { SharedMemoryMapType_ReadOnly, SharedMemoryMapType_ReadWrite, SharedMemoryMapType_CopyOnWrite, SharedMemoryMapType_LazyCopyOnWrite };

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class SharedMemoryView
	{
	public:
		SharedMemoryView(FileMappingBackend& backend);
		SharedMemoryView(SharedMemoryView&&) = default;

		bool Init(SharedMemoryAllocatorHandle handle, u64 capacity, void* baseAddress = nullptr, const tchar* hint = TC(""));
		bool Init(SharedMemoryAllocatorHandle handle, u8* sharedMemory, BinaryReader& reader, u64 capacity, const tchar* hint = TC(""), SharedMemoryMapType type = SharedMemoryMapType_ReadOnly);

		bool AddRequestedMemory(SharedMemoryAllocatorHandle handle, BinaryReader& reader, SharedMemoryMapType mapType = SharedMemoryMapType_ReadWrite, u64 offset = 0, const tchar* hint = TC(""), u8 init = 0);
		void Reset();

		u8* GetMemory() { return m_memory; }
		u64 GetMappedSize() { return m_mappedSize; }
		u64 GetCapacity() { return m_capacity; }

		void SetMemory(u8* memory, u64 capacity);
		u8* DetachMemory();
		~SharedMemoryView();

	private:
		bool InternalInit(SharedMemoryAllocatorHandle handle, u64 capacity, void* baseAddress, const tchar* hint);
		bool AddRequestedMemory(SharedMemoryAllocatorHandle handle, BinaryReader& reader, u32 sliceCount, SharedMemoryMapType mapType, u64 offset, const tchar* hint, u8 init);
		void ReferenceExternalMemory(u8* memory, u64 mappedSize);
		bool ExtendMemory(SharedMemoryAllocatorHandle handle, u64 offset, u64 size, SharedMemoryMapType mapType, const tchar* hint, u8 init = 0);

		FileMappingBackend& m_fileMappingBackend;
		u8* m_memory = nullptr;
		u64 m_mappedSize = 0;
		u64 m_capacity = 0;

		Vector<u64> m_additionalMappings;

		SharedMemoryView(const SharedMemoryView&) = delete;
		SharedMemoryView& operator=(const SharedMemoryView&) = delete;
		friend class SharedMemoryAllocator;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool IsMemoryHandle(const tchar* name);
	bool IsFreeableMemoryHandle(const tchar* name);
	void GetMappingHandleAndOffset(const tchar* str, u64& outHandle, u64& outOffset);
	inline constexpr tchar MemoryHandleChar = '^';
	inline constexpr tchar FreeableMemoryHandleChar = '>';
	inline constexpr tchar WrittenMemoryHandleChar = ':';

	////////////////////////////////////////////////////////////////////////////////////////////////////
}

template<> struct std::hash<uba::SharedMemoryHandle> { size_t operator()(const uba::SharedMemoryHandle& h) const { return h.internalHandle; } };
