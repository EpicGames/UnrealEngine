// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaString.h"
#include <stdarg.h>

namespace uba
{
	struct BinaryReader;
	struct StringView;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool StartsWith(const tchar* data, const tchar* str, bool ignoreCase = true);
	bool EndsWith(const tchar* str, u64 strLen, const tchar* value, u64 valueLen, bool ignoreCase);
	bool Contains(const tchar* str, const tchar* sub, bool ignoreCase = true, const tchar** pos = nullptr);
	bool Equals(const tchar* str1, const tchar* str2, bool ignoreCase = true);
	bool Equals(const tchar* str1, const tchar* str2, u64 count, bool ignoreCase = true);
	void Replace(tchar* str, tchar from, tchar to);
	void FixPathSeparators(tchar* str);
	bool Parse(u64& out, const tchar* str, u64 strLen);
	inline void ToLower(tchar* str) { while (tchar c = *str) { if (c >= 'A' && c <= 'Z') *str = c - 'A' + 'a'; ++str; } }
	inline tchar ToLower(tchar c) { return (c >= 'A' && c <= 'Z') ? (c - 'A' + 'a') : c;}
	inline tchar ToUpper(tchar c) { return (c >= 'a' && c <= 'z') ? (c - 'a' + 'A') : c;}
	const tchar* GetFileName(const tchar* path);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class StringBufferBase
	{
	public:
		StringBufferBase& Append(const StringBufferBase& str) { return Append(str.data, str.count); }
		StringBufferBase& Append(const TString& str) { return Append(str.data(), str.size()); }
		StringBufferBase& Append(const tchar* str);
		StringBufferBase& Append(const tchar c) { return Append(&c, 1); }
		StringBufferBase& Append(const tchar* str, u64 charCount);
		StringBufferBase& Append(StringView view);
		StringBufferBase& Appendf(const tchar* format, ...);
		StringBufferBase& AppendDir(const StringBufferBase& str);
		StringBufferBase& AppendDir(const tchar* dir);
		StringBufferBase& AppendFileName(const tchar* str);
		StringBufferBase& AppendHex(u64 v);
		StringBufferBase& AppendBase62(u64 v);
		StringBufferBase& AppendValue(u64 v);
		StringBufferBase& Append(const tchar* format, va_list& args);
		StringBufferBase& Prepend(StringView view, u32 overwriteCount = 0);
		StringBufferBase& Resize(u64 newSize);
		StringBufferBase& Clear();

		StringBufferBase& ReadString(BinaryReader& reader);

		#if PLATFORM_WINDOWS
		StringBufferBase& Append(const char* str);
		StringBufferBase& Append(const char* str, u32 charCount);
		#endif

		template<typename Container, typename Func>
		StringBufferBase& Join(const Container& c, const Func& f, const tchar* separator = TC(","));

		tchar operator[](u64 i) const { return data[i]; }
		tchar& operator[](u64 i) { return data[i]; }

		bool IsEmpty() const { return count == 0; }
		bool StartsWith(const tchar* str, bool ignoreCase = true) const { return uba::StartsWith(data, str, ignoreCase); }
		bool StartsWith(StringView str, bool ignoreCase = true) const;
		bool EndsWith(const tchar* value, bool ignoreCase = true) const;
		bool EndsWith(StringView value, bool ignoreCase = true) const;
		bool Contains(tchar c) const;
		bool Contains(const tchar* str, bool ignoreCase = true, const tchar** pos = nullptr) const { return uba::Contains(data, str, ignoreCase, pos); }
		bool Equals(const tchar* str, bool ignoreCase = true) const { return uba::Equals(data, str, ignoreCase); }
		bool Equals(StringView str, bool ignoreCase = true) const;
		const tchar* First(tchar c, u64 offset = 0) const;
		const tchar* Last(tchar c, u64 offset = 0) const;
		const tchar* GetFileName() const { return uba::GetFileName(data); }
		inline StringBufferBase& Replace(tchar from, tchar to) { uba::Replace(data, from, to);  return *this; }

		StringBufferBase& EnsureEndsWithSlash();
		StringBufferBase& FixPathSeparators();
		StringBufferBase& MakeLower();

		bool Parse(u64& out) const;
		bool Parse(u32& out, u64 offset = 0) const;
		bool Parse(u16& out, u64 offset = 0) const;
		bool Parse(bool& out) const;
		bool Parse(float& out) const;
		bool Parse(TString& out, u64 offset = 0) const;
		bool Parse(StringBufferBase& out, u64 offset = 0) const;
		u32 Parse(char* out, u64 outCapacity) const; // Note, return value contains length + null termination.. zero if failed

		TString ToString() const { return TString(data, data + count); }
		TString ToStringAndClear() { TString s(data, data + count); Clear(); return s; }
		bool ToFalse() const { return false; }

		u32 count;
		u32 capacity;
		tchar data[1];

	protected:
		StringBufferBase(u32 c) : capacity(c) { count = 0; *data = 0; }
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	enum NoInitEnum { NoInit };

	struct StringView
	{
		constexpr StringView() : data(TC("")), count(0) {}
		constexpr StringView(NoInitEnum) {}
		constexpr StringView(const StringView&) = default;
		constexpr StringView(const tchar* d, u32 c) : data(d), count(c) {}
		constexpr StringView(const StringBufferBase& sb) : data(sb.data), count(sb.count) {}
		constexpr StringView(const tchar* d) = delete;
		StringView(const TString& str) : data(str.data()), count(u32(str.size())) {}

		tchar operator[](u64 i) const { return data[i]; }
		tchar Last() { return count ? data[count - 1] : 0; }

		bool IsEmpty() const { return count == 0; }
		bool IsTerminated() const { return data[count] == 0; }
		bool StartsWith(StringView str, bool ignoreCase = true) const; // Requires terminated StringViews
		bool StartsWith(const tchar* str, bool ignoreCase = true) const { return uba::StartsWith(data, str, ignoreCase); } // Requires terminated StringView
		bool EndsWith(StringView value, bool ignoreCase = true) const { return uba::EndsWith(data, count, value.data, value.count, ignoreCase); }
		bool EndsWith(const tchar* value, bool ignoreCase = true) const;
		bool Contains(tchar c, u32* outPos = nullptr) const;
		bool Contains(const tchar* str, bool ignoreCase = true) const { return uba::Contains(data, str, ignoreCase); } // Requires terminated StringViews
		bool Contains(StringView str, bool ignoreCase = true) const { return uba::Contains(data, str.data, ignoreCase); } // Requires terminated StringViews
		bool Equals(const tchar* str, bool ignoreCase = true) const { return uba::Equals(data, str, ignoreCase); } // Requires terminated StringViews
		bool Equals(StringView str, bool ignoreCase = true) const { return count == str.count && uba::Equals(data, str.data, count, ignoreCase); }
		StringView GetFileName() const;
		StringView GetPath(bool keepLastSlash = false) const;
		StringView Skip(u32 skipCount) const { return StringView(data + skipCount, count - skipCount); }
		StringView RemoveSuffix(u32 removeCount) { return StringView(data, count - removeCount); }
		StringView SubStr(u32 startOffset, u32 length) const { return StringView(data + startOffset, length); }
		StringView Trim();

		const tchar* CheckTerminated() const;

		TString ToString() const { return TString(data, data + count); }

		bool Parse(u64& out) const;
		bool Parse(u32& out, u64 offset = 0) const;
		bool Parse(u16& out, u64 offset = 0) const;
		bool Parse(bool& out) const;
		bool Parse(float& out) const;
		bool Parse(TString& out, u64 offset = 0) const;
		bool Parse(StringBufferBase& out, u64 offset = 0) const;
		u32 Parse(char* out, u64 outCapacity) const; // Note, return value contains length + null termination.. zero if failed

		const tchar* data;
		u32 count;
	};

	StringView ToView(const tchar* s);
	StringView ToView(StringView sv);
	StringView ToView(StringBufferBase& sb);

	template<typename T, size_t size>
	constexpr StringView AsView(T (&buffer)[size]) { return StringView(buffer, size-1); }

	#define TCV(x) AsView(TC(x))

	////////////////////////////////////////////////////////////////////////////////////////////////////

	template<u32 Capacity = 512>
	class StringBuffer : public StringBufferBase
	{
	public:
		StringBuffer() : StringBufferBase(Capacity) { *buf = 0; }
		explicit StringBuffer(const TString& str) : StringBufferBase(Capacity) { *buf = 0; Append(str); }
		explicit StringBuffer(const tchar* str) : StringBufferBase(Capacity) { *buf = 0; if (str) Append(str); }
		StringBuffer(const StringBufferBase& str) : StringBufferBase(Capacity) { *buf = 0; Append(str); }
		StringBuffer(StringView str) : StringBufferBase(Capacity) { *buf = 0; Append(str); }
	private:
		tchar buf[Capacity];
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct LastErrorToText : StringBuffer<256>
	{
		LastErrorToText();
		LastErrorToText(u32 lastError);
		operator const tchar* () const { return data; };
	};

	template<typename Container, typename Func>
	StringBufferBase& StringBufferBase::Join(const Container& c, const Func& f, const tchar* separator)
	{
		bool isFirst = true;
		for (auto& e : c)
		{
			if (!isFirst)
				Append(separator);
			isFirst = false;
			f(e);
		}
		return *this;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}