// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBase.h"
#include <atomic>
#include <utility>

#define UBA_TRACK_CONTENTION 0

#define STRING_JOIN(arg1, arg2) STRING_JOIN_INNER(arg1, arg2)
#define STRING_JOIN_INNER(arg1, arg2) arg1 ## arg2

namespace uba
{
	template<typename Type>
	using Atomic = std::atomic<Type>;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct AtomicU64 : Atomic<u64>
	{
		AtomicU64(u64 initialValue = 0) : Atomic<u64>(initialValue) {}
		AtomicU64(AtomicU64&& o) noexcept : Atomic<u64>(o.load()) {}
		void operator=(u64 o) { store(o); }
		void operator=(const AtomicU64& o) { store(o); }
	};

	template<typename T>
	void AtomicMax(Atomic<T>& target, T value);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	#if UBA_USE_PARKINGLOT
	class CriticalSection
	{
	public:
		CriticalSection() = default;

		[[nodiscard]] bool TryEnter();
		void Enter();
		void Leave();

	private:
		void EnterSlow(u32 currentState, u32 currentThreadId);
		void WakeWaitingThread();

		static constexpr u32 mayHaveWaitingLockFlag = 1 << 0;
		static constexpr u32 lockCountShift = 1;
		static constexpr u32 lockCountMask = 0xffff'fffe;

		Atomic<u32> state = 0;
		Atomic<u32> threadId = 0;

		CriticalSection(const CriticalSection&) = delete;
		CriticalSection& operator=(const CriticalSection&) = delete;
	};
	#else
	class CriticalSection
	{
	public:
		CriticalSection(bool recursive = true);
		~CriticalSection();

		void Enter();
		void Leave();

		bool TryEnter() const;

	private:
		#if PLATFORM_WINDOWS
		u64 data[5];
		#else
		u64 data[10];
		#endif

		CriticalSection(const CriticalSection&) = delete;
		CriticalSection& operator=(const CriticalSection&) = delete;
	};
	#endif

	////////////////////////////////////////////////////////////////////////////////////////////////////

	#if UBA_USE_PARKINGLOT
	class ReaderWriterLock
	{
	public:
		ReaderWriterLock() = default;

		inline bool TryEnter()
		{
			u32 expected = state.load(std::memory_order_relaxed);
			return !(expected & (isLockedFlag | readLockCountMask)) &&
				state.compare_exchange_strong(expected, expected | isLockedFlag,
					std::memory_order_acquire, std::memory_order_relaxed);
		}

		inline void Enter()
		{
			u32 expected = 0;
			if (state.compare_exchange_weak(expected, isLockedFlag, std::memory_order_acquire, std::memory_order_relaxed))
			{
				return;
			}
			EnterSlow();
		}

		inline void Leave()
		{
			// Unlock immediately to allow other threads to acquire the lock while this thread looks for a thread to wake.
			u32 lastState = state.fetch_sub(isLockedFlag, std::memory_order_release);
			if (!(lastState & (mayHaveWaitingLockFlag | mayHaveWaitingReadLockFlag)))
			{
				return;
			}
			WakeWaitingThreads(lastState);
		}

		inline void EnterRead() const
		{
			u32 expected = state.load(std::memory_order_relaxed);
			if (!(expected & (isLockedFlag | mayHaveWaitingLockFlag)) &&
				state.compare_exchange_weak(expected, expected + (1 << readLockCountShift),
					std::memory_order_acquire, std::memory_order_relaxed))
			{
				return;
			}
			EnterReadSlow();
		}

		inline void LeaveRead() const
		{
			// Unlock immediately to allow other threads to acquire the lock while this thread looks for a thread to wake.
			const u32 lastState = state.fetch_sub(1 << readLockCountShift, std::memory_order_release);
			constexpr u32 wakeState = mayHaveWaitingLockFlag | (1 << readLockCountShift);
			if ((lastState & ~mayHaveWaitingReadLockFlag) != wakeState)
			{
				return;
			}
			WakeWaitingThread();
		}

	private:
		void EnterSlow();
		void EnterReadSlow() const;
		void WakeWaitingThread() const;
		void WakeWaitingThreads(u32 lastState);

		const void* GetLockAddress() const;
		const void* GetReadLockAddress() const;

		static constexpr u32 isLockedFlag = 1 << 0;
		static constexpr u32 mayHaveWaitingLockFlag = 1 << 1;
		static constexpr u32 mayHaveWaitingReadLockFlag = 1 << 2;
		static constexpr u32 readLockCountShift = 3;
		static constexpr u32 readLockCountMask = 0xffff'fff8;

		mutable Atomic<u32> state = 0;

		ReaderWriterLock(const ReaderWriterLock&) = delete;
		ReaderWriterLock& operator=(const ReaderWriterLock&) = delete;
	};
	#else
	class ReaderWriterLock
	{
	public:
		ReaderWriterLock();
		~ReaderWriterLock();

		void EnterRead() const;
		void LeaveRead() const;

		void Enter();
		void Leave();

		bool TryEnter();

	private:

		#if PLATFORM_WINDOWS
		u64 data[1];
		#elif PLATFORM_LINUX
		u64 data[7];
		#else
		u64 data[25];
		#endif

		ReaderWriterLock(const ReaderWriterLock&) = delete;
		ReaderWriterLock& operator=(const ReaderWriterLock&) = delete;
	};
	#endif

	////////////////////////////////////////////////////////////////////////////////////////////////////

	#if !UBA_TRACK_CONTENTION
	#define SCOPED_READ_LOCK(readerWriterLock, name) ScopedReadLock name(readerWriterLock);
	#define SCOPED_WRITE_LOCK(readerWriterLock, name) ScopedWriteLock name(readerWriterLock);
	#define SCOPED_CRITICAL_SECTION(cs, name) ScopedCriticalSection name(cs);
	#define SCOPED_FUTEX(cs, name) ScopedFutex name(cs);
	#define SCOPED_FUTEX_READ(cs, name) ScopedFutexRead name(cs);
	#else
	u64 GetTime();
	struct ContentionTracker { void Add(u64 t) { time += t; ++count; }; Atomic<u64> time; Atomic<u64> count; const char* file; u64 line; };
	ContentionTracker& GetContentionTracker(const char* file, u64 line);

	#define SCOPED_LOCK_INTERNAL(lock, lockType, name) \
		u64 STRING_JOIN(contentionStart, __LINE__) = GetTime(); \
		lockType name(lock); \
		static ContentionTracker& STRING_JOIN(tracker, __LINE__) = GetContentionTracker(__FILE__, __LINE__); \
		STRING_JOIN(tracker, __LINE__).Add(GetTime() - STRING_JOIN(contentionStart, __LINE__));

	#define SCOPED_READ_LOCK(readerWriterLock, name) SCOPED_LOCK_INTERNAL(readerWriterLock, ScopedReadLock, name)
	#define SCOPED_WRITE_LOCK(readerWriterLock, name) SCOPED_LOCK_INTERNAL(readerWriterLock, ScopedWriteLock, name)
	#define SCOPED_CRITICAL_SECTION(cs, name) SCOPED_LOCK_INTERNAL(cs, ScopedCriticalSection, name)
	#define SCOPED_FUTEX(futex, name) SCOPED_LOCK_INTERNAL(futex, ScopedFutex, name)
	#define SCOPED_FUTEX_READ(futex, name) SCOPED_LOCK_INTERNAL(futex, ScopedFutex, name)
	#endif

	////////////////////////////////////////////////////////////////////////////////////////////////////

	template <typename MutexType>
	class ScopedCriticalSection 
	{
	public:
		ScopedCriticalSection(MutexType& mutex) : m_mutex(mutex), m_active(true) { mutex.Enter(); }
		~ScopedCriticalSection() { Leave(); }
		void Enter() { if (m_active) return; m_mutex.Enter(); m_active = true; }
		void Leave() { if (!m_active) return; m_mutex.Leave(); m_active = false; }
	private:
		MutexType& m_mutex;
		bool m_active;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class ScopedReadLock
	{
	public:
		ScopedReadLock(const ReaderWriterLock& lock) : m_lock(lock) { lock.EnterRead(); }
		~ScopedReadLock() { Leave(); }
		inline void Enter() { if (m_active) return; m_active = true; m_lock.EnterRead(); }
		inline void Leave() { if (!m_active) return; m_active = false; m_lock.LeaveRead(); }

		const ReaderWriterLock& m_lock;
		bool m_active = true;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class ScopedWriteLock
	{
	public:
		ScopedWriteLock(ReaderWriterLock& lock) : m_lock(lock) { lock.Enter(); }
		~ScopedWriteLock() { Leave(); }
		inline void Enter() { if (m_active) return; m_active = true; m_lock.Enter(); }
		inline void Leave() { if (!m_active) return; m_active = false; m_lock.Leave(); }

		ReaderWriterLock& m_lock;
		bool m_active = true;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	#if UBA_USE_PARKINGLOT || PLATFORM_WINDOWS
	using Futex = ReaderWriterLock;
	using ScopedFutex = ScopedWriteLock;
	using ScopedFutexRead = ScopedReadLock;
	#else
	class Futex : public CriticalSection { public: Futex() : CriticalSection(false) {} };
	using ScopedFutex = ScopedCriticalSection<Futex>;
	using ScopedFutexRead = ScopedCriticalSection<Futex>;
	#endif


	////////////////////////////////////////////////////////////////////////////////////////////////////

	template<typename Lambda>
	struct ScopeGuard
	{
		void Cancel() { m_called = true; }
		auto Execute() { bool called = m_called; m_called = true; if constexpr (!std::is_void_v<std::invoke_result_t<Lambda>>) return (called ? decltype(m_lambda()){} : m_lambda()); else if (!called) m_lambda(); }

		ScopeGuard(Lambda lambda) : m_lambda(lambda) {}
		ScopeGuard(ScopeGuard&& o) { m_lambda = std::move(o.m_lambda); o.m_called = true; }
		ScopeGuard() = delete;
		ScopeGuard(const ScopeGuard& o) = delete;
		void operator=(const ScopeGuard&) = delete;
		void operator=(ScopeGuard&&) = delete;
		~ScopeGuard() { if (!m_called) m_lambda(); }
	private:
		Lambda m_lambda;
		bool m_called = false;
	};
	template<typename Lambda>
	ScopeGuard<Lambda> MakeGuard(Lambda&& lambda) { return ScopeGuard<Lambda>(std::move(lambda)); }

	////////////////////////////////////////////////////////////////////////////////////////////////////

	template<typename T>
	void AtomicMax(Atomic<T>& target, T value)
	{
		T current = target.load(std::memory_order_relaxed);
		while (current < value && !target.compare_exchange_weak(current, value, std::memory_order_relaxed, std::memory_order_relaxed)) {}
	}

	template<typename T>
	void AtomicMin(Atomic<T>& target, T value)
	{
		T current = target.load(std::memory_order_relaxed);
		while (current > value && !target.compare_exchange_weak(current, value, std::memory_order_relaxed, std::memory_order_relaxed)) {}
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
