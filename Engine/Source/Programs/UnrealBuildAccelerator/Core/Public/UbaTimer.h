// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaSynchronization.h"

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	u64 GetTime();							// Counter for time. Note this has no specific start time for 0
	u64 GetFrequency();						// Number of counts per second. GetTime() / GetFrequency() = seconds
	u64 GetSystemTimeUs();					// Get time in microseconds since 1 Jan 1970 00:00 UTC

	inline u64 TimeToUs(u64 time, u64 frequency)	{ return time * 1'000'000 / frequency; }
	inline u64 TimeToMs(u64 time, u64 frequency)	{ return time * 1'000 / frequency; }
	inline float TimeToS(u64 time, u64 frequency)	{ return float(double(time) / double(frequency)); }

	inline u64 TimeToUs(u64 time)					{ return TimeToUs(time, GetFrequency()); }
	inline u64 TimeToMs(u64 time)					{ return TimeToMs(time, GetFrequency()); }
	inline float TimeToS(u64 time)					{ return TimeToS(time, GetFrequency()); }

	inline u64 UsToTime(u64 us)						{ return us * GetFrequency() / 1'000'000; }
	inline u64 MsToTime(u64 ms)						{ return ms * GetFrequency() / 1'000; }

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct Timer
	{
        Timer(Timer &o) { time.store(o.time.load()); count.store(o.count.load());}
        Timer() = default;
		Timer(u64 t, u32 c) : time(t), count(c) {}
		Timer(Timer&& o) noexcept : time(o.time.load()), count(o.count.load()) {}
		void operator=(const Timer& o) { time.store(o.time.load()); count.store(o.count.load()); }
		Atomic<u64> time;
		Atomic<u32> count;
		void operator+=(const Timer& o) { time += o.time; count += o.count; }
		bool operator==(const Timer& o) const { return time == o.time && count == o.count; }
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct ExtendedTimer : Timer
	{
		AtomicU64 longest;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct TimerScope
	{
		TimerScope(Timer& t) : timer(t), start(GetTime()) { }
		~TimerScope() { if (start == ~u64(0)) return; u64 t = GetTime(); timer.time += t - start; ++timer.count; }
		void Leave() { u64 t = GetTime(); if (t > start) timer.time += t - start; start = ~u64(0); ++timer.count; }
		void Cancel() { start = ~u64(0); }
		Timer& timer;
		u64 start;
		TimerScope(const TimerScope&) = delete;
		void operator=(const TimerScope&) = delete;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct ExtendedTimerScope 
	{
		ExtendedTimerScope(ExtendedTimer& t) : timer(t), start(GetTime()) { ++timer.count; }
		~ExtendedTimerScope()
		{
			u64 t = GetTime() - start;
			timer.time += t;
			u64 prev = timer.longest.load();
			while(prev < t && !timer.longest.compare_exchange_weak(prev, t)) {}
		}
		ExtendedTimer& timer;
		u64 start;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
