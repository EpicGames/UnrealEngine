// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaMemory.h"
#include <tuple>
#include <utility>

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	namespace detail
	{
		template<typename T>
		inline T next_pow2(T v)
		{
			if (v < 2)return 2;
			--v;
			v |= v >> 1; v |= v >> 2; v |= v >> 4;
			if constexpr (sizeof(T) >= 2) v |= v >> 8;
			if constexpr (sizeof(T) >= 4) v |= v >> 16;
			if constexpr (sizeof(T) >= 8) v |= v >> 32;
			return v + 1;
		}
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	template<typename Key, typename Value, typename Hash = std::hash<Key>, typename KeyEqual = std::equal_to<>, typename NodeAlloc = Allocator, typename BucketAlloc = Allocator, typename SizeType = u64>
	class UnorderedMap
	{
	public:
		using key_type    = Key;
		using mapped_type = Value;
		using value_type  = std::pair<const Key, Value>;
		using size_type   = SizeType;
		using hasher      = Hash;
		using key_equal   = KeyEqual;

	private:
		struct Node
		{
			value_type kv;
			Node* next;

			template<typename K_, typename... VArgs>
			Node(Node* n, std::piecewise_construct_t, std::tuple<K_> kt, std::tuple<VArgs...> vt)
				: kv(std::piecewise_construct, std::move(kt), std::move(vt))
				, next(n) {}

			template<typename P>
			Node(Node* n, P&& p)
				: kv(std::forward<P>(p))
				, next(n) {}
		};

	public:
		class const_iterator;

		class iterator
		{
		public:
			using value_type = UnorderedMap::value_type;
			using reference  = value_type&;
			using pointer    = value_type*;

			iterator() = default;
			iterator(const UnorderedMap* m, size_type b, Node* n) : m_map(m), m_bucket(b), m_node(n) {}

			reference operator*()  const { UBA_ASSUME(m_node); return m_node->kv; }
			pointer   operator->() const { UBA_ASSUME(m_node); return &m_node->kv; }

			iterator& operator++()
			{
				if (m_node->next) { m_node = m_node->next; return *this; }
				size_type bc = m_map->m_bucketCount;
				for (++m_bucket; m_bucket < bc; ++m_bucket)
					if (m_map->m_buckets[m_bucket]) { m_node = m_map->m_buckets[m_bucket]; return *this; }
				m_node = nullptr;
				m_bucket = bc;
				return *this;
			}
			iterator operator++(int) { iterator t = *this; ++(*this); return t; }

			bool operator==(const iterator& o) const { return m_node == o.m_node; }
			bool operator!=(const iterator& o) const { return m_node != o.m_node; }

			const UnorderedMap* m_map = nullptr;
			size_type m_bucket = 0;
			Node* m_node = nullptr;
		};

		class const_iterator
		{
		public:
			using value_type = UnorderedMap::value_type;
			using reference  = const value_type&;
			using pointer    = const value_type*;

			const_iterator() = default;
			const_iterator(const UnorderedMap* m, size_type b, const Node* n) : m_map(m), m_bucket(b), m_node(n) {}
			const_iterator(iterator it) : m_map(it.m_map), m_bucket(it.m_bucket), m_node(it.m_node) {}

			reference operator*()  const { UBA_ASSUME(m_node); return m_node->kv; }
			pointer   operator->() const { UBA_ASSUME(m_node); return &m_node->kv; }

			const_iterator& operator++()
			{
				if (m_node->next) { m_node = m_node->next; return *this; }
				size_type bc = m_map->m_bucketCount;
				for (++m_bucket; m_bucket < bc; ++m_bucket)
					if (m_map->m_buckets[m_bucket]) { m_node = m_map->m_buckets[m_bucket]; return *this; }
				m_node = nullptr;
				m_bucket = bc;
				return *this;
			}
			const_iterator operator++(int) { const_iterator t = *this; ++(*this); return t; }

			bool operator==(const const_iterator& o) const { return m_node == o.m_node; }
			bool operator!=(const const_iterator& o) const { return m_node != o.m_node; }

			const UnorderedMap* m_map = nullptr;
			size_type m_bucket = 0;
			const Node* m_node = nullptr;
		};

		UnorderedMap() = default;
		explicit UnorderedMap(const NodeAlloc& na, const BucketAlloc& ba = {}) : m_nodeAlloc(na), m_bucketAlloc(ba) {}
		explicit UnorderedMap(size_type reserveCount) { if (reserveCount) reserve(reserveCount); }
		UnorderedMap(const UnorderedMap& o) { for (const auto& kv : o) emplace(kv.first, kv.second); }
		UnorderedMap(UnorderedMap&& o) noexcept { swap(o); }

		~UnorderedMap()
		{
			clear();
			if (m_buckets) m_bucketAlloc.Deallocate(m_buckets, sizeof(Node*)*m_bucketCount, alignof(Node*));
		}

		UnorderedMap& operator=(const UnorderedMap& o)
		{
			if (this != &o)
			{
				clear();
				for (const auto& kv : o) emplace(kv.first, kv.second);
			}
			return *this;
		}

		UnorderedMap& operator=(UnorderedMap&& o) noexcept
		{
			if (this != &o)
			{
				clear();
				if (m_buckets) m_bucketAlloc.Deallocate(m_buckets, m_bucketCount);
				m_buckets = nullptr; m_bucketCount = 0; m_size = 0;
				swap(o);
			}
			return *this;
		}

		// ---- size / iteration ----

		size_type size() const { return m_size; }
		bool empty() const { return m_size == 0; }

		iterator       begin()       { return first_iter<iterator>(); }
		iterator       end()         { return iterator(this, m_bucketCount, nullptr); }
		const_iterator begin() const { return first_iter<const_iterator>(); }
		const_iterator end()   const { return const_iterator(this, m_bucketCount, nullptr); }
		const_iterator cbegin() const { return begin(); }
		const_iterator cend()   const { return end(); }

		// ---- lookup ----

		iterator find(const Key& k)
		{
			if (!m_bucketCount) return end();
			size_type b = bucket_of(k);
			for (Node* n = m_buckets[b]; n; n = n->next)
				if (KeyEqual{}(n->kv.first, k)) return iterator(this, b, n);
			return end();
		}

		const_iterator find(const Key& k) const
		{
			if (!m_bucketCount) return end();
			size_type b = bucket_of(k);
			for (const Node* n = m_buckets[b]; n; n = n->next)
				if (KeyEqual{}(n->kv.first, k)) return const_iterator(this, b, n);
			return end();
		}

		bool contains(const Key& k) const { return find(k) != end(); }

		size_type count(const Key& k) const { return contains(k) ? 1 : 0; }

		// ---- insertion ----

		Value& operator[](const Key& k)
		{
			ensure_capacity();
			size_type b = bucket_of(k);
			for (Node* n = m_buckets[b]; n; n = n->next)
				if (KeyEqual{}(n->kv.first, k)) return n->kv.second;
			Node* nn = create_node_piecewise(m_buckets[b], k);
			m_buckets[b] = nn;
			++m_size;
			return nn->kv.second;
		}

		std::pair<iterator, bool> insert(const value_type& p)
		{
			return emplace_impl(p.first, p);
		}

		std::pair<iterator, bool> insert(value_type&& p)
		{
			return emplace_impl(p.first, std::move(p));
		}

		// Also accept `std::pair<K, V>` (as opposed to pair<const K, V>) so that
		// braced-init like `map.insert({k, v})` and `insert(std::make_pair(k,v))`
		// work. Templated to avoid ambiguity with the value_type overloads.
		template<typename P,
		         typename = std::enable_if_t<
		             !std::is_same_v<std::decay_t<P>, value_type> &&
		             std::is_constructible_v<value_type, P&&>>>
		std::pair<iterator, bool> insert(P&& p)
		{
			return emplace_impl(p.first, std::forward<P>(p));
		}

		// Range insert: insert each pair in [first, last).
		template<typename InputIt,
		         typename = std::enable_if_t<
		             !std::is_same_v<std::decay_t<InputIt>, value_type> &&
		             !std::is_same_v<std::decay_t<InputIt>, std::pair<Key, Value>>>>
		void insert(InputIt first, InputIt last)
		{
			for (; first != last; ++first) insert(*first);
		}

		std::pair<iterator, bool> try_emplace(const Key& k)
		{
			ensure_capacity();
			size_type b = bucket_of(k);
			for (Node* n = m_buckets[b]; n; n = n->next)
				if (KeyEqual{}(n->kv.first, k)) return { iterator(this, b, n), false };
			Node* nn = create_node_piecewise(m_buckets[b], k);
			m_buckets[b] = nn;
			++m_size;
			return { iterator(this, b, nn), true };
		}

		template<typename... Args>
		std::pair<iterator, bool> try_emplace(const Key& k, Args&&... args)
		{
			ensure_capacity();
			size_type b = bucket_of(k);
			for (Node* n = m_buckets[b]; n; n = n->next)
				if (KeyEqual{}(n->kv.first, k)) return { iterator(this, b, n), false };
			Node* nn = create_node_piecewise_args(m_buckets[b], k, std::forward<Args>(args)...);
			m_buckets[b] = nn;
			++m_size;
			return { iterator(this, b, nn), true };
		}

		template<typename... Args>
		std::pair<iterator, bool> emplace(Args&&... args)
		{
			// emplace: construct value_type from args, then insert by its key.
			value_type tmp(std::forward<Args>(args)...);
			return insert(std::move(tmp));
		}

		// ---- removal ----

		size_type erase(const Key& k)
		{
			if (!m_bucketCount) return 0;
			size_type b = bucket_of(k);
			Node** prev = &m_buckets[b];
			while (*prev)
			{
				if (KeyEqual{}((*prev)->kv.first, k))
				{
					Node* t = *prev;
					*prev = t->next;
					destroy_node(t);
					--m_size;
					return 1;
				}
				prev = &(*prev)->next;
			}
			return 0;
		}

		iterator erase(const_iterator it)
		{
			if (it.m_node == nullptr) return end();
			size_type b = it.m_bucket;
			const Node* target = it.m_node;
			Node** prev = &m_buckets[b];
			while (*prev && *prev != target) prev = &(*prev)->next;
			if (!*prev) return end();
			Node* nextNode = (*prev)->next;
			Node* dead = *prev;
			*prev = nextNode;
			destroy_node(dead);
			--m_size;
			if (nextNode) return iterator(this, b, nextNode);
			for (size_type i = b + 1; i < m_bucketCount; ++i)
				if (m_buckets[i]) return iterator(this, i, m_buckets[i]);
			return end();
		}

		iterator erase(iterator it) { return erase(const_iterator(it)); }

		void clear()
		{
			if (!m_buckets) return;
			for (size_type i = 0; i < m_bucketCount; ++i)
			{
				Node* n = m_buckets[i];
				while (n) { Node* nx = n->next; destroy_node(n); n = nx; }
				m_buckets[i] = nullptr;
			}
			m_size = 0;
		}

		void reserve(size_type n)
		{
			size_type want = detail::next_pow2<size_type>(n ? n : 2);
			if (want > m_bucketCount) rehash(want);
		}

		// UB if m_buckets is non-null.
		void set_bucket_alloc(const BucketAlloc& alloc)
		{
			m_bucketAlloc = alloc;
		}

		void compact_buckets(const BucketAlloc& newAlloc, size_type minBucketCount = 0)
		{
			size_type want = m_size;
			if (minBucketCount > want) want = minBucketCount;
			size_type target = detail::next_pow2<size_type>(want ? want : 2);

			BucketAlloc swapAlloc = newAlloc;
			Node** newBuckets = (Node**)swapAlloc.Allocate(sizeof(Node*)*target, alignof(Node*));
			for (size_type i = 0; i < target; ++i) newBuckets[i] = nullptr;
			if (m_buckets)
			{
				for (size_type i = 0; i < m_bucketCount; ++i)
				{
					Node* n = m_buckets[i];
					while (n)
					{
						Node* nx = n->next;
						size_type b = size_type(Hash{}(n->kv.first)) & (target - 1);
						n->next = newBuckets[b];
						newBuckets[b] = n;
						n = nx;
					}
				}
				m_bucketAlloc.Deallocate(m_buckets, sizeof(Node*)*m_bucketCount, alignof(Node*));
			}
			m_buckets = newBuckets;
			m_bucketCount = target;
			m_bucketAlloc = swapAlloc;
		}

		void swap(UnorderedMap& o) noexcept
		{
			Node** tb = m_buckets;      m_buckets = o.m_buckets;         o.m_buckets = tb;
			size_type tc = m_bucketCount; m_bucketCount = o.m_bucketCount; o.m_bucketCount = tc;
			size_type ts = m_size;      m_size = o.m_size;               o.m_size = ts;
		}

		void merge_move(UnorderedMap& o)
		{
			if (&o == this || !o.m_buckets)
				return;
			size_type combined = m_size + o.m_size;
			if (combined > m_bucketCount)
				reserve(combined);
			Node* rejects = nullptr;
			size_type rejectCount = 0;
			for (size_type i = 0; i < o.m_bucketCount; ++i)
			{
				Node* n = o.m_buckets[i];
				o.m_buckets[i] = nullptr;
				while (n)
				{
					Node* nx = n->next;
					size_type b = bucket_of(n->kv.first);
					Node* e = m_buckets[b];
					for (; e; e = e->next)
						if (KeyEqual{}(e->kv.first, n->kv.first)) break;
					if (e)
					{
						n->next = rejects;
						rejects = n;
						++rejectCount;
					}
					else
					{
						n->next = m_buckets[b];
						m_buckets[b] = n;
						++m_size;
					}
					n = nx;
				}
			}
			o.m_buckets[0] = rejects;
			o.m_size = rejectCount;
		}

	private:
		template<typename It>
		It first_iter() const
		{
			for (size_type i = 0; i < m_bucketCount; ++i)
				if (m_buckets[i]) return It(this, i, m_buckets[i]);
			return It(this, m_bucketCount, nullptr);
		}

		size_type bucket_of(const Key& k) const
		{
			return size_type(Hash{}(k)) & (m_bucketCount - 1);
		}

		template<typename... Args>
		std::pair<iterator, bool> emplace_impl(const Key& k, Args&&... args)
		{
			ensure_capacity();
			size_type b = bucket_of(k);
			for (Node* n = m_buckets[b]; n; n = n->next)
				if (KeyEqual{}(n->kv.first, k)) return { iterator(this, b, n), false };
			Node* nn = create_node_from_pair(m_buckets[b], std::forward<Args>(args)...);
			m_buckets[b] = nn;
			++m_size;
			return { iterator(this, b, nn), true };
		}

		Node* create_node_piecewise(Node* next, const Key& k)
		{
			Node* n = (Node*)m_nodeAlloc.Allocate(sizeof(Node), alignof(Node));
			new (n) Node(next, std::piecewise_construct, std::forward_as_tuple(k), std::forward_as_tuple());
			return n;
		}

		template<typename... Args>
		Node* create_node_piecewise_args(Node* next, const Key& k, Args&&... args)
		{
			Node* n = (Node*)m_nodeAlloc.Allocate(sizeof(Node), alignof(Node));
			new (n) Node(next, std::piecewise_construct,
			             std::forward_as_tuple(k),
			             std::forward_as_tuple(std::forward<Args>(args)...));
			return n;
		}

		template<typename P>
		Node* create_node_from_pair(Node* next, P&& p)
		{
			Node* n = (Node*)m_nodeAlloc.Allocate(sizeof(Node), alignof(Node));
			new (n) Node(next, std::forward<P>(p));
			return n;
		}

		void destroy_node(Node* n)
		{
			n->~Node();
			m_nodeAlloc.Deallocate(n, sizeof(Node), alignof(Node));
		}

		void ensure_capacity()
		{
			if (m_bucketCount == 0) { rehash(16); return; }
			// Max load factor = 1.0 — rehash when size+1 would exceed bucket count.
			if (m_size + 1 > m_bucketCount) rehash(m_bucketCount * 2);
		}

		void rehash(size_type newCount)
		{
			Node** newBuckets = (Node**)m_bucketAlloc.Allocate(sizeof(Node*)*newCount, alignof(Node*));
			for (size_type i = 0; i < newCount; ++i) newBuckets[i] = nullptr;
			if (m_buckets)
			{
				for (size_type i = 0; i < m_bucketCount; ++i)
				{
					Node* n = m_buckets[i];
					while (n)
					{
						Node* nx = n->next;
						size_type b = size_type(Hash{}(n->kv.first)) & (newCount - 1);
						n->next = newBuckets[b];
						newBuckets[b] = n;
						n = nx;
					}
				}
				m_bucketAlloc.Deallocate(m_buckets, sizeof(Node*)*m_bucketCount, alignof(Node*));
			}
			m_buckets = newBuckets;
			m_bucketCount = newCount;
		}

		Node**    m_buckets = nullptr;
		size_type m_bucketCount = 0;
		size_type m_size = 0;
		NodeAlloc   m_nodeAlloc;
		BucketAlloc m_bucketAlloc;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	template<typename Key, typename Hash = std::hash<Key>, typename KeyEqual = std::equal_to<>, typename NodeAlloc = Allocator, typename BucketAlloc = Allocator>
	class UnorderedSet
	{
	public:
		using key_type   = Key;
		using value_type = Key;
		using size_type  = u64;

	private:
		struct Node
		{
			Key key;
			Node* next;
			template<typename K_>
			Node(Node* n, K_&& k) : key(std::forward<K_>(k)), next(n) {}
		};

	public:
		class const_iterator
		{
		public:
			const_iterator() = default;
			const_iterator(const UnorderedSet* m, size_type b, const Node* n) : m_set(m), m_bucket(b), m_node(n) {}

			const Key& operator*()  const { UBA_ASSUME(m_node); return m_node->key; }
			const Key* operator->() const { UBA_ASSUME(m_node); return &m_node->key; }

			const_iterator& operator++()
			{
				if (m_node->next) { m_node = m_node->next; return *this; }
				size_type bc = m_set->m_bucketCount;
				for (++m_bucket; m_bucket < bc; ++m_bucket)
					if (m_set->m_buckets[m_bucket]) { m_node = m_set->m_buckets[m_bucket]; return *this; }
				m_node = nullptr;
				m_bucket = bc;
				return *this;
			}
			const_iterator operator++(int) { const_iterator t = *this; ++(*this); return t; }

			bool operator==(const const_iterator& o) const { return m_node == o.m_node; }
			bool operator!=(const const_iterator& o) const { return m_node != o.m_node; }

			const UnorderedSet* m_set = nullptr;
			size_type m_bucket = 0;
			const Node* m_node = nullptr;
		};
		using iterator = const_iterator; // set iterators expose const keys only

		UnorderedSet() = default;
		explicit UnorderedSet(const NodeAlloc& na, const BucketAlloc& ba = {}) : m_nodeAlloc(na), m_bucketAlloc(ba) {}
		explicit UnorderedSet(size_type reserveCount) { if (reserveCount) reserve(reserveCount); }
		UnorderedSet(const UnorderedSet& o) { for (const auto& k : o) insert(k); }
		UnorderedSet(UnorderedSet&& o) noexcept { swap(o); }

		~UnorderedSet()
		{
			clear();
			if (m_buckets) m_bucketAlloc.Deallocate(m_buckets, sizeof(Node*)*m_bucketCount, alignof(Node*));
		}

		UnorderedSet& operator=(const UnorderedSet& o)
		{
			if (this != &o) { clear(); for (const auto& k : o) insert(k); }
			return *this;
		}

		UnorderedSet& operator=(UnorderedSet&& o) noexcept
		{
			if (this != &o)
			{
				clear();
				if (m_buckets) m_bucketAlloc.Deallocate(m_buckets, sizeof(Node*)*m_bucketCount, alignof(Node*));
				m_buckets = nullptr; m_bucketCount = 0; m_size = 0;
				swap(o);
			}
			return *this;
		}

		size_type size() const { return m_size; }
		bool empty() const { return m_size == 0; }

		iterator begin()       { return first_iter(); }
		iterator end()         { return iterator(this, m_bucketCount, nullptr); }
		const_iterator begin() const { return first_iter(); }
		const_iterator end()   const { return const_iterator(this, m_bucketCount, nullptr); }
		const_iterator cbegin() const { return begin(); }
		const_iterator cend()   const { return end(); }

		const_iterator find(const Key& k) const
		{
			if (!m_bucketCount) return end();
			size_type b = bucket_of(k);
			for (const Node* n = m_buckets[b]; n; n = n->next)
				if (KeyEqual{}(n->key, k)) return const_iterator(this, b, n);
			return end();
		}

		iterator find(const Key& k)
		{
			return static_cast<const UnorderedSet*>(this)->find(k);
		}

		bool contains(const Key& k) const { return find(k) != end(); }

		size_type count(const Key& k) const { return contains(k) ? 1 : 0; }

		std::pair<const_iterator, bool> insert(const Key& k)
		{
			ensure_capacity();
			size_type b = bucket_of(k);
			for (Node* n = m_buckets[b]; n; n = n->next)
				if (KeyEqual{}(n->key, k)) return { const_iterator(this, b, n), false };
			Node* nn = create_node(m_buckets[b], k);
			m_buckets[b] = nn;
			++m_size;
			return { const_iterator(this, b, nn), true };
		}

		std::pair<const_iterator, bool> insert(Key&& k)
		{
			ensure_capacity();
			size_type b = bucket_of(k);
			for (Node* n = m_buckets[b]; n; n = n->next)
				if (KeyEqual{}(n->key, k)) return { const_iterator(this, b, n), false };
			Node* nn = create_node(m_buckets[b], std::move(k));
			m_buckets[b] = nn;
			++m_size;
			return { const_iterator(this, b, nn), true };
		}

		// Range insert: insert each key in [first, last).
		template<typename InputIt,
		         typename = std::enable_if_t<!std::is_same_v<std::decay_t<InputIt>, Key>>>
		void insert(InputIt first, InputIt last)
		{
			for (; first != last; ++first) insert(*first);
		}

		template<typename... Args>
		std::pair<const_iterator, bool> emplace(Args&&... args)
		{
			Key tmp(std::forward<Args>(args)...);
			return insert(std::move(tmp));
		}

		size_type erase(const Key& k)
		{
			if (!m_bucketCount) return 0;
			size_type b = bucket_of(k);
			Node** prev = &m_buckets[b];
			while (*prev)
			{
				if (KeyEqual{}((*prev)->key, k))
				{
					Node* t = *prev;
					*prev = t->next;
					destroy_node(t);
					--m_size;
					return 1;
				}
				prev = &(*prev)->next;
			}
			return 0;
		}

		const_iterator erase(const_iterator it)
		{
			if (it.m_node == nullptr) return end();
			size_type b = it.m_bucket;
			const Node* target = it.m_node;
			Node** prev = &m_buckets[b];
			while (*prev && *prev != target) prev = &(*prev)->next;
			if (!*prev) return end();
			Node* nextNode = (*prev)->next;
			Node* dead = *prev;
			*prev = nextNode;
			destroy_node(dead);
			--m_size;
			if (nextNode) return const_iterator(this, b, nextNode);
			for (size_type i = b + 1; i < m_bucketCount; ++i)
				if (m_buckets[i]) return const_iterator(this, i, m_buckets[i]);
			return end();
		}

		void clear()
		{
			if (!m_buckets) return;
			for (size_type i = 0; i < m_bucketCount; ++i)
			{
				Node* n = m_buckets[i];
				while (n) { Node* nx = n->next; destroy_node(n); n = nx; }
				m_buckets[i] = nullptr;
			}
			m_size = 0;
		}

		void reserve(size_type n)
		{
			size_type want = detail::next_pow2<size_type>(n ? n : 2);
			if (want > m_bucketCount) rehash(want);
		}

		void swap(UnorderedSet& o) noexcept
		{
			Node** tb = m_buckets;        m_buckets = o.m_buckets;         o.m_buckets = tb;
			size_type tc = m_bucketCount; m_bucketCount = o.m_bucketCount; o.m_bucketCount = tc;
			size_type ts = m_size;        m_size = o.m_size;               o.m_size = ts;
		}

		void merge_move(UnorderedSet& o)
		{
			if (&o == this || !o.m_buckets)
				return;
			size_type combined = m_size + o.m_size;
			if (combined > m_bucketCount)
				reserve(combined);
			Node* rejects = nullptr;
			size_type rejectCount = 0;
			for (size_type i = 0; i < o.m_bucketCount; ++i)
			{
				Node* n = o.m_buckets[i];
				o.m_buckets[i] = nullptr;
				while (n)
				{
					Node* nx = n->next;
					size_type b = bucket_of(n->key);
					Node* e = m_buckets[b];
					for (; e; e = e->next)
						if (KeyEqual{}(e->key, n->key)) break;
					if (e)
					{
						n->next = rejects;
						rejects = n;
						++rejectCount;
					}
					else
					{
						n->next = m_buckets[b];
						m_buckets[b] = n;
						++m_size;
					}
					n = nx;
				}
			}
			o.m_buckets[0] = rejects;
			o.m_size = rejectCount;
		}

	private:
		const_iterator first_iter() const
		{
			for (size_type i = 0; i < m_bucketCount; ++i)
				if (m_buckets[i]) return const_iterator(this, i, m_buckets[i]);
			return const_iterator(this, m_bucketCount, nullptr);
		}

		size_type bucket_of(const Key& k) const
		{
			return size_type(Hash{}(k)) & (m_bucketCount - 1);
		}

		template<typename K_>
		Node* create_node(Node* next, K_&& k)
		{
			Node* n = (Node*)m_nodeAlloc.Allocate(sizeof(Node), alignof(Node));
			new (n) Node(next, std::forward<K_>(k));
			return n;
		}

		void destroy_node(Node* n)
		{
			n->~Node();
			m_nodeAlloc.Deallocate(n, sizeof(Node), alignof(Node));
		}

		void ensure_capacity()
		{
			if (m_bucketCount == 0) { rehash(16); return; }
			if (m_size + 1 > m_bucketCount) rehash(m_bucketCount * 2);
		}

		void rehash(size_type newCount)
		{
			Node** newBuckets = (Node**)m_bucketAlloc.Allocate(sizeof(Node*)*newCount, alignof(Node*));
			for (size_type i = 0; i < newCount; ++i) newBuckets[i] = nullptr;
			if (m_buckets)
			{
				for (size_type i = 0; i < m_bucketCount; ++i)
				{
					Node* n = m_buckets[i];
					while (n)
					{
						Node* nx = n->next;
						size_type b = size_type(Hash{}(n->key)) & (newCount - 1);
						n->next = newBuckets[b];
						newBuckets[b] = n;
						n = nx;
					}
				}
				m_bucketAlloc.Deallocate(m_buckets, sizeof(Node*)*m_bucketCount, alignof(Node*));
			}
			m_buckets = newBuckets;
			m_bucketCount = newCount;
		}

		Node**    m_buckets = nullptr;
		size_type m_bucketCount = 0;
		size_type m_size = 0;
		NodeAlloc   m_nodeAlloc;
		BucketAlloc m_bucketAlloc;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	template<typename Key, typename Value, typename Hash = std::hash<Key>, typename KeyEqual = std::equal_to<Key>, typename NodeAlloc = GrowingAllocator>
	using GrowingUnorderedMap = UnorderedMap<Key, Value, Hash, KeyEqual, NodeAlloc>;

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
