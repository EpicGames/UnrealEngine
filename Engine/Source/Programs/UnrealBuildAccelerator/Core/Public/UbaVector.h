// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaMemory.h"
#include <initializer_list>
#include <string.h>
#include <type_traits>
#include <utility>

namespace uba
{
	namespace detail
	{
		// Detect whether an allocator exposes `try_grow_in_place(p, oldN, newN)`. Enables
		// Vector to extend in-place on bump allocators (GrowingAllocator[NoLock]) without
		// reallocating + copying the old buffer.
		template<typename A, typename = void>
		struct HasTryGrowInPlace : std::false_type {};

		template<typename A>
		struct HasTryGrowInPlace<A, std::void_t<
			decltype(std::declval<A&>().TryGrowInPlace(
				std::declval<typename A::value_type*>(),
				std::declval<u64>(),
				std::declval<u64>()))>> : std::true_type {};
	}

	// Minimal std::vector replacement. Contiguous storage; doubles capacity on grow.
	template<typename T, typename Alloc = Allocator>
	class Vector
	{
	public:
		using value_type = T;
		using size_type = u64;
		using iterator = T*;
		using const_iterator = const T*;
		using reference = T&;
		using const_reference = const T&;

		Vector() = default;

		explicit Vector(const Alloc& a) : m_alloc(a) {}

		explicit Vector(size_type n) { resize(n); }

		Vector(size_type n, const T& v) { resize(n, v); }

		Vector(std::initializer_list<T> il)
		{
			reserve(il.size());
			for (auto& v : il) emplace_back(v);
		}

		template<typename It>
		Vector(It first, It last)
		{
			for (; first != last; ++first) emplace_back(*first);
		}

		Vector(const Vector& o)
		{
			reserve(o.m_size);
			for (size_type i = 0; i < o.m_size; ++i) emplace_back(o.m_data[i]);
		}

		Vector(Vector&& o) noexcept { swap(o); }

		~Vector()
		{
			clear();
			if (m_data) m_alloc.Deallocate(m_data, sizeof(T)*m_capacity, alignof(T));
		}

		Vector& operator=(const Vector& o)
		{
			if (this != &o)
			{
				clear();
				reserve(o.m_size);
				for (size_type i = 0; i < o.m_size; ++i) emplace_back(o.m_data[i]);
			}
			return *this;
		}

		Vector& operator=(Vector&& o) noexcept
		{
			if (this != &o)
			{
				clear();
				if (m_data) m_alloc.Deallocate(m_data, sizeof(T)*m_capacity, alignof(T));
				m_data = nullptr; m_size = 0; m_capacity = 0;
				swap(o);
			}
			return *this;
		}

		Vector& operator=(std::initializer_list<T> il)
		{
			clear();
			reserve(il.size());
			for (auto& v : il) emplace_back(v);
			return *this;
		}

		T*       data()       { return m_data; }
		const T* data() const { return m_data; }
		size_type size()     const { return m_size; }
		size_type capacity() const { return m_capacity; }
		bool      empty()    const { return m_size == 0; }

		T&       operator[](size_type i)       { return m_data[i]; }
		const T& operator[](size_type i) const { return m_data[i]; }
		T&       at(size_type i)       { return m_data[i]; }
		const T& at(size_type i) const { return m_data[i]; }

		T&       front()       { return m_data[0]; }
		const T& front() const { return m_data[0]; }
		T&       back()       { return m_data[m_size - 1]; }
		const T& back() const { return m_data[m_size - 1]; }

		iterator       begin()       { return m_data; }
		iterator       end()         { return m_data + m_size; }
		const_iterator begin()  const { return m_data; }
		const_iterator end()    const { return m_data + m_size; }
		const_iterator cbegin() const { return m_data; }
		const_iterator cend()   const { return m_data + m_size; }

		// Minimal reverse iterators — forward-incrementing, stepping backward.
		class reverse_iterator
		{
		public:
			reverse_iterator() = default;
			reverse_iterator(T* p) : m_p(p) {}
			T& operator*()  const { return *(m_p - 1); }
			T* operator->() const { return m_p - 1; }
			reverse_iterator& operator++() { --m_p; return *this; }
			reverse_iterator  operator++(int) { reverse_iterator t = *this; --m_p; return t; }
			bool operator==(const reverse_iterator& o) const { return m_p == o.m_p; }
			bool operator!=(const reverse_iterator& o) const { return m_p != o.m_p; }
			T* m_p = nullptr;
		};
		class const_reverse_iterator
		{
		public:
			const_reverse_iterator() = default;
			const_reverse_iterator(const T* p) : m_p(p) {}
			const T& operator*()  const { return *(m_p - 1); }
			const T* operator->() const { return m_p - 1; }
			const_reverse_iterator& operator++() { --m_p; return *this; }
			const_reverse_iterator  operator++(int) { const_reverse_iterator t = *this; --m_p; return t; }
			bool operator==(const const_reverse_iterator& o) const { return m_p == o.m_p; }
			bool operator!=(const const_reverse_iterator& o) const { return m_p != o.m_p; }
			const T* m_p = nullptr;
		};
		reverse_iterator       rbegin()       { return reverse_iterator(m_data + m_size); }
		reverse_iterator       rend()         { return reverse_iterator(m_data); }
		const_reverse_iterator rbegin() const { return const_reverse_iterator(m_data + m_size); }
		const_reverse_iterator rend()   const { return const_reverse_iterator(m_data); }

		void clear()
		{
			if constexpr (!std::is_trivially_destructible_v<T>)
				for (size_type i = 0; i < m_size; ++i) m_data[i].~T();
			m_size = 0;
		}

		void reserve(size_type n)
		{
			if (n <= m_capacity) return;

			// Bump-allocator fast path: extend the existing buffer in place when it sits at
			// the allocator's tail. Zero copies, no wasted old buffer left behind.
			if constexpr (detail::HasTryGrowInPlace<Alloc>::value)
			{
				if (m_data && m_alloc.TryGrowInPlace(m_data, sizeof(T)*m_capacity, sizeof(T)*n))
				{
					m_capacity = n;
					return;
				}
			}

			T* newData = (T*)m_alloc.Allocate(sizeof(T)*n, alignof(T));
			if (m_data)
			{
				if constexpr (std::is_trivially_copyable_v<T>)
				{
					// Single memcpy matches what std::vector does for trivially-copyable T;
					// faster than the per-element move loop (which may or may not vectorize).
					if (m_size) memcpy(newData, m_data, m_size * sizeof(T));
				}
				else
				{
					for (size_type i = 0; i < m_size; ++i)
					{
						new (&newData[i]) T(std::move(m_data[i]));
						if constexpr (!std::is_trivially_destructible_v<T>)
							m_data[i].~T();
					}
				}
				m_alloc.Deallocate(m_data, sizeof(T)*m_capacity, alignof(T));
			}
			m_data = newData;
			m_capacity = n;
		}

		void resize(size_type n)
		{
			if (n < m_size)
			{
				if constexpr (!std::is_trivially_destructible_v<T>)
					for (size_type i = n; i < m_size; ++i) m_data[i].~T();
				m_size = n;
			}
			else if (n > m_size)
			{
				grow_to(n);
				for (size_type i = m_size; i < n; ++i) new (&m_data[i]) T();
				m_size = n;
			}
		}

		void resize(size_type n, const T& v)
		{
			if (n < m_size)
			{
				if constexpr (!std::is_trivially_destructible_v<T>)
					for (size_type i = n; i < m_size; ++i) m_data[i].~T();
				m_size = n;
			}
			else if (n > m_size)
			{
				grow_to(n);
				for (size_type i = m_size; i < n; ++i) new (&m_data[i]) T(v);
				m_size = n;
			}
		}

		void shrink_to_fit() { /* no-op: minimal impl */ }

		void push_back(const T& v)
		{
			if (m_size == m_capacity) grow_one();
			new (&m_data[m_size]) T(v);
			++m_size;
		}

		void push_back(T&& v)
		{
			if (m_size == m_capacity) grow_one();
			new (&m_data[m_size]) T(std::move(v));
			++m_size;
		}

		template<typename... Args>
		T& emplace_back(Args&&... args)
		{
			if (m_size == m_capacity) grow_one();
			T* slot = &m_data[m_size];
			new (slot) T(std::forward<Args>(args)...);
			++m_size;
			return *slot;
		}

		void pop_back()
		{
			if constexpr (!std::is_trivially_destructible_v<T>)
				m_data[m_size - 1].~T();
			--m_size;
		}

		iterator insert(const_iterator pos, const T& v)
		{
			size_type idx = size_type(pos - m_data);
			if (m_size == m_capacity) grow_one();
			if (idx < m_size)
			{
				new (&m_data[m_size]) T(std::move(m_data[m_size - 1]));
				for (size_type i = m_size - 1; i > idx; --i)
					m_data[i] = std::move(m_data[i - 1]);
				m_data[idx] = v;
			}
			else
			{
				new (&m_data[idx]) T(v);
			}
			++m_size;
			return m_data + idx;
		}

		iterator insert(const_iterator pos, T&& v)
		{
			size_type idx = size_type(pos - m_data);
			if (m_size == m_capacity) grow_one();
			if (idx < m_size)
			{
				new (&m_data[m_size]) T(std::move(m_data[m_size - 1]));
				for (size_type i = m_size - 1; i > idx; --i)
					m_data[i] = std::move(m_data[i - 1]);
				m_data[idx] = std::move(v);
			}
			else
			{
				new (&m_data[idx]) T(std::move(v));
			}
			++m_size;
			return m_data + idx;
		}

		// Range insert: insert [first, last) at `pos`.
		template<typename InputIt>
		iterator insert(const_iterator pos, InputIt first, InputIt last)
		{
			size_type idx = size_type(pos - m_data);
			size_type count = 0;
			for (InputIt it = first; it != last; ++it) ++count;
			if (count == 0) return m_data + idx;
			grow_to(m_size + count);
			// Shift tail right by `count`.
			for (size_type i = m_size; i-- > idx; )
			{
				new (&m_data[i + count]) T(std::move(m_data[i]));
				if constexpr (!std::is_trivially_destructible_v<T>) m_data[i].~T();
			}
			size_type j = idx;
			for (InputIt it = first; it != last; ++it, ++j)
				new (&m_data[j]) T(*it);
			m_size += count;
			return m_data + idx;
		}

		iterator erase(const_iterator pos)
		{
			iterator p = const_cast<iterator>(pos);
			for (iterator it = p; it + 1 != end(); ++it)
				*it = std::move(*(it + 1));
			if constexpr (!std::is_trivially_destructible_v<T>)
				m_data[m_size - 1].~T();
			--m_size;
			return p;
		}

		iterator erase(const_iterator first, const_iterator last)
		{
			iterator f = const_cast<iterator>(first);
			iterator l = const_cast<iterator>(last);
			size_type removed = size_type(l - f);
			iterator src = l;
			iterator dst = f;
			iterator e = end();
			while (src != e) *dst++ = std::move(*src++);
			if constexpr (!std::is_trivially_destructible_v<T>)
				for (iterator it = dst; it != e; ++it) it->~T();
			m_size -= removed;
			return f;
		}

		void swap(Vector& o) noexcept
		{
			T* td = m_data;            m_data = o.m_data;         o.m_data = td;
			size_type ts = m_size;     m_size = o.m_size;         o.m_size = ts;
			size_type tc = m_capacity; m_capacity = o.m_capacity; o.m_capacity = tc;
		}

	private:
		// Geometric growth helper used by resize/insert/push_back. Matches std::vector's
		// policy: capacity at least doubles on every grow, so N successive resize(1..N)
		// calls cost O(N) amortized instead of O(N^2) that a plain reserve(n) would give.
		void grow_to(size_type n)
		{
			if (n <= m_capacity) return;
			size_type doubled = m_capacity ? m_capacity * 2 : 4;
			reserve(n > doubled ? n : doubled);
		}

		void grow_one() { grow_to(m_capacity + 1); }

		T* m_data = nullptr;
		size_type m_size = 0;
		size_type m_capacity = 0;
		Alloc m_alloc;
	};

	template<typename T, typename A>
	bool operator==(const Vector<T, A>& a, const Vector<T, A>& b)
	{
		if (a.size() != b.size()) return false;
		for (u64 i = 0; i < a.size(); ++i) if (!(a[i] == b[i])) return false;
		return true;
	}
	template<typename T, typename A>
	bool operator!=(const Vector<T, A>& a, const Vector<T, A>& b) { return !(a == b); }
}
