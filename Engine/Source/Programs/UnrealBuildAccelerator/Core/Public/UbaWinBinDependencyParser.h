// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaAlgorithm.h"
#include "UbaFunctional.h"
#include "UbaPlatform.h"
#include "UbaStringBuffer.h"

#if !defined(UBA_IS_DETOURED_INCLUDE)
#define True_CreateFileMappingW ::CreateFileMappingW
#define True_MapViewOfFile ::MapViewOfFile
#define True_UnmapViewOfFile ::UnmapViewOfFile
#endif

namespace uba
{
	struct BinaryInfo
	{
	};

	// These dlls should _always_ exist on all machines so we can filter them out from the list of imports to copy
	// It should somewhat match what is found in Computer\HKEY_LOCAL_MACHINE\SYSTEM\CurrentControlSet\Control\Session Manager\KnownDLLs
	// but also take info from wine source code which dlls that are custom implemented for wine.
	static const wchar_t* g_knownSystemFiles[] = // Sorted lowercase
	{
		L"advapi32.dll",
		L"bcrypt.dll",
		L"bcryptprimitives.dll",
		L"combase.dll",
		L"concrt140.dll",
		L"crypt32.dll",
		L"cryptbase.dll",
		L"dbghelp.dll",
		L"dnsapi.dll",
		L"dwmapi.dll",
		L"dxcore.dll",
		L"dxgi.dll",
		L"fwpuclnt.dll",
		L"gdi32.dll",
		L"gdi32full.dll",
		L"glu32.dll",
		L"imagehlp.dll",
		L"imm32.dll",
		L"iphlpapi.dll",
		L"kernel32.dll",
		L"kernelbase.dll",
		L"mscoree.dll",
		L"msvcp_win.dll",
		L"msvcr120.dll",
		L"msvcrt.dll",
		L"mswsock.dll",
		L"ncrypt.dll",
		L"netapi32.dll",
		L"nsi.dll",
		L"ntasn1.dll",
		L"ntdll.dll",
		L"ole32.dll",
		L"oleaut32.dll",
		L"ondemandconnroutehelper.dll",
		L"opengl32.dll",
		L"powrprof.dll",
		L"psapi.dll",
		L"rasadhlp.dll",
		L"rpcrt4.dll",
		L"rsaenh.dll",
		L"rstrtmgr.dll",
		L"sechost.dll",
		L"setupapi.dll",
		L"shell32.dll",
		L"shlwapi.dll",
		L"sspicli.dll",
		L"tbs.dll",
		L"traffic.dll",
		L"ucrtbase.dll",
		L"umpdc.dll",
		L"umppc17706.dll",
		L"user32.dll",
		L"userenv.dll",
		L"uxtheme.dll",
		L"version.dll",
		L"webio.dll",
		L"win32u.dll",
		L"winhttp.dll",
		L"winmm.dll",
		L"winnsi.dll",
		L"ws2_32.dll",
	};

	inline bool IsKnownSystemFile(const tchar* fileName)
	{
		StringBuffer<> nameLower;
		if (auto lastSeparator = TStrrchr(fileName, PathSeparator))
			nameLower.Append(lastSeparator + 1);
		else
			nameLower.Append(fileName);
		nameLower.MakeLower();
		return uba::BinarySearch(g_knownSystemFiles, g_knownSystemFiles + sizeof_array(g_knownSystemFiles), nameLower.data, [](const wchar_t* a, const wchar_t* b) { return TStrcmp(a, b) < 0; });
	}

	inline DWORD GetOffset(DWORD rva, PIMAGE_SECTION_HEADER psh, PIMAGE_NT_HEADERS pnt)
	{
		if(rva == 0)
			return rva;
		PIMAGE_SECTION_HEADER seh = psh;
		for(size_t i=0; i<pnt->FileHeader.NumberOfSections; ++i, ++seh)
			if (rva >= seh->VirtualAddress && rva < seh->VirtualAddress + seh->Misc.VirtualSize)
				break;
		return rva - seh->VirtualAddress + seh->PointerToRawData;
	} 

	DWORD UbaExceptionHandlerAllowAccessViolation(DWORD code, _EXCEPTION_POINTERS* info, const tchar* fileName);

	inline bool FindImportsInMem(StringView fileName, void* mem, u64 memSize, const Function<void(const tchar* import, bool isKnown, const tchar* const* importLoaderPaths)>& func, StringBufferBase& outError)
	{
		auto hdrs = (PIMAGE_NT_HEADERS)(PCHAR(mem) + PIMAGE_DOS_HEADER(mem)->e_lfanew);
		auto pSech=IMAGE_FIRST_SECTION(hdrs);
		// Local copy: clang ICEs if the __except filter below references the fileName parameter directly.
		const tchar* fileNameData = fileName.data;
		__try
		{
			auto& dataDir = hdrs->OptionalHeader.DataDirectory[IMAGE_DIRECTORY_ENTRY_IMPORT];
			if(dataDir.Size == 0)
				return true; // No import table

			auto importDesc = (PIMAGE_IMPORT_DESCRIPTOR)((DWORD_PTR)mem + GetOffset(dataDir.VirtualAddress, pSech, hdrs));

			while(true)
			{
				if (u64((u8*)importDesc - (u8*)mem) + sizeof(IMAGE_IMPORT_DESCRIPTOR) > memSize)
					return outError.Appendf(TC("Import parsing error1: %.*s"), fileName.count, fileNameData).ToFalse();
				if (importDesc->Name == NULL)
					break;
				char* name = (PCHAR)((DWORD_PTR)mem + GetOffset(importDesc->Name, pSech, hdrs));
				if (u64((u8*)name - (u8*)mem) + 2 > memSize)
					return outError.Appendf(TC("Import parsing error2: %.*s"), fileName.count, fileNameData).ToFalse();
				StringBuffer<256> wname;
				wname.Append(name);
				func(wname.data, IsKnownSystemFile(wname.data), nullptr);
				++importDesc;
			}
		}
		__except(UbaExceptionHandlerAllowAccessViolation(GetExceptionCode(), GetExceptionInformation(), fileNameData))
		{
			return false;
		}
		return true;
	}

	inline bool ParseBinary(const StringView& filePath, const StringView& originalPath, BinaryInfo& outInfo, const Function<void(const tchar* import, bool isKnown, const tchar* const* importLoaderPaths)>& func, StringBufferBase& outError)
	{
		if (filePath.EndsWith(TCV(".bat")))
		{
			outError.Appendf(TC("Batch file has no dependencies: %.*s"), filePath.count, filePath.data);
			return true;
		}
		HANDLE fileHandle = ::CreateFileW(filePath.data, GENERIC_READ, FILE_SHARE_READ, 0, OPEN_EXISTING, FILE_ATTRIBUTE_NORMAL | FILE_FLAG_SEQUENTIAL_SCAN, 0);
		if (fileHandle == INVALID_HANDLE_VALUE)
		{
			outError.Appendf(TC("File not found: %.*s"), filePath.count, filePath.data);
			return true;
		}

		auto closeFileHandle = MakeGuard([&]() { CloseHandle(fileHandle); });
		HANDLE fileMapping = True_CreateFileMappingW(fileHandle, NULL, PAGE_READONLY, 0, 0, NULL);
		if (!fileMapping)
			return outError.Appendf(TC("Failed to create mapping: %.*s"), filePath.count, filePath.data).ToFalse();
		auto closeMappingHandle = MakeGuard([&]() { CloseHandle(fileMapping); });
		void* mem = True_MapViewOfFile(fileMapping, FILE_MAP_READ, 0, 0, 0);
		if (!mem)
			return outError.Appendf(TC("Failed to map view of mapping: %.*s"), filePath.count, filePath.data).ToFalse();
		auto unmap = MakeGuard([&]() { True_UnmapViewOfFile(mem); });
		LARGE_INTEGER fileSize;
		if (!::GetFileSizeEx(fileHandle, &fileSize))
			return outError.Appendf(TC("Failed to get size of file: %.*s"), filePath.count, filePath.data).ToFalse();

		return FindImportsInMem(filePath, mem, fileSize.QuadPart, func, outError);
	}
}
