// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaPlatform.h"
#include <winternl.h>

struct FILE_DIRECTORY_INFORMATION {
	ULONG         NextEntryOffset;
	ULONG         FileIndex;
	LARGE_INTEGER CreationTime;
	LARGE_INTEGER LastAccessTime;
	LARGE_INTEGER LastWriteTime;
	LARGE_INTEGER ChangeTime;
	LARGE_INTEGER EndOfFile;
	LARGE_INTEGER AllocationSize;
	ULONG         FileAttributes;
	ULONG         FileNameLength;
	WCHAR         FileName[1];
};

struct FILE_FULL_DIR_INFORMATION {
	ULONG         NextEntryOffset;
	ULONG         FileIndex;
	LARGE_INTEGER CreationTime;
	LARGE_INTEGER LastAccessTime;
	LARGE_INTEGER LastWriteTime;
	LARGE_INTEGER ChangeTime;
	LARGE_INTEGER EndOfFile;
	LARGE_INTEGER AllocationSize;
	ULONG         FileAttributes;
	ULONG         FileNameLength;
	ULONG         EaSize;
	WCHAR         FileName[1];
};

struct FILE_ID_BOTH_DIR_INFORMATION {
	ULONG         NextEntryOffset;
	ULONG         FileIndex;
	LARGE_INTEGER CreationTime;
	LARGE_INTEGER LastAccessTime;
	LARGE_INTEGER LastWriteTime;
	LARGE_INTEGER ChangeTime;
	LARGE_INTEGER EndOfFile;
	LARGE_INTEGER AllocationSize;
	ULONG         FileAttributes;
	ULONG         FileNameLength;
	ULONG         EaSize;
	CCHAR         ShortNameLength;
	WCHAR         ShortName[12];
	LARGE_INTEGER FileId;
	WCHAR         FileName[1];
};

struct FILE_BASIC_INFORMATION {
	LARGE_INTEGER CreationTime;
	LARGE_INTEGER LastAccessTime;
	LARGE_INTEGER LastWriteTime;
	LARGE_INTEGER ChangeTime;
	ULONG FileAttributes;
};

struct FILE_FS_DEVICE_INFORMATION
{
	DEVICE_TYPE DeviceType;
	ULONG Characteristics;
};
constexpr ULONG FILE_REMOTE_DEVICE = 0x00000010;
constexpr ULONG FILE_DEVICE_IS_MOUNTED = 0x00000020;


struct FILE_FS_ATTRIBUTE_INFORMATION
{
	ULONG FileSystemAttributes;
	LONG  MaximumComponentNameLength;
	ULONG FileSystemNameLength;
	WCHAR FileSystemName[1];
};

struct FILE_NETWORK_OPEN_INFORMATION
{
	LARGE_INTEGER CreationTime;
	LARGE_INTEGER LastAccessTime;
	LARGE_INTEGER LastWriteTime;
	LARGE_INTEGER ChangeTime;
	LARGE_INTEGER AllocationSize;
	LARGE_INTEGER EndOfFile;
	ULONG FileAttributes;
};

struct FILE_POSITION_INFORMATION
{
	LARGE_INTEGER CurrentByteOffset;
};

struct FILE_END_OF_FILE_INFORMATION
{
	LARGE_INTEGER EndOfFile;
};

struct FILE_STANDARD_INFORMATION {
	LARGE_INTEGER AllocationSize;
	LARGE_INTEGER EndOfFile;
	ULONG         NumberOfLinks;
	BOOLEAN       DeletePending;
	BOOLEAN       Directory;
};

struct FILE_RENAME_INFORMATION {
	union {
		BOOLEAN ReplaceIfExists;  // FileRenameInformation
		ULONG Flags;              // FileRenameInformationEx
	} DUMMYUNIONNAME;
	HANDLE RootDirectory;
	ULONG FileNameLength;
	WCHAR FileName[1];
};

struct FILE_IS_REMOTE_DEVICE_INFORMATION
{
	BOOLEAN IsRemote;
};

struct FILE_ID_INFORMATION
{
	ULONGLONG   VolumeSerialNumber;
	FILE_ID_128 FileId;
};

struct FILE_NAME_INFORMATION
{
	ULONG FileNameLength;
	WCHAR FileName[1];
};

struct FILE_INTERNAL_INFORMATION
{
	LARGE_INTEGER IndexNumber;
};

struct FILE_EA_INFORMATION
{
	ULONG EaSize;
};

struct FILE_ACCESS_INFORMATION
{
	ACCESS_MASK AccessFlags;
};

struct FILE_MODE_INFORMATION
{
	ULONG Mode;
};

struct FILE_ALIGNMENT_INFORMATION
{
	ULONG AlignmentRequirement;
};

struct FILE_DISPOSITION_INFORMATION
{
	BOOLEAN DeleteFile;
};

struct FILE_ALLOCATION_INFORMATION
{
	LARGE_INTEGER AllocationSize;
};

struct FILE_ALL_INFORMATION
{
	FILE_BASIC_INFORMATION     BasicInformation;
	FILE_STANDARD_INFORMATION  StandardInformation;
	FILE_INTERNAL_INFORMATION  InternalInformation;
	FILE_EA_INFORMATION        EaInformation;
	FILE_ACCESS_INFORMATION    AccessInformation;
	FILE_POSITION_INFORMATION  PositionInformation;
	FILE_MODE_INFORMATION      ModeInformation;
	FILE_ALIGNMENT_INFORMATION AlignmentInformation;
	FILE_NAME_INFORMATION      NameInformation;
};

struct FILE_FS_VOLUME_INFORMATION {
	LARGE_INTEGER VolumeCreationTime;
	ULONG         VolumeSerialNumber;
	ULONG         VolumeLabelLength;
	BOOLEAN       SupportsObjects;
	WCHAR         VolumeLabel[1];
};

#define FILE_INFO_CLASSES \
	FILE_INFO_CLASS(FileBasicInformation, 4) \
	FILE_INFO_CLASS(FileStandardInformation, 5) \
	FILE_INFO_CLASS(FileNameInformation, 9) \
	FILE_INFO_CLASS(FileRenameInformation, 10) \
	FILE_INFO_CLASS(FileDispositionInformation, 13) \
	FILE_INFO_CLASS(FilePositionInformation, 14) \
	FILE_INFO_CLASS(FileAllInformation, 18) \
	FILE_INFO_CLASS(FileAllocationInformation, 19) \
	FILE_INFO_CLASS(FileEndOfFileInformation, 20) \
	FILE_INFO_CLASS(FileNetworkOpenInformation, 34) \
	FILE_INFO_CLASS(FileNormalizedNameInformation, 35) \
	FILE_INFO_CLASS(FileIdBothDirectoryInformation, 37) \
	FILE_INFO_CLASS(FileIsRemoteDeviceInformation, 51) \
	FILE_INFO_CLASS(FileIdInformation, 59) \
	FILE_INFO_CLASS(FileRenameInformationEx, 65) \


// const (not constexpr): clang rejects the out-of-range cast to this fixed-type-less winternl enum in a constant expression.
#define FILE_INFO_CLASS(name, value) [[maybe_unused]] const FILE_INFORMATION_CLASS name = (FILE_INFORMATION_CLASS)value;
FILE_INFO_CLASSES
#undef FILE_INFO_CLASS

constexpr OBJECT_INFORMATION_CLASS ObjectNameInformation = (OBJECT_INFORMATION_CLASS)1;

enum FS_INFORMATION_CLASS : int {};

constexpr FS_INFORMATION_CLASS FileFsVolumeInformation = (FS_INFORMATION_CLASS)1;
constexpr FS_INFORMATION_CLASS FileFsDeviceInformation = (FS_INFORMATION_CLASS)4;
constexpr FS_INFORMATION_CLASS FileFsAttributeInformation = (FS_INFORMATION_CLASS)5;

enum RTL_PATH_TYPE
{
	RtlPathTypeUnknown = 0,
	RtlPathTypeUncAbsolute,     // "//foo"
	RtlPathTypeDriveAbsolute,   // "c:/foo"
	RtlPathTypeDriveRelative,   // "c:foo"
	RtlPathTypeRooted,          // "/foo"
	RtlPathTypeRelative,        // "foo"
	RtlPathTypeLocalDevice,     // "//./foo"
	RtlPathTypeRootLocalDevice  // "//."
};

struct RTL_RELATIVE_NAME_U
{
	UNICODE_STRING RelativeName;
	HANDLE ContainingDirectory;
	PVOID CurDirRef;
};

struct FILE_IO_COMPLETION_INFORMATION
{
	PVOID KeyContext;
	PVOID ApcContext;
	IO_STATUS_BLOCK IoStatusBlock;
};

struct USER_THREAD_START_ROUTINE;

struct PS_ATTRIBUTE_LIST;