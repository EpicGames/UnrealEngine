<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <!-- Epic Games, Inc. UnrealBuildAccelerator Visualizers -->
  <!--
    uba::tchar is wchar_t on Windows (the only platform natvis runs on), so
    strings are displayed with the ,su (UTF-16) format specifier. count is the
    character length; data is null-terminated but [count] keeps display honest
    for non-terminated views.
  -->

  <!-- StringBufferBase covers StringBuffer<N> and LastErrorToText via inheritance -->
  <Type Name="uba::StringBufferBase">
    <DisplayString Condition="count == 0">""</DisplayString>
    <DisplayString>{data,[count]su}</DisplayString>
    <StringView>data,[count]su</StringView>
    <Expand>
      <Item Name="[count]">count</Item>
      <Item Name="[capacity]">capacity</Item>
      <ArrayItems>
        <Size>count</Size>
        <ValuePointer>data</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <Type Name="uba::StringView">
    <DisplayString Condition="count == 0">""</DisplayString>
    <DisplayString>{data,[count]su}</DisplayString>
    <StringView>data,[count]su</StringView>
    <Expand>
      <Item Name="[count]">count</Item>
      <ArrayItems>
        <Size>count</Size>
        <ValuePointer>data</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <Type Name="uba::TString">
    <DisplayString Condition="m_size == 0">""</DisplayString>
    <DisplayString>{m_data,[m_size]su}</DisplayString>
    <StringView>m_data,[m_size]su</StringView>
    <Expand>
      <Item Name="[size]">m_size</Item>
      <Item Name="[capacity]">m_capacity ? m_capacity - 1 : 0</Item>
      <ArrayItems>
        <Size>m_size</Size>
        <ValuePointer>m_data</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <!-- ===== Containers ===== -->

  <Type Name="uba::Vector&lt;*&gt;">
    <DisplayString>{{ size={m_size} }}</DisplayString>
    <Expand>
      <Item Name="[size]">m_size</Item>
      <Item Name="[capacity]">m_capacity</Item>
      <ArrayItems>
        <Size>m_size</Size>
        <ValuePointer>m_data</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <Type Name="uba::List&lt;*&gt;">
    <DisplayString>{{ size={m_size} }}</DisplayString>
    <Expand>
      <Item Name="[size]">m_size</Item>
    </Expand>
  </Type>

  <!-- Open-chaining hash map: Node* m_buckets[m_bucketCount], chained via next, payload in kv -->
  <Type Name="uba::UnorderedMap&lt;*&gt;">
    <DisplayString>{{ size={m_size} }}</DisplayString>
    <Expand>
      <Item Name="[size]">m_size</Item>
      <Item Name="[bucket_count]">m_bucketCount</Item>
      <CustomListItems MaxItemsPerView="5000" Condition="m_size != 0">
        <Variable Name="b" InitialValue="0"/>
        <Variable Name="node" InitialValue="*m_buckets"/>
        <Loop Condition="b &lt; m_bucketCount">
          <Exec>node = m_buckets[b]</Exec>
          <Loop Condition="node != 0">
            <Item>node->kv</Item>
            <Exec>node = node->next</Exec>
          </Loop>
          <Exec>b = b + 1</Exec>
        </Loop>
      </CustomListItems>
    </Expand>
  </Type>

  <Type Name="uba::UnorderedSet&lt;*&gt;">
    <DisplayString>{{ size={m_size} }}</DisplayString>
    <Expand>
      <Item Name="[size]">m_size</Item>
      <Item Name="[bucket_count]">m_bucketCount</Item>
      <CustomListItems MaxItemsPerView="5000" Condition="m_size != 0">
        <Variable Name="b" InitialValue="0"/>
        <Variable Name="node" InitialValue="*m_buckets"/>
        <Loop Condition="b &lt; m_bucketCount">
          <Exec>node = m_buckets[b]</Exec>
          <Loop Condition="node != 0">
            <Item>node->key</Item>
            <Exec>node = node->next</Exec>
          </Loop>
          <Exec>b = b + 1</Exec>
        </Loop>
      </CustomListItems>
    </Expand>
  </Type>

  <!-- ===== Hash / key value types ===== -->

  <Type Name="uba::StringKey">
    <DisplayString>{a,Xb}{b,Xb}</DisplayString>
  </Type>

  <Type Name="uba::CasKey">
    <DisplayString>{a,Xb}{b,Xb}{c,Xb}</DisplayString>
  </Type>

  <Type Name="uba::Guid">
    <DisplayString>{data1,Xb}-{data2,Xb}-{data3,Xb}-{data4[0],Xb}{data4[1],Xb}-{data4[2],Xb}{data4[3],Xb}{data4[4],Xb}{data4[5],Xb}{data4[6],Xb}{data4[7],Xb}</DisplayString>
  </Type>

  <!-- ===== Handles ===== -->

  <Type Name="uba::FileMappingHandle">
    <DisplayString Condition="shmFd == 0 &amp;&amp; lockFd == 0">{internal}</DisplayString>
    <DisplayString>{internal} (shmFd={shmFd} lockFd={lockFd})</DisplayString>
    <Expand>
      <Item Name="[internal]">internal</Item>
      <Item Name="[shmFd]">shmFd</Item>
      <Item Name="[lockFd]">lockFd</Item>
    </Expand>
  </Type>

  <!-- ===== Binary cursors (StackBinary* covered via inheritance) ===== -->

  <Type Name="uba::BinaryWriter">
    <DisplayString>{{ pos={m_pos - m_begin} capacity={m_end - m_begin} }}</DisplayString>
    <Expand>
      <Item Name="[position]">m_pos - m_begin</Item>
      <Item Name="[capacity]">m_end - m_begin</Item>
      <Item Name="[remaining]">m_end - m_pos</Item>
      <Item Name="[begin]">m_begin</Item>
    </Expand>
  </Type>

  <Type Name="uba::BinaryReader">
    <DisplayString>{{ pos={m_pos - m_begin} size={m_end - m_begin} }}</DisplayString>
    <Expand>
      <Item Name="[position]">m_pos - m_begin</Item>
      <Item Name="[size]">m_end - m_begin</Item>
      <Item Name="[remaining]">m_end - m_pos</Item>
      <Item Name="[begin]">m_begin</Item>
    </Expand>
  </Type>

</AutoVisualizer>
