// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaCore : ModuleRules
{
	public UbaCore(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		StaticAnalyzerDisabledCheckers.Clear();

		bRequiresPlatformSDK = true;

		PrivatePCHHeaderFile = "Public/UbaCorePch.h";
		SharedPCHHeaderFile = "Public/UbaCorePch.h";

		PrivateDependencyModuleNames.AddRange(new string[] {
			"UbaMimalloc",
			"BLAKE3",
		});

		PublicDependencyModuleNames.AddRange(new string[] {
			"UbaVersion",
		});

		PrivateDefinitions.AddRange(new string[] {
			"_CONSOLE",
		});

		PublicIncludePathModuleNames.AddRange(new string[]
		{
			"UbaMimalloc",
			"OodleDataCompression",
		});

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PublicSystemLibraries.Add("ntdll.lib");
		}
	}
}
