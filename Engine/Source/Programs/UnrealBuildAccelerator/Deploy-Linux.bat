@echo off
setlocal 

pushd "%~dp0"

set config=Shipping

if "%~1"=="" goto noconfig
set config=%1
:noconfig

set options=""
if /I not "%config%"=="debug" goto nooptions
set options="-enabletsan"
:nooptions


echo.
echo === Building Targets ===
pushd "../../../.."
call Engine/Build/BatchFiles/RunUBT.bat -NoUba -NoUbaLocal -NoSNDBS -NoXGE ^
	-Target="UbaAgent Linux %config% %options%" ^
	-Target="UbaCli Linux %config% %options%" ^
	-Target="UbaDetours Linux %config%" ^
	-Target="UbaHost Linux %config%" ^
	-Target="UbaTest Linux %config% %options%" ^
	-Target="UbaTestApp Linux %config%" ^
	-Target="UbaCacheService Linux %config%" ^
	-Target="UbaCoordinatorHorde Linux %config% %options%"

if %ERRORLEVEL% NEQ 0 exit /b %ERRORLEVEL%
popd

echo.
echo === Reconciling artifacts ===
p4 reconcile ../../../Binaries/Linux/UnrealBuildAccelerator/...

popd

endlocal

exit /b 0
