#!/bin/bash -f

ws_root=`dirname $0`/../../../..
ws_root=`realpath ${ws_root}`

config=${1:-Shipping}
options=""

if [[ "${config,,}" = "debug" ]]; then
	options="-EnableTSan"
fi

echo "=== Building Targets ==="

(cd $ws_root && ./RunUBT.sh -NoUba -NoUbaLocal -NoSNDBS -NoXGE \
	-Target="UbaAgent Linux $config $options" \
	-Target="UbaCli Linux $config $options" \
	-Target="UbaDetours Linux $config" \
	-Target="UbaHost Linux $config" \
	-Target="UbaTest Linux $config $options" \
	-Target="UbaTestApp Linux $config" \
	-Target="UbaCacheService Linux $config" \
	-Target="UbaCoordinatorHorde Linux $config $options")

echo "=== Reconciling artifacts ==="
p4 reconcile ${ws_root}/Engine/Binaries/Linux/UnrealBuildAccelerator/...

echo "=== Done ==="

