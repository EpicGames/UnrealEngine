@echo off
setlocal 

set config=Shipping

if "%~1"=="" goto blank

set config=%1

:blank

echo.
echo === Building Targets ===
pushd "%~dp0../../../.."
call Engine/Build/BatchFiles/RunUBT.bat -NoUba -NoUbaLocal -NoSNDBS -NoXGE -novfs ^
	-Target="UbaAgent Win64 %config%" ^
	-Target="UbaCli Win64 %config%" ^
	-Target="UbaDetours Win64 %config%" ^
	-Target="UbaHost Win64 %config%" ^
	-Target="UbaVisualizer Win64 %config%" ^
	-Target="UbaObjTool Win64 %config%" ^
	-Target="UbaCacheService Win64 %config%"

if %ERRORLEVEL% NEQ 0 exit /b %ERRORLEVEL%
popd

echo.
echo === Reconciling artifacts ===
pushd "%~dp0"
p4 reconcile ../../../Binaries/Win64/UnrealBuildAccelerator/...
popd

endlocal

exit /b 0
