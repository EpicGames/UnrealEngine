.code

EXTERN Local_NdrClientCall2 : PROC
EXTERN True2_NdrClientCall2 : QWORD

Detoured_NdrClientCall2 PROC

	; Store state
	mov rax,rsp
	mov [rax+8],rcx
	mov [rax+16],rdx
	mov [rax+24],r8
	mov [rax+32],r9
	sub rsp,40

	lea r8,[rsp+64]
	call Local_NdrClientCall2
	test al,al
	jnz SkipTrue

	lea rax,[rsp+40]

	; Restore state
	mov rcx,[rax+8]
	mov rdx,[rax+16]
	mov r8,[rax+24]
	mov r9,[rax+32]

	; Call true function
	add rsp,40
	jmp [True2_NdrClientCall2]

SkipTrue:
	add rsp,40
	ret

Detoured_NdrClientCall2 ENDP

end
