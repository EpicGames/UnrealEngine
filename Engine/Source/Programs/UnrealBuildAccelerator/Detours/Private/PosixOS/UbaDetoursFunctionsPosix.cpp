// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaDetoursSharedPosix.h" // Must be first

#include "UbaFileMapping.h"
#include "UbaLinuxRawSyscall.h"
#include "UbaPlatform.h"
#include "UbaProcessStats.h"
#include "UbaProcessUtils.h"
#include "UbaProtocol.h"
#include "UbaStringBuffer.h"
#include "UbaTimer.h"
#include <sys/wait.h>
#include <dirent.h> 
#include <glob.h>
#include <dlfcn.h>

#include <sys/file.h>
#include <sys/stat.h>
#include <sys/xattr.h>

// undefs for mimalloc
#undef realpath
#undef malloc

#include <signal.h>
#include <execinfo.h>
#include <sys/syscall.h>
#include <unistd.h>

#if PLATFORM_LINUX
#include <ucontext.h>
#include <sys/prctl.h>
#include <sys/sysmacros.h>
#include <sys/mount.h>
#include <sys/statvfs.h>
#else
#include <crt_externs.h>
extern char** environ;
#endif

#define UBA_DETOUR_DEBUG 0//UBA_DEBUG

namespace uba
{
	#include "UbaFileMapping.inl"

	constexpr bool g_logToScreen = false;

	bool g_isInitialized;
	bool g_isCancelled;
	u32 g_processId;
	pid_t g_pid;
	void Deinit();
	bool TakeLockForRpc();

	SharedMemoryAllocatorHandle g_sharedMemoryAllocator;
	u8* g_sharedMemory;

	extern SharedEvent* g_cancelEvent;
	extern SharedEvent* g_readEvent;
	extern SharedEvent* g_writeEvent;
	extern u8* g_messageMappingMem;
}

using namespace uba;

#include "UbaDetoursTrueWrappersPosix.h"
#include "UbaDetoursRealPosix.h"

#define UBA_EXPORT __attribute__((visibility("default")))

// TODO: may need to detour fadvise64 / copy_file_range (used by cp).

#if PLATFORM_LINUX
// Storage definitions for True_##func globals (declared extern in UbaDetoursTrueWrappersPosix.h).
#define DETOURED_FUNCTION(func) Symbol_##func* True_##func;
DETOURED_FUNCTIONS
#undef DETOURED_FUNCTION
#endif

#include "UbaBinaryParser.h"
#include "UbaDetoursGoPosix.h"

void CloseCom();

namespace uba
{
	bool CanDetour(StringView file)
	{
		if (DisallowDetourDepth())
			return false;
		return g_rules->CanDetour(file, g_runningRemote);
	}

}


template<typename TrueOpen>
FILE* Shared_fopen(const char* funcName, const char* path, const char* mode, const char* trueOpenName, const TrueOpen& trueOpen)
{
	bool p = strchr(mode, '+');

	int flags = 0;
	if (strchr(mode, 'r'))
	{
		if (p)
			flags = O_RDWR;
		else
			flags = O_RDONLY | O_NONBLOCK;
	}
	else if (strchr(mode, 'w'))
	{
		flags = O_CREAT | O_TRUNC;
		if (p)
			flags |= O_RDWR;
		else
			flags |= O_WRONLY;
	}
	else if (strchr(mode, 'a'))
	{
		flags = O_CREAT | O_APPEND;
		if (p)
			flags |= O_RDWR;
		else
			flags |= O_WRONLY;
	}

	int openMode = S_IRUSR | S_IWUSR | S_IRGRP | S_IROTH;
	int fd = Shared_open(trueOpenName, path, flags, openMode, trueOpen);
	if (fd == -1)
	{
		DEBUG_LOG_DETOURED(funcName, "(%s) -> FAILED", path);
		return nullptr;
	}


	TimerScope ts(g_kernelStats.createFile);
	FILE* res = TRUE_WRAPPER(fdopen)(fd, mode);
	ts.Leave();

	DEBUG_LOG_TRUE(funcName, "%i (%s %s) -> %p", fd, path, mode, res);
	return res;
}

bool LdpreloadSnapshot(int fd, u64& outSize, u64& outLastWritten, u32& outAttributes)
{
	int dupFd = Real_dup(errno, fd);
	if (dupFd == -1)
		return false;
	struct stat attr;
	int rc = Real_fstat(errno,dupFd, &attr);
	Real_close(errno,dupFd);
	if (rc != 0)
		return false;
	outSize = attr.st_size;
	outLastWritten = FromTimeSpec(attr.st_mtimespec);
	outAttributes = attr.st_mode;
	return true;
}

// freopen reuses caller's FILE* — can't decompose to close+fopen. Retire old fd's UBA state, then trueFreopen on prep.tempFileName so remote staging works.
template<typename TrueFreopen, typename TrueOpen>
FILE* Shared_freopen(const char* funcName, const char* file, const char* mode, FILE* stream, const TrueFreopen& trueFreopen, const TrueOpen& trueOpen)
{
	bool p = strchr(mode, '+');
	int flags = 0;
	if (strchr(mode, 'r'))
		flags = p ? O_RDWR : (O_RDONLY | O_NONBLOCK);
	else if (strchr(mode, 'w'))
		flags = O_CREAT | O_TRUNC | (p ? O_RDWR : O_WRONLY);
	else if (strchr(mode, 'a'))
		flags = O_CREAT | O_APPEND | (p ? O_RDWR : O_WRONLY);

	int openMode = S_IRUSR | S_IWUSR | S_IRGRP | S_IROTH;

	int oldFd = fileno(stream);
	fflush(stream);
	Shared_close(funcName, oldFd, []() { return 0; }, LdpreloadSnapshot); // retire UBA state; glibc closes oldFd inside trueFreopen below

	SharedOpenPrep prep;
	Shared_prepareOpen(funcName, file, flags, openMode, trueOpen, prep);

	#if UBA_DEBUG_LOG_ENABLED
	const char* isWriteStr = prep.isWrite ? " WRITE" : ""; (void)isWriteStr;
	#endif

	switch (prep.action)
	{
	case SharedOpenAction::Error:
		errno = prep.err;
		return nullptr;

	case SharedOpenAction::NoDetourFallback:
	{
		TimerScope ts(g_kernelStats.createFile);
		FILE* res = trueFreopen(file, mode, stream);
		ts.Leave();
		DEBUG_LOG_TRUE(funcName, "NODETOUR%s (%s, %s) -> %p", isWriteStr, file, mode, res);
		if (res)
			TrackInput(prep.fileName);
		return res;
	}

	case SharedOpenAction::ProcFallback:
	{
		FILE* res = trueFreopen(file, mode, stream);
		DEBUG_LOG_TRUE(funcName, "NODETOUR (%s, %s) -> %p", file, mode, res);
		return res;
	}

	case SharedOpenAction::DirectoryFd:
	{
		errno = EISDIR;
		DEBUG_LOG_DETOURED(funcName, "(DIR-REJECT) %s -> nullptr (EISDIR)", prep.fileName.data);
		return nullptr;
	}

	case SharedOpenAction::UnsupportedFail:
		return nullptr;

	case SharedOpenAction::NormalOpen:
	{
		UBA_ASSERTF(!prep.isWrite || prep.closeId != 0, "%s: NormalOpen write path produced closeId=0 for %s (mode=%s) — Rpc_UpdateCloseHandle will be skipped", funcName, prep.fileName.data, mode);
		TimerScope ts(g_kernelStats.createFile);
		FILE* res = trueFreopen(prep.tempFileName, mode, stream);
		ts.Leave();

		if (!res)
		{
			DEBUG_LOG_TRUE(funcName, "%s%s (%s) %s -> nullptr (%s)", file, isWriteStr, prep.tempFileName, mode, strerror(errno));
			return nullptr;
		}

		int fd = fileno(res);
		DEBUG_LOG_TRUE(funcName, "%s%s (%s) %s -> %p fd=%i closeId=%i", file, isWriteStr, prep.tempFileName, mode, res, fd, prep.closeId);

		{
			SCOPED_READ_LOCK(g_fileHandlesLock, lock);
			UBA_ASSERTF(g_fileHandles.find(fd) == g_fileHandles.end(),
				"%s: fd=%i already has a g_fileHandles entry after trueFreopen (%s) — "
				"glibc's internal open/dup2/close went through detours and double-registered",
				funcName, fd, prep.tempFileName);
		}

		if (auto toWrite = prep.dataToWrite.size())
		{
			int wrres = write(fd, prep.dataToWrite.data(), toWrite);
			UBA_ASSERT(wrres == (int)toWrite);
		}

		TrackInput(prep.fileName);
		CreateFileTableEntry(fd, prep, false, funcName, prep.tempFileName);
		if (prep.isWrite)
			Rpc_RegisterFileForWrite(prep.fileName, prep.fileNameKey);

		return res;
	}
	}
	return nullptr;
}

template<typename True_fstat>
int Shared_fstat(const char* funcName, int fd, struct stat* attr, const True_fstat& trueFstat)
{
	SCOPED_READ_LOCK(g_fileHandlesLock, lock);
	auto findIt = g_fileHandles.find(fd);
	if (findIt == g_fileHandles.end())
	{
		TimerScope ts(g_kernelStats.getFileInfo);
		int res = trueFstat(fd, attr);
		ts.Leave();
		DEBUG_LOG_TRUE(funcName, "(%i) NODETOUR (size: %llu) -> %i (%s)", fd, attr->st_size, res, StrError(res, errno));
		return res;
	}

	FileObject& fo = *findIt->second.fileObject;
	FileInfo& fi = *fo.fileInfo;

	if (fo.desiredAccess & AccessFlag_Write)
	{
		TimerScope ts(g_kernelStats.getFileInfo);
		int res = trueFstat(fd, attr);
		ts.Leave();
		DEBUG_LOG_TRUE(funcName, "(%i) AFW (%s size: %llu) -> %i (%s)", fd, fi.originalName, attr->st_size, res, StrError(res, errno));
		return res;
	}

	FileAttributes fileAttr = {};
	const char* realName = Shared_GetFileAttributes(fileAttr, ToView(fi.originalName));

	int res = fileAttr.lastError == 0 ? 0 : -1;

	DEBUG_LOG_DETOURED(funcName, "(%i) (%s size: %llu id: %llu dev: %u) -> %i (%s)", fd, fi.originalName, fileAttr.data.st_size, fileAttr.data.st_ino, fileAttr.data.st_dev, res, StrError(res, fileAttr.lastError));

#if UBA_SUPPORT_SYMLINKS
	UBA_ASSERT(!IsSymlink(fileAttr.data.st_mode));
#endif

	errno = fileAttr.lastError;
	if (res == 0)
		memcpy(attr, &fileAttr.data, sizeof(struct stat));

	// Overlay InvalidValue (~34GiB) sentinel leaking out makes GNU cp ftruncate dest to 34GiB; fall back to kernel fstat.
	if (res == 0 && attr->st_size == (off_t)InvalidValue)
	{
		struct stat realAttr;
		if (trueFstat(fd, &realAttr) == 0)
		{
			memcpy(attr, &realAttr, sizeof(*attr)); // full attr: sqlite verifyDbFile compares st_ino fd↔path
			DEBUG_LOG_DETOURED(funcName, "(%i) overlay-size=InvalidValue, kernel size=%lld ino=%llu", fd, (long long)attr->st_size, (unsigned long long)attr->st_ino);
		}
		else
		{
			attr->st_size = 0;
			DEBUG_LOG_DETOURED(funcName, "(%i) overlay-size=InvalidValue, kernel fstat failed -> 0", fd);
		}
	}

	// Diag: dirtable size > kernel size → mmap past EOF SIGBUS. DEBUG_LOG_DETOURED only (holding g_fileHandlesLock read-lock — ::open would upgrade and deadlock).
	if (res == 0 && !g_runningRemote && !IsVfsEnabled() && !S_ISDIR(attr->st_mode))
	{
		struct stat truthAttr;
		if (trueFstat(fd, &truthAttr) == 0 && attr->st_size > truthAttr.st_size)
		{
			DEBUG_LOG_DETOURED(funcName, "(%i) UBA size %lld > kernel size %lld for %s — mmap past EOF will SIGBUS",
				fd, (long long)attr->st_size, (long long)truthAttr.st_size, fi.originalName);
		}
	}

	#if UBA_DEBUG_VALIDATE
	if (!g_runningRemote && !IsVfsEnabled())
	{
		struct stat attr2;
		int res2 = trueFstat(fd, &attr2);
		UBA_ASSERTF(res == res2, "fstat: return value differs for %s (%i vs %i)", fi.originalName, res, res2);
		if (res != -1)
		{
			bool isDir = S_ISDIR(attr->st_mode);
			UBA_ASSERTF(isDir == S_ISDIR(attr2.st_mode), "fstat: isDir not matching");
			UBA_ASSERTF(attr->st_ino == attr2.st_ino, "fstat: st_ino mismatch for %s (%llu vs %llu)", fi.originalName, attr->st_ino, attr2.st_ino);
			UBA_ASSERTF(isDir || attr->st_size == attr2.st_size, "fstat: size not matching");
			UBA_ASSERTF(isDir || FromTimeSpec(attr->st_mtimespec) == FromTimeSpec(attr2.st_mtimespec), "fstat: st_mtim mismatch for %s (%llu vs %llu)", fi.originalName, FromTimeSpec(attr->st_mtimespec), FromTimeSpec(attr2.st_mtimespec));
		}
		else
			UBA_ASSERTF(fileAttr.lastError == errno, "fstat: error not matching");
	}
	#endif

	return res;
}

// Shared_stat template body moved to UbaDetoursSharedPosix.inl (alongside the other Shared_* templates).

#if PLATFORM_LINUX
int LdpreloadStatx(const char* funcName, int dirfd, const char* pathname, int flags, unsigned int mask, struct statx* statxbuf)
{
	pathname = ResolveProcSelfFd(pathname);
	StringBuffer<> temp;
	const char* resolvedPath = ResolvePath(dirfd, pathname, temp);
	if (!resolvedPath)
	{
		int res = (int)Real_statx(errno, dirfd, pathname, flags, mask, statxbuf);
		if (res < 0 && errno == EAGAIN) // tmpfs EAGAIN backoff (~500ms cap)
		{
			struct timespec ts;
			ts.tv_sec = 0;
			long usec = 100;
			for (int retries = 0; retries < 25 &&res < 0 && errno == EAGAIN; ++retries)
			{
				ts.tv_nsec = usec * 1000;
				nanosleep(&ts, nullptr);
				res = (int)Real_statx(errno, dirfd, pathname, flags, mask, statxbuf);
				usec *= 2;
				if (usec > 50000) usec = 50000;
			}
		}
		DEBUG_LOG_TRUE("syscall.statx", "(NODETOUR) %i %s -> %d", dirfd, pathname, res);
		return res;
	}
	bool trueStatCalled = false;
	struct stat st;
	int res = Shared_stat(funcName, resolvedPath, &st, [&](const char* path, struct stat*) -> int
		{
			trueStatCalled = true;
			return (int)Real_statx(errno, AT_FDCWD, path, flags, mask, statxbuf);
		});
	if (trueStatCalled)
		return res;
	if (res != 0)
		return res;

	memset(statxbuf, 0, sizeof(*statxbuf));
	statxbuf->stx_mask       = STATX_BASIC_STATS;
	statxbuf->stx_blksize    = (u32)st.st_blksize;
	statxbuf->stx_nlink      = (u32)st.st_nlink;
	statxbuf->stx_uid        = st.st_uid;
	statxbuf->stx_gid        = st.st_gid;
	statxbuf->stx_mode       = (u16)st.st_mode;
	statxbuf->stx_ino        = st.st_ino;
	statxbuf->stx_size       = (u64)st.st_size;
	statxbuf->stx_blocks     = (u64)st.st_blocks;
	statxbuf->stx_atime      = { st.st_atim.tv_sec, (u32)st.st_atim.tv_nsec };
	statxbuf->stx_mtime      = { st.st_mtim.tv_sec, (u32)st.st_mtim.tv_nsec };
	statxbuf->stx_ctime      = { st.st_ctim.tv_sec, (u32)st.st_ctim.tv_nsec };
	statxbuf->stx_rdev_major = major(st.st_rdev);
	statxbuf->stx_rdev_minor = minor(st.st_rdev);
	statxbuf->stx_dev_major  = major(st.st_dev);
	statxbuf->stx_dev_minor  = minor(st.st_dev);
	return res;
}
#endif

// strict=true: enforce R/W/X_OK against dirtable mode bits (eaccess — bash PATH search depends on X_OK actually rejecting non-executables).
int Shared_access(const char* funcName, const char* pathname, int mode, bool strict = false)
{
	pathname = ResolveProcSelfFd(pathname);
	StringView original(ToView(pathname));

	StringBuffer<> fixedPath;
	bool fixPathOk = FixPath(fixedPath, original);
	if (!fixPathOk || !CanDetour2(fixedPath))
	{
		TimerScope ts(g_kernelStats.getFileInfo);
		const char* kernelPath = fixPathOk ? fixedPath.data : original.data;
		auto res = Real_access(errno, kernelPath, mode);
		ts.Leave();
		DEBUG_LOG_TRUE(funcName, "%s %i -> %i (%s)", kernelPath, mode, res, StrError(res, errno));
		return res;
	}

	bool checkIfDir = false;
	StringBuffer<> temp;
	if (g_runningRemote && fixedPath.StartsWith(g_exeDir.data))
	{
		temp.Append(g_virtualApplicationDir).Append(fixedPath.data + g_exeDir.count);

		if (temp.count == g_virtualApplicationDir.count) // Remove last slash from temp if this is a access call for the actual applicaiton directory
		{
			checkIfDir = true;
			temp.Resize(temp.count - 1);
		}
		fixedPath.Clear().Append(temp);
	}

	DevirtualizePath(fixedPath);

	if (!CanDetour(original))
	{
		TimerScope ts(g_kernelStats.getFileInfo);
		auto res = Real_access(errno, fixedPath.data, mode);
		ts.Leave();
		DEBUG_LOG_TRUE(funcName, "%s %i -> %i (%s)", fixedPath.data, mode, res, StrError(res, errno));
		return res;
	}

	FileAttributes attr;
	bool followSymlink = true;
	const char* realName = Shared_GetFileAttributes(attr, fixedPath, followSymlink, checkIfDir);

	int res = attr.lastError == 0 ? 0 : -1;

	if (strict && res == 0 && S_ISREG(attr.data.st_mode)) // eaccess: permission-bit check on regular files; dirs skip (traverse perm always granted)
	{
		const u32 m = attr.data.st_mode;
		bool deny = false;
		if ((mode & X_OK) && !(m & (S_IXUSR | S_IXGRP | S_IXOTH))) deny = true;
		if ((mode & R_OK) && !(m & (S_IRUSR | S_IRGRP | S_IROTH))) deny = true;
		if ((mode & W_OK) && !(m & (S_IWUSR | S_IWGRP | S_IWOTH))) deny = true;
		if (deny)
		{
			errno = EACCES;
			DEBUG_LOG_DETOURED(funcName, "%s %i (%s mode: %o) -> -1 (EACCES)", original.data, mode, realName, m);
			return -1;
		}
	}

	DEBUG_LOG_DETOURED(funcName, "%s %i (%s mode: %o) -> %i %s", original.data, mode, realName, attr.data.st_mode, res, StrError(res, attr.lastError));

	#if UBA_DEBUG_VALIDATE
	if (!g_runningRemote)
	{
		auto res2 = Real_access(errno, realName, mode);
		DEBUG_LOG_DETOURED(funcName, "%s %i (%s) -> %i %s", original.data, mode, realName, res, StrError(res, attr.lastError));
		UBA_ASSERTF(res2 == res, "MISMATCH OF RESULTS for %s - %i %i (err = %s) (exedir %s)", realName, res2, res, StrError(res, attr.lastError), g_exeDir.data);
	}
	#endif
	
	errno = attr.lastError;
	return res;
}


#if PLATFORM_MAC
UBA_EXPORT int UBA_WRAPPER(_NSGetExecutablePath)(char* buf, uint32_t* bufsize)
{
	if (!g_isDetouring)
	{
		DEBUG_LOG_TRUE("NSGetExecutablePath", "");
		return TRUE_WRAPPER(_NSGetExecutablePath)(buf, bufsize);
	}
	if (bufsize == nullptr)
		return -1;
	const uint32_t requiredBufsize = g_virtualApplication.count + 1;
	const uint32_t initialBufsize = *bufsize;
	*bufsize = requiredBufsize;
	if (initialBufsize < requiredBufsize)
		return -1;
	if (buf != nullptr)
		memcpy(buf, g_virtualApplication.data, requiredBufsize);
	DEBUG_LOG_DETOURED("NSGetExecutablePath", "%s", buf);
	return 0;
}

#endif


UBA_EXPORT int UBA_WRAPPER(chdir)(const char* path)
{
	UBA_INIT_DETOUR(chdir, path);
	if (path == nullptr || *path == '\0')
	{
		errno = ENOENT;
		DEBUG_LOG_TRUE("chdir", "(BAD_PATH) -> -1");
		return -1;
	}
	int rc = Shared_chdir("chdir", path, errno);

#if PLATFORM_LINUX
	if (rc == 0) // mirror to kernel when cwd is under a pass-through subtree so NODETOUR relative ops resolve there
	{
		StringBuffer<MaxPath> devirt;
		devirt.Append(g_virtualWorkingDir);
		uba::DevirtualizePath(devirt);
		if (uba::IsUnderPassThroughRoot(StringView(devirt.data, devirt.count)))
		{
			long krc = ::syscall(SYS_chdir, g_virtualWorkingDir.data); // NS-virtual form (post-pivot kernel)
			DEBUG_LOG_TRUE("chdir", "(passthrough kernel) %s -> %ld", g_virtualWorkingDir.data, krc);
		}
	}
#endif
	return rc;
}

UBA_EXPORT int UBA_WRAPPER(fchdir)(int fd)
{
	UBA_INIT_DETOUR(fchdir, fd);
	int err = 0;
	int rc = Shared_fchdir(fd, err);
#if PLATFORM_LINUX
	if (rc < 0 && err == EBADF) // NODETOUR-opened fd: kernel fchdir + readlink /proc/self/fd to resync virtual cwd
	{
		long krc = ::syscall(SYS_fchdir, fd);
		if (krc == 0)
		{
			char proc[64];
			int n = snprintf(proc, sizeof(proc), "/proc/self/fd/%d", fd);
			if (n > 0 && (size_t)n < sizeof(proc))
			{
				char buf[1024];
				long rl = ::syscall(SYS_readlinkat, AT_FDCWD, proc, buf, (size_t)(sizeof(buf) - 1));
				if (rl > 0)
				{
					buf[rl] = 0;
					int e2 = 0;
					Shared_chdir("fchdir", buf, e2);
				}
			}
			DEBUG_LOG_TRUE("fchdir", "(passthrough kernel) fd=%d -> 0", fd);
			return 0;
		}
		DEBUG_LOG_TRUE("fchdir", "(kernel fallback) fd=%d -> -1 errno=%d", fd, errno);
		return -1;
	}
#endif

	if (rc < 0) errno = err;
	return rc;
}

UBA_EXPORT int UBA_WRAPPER(mkdir)(const char* path, mode_t mode)
{
	UBA_INIT_DETOUR(mkdir, path, mode);
	int err = 0;
	int rc = Shared_mkdir("mkdir", path, mode, err, [](const char* p, mode_t m) { return Real_mkdir(errno,p, m); });
	if (rc == -1 && err)
		errno = err;
	return rc;
}

UBA_EXPORT int UBA_WRAPPER(mkdirat)(int dirfd, const char* path, mode_t mode)
{
	UBA_INIT_DETOUR(mkdirat, dirfd, path, mode);
	StringBuffer<> temp;
	if (const char* resolved = ResolvePath(dirfd, path, temp))
	{
		int err = 0;
		int rc = Shared_mkdir("mkdirat", resolved, mode, err, [](const char* p, mode_t m) { return Real_mkdir(errno,p, m); });
		if (rc == -1 && err)
			errno = err;
		return rc;
	}
	UBA_ASSERTF(!g_runningRemote, TC("mkdirat: dirfd=%d not in g_fileHandles, path='%s' cwd='%s'"), dirfd, path, g_virtualWorkingDir.data);
	int res = Real_mkdirat(errno,dirfd, path, mode);
	DEBUG_LOG_TRUE("mkdirat", "(%i) %s -> %i", dirfd, path, res);
	return res;
}

UBA_EXPORT int UBA_WRAPPER(rmdir)(const char* path)
{
	UBA_INIT_DETOUR(rmdir, path);
	int err = 0;
	int rc = Shared_rmdir("rmdir", path, err, [](const char* p) { return Real_rmdir(errno,p); });
	if (rc == -1 && err)
		errno = err;
	return rc;
}

// Also called from raw-syscall dispatcher (case 161); no UBA_INIT_DETOUR here.
static int Shared_chroot(const char* path)
{
	StringBuffer<> abs;
	if (!FixPath(abs, path))
	{
		DEBUG_LOG_DETOURED("chroot", "%s -> -1 (FixPath fail)", path);
		errno = EINVAL;
		return -1;
	}
	bool ok = AddVfsEntry(StringView(TC("/"), 1), StringView(abs.data, abs.count)); // VFS overlay "/"→path; skip kernel chroot so host RPC keeps working
	DEBUG_LOG_DETOURED("chroot", "%s -> %d (vfs / -> %s)", abs.data, ok ? 0 : -1, abs.data);
	if (!ok) { errno = ENOMEM; return -1; }
	return 0;
}

UBA_EXPORT int UBA_WRAPPER(chroot)(const char* path)
{
	UBA_INIT_DETOUR(chroot, path);
	return Shared_chroot(path);
}

#if PLATFORM_LINUX
// nsjail calls mount() directly; dispatch by flag shape (prop/REMOUNT/BIND/fresh-fs).
static int Shared_mount(const char* source, const char* target, const char* fstype, unsigned long flags, const void* data)
{
	constexpr unsigned long kMountPropagationFollower = (1<<19); // Unacceptable word for flag so can't use the real flag

	constexpr unsigned long kPropFlags = MS_PRIVATE | MS_SHARED | MS_UNBINDABLE | kMountPropagationFollower;
	constexpr unsigned long kStructuralFlags = MS_BIND | MS_MOVE | MS_REMOUNT;
	bool isPropOnly = (flags & kPropFlags) != 0 && (flags & kStructuralFlags) == 0 && !fstype; // MS_PRIVATE|MS_REC etc — pass through
	if (isPropOnly)
	{
		int res = TRUE_WRAPPER(mount)(source, target, fstype, flags, data);
		int err = res < 0 ? errno : 0;
		DEBUG_LOG_TRUE("mount", "(prop) src=%s tgt=%s flags=0x%lx -> %d %s",
			source ? source : "(null)", target ? target : "(null)", flags,
			res, StrError(res, err));
		if (res < 0) errno = err;
		return res;
	}
	// MS_REMOUNT no-op (VFS overlay doesn't track per-mount flags); MUST NOT fall through to MS_BIND.
	if ((flags & MS_REMOUNT) != 0)
	{
		DEBUG_LOG_DETOURED("mount", "(remount) tgt=%s flags=0x%lx -> 0 (no-op)",
			target ? target : "(null)", flags);
		return 0;
	}
	if ((flags & MS_BIND) != 0) // MS_BIND → VFS overlay; skip kernel mount
	{
		StringBuffer<> srcAbs, dstAbs;
		if (!source || !target || !FixPath(srcAbs, source) || !FixPath(dstAbs, target))
		{
			DEBUG_LOG_DETOURED("mount", "(bind) src=%s tgt=%s -> -1 (FixPath fail)",
				source ? source : "(null)", target ? target : "(null)");
			errno = EINVAL;
			return -1;
		}
		bool ok = AddVfsEntry(StringView(dstAbs.data, dstAbs.count), StringView(srcAbs.data, srcAbs.count));
		DEBUG_LOG_DETOURED("mount", "(bind) src=%s tgt=%s flags=0x%lx -> %d (vfs %s)",
			srcAbs.data, dstAbs.data, flags, ok ? 0 : -1, ok ? "added" : "table-full");
		if (!ok) { errno = ENOMEM; return -1; }
		return 0;
	}
	// Fresh-fs mount (tmpfs/proc/...) — VFS-only redirect to a writable subdir under session temp; on-disk target may not exist on remote helpers.
	if (fstype && *fstype && (flags & MS_BIND) == 0)
	{
		StringBuffer<> tgtAbs;
		if (!target || !FixPath(tgtAbs, target))
		{
			DEBUG_LOG_DETOURED("mount", "(fs=%s) tgt=%s -> -1 (FixPath fail)",
				fstype, target ? target : "(null)");
			errno = EINVAL;
			return -1;
		}

		StringBuffer<> phys;
		phys.Append(g_systemTemp);
		// No trailing PathSeparator — Shared_mkdir's Last(PathSeparator) would otherwise split the path itself as parent.
		for (u32 i = 0; i < tgtAbs.count; ++i)
			phys.Append(tgtAbs[i] == PathSeparator ? '_' : tgtAbs[i]);

		// Must go through Shared_mkdir so DIRTABLE records it — TRUE_WRAPPER(mkdir) leaves subsequent DIRTABLE-routed opens of phys ENOENT.
		int mkdirErr = 0;
		int rc = Shared_mkdir("mount(tmpfs)/mkdir", phys.data, 0700, mkdirErr, [](const char* p, mode_t m) { return Real_mkdir(errno,p, m); });
		(void)rc;

		bool ok = AddVfsEntry(StringView(tgtAbs.data, tgtAbs.count), StringView(phys.data, phys.count));
		DEBUG_LOG_DETOURED("mount", "(fs=%s vfs-only) tgt=%s -> phys=%s (vfs %s)",
			fstype, tgtAbs.data, phys.data, ok ? "added" : "table-full");
		if (!ok) { errno = ENOMEM; return -1; }
		return 0;
	}
	UBA_ASSERTF(false, "mount not implemented (src=%s tgt=%s fstype=%s flags=0x%lx)",
		source ? source : "(null)", target ? target : "(null)",
		fstype ? fstype : "(null)", flags);
	return TRUE_WRAPPER(mount)(source, target, fstype, flags, data);
}

UBA_EXPORT int UBA_WRAPPER(mount)(const char* source, const char* target, const char* fstype, unsigned long flags, const void* data)
{
	UBA_INIT_DETOUR(mount, source, target, fstype, flags, data);
	return Shared_mount(source, target, fstype, flags, data);
}

static int Shared_umount(const char* target)
{
	StringBuffer<> abs;
	const char* tgtForKernel = target;
	if (target && FixPath(abs, target))
	{
		if (abs.count == 1 && abs.data[0] == '/') // post-pivot umount("/") — keep vfs="/" or NS lookups lose pivot translation
		{
			DEBUG_LOG_DETOURED("umount", "tgt=/ -> 0 (post-pivot cleanup, vfs preserved)");
			return 0;
		}
		if (RemoveVfsEntry(StringView(abs.data, abs.count))) // virtual overlay (MS_BIND/pivot_root/chroot): kernel never had a mount
		{
			DEBUG_LOG_DETOURED("umount", "tgt=%s -> 0 (virtual)", abs.data);
			return 0;
		}
		DevirtualizePath(abs);
		tgtForKernel = abs.data;
	}
	int res = TRUE_WRAPPER(umount)(tgtForKernel);
	int err = res < 0 ? errno : 0;
	DEBUG_LOG_TRUE("umount", "tgt=%s -> %d %s",
		tgtForKernel ? tgtForKernel : "(null)", res, StrError(res, err));
	if (res < 0) errno = err;
	return res;
}

UBA_EXPORT int UBA_WRAPPER(umount)(const char* target)
{
	UBA_INIT_DETOUR(umount, target);
	return Shared_umount(target);
}

static int Shared_umount2(const char* target, int flags)
{
	StringBuffer<> abs;
	const char* tgtForKernel = target;
	if (target && FixPath(abs, target))
	{
		if (abs.count == 1 && abs.data[0] == '/') // see Shared_umount
		{
			DEBUG_LOG_DETOURED("umount2", "tgt=/ flags=0x%x -> 0 (post-pivot cleanup, vfs preserved)", flags);
			return 0;
		}
		if (RemoveVfsEntry(StringView(abs.data, abs.count)))
		{
			DEBUG_LOG_DETOURED("umount2", "tgt=%s flags=0x%x -> 0 (virtual)", abs.data, flags);
			return 0;
		}
		DevirtualizePath(abs);
		tgtForKernel = abs.data;
	}
	int res = TRUE_WRAPPER(umount2)(tgtForKernel, flags);
	int err = res < 0 ? errno : 0;
	DEBUG_LOG_TRUE("umount2", "tgt=%s flags=0x%x -> %d %s",
		tgtForKernel ? tgtForKernel : "(null)", flags, res, StrError(res, err));
	if (res < 0) errno = err;
	return res;
}

UBA_EXPORT int UBA_WRAPPER(umount2)(const char* target, int flags)
{
	UBA_INIT_DETOUR(umount2, target, flags);
	return Shared_umount2(target, flags);
}

static int Shared_pivot_root(const char* new_root, const char* put_old)
{
	StringBuffer<> abs;
	if (!FixPath(abs, new_root))
	{
		DEBUG_LOG_DETOURED("pivot_root", "new=%s -> -1 (FixPath fail)", new_root);
		errno = EINVAL;
		return -1;
	}
	// Same shape as chroot: vfs="/"→new_root. put_old ignored — kernel never pivots.
	bool ok = AddVfsEntry(StringView(TC("/"), 1), StringView(abs.data, abs.count));
	DEBUG_LOG_DETOURED("pivot_root", "new=%s put_old=%s -> %d (vfs / -> %s)",
		abs.data, put_old ? put_old : "(null)", ok ? 0 : -1, abs.data);
	if (!ok) { errno = ENOMEM; return -1; }
	return 0;
}

UBA_EXPORT int UBA_WRAPPER(pivot_root)(const char* new_root, const char* put_old)
{
	UBA_INIT_DETOUR(pivot_root, new_root, put_old);
	return Shared_pivot_root(new_root, put_old);
}

// Synthetic statvfs (nsjail uses it pre-MS_REMOUNT).
static void FillFakeStatvfs(struct statvfs* buf)
{
	if (!buf) return;
	memset(buf, 0, sizeof(*buf));
	buf->f_bsize   = 4096;
	buf->f_frsize  = 4096;
	buf->f_blocks  = 1ull << 30;   // ~4TiB total
	buf->f_bfree   = 1ull << 29;   // ~2TiB free
	buf->f_bavail  = 1ull << 29;
	buf->f_files   = 1ull << 24;
	buf->f_ffree   = 1ull << 23;
	buf->f_favail  = 1ull << 23;
	buf->f_fsid    = 0;
	buf->f_flag    = 0;            // no ST_RDONLY / ST_NOSUID — fully writable
	buf->f_namemax = 255;
}

UBA_EXPORT int UBA_WRAPPER(statvfs)(const char* path, struct statvfs* buf)
{
	UBA_INIT_DETOUR(statvfs, path, buf);
	FillFakeStatvfs(buf);
	DEBUG_LOG_DETOURED("statvfs", "%s -> 0 (synthetic)", path);
	return 0;
}

UBA_EXPORT int UBA_WRAPPER(fstatvfs)(int fd, struct statvfs* buf)
{
	UBA_INIT_DETOUR(fstatvfs, fd, buf);
	FillFakeStatvfs(buf);
	DEBUG_LOG_DETOURED("fstatvfs", "fd=%d -> 0 (synthetic)", fd);
	return 0;
}
#endif

UBA_EXPORT char* UBA_WRAPPER(getcwd)(char* buf, size_t size)
{
	UBA_INIT_DETOUR(getcwd, buf, size);

	// g_virtualWorkingDir always ends with '/' (FixPath2 requires it); POSIX getcwd must NOT return trailing slash except for "/".
	u32 len = g_virtualWorkingDir.count;
	if (len > 1 && g_virtualWorkingDir.data[len - 1] == '/')
		--len;

	if (size == 0)
	{
		if (buf == nullptr)
		{
			buf = (char*)malloc(len + 1);
			memcpy(buf, g_virtualWorkingDir.data, len);
			buf[len] = 0;
			DEBUG_LOG_DETOURED("getcwd", "(null) -> %s ", buf);
			return buf;
		}

		errno = EINVAL;
		DEBUG_LOG_DETOURED("getcwd", "(null) -> null (EINVAL)");
		return nullptr;
	}
	if (size < len + 1)
	{
		DEBUG_LOG_DETOURED("getcwd", "-> null (Buffer too small: %u)", u32(size));
		errno = ERANGE;
		return nullptr;
	}
	if (!buf)
		buf = (char*)malloc(size);

	memcpy(buf, g_virtualWorkingDir.data, len);
	buf[len] = 0;
	DEBUG_LOG_DETOURED("getcwd", "%s -> %p", buf, buf);
	return buf;
}

UBA_EXPORT char* UBA_WRAPPER(getenv)(const char* name)
{
	UBA_INIT_DETOUR(getenv, name);
	auto res = TRUE_WRAPPER(getenv)(name);
	DEBUG_LOG_TRUE("getenv", "(%s) -> %s", name, res ? res : "<null>");
	return res;
}

UBA_EXPORT int UBA_WRAPPER(setenv)(const char* name, const char* value, int replace)
{
	UBA_INIT_DETOUR(setenv, name, value, replace);
	auto res = TRUE_WRAPPER(setenv)(name, value, replace);
	DEBUG_LOG_TRUE("setenv", "(%s) -> %s (%i)", name, value, res);
	return res;
}

UBA_EXPORT int UBA_WRAPPER(unsetenv)(const char* name)
{
	UBA_INIT_DETOUR(unsetenv, name);
	auto res = TRUE_WRAPPER(unsetenv)(name);
	DEBUG_LOG_TRUE("unsetenv", "(%s) -> (%i)", name, res);
	return res;
}

UBA_EXPORT char* UBA_WRAPPER(realpath)(const char* path, char* resolved_path)
{
	UBA_INIT_DETOUR(realpath, path, resolved_path);

	// /proc/self/exe under VFS resolves host-real; callers (clang resource-dir, Go GOROOT, JAVA_HOME) need the NS-virtual path.
	if (path && Equals(path, "/proc/self/exe") && g_virtualApplication.count > 0
		&& (IsVfsEnabled() || g_runningRemote))
	{
		if (!resolved_path)
			resolved_path = (char*)malloc(g_virtualApplication.count + 1);
		memcpy(resolved_path, g_virtualApplication.data, g_virtualApplication.count);
		resolved_path[g_virtualApplication.count] = 0;
		DEBUG_LOG_DETOURED("realpath", "(%s) -> %s (vfs-virt)", path, resolved_path);
		return resolved_path;
	}

	// TODO: This entire thing should be removed when UBA_SUPPORT_SYMLINKS is enabled.. which eventually should be enabled on all posix platforms
	if (!g_runningRemote && !IsVfsEnabled())
	{
		// Apple's realpath implementation calls open(O_RDONLY|O_NONBLOCK) + fcntl(F_GETPATH)
		// internally. Our open detour would otherwise interfere (path rewrites, dirtable
		// lookups, etc.). Suppress detouring around the libc call so it sees a clean libc.
		SuppressDetourScope sds;
		char ourBuf[PATH_MAX];
		auto res = TRUE_WRAPPER(realpath)(path, ourBuf);
		if (!res)
		{
			DEBUG_LOG_TRUE("realpath", "(%s) -> (null) (%s)", path, StrError(-1, errno));
			return nullptr;
		}
		if (!resolved_path)
		{
			size_t n = strlen(ourBuf) + 1;
			resolved_path = (char*)malloc(n);
			memcpy(resolved_path, ourBuf, n);
		}
		else
		{
			strcpy(resolved_path, ourBuf);
		}
		DEBUG_LOG_TRUE("realpath", "(%s) -> %s", path, resolved_path);
		return resolved_path;
	}

	StringBuffer<> fixedPath;
	FixPath(fixedPath, path);

	StringBuffer<> devirtualizedPath(fixedPath);
	DevirtualizePath(devirtualizedPath);

#if UBA_SUPPORT_SYMLINKS
	// Walk the path through the DIRTABLE, dereferencing any DT_LNK entries along
	// the way (matches the opendir precedent at ~line 2270). Rpc_GetEntryInformation
	// does both the existence check and the symlink resolution in one round-trip.
	DirectoryTableEntryInformation einfo;
	StringBuffer<> resolved;
	StringKey resolvedKey;
	if (!Rpc_GetEntryInformation(einfo, ToFilenameKey(devirtualizedPath), devirtualizedPath, false, true, &resolved, &resolvedKey) || einfo.attributes == 0)
	{
		errno = ENOENT;
		DEBUG_LOG_DETOURED("realpath", "(%s) -> nullptr (ENOENT)", path);
		return nullptr;
	}
	if (resolved.count && !resolved.Equals(devirtualizedPath))
		devirtualizedPath.Clear().Append(resolved);
#else
	FileAttributes fileAttr;
	Shared_GetFileAttributes(fileAttr, devirtualizedPath);
	if (!fileAttr.exists)
	{
		errno = ENOENT;
		DEBUG_LOG_DETOURED("realpath", "(%s) -> nullptr (ENOENT)", path);
		return nullptr;
	}
#endif

	if (!resolved_path)
		resolved_path = (char*)malloc(devirtualizedPath.count + 1);
	memcpy(resolved_path, devirtualizedPath.data, devirtualizedPath.count + 1);
	DEBUG_LOG_DETOURED("realpath", "(%s) -> %s", path, resolved_path);
	return resolved_path;
}

UBA_EXPORT ssize_t UBA_WRAPPER(readlink)(const char* pathname, char* buf, size_t bufsiz)
{
	UBA_INIT_DETOUR(readlink, pathname, buf, bufsiz);

	// je_malloc/dlsym recursion deadlock during detour init: ENOENT on /etc/je_malloc.conf breaks the loop.
	if (!g_isDetouring && Equals(pathname, "/etc/je_malloc.conf"))
	{
		errno = ENOENT;
		return -1;
	}

	{
		u64 outLen = 0;
		switch (Shared_prepareReadlink("readlink", pathname, buf, bufsiz, outLen, errno))
		{
		case SharedReadlinkAction::Synthesized:
			return (ssize_t)outLen;
		case SharedReadlinkAction::Error:
			return -1;
		case SharedReadlinkAction::Passthrough:
			break;
		}
	}

	// chdir is virtualized — pass absolute path so kernel doesn't resolve against its stale cwd.
	StringBuffer<MaxPath> resolved;
	if (FixPath(resolved, ToView(pathname)))
		pathname = resolved.data;
	auto res = Real_readlink(errno, pathname, buf, bufsiz);
	DEBUG_LOG_TRUE("readlink", "(%s) (%s) -> %lli (%s)", pathname, buf, res, StrError(int(res), errno));

	if (res && res < bufsiz && IsVfsEnabled())
	{
		StringBuffer<> temp(buf);
		if (!uba::IsUnderPassThroughRoot(StringView(temp.data, temp.count))) // pass-through subtrees legitimately devirtualize via "/" overlay
			UBA_ASSERT(!DevirtualizePath(temp));
	}

	return res;
}

UBA_EXPORT ssize_t UBA_WRAPPER(readlinkat)(int dirfd, const char* pathname, char* buf, size_t bufsiz)
{
	UBA_INIT_DETOUR(readlinkat, dirfd, pathname, buf, bufsiz);

	StringBuffer<> temp;
	if (const char* resolvedPath = ResolvePath(dirfd, pathname, temp))
	{
		u64 outLen = 0;
		switch (Shared_prepareReadlink("readlinkat", resolvedPath, buf, bufsiz, outLen, errno))
		{
		case SharedReadlinkAction::Synthesized:
			return (ssize_t)outLen;
		case SharedReadlinkAction::Error:
			return -1;
		case SharedReadlinkAction::Passthrough:
			break;
		}
	}

	UBA_ASSERTF(!g_runningRemote, TC("readlinkat: dirfd=%d not in g_fileHandles, path='%s' cwd='%s'"), dirfd, pathname, g_virtualWorkingDir.data);
	auto res = Real_readlinkat(errno, dirfd, pathname, buf, bufsiz);
	DEBUG_LOG_TRUE("readlinkat", "(%i %s) -> %lli (%s)", dirfd, pathname, (long long)res, StrError(int(res), errno));
	return res;
}

UBA_EXPORT int UBA_WRAPPER(fcntl)(int fd, int cmd, ...)
{
	va_list ap;
	va_start(ap, cmd);
	void* arg = va_arg(ap, void*);
	va_end(ap);
	UBA_INIT_DETOUR(fcntl, fd, cmd, arg);
	int res = Real_fcntl(errno, fd, cmd, (long)arg);
	if (res != -1 && (cmd == F_DUPFD || cmd == F_DUPFD_CLOEXEC))
		DuplicateFd(fd, res, /*newCloexec*/cmd == F_DUPFD_CLOEXEC);
	if (res != -1 && cmd == F_SETFD)
	{
		SCOPED_WRITE_LOCK(g_fileHandlesLock, lock);
		auto it = g_fileHandles.find(fd);
		if (it != g_fileHandles.end())
			it->second.hasCloexec = ((u64)(uintptr_t)arg & FD_CLOEXEC) != 0;
	}
	DEBUG_LOG_TRUE("fcntl", "(%i, %i) -> %i", fd, cmd, res);
	return res;
}

#if PLATFORM_LINUX

UBA_EXPORT int UBA_WRAPPER(fcntl64)(int fd, int cmd, ...)
{
	va_list ap;
	va_start(ap, cmd);
	void* arg = va_arg(ap, void*);
	va_end(ap);
	UBA_INIT_DETOUR(fcntl64, fd, cmd, arg);
	int res = Real_fcntl(errno, fd, cmd, (long)arg);
	if (res != -1 && (cmd == F_DUPFD || cmd == F_DUPFD_CLOEXEC))
		DuplicateFd(fd, res, /*newCloexec*/cmd == F_DUPFD_CLOEXEC);
	if (res != -1 && cmd == F_SETFD)
	{
		SCOPED_WRITE_LOCK(g_fileHandlesLock, lock);
		auto it = g_fileHandles.find(fd);
		if (it != g_fileHandles.end())
			it->second.hasCloexec = ((u64)(uintptr_t)arg & FD_CLOEXEC) != 0;
	}
	DEBUG_LOG_TRUE("fcntl64", "(%i, %i) -> %i", fd, cmd, res);
	return res;
}

UBA_EXPORT ssize_t UBA_WRAPPER(pread64)(int __fd, void * __buf, size_t __nbyte, off_t __offset)
{
	return Real_pread(errno,__fd, __buf, __nbyte, __offset);
}

UBA_EXPORT int UBA_WRAPPER(open64)(const char* file, int flags, ...)
{
	va_list args;
	va_start(args, flags);
	int mode = va_arg(args, int);
	va_end(args);
	UBA_INIT_DETOUR(open64, file, flags, mode);
	return Shared_open("open64", file, flags, mode, [](const char* realFile, int flags, int mode) { return Real_open(errno,realFile, flags, mode); });
}
UBA_EXPORT int UBA_WRAPPER(creat64)(const char *path, mode_t mode)
{
	UBA_INIT_DETOUR(creat64, path, mode);
	return Shared_open("creat64", path, O_WRONLY | O_CREAT | O_TRUNC, mode, [](const char* p, int f, int m) { return Real_open(errno,p, f, m); });
}
UBA_EXPORT char* UBA_WRAPPER(secure_getenv)(const char* name)
{
	UBA_INIT_DETOUR(secure_getenv, name);
	auto res = TRUE_WRAPPER(secure_getenv)(name);
	DEBUG_LOG_TRUE("secure_getenv", "(%s) -> %s", name, res ? res : "<null>");
	return res;
}

#endif

// Chain SIGBUS/SIGSEGV/SIGILL/SIGFPE so user-installed handlers (clang.real) don't clobber UBA's crash backtrace. Handler I/O must use RawSyscallDirect — detoured open/write would desync the dirtable RPC stream.
#if PLATFORM_LINUX
namespace {
	struct ChainedHandler { struct sigaction user{}; bool valid = false; };
	static ChainedHandler g_chainedHandlers[NSIG];

	// Address range of libUbaDetours.so — used to decide whether SEGV/SIGBUS originated in UBA code (worth symbolicating) vs. user code (JVM safepoint etc).
	static unsigned long long g_ubaLibStart = 0;
	static unsigned long long g_ubaLibEnd = 0;
	static bool g_isCrashSignal(int sig) {
		return sig == SIGBUS || sig == SIGSEGV || sig == SIGILL || sig == SIGFPE;
	}

	// Async-signal-safe; no malloc/varargs.
	static char* appendDec(char* p, unsigned int v) {
		char digs[16]; int dl = 0;
		if (!v) digs[dl++] = '0';
		while (v) { digs[dl++] = (char)('0' + v % 10); v /= 10; }
		while (dl--) *p++ = digs[dl];
		return p;
	}

	static char* appendHex(char* p, unsigned long long v) {
		const char* hd = "0123456789abcdef";
		*p++ = '0'; *p++ = 'x';
		char tmp[24]; int tl = 0;
		if (!v) tmp[tl++] = '0';
		while (v) { tmp[tl++] = hd[v & 0xf]; v >>= 4; }
		while (tl--) *p++ = tmp[tl];
		return p;
	}

	static void rawWrite(long fd, const void* buf, long len) {
		RawSyscallDirect(SYS_write, fd, (long)buf, len); // short writes accepted; log is best-effort
	}

	#ifndef UBA_CHAINED_SIGNAL_LOG_FILE // Set to 1 to also dump /tmp/uba_crash.<pid>.log; default routes the callstack through Rpc_ResolveCallstack.
	#define UBA_CHAINED_SIGNAL_LOG_FILE 0
	#endif

	static void ChainedSigaction(int sig, siginfo_t* info, void* uc)
	{
		ucontext_t* ucp = (ucontext_t*)uc;
		#if PLATFORM_CPU_ARM_FAMILY
		unsigned long long rip = ucp ? (unsigned long long)ucp->uc_mcontext.pc : 0;
		#else
		unsigned long long rip = ucp ? (unsigned long long)ucp->uc_mcontext.gregs[REG_RIP] : 0;
		#endif
		unsigned long long fault_addr = info ? (unsigned long long)info->si_addr : 0;
		int tid = (int)RawSyscallDirect(SYS_gettid, 0, 0, 0).r1;
		int pid = (int)RawSyscallDirect(SYS_getpid, 0, 0, 0).r1;
		int sicode = info ? info->si_code : 0;

		#if UBA_CHAINED_SIGNAL_LOG_FILE
		char path[64]; char* p = path;
		const char prefix[] = "/tmp/uba_crash.";
		for (size_t i = 0; i < sizeof(prefix) - 1; ++i) *p++ = prefix[i];
		p = appendDec(p, (unsigned int)pid);
		const char suffix[] = ".log";
		for (size_t i = 0; i < sizeof(suffix); ++i) *p++ = suffix[i]; // includes \0

		long fd = RawSyscallDirect(SYS_openat, (long)AT_FDCWD, (long)path,
			O_WRONLY | O_CREAT | O_APPEND, 0644).r1;
		if (fd >= 0) {
			char buf[256]; char* b = buf;
			const char h0[] = "[CHAINED-SIGNAL ";
			for (size_t i = 0; i < sizeof(h0) - 1; ++i) *b++ = h0[i];
			b = appendDec(b, (unsigned int)sig);
			const char h1[] = "] pid=";
			for (size_t i = 0; i < sizeof(h1) - 1; ++i) *b++ = h1[i];
			b = appendDec(b, (unsigned int)pid);
			const char h2[] = " tid=";
			for (size_t i = 0; i < sizeof(h2) - 1; ++i) *b++ = h2[i];
			b = appendDec(b, (unsigned int)tid);
			const char h3[] = " rip=";
			for (size_t i = 0; i < sizeof(h3) - 1; ++i) *b++ = h3[i];
			b = appendHex(b, rip);
			const char h4[] = " fault_addr=";
			for (size_t i = 0; i < sizeof(h4) - 1; ++i) *b++ = h4[i];
			b = appendHex(b, fault_addr);
			const char h5[] = " si_code=";
			for (size_t i = 0; i < sizeof(h5) - 1; ++i) *b++ = h5[i];
			b = appendDec(b, (unsigned int)sicode);
			*b++ = '\n';
			rawWrite(fd, buf, b - buf);

			// SIGBUS BUS_ADRERR diagnostic: walk /proc/self/maps for the mapping containing si_addr, stat the file via raw syscall — distinguishes truncation/replacement from other faults.
			if (sig == SIGBUS && fault_addr) {
				long mfd = RawSyscallDirect(SYS_openat, (long)AT_FDCWD,
					(long)"/proc/self/maps", O_RDONLY, 0).r1;
				if (mfd >= 0) {
					char mbuf[16384];
					long total = 0;
					while (total < (long)sizeof(mbuf) - 1) {
						long r = RawSyscallDirect(SYS_read, mfd,
							(long)(mbuf + total),
							(long)(sizeof(mbuf) - 1 - total)).r1;
						if (r <= 0) break;
						total += r;
					}
					mbuf[total] = 0;
					RawSyscallDirect(SYS_close, mfd, 0, 0);

					auto hexNibble = [](char c) -> int {
						if (c >= '0' && c <= '9') return c - '0';
						if (c >= 'a' && c <= 'f') return c - 'a' + 10;
						if (c >= 'A' && c <= 'F') return c - 'A' + 10;
						return -1;
					};
					char* lp = mbuf;
					while (lp < mbuf + total) {
						char* le = lp;
						while (le < mbuf + total && *le != '\n') le++;
						if (le < mbuf + total) *le = 0;

						unsigned long long s = 0, e = 0;
						char* q = lp;
						int dg;
						while (*q && (dg = hexNibble(*q)) >= 0) { s = (s << 4) | dg; q++; }
						if (*q == '-') { q++;
							while (*q && (dg = hexNibble(*q)) >= 0) { e = (e << 4) | dg; q++; } }

						if (fault_addr >= s && fault_addr < e) {
							char* pe = le;
							char* ps = pe - 1;
							while (ps > lp && *ps != '\n') ps--;
							ps = le; // start at end-of-line, walk back
							while (ps > lp && *ps != '/') ps--;
							const char* mappedPath = (*ps == '/') ? ps : nullptr;
							if (mappedPath) {
								char* pstart = (char*)mappedPath;
								while (pstart > lp && *(pstart - 1) != ' ' && *(pstart - 1) != '\t') pstart--;
								mappedPath = pstart;
							}

							char db[1536]; char* d = db;
							const char hdr[] = "[MMAP_FAULT] ";
							for (size_t i = 0; i < sizeof(hdr) - 1; ++i) *d++ = hdr[i];
							const char km[] = "mapping=";
							for (size_t i = 0; i < sizeof(km) - 1; ++i) *d++ = km[i];
							d = appendHex(d, s); *d++ = '-'; d = appendHex(d, e);
							const char ksz[] = " mapping_size=";
							for (size_t i = 0; i < sizeof(ksz) - 1; ++i) *d++ = ksz[i];
							d = appendDec(d, (unsigned long long)(e - s));
							const char kfo[] = " fault_offset_in_mapping=";
							for (size_t i = 0; i < sizeof(kfo) - 1; ++i) *d++ = kfo[i];
							d = appendDec(d, (unsigned long long)(fault_addr - s));
							if (mappedPath) {
								const char kp[] = " path=";
								for (size_t i = 0; i < sizeof(kp) - 1; ++i) *d++ = kp[i];
								const char* pp = mappedPath;
								while (*pp && d < db + sizeof(db) - 64) *d++ = *pp++;

								struct stat st{};
								long sres = RawSyscallDirect(SYS_newfstatat,
									(long)AT_FDCWD, (long)mappedPath, (long)&st, 0).r1;
								if (sres == 0) {
									const char ks[] = " stat_size=";
									for (size_t i = 0; i < sizeof(ks) - 1; ++i) *d++ = ks[i];
									d = appendDec(d, (unsigned long long)st.st_size);
									const char ki[] = " stat_inode=";
									for (size_t i = 0; i < sizeof(ki) - 1; ++i) *d++ = ki[i];
									d = appendDec(d, (unsigned long long)st.st_ino);
									const char kt[] = " stat_mtime_sec=";
									for (size_t i = 0; i < sizeof(kt) - 1; ++i) *d++ = kt[i];
									d = appendDec(d, (unsigned long long)st.st_mtim.tv_sec);

									unsigned long long fileOff = (unsigned long long)(fault_addr - s); // fault byte's offset within the file

									if ((long long)st.st_size < (long long)fileOff + 1) {
										const char kv[] = " VERDICT=truncated_under_mmap";
										for (size_t i = 0; i < sizeof(kv) - 1; ++i) *d++ = kv[i];
									} else {
										const char kv[] = " VERDICT=file_size_ok_so_io_or_oom";
										for (size_t i = 0; i < sizeof(kv) - 1; ++i) *d++ = kv[i];
									}
								} else {
									const char ke[] = " stat_FAILED errno=";
									for (size_t i = 0; i < sizeof(ke) - 1; ++i) *d++ = ke[i];
									d = appendDec(d, (unsigned long long)(-sres));
								}
							} else {
								const char ka[] = " path=<anonymous-mapping>";
								for (size_t i = 0; i < sizeof(ka) - 1; ++i) *d++ = ka[i];
							}
							*d++ = '\n';
							rawWrite(fd, db, d - db);
							break;
						}
						lp = le + 1;
					}
				}
			}

			void* frames[40]; // backtrace's libgcc_s dlopen pre-warmed in PreInit so this signal-context call doesn't trigger it
			int n = backtrace(frames, 40);
			for (int i = 0; i < n; ++i) {
				char fb[64]; char* fp = fb;
				*fp++ = ' '; *fp++ = ' '; *fp++ = '#';
				fp = appendDec(fp, (unsigned int)i);
				*fp++ = ' ';
				fp = appendHex(fp, (unsigned long long)frames[i]);
				*fp++ = '\n';
				rawWrite(fd, fb, fp - fb);
			}

			// Capped dump of /proc/self/maps so frame addresses survive ASLR.
			const char mapsHdr[] = "[maps]\n";
			rawWrite(fd, mapsHdr, sizeof(mapsHdr) - 1);
			long mapsFd = RawSyscallDirect(SYS_openat, (long)AT_FDCWD,
				(long)"/proc/self/maps", O_RDONLY, 0).r1;
			if (mapsFd >= 0) {
				char mbuf[1024];
				int written = 0;
				while (written < 65536) {
					long r = RawSyscallDirect(SYS_read, mapsFd, (long)mbuf, sizeof(mbuf)).r1;
					if (r <= 0) break;
					rawWrite(fd, mbuf, r);
					written += (int)r;
				}
				RawSyscallDirect(SYS_close, mapsFd, 0, 0);
			}
			RawSyscallDirect(SYS_close, fd, 0, 0);
		}
		#endif // UBA_CHAINED_SIGNAL_LOG_FILE

		if (sig == SIGUSR1) // diagnostic stack-dump (kill -USR1 <pid>) — don't terminate
			return;

		// Rpc_ResolveCallstack only if libUbaDetours.so is in top frames; JVM/V8/Go SEGVs are workload traps. Init failure → log everything.
		bool ubaInvolved = (g_ubaLibStart == 0 || g_ubaLibEnd == 0);
		if (!ubaInvolved) {
			if (rip >= g_ubaLibStart && rip < g_ubaLibEnd) {
				ubaInvolved = true;
			} else {
				void* frames[6];
				int nf = backtrace(frames, 6);
				for (int i = 2; i < nf && i < 6 && !ubaInvolved; ++i) { // [0]=handler, [1]=libc trampoline

					auto a = (unsigned long long)frames[i];
					if (a >= g_ubaLibStart && a < g_ubaLibEnd) ubaInvolved = true;
				}
			}
		}
		// Log unconditionally if the signal would terminate the process (no chained handler, or SIG_DFL/SIG_IGN).
		bool willTerminate = true;
		if (sig >= 0 && sig < NSIG && g_chainedHandlers[sig].valid) {
			const struct sigaction& u = g_chainedHandlers[sig].user;
			if (u.sa_flags & SA_SIGINFO) {
				if (u.sa_sigaction) willTerminate = false;
			} else if (u.sa_handler && u.sa_handler != SIG_DFL && u.sa_handler != SIG_IGN) {
				willTerminate = false;
			}
		}
		const bool shouldLog = ubaInvolved || willTerminate;
		if (g_isInitialized && getpid() == g_pid && shouldLog)
		{
			StringBuffer<8 * 1024> sb;
			sb.Append(TCV("[CHAINED-SIGNAL ")).AppendValue(u64(sig)).Append(TCV("] pid="))
				.AppendValue(u64(pid)).Append(TCV(" tid=")).AppendValue(u64(tid))
				.Append(TCV(" rip=0x")).AppendHex(rip).Append(TCV(" fault_addr=0x")).AppendHex(fault_addr)
				.Append(TCV(" si_code=")).AppendValue(u64(u32(sicode)));
			Rpc_ResolveCallstack(sb, 0, uc);
			Rpc_WriteLog(sb.data, sb.count, true, true);
		}

		if (sig >= 0 && sig < NSIG && g_chainedHandlers[sig].valid)
		{
			const struct sigaction& u = g_chainedHandlers[sig].user;
			if (u.sa_flags & SA_SIGINFO) {
				if (u.sa_sigaction) u.sa_sigaction(sig, info, uc);
			} else {
				if (u.sa_handler == SIG_DFL) {
					struct sigaction dfl{}; // restore default + re-raise for kernel core dump
					dfl.sa_handler = SIG_DFL;
					sigemptyset(&dfl.sa_mask);
					TRUE_WRAPPER(sigaction)(sig, &dfl, nullptr);
					raise(sig);
				} else if (u.sa_handler != SIG_IGN) {
					if (u.sa_handler) u.sa_handler(sig);
				}
			}
		}
		else
		{
			// No chained handler — restore default and re-raise so kernel dumps core.
			struct sigaction dfl{};
			dfl.sa_handler = SIG_DFL;
			sigemptyset(&dfl.sa_mask);
			TRUE_WRAPPER(sigaction)(sig, &dfl, nullptr);
			raise(sig);
		}
	}

	// Scan /proc/self/maps, OR all r-xp+r--p segments of libUbaDetours.so into one [start,end) range.
	static void InitUbaLibRange()
	{
		long fd = Real_open(errno, "/proc/self/maps", O_RDONLY, 0);
		if (fd < 0) return;
		char buf[32 * 1024]; long total = 0;
		while (total < (long)sizeof(buf) - 1) {
			ssize_t r = Real_read(errno,(int)fd, buf + total, sizeof(buf) - 1 - (size_t)total);
			if (r <= 0) break;
			total += r;
		}
		buf[total] = 0;
		Real_close(errno,(int)fd);

		auto hexNibble = [](char c) -> int {
			if (c >= '0' && c <= '9') return c - '0';
			if (c >= 'a' && c <= 'f') return c - 'a' + 10;
			if (c >= 'A' && c <= 'F') return c - 'A' + 10;
			return -1;
		};
		char* lp = buf;
		while (lp < buf + total) {
			char* le = lp;
			while (le < buf + total && *le != '\n') le++;
			if (le < buf + total) *le = 0;
			// Path is the last whitespace-separated field; check for libUbaDetours.so suffix.
			char* path = le;
			while (path > lp && path[-1] != ' ' && path[-1] != '\t') path--;
			const char marker[] = "libUbaDetours.so";
			constexpr size_t mlen = sizeof(marker) - 1;
			size_t pathLen = (size_t)(le - path);
			bool isUba = pathLen >= mlen && (
				TStrcmp(path + pathLen - mlen, marker) == 0 ||
				(pathLen > mlen && memmem(path, pathLen, marker, mlen))); // allow .so.N suffix
			if (isUba) {
				unsigned long long s = 0, e = 0;
				char* q = lp; int dg;
				while (*q && (dg = hexNibble(*q)) >= 0) { s = (s << 4) | dg; q++; }
				if (*q == '-') { q++;
					while (*q && (dg = hexNibble(*q)) >= 0) { e = (e << 4) | dg; q++; } }
				if (s && e > s) {
					if (g_ubaLibStart == 0 || s < g_ubaLibStart) g_ubaLibStart = s;
					if (e > g_ubaLibEnd) g_ubaLibEnd = e;
				}
			}
			lp = le + 1;
		}
	}

	// Called from PreInit AFTER TRUE_WRAPPER pointers populated; ensures processes that never call sigaction (clang.real) still get a crash log.
	void InstallCrashHandlers()
	{
		void* dummy[1]; // force libgcc_s.so.1 dlopen now so backtrace() in the handler doesn't run dlopen from signal context
		(void)backtrace(dummy, 1);

		InitUbaLibRange();

		struct sigaction sa{};
		sa.sa_flags = SA_SIGINFO;
		sa.sa_sigaction = ChainedSigaction;
		sigemptyset(&sa.sa_mask);
		TRUE_WRAPPER(sigaction)(SIGBUS, &sa, nullptr); // bypass UBA_WRAPPER(sigaction) — would record our handler in g_chainedHandlers and chain to itself
		TRUE_WRAPPER(sigaction)(SIGSEGV, &sa, nullptr);
		TRUE_WRAPPER(sigaction)(SIGILL, &sa, nullptr);
		TRUE_WRAPPER(sigaction)(SIGFPE, &sa, nullptr);
		TRUE_WRAPPER(sigaction)(SIGUSR1, &sa, nullptr); // diagnostic stack-dump (see SIGUSR1 branch above)
	}
}
#else
namespace { void InstallCrashHandlers() {} }
#endif

UBA_EXPORT int UBA_WRAPPER(sigaction)(int sig, const struct sigaction* act, struct sigaction* oldact)
{
	UBA_INIT_DETOUR(sigaction, sig, act, oldact);
#if PLATFORM_LINUX
	if (sig >= 0 && sig < NSIG && g_isCrashSignal(sig) && act)
	{
		g_chainedHandlers[sig].user = *act;
		g_chainedHandlers[sig].valid = true;
		struct sigaction wrapped = *act;
		wrapped.sa_flags |= SA_SIGINFO;
		wrapped.sa_sigaction = ChainedSigaction;
		int res = TRUE_WRAPPER(sigaction)(sig, &wrapped, oldact);
		if (oldact && g_chainedHandlers[sig].valid)
			*oldact = g_chainedHandlers[sig].user; // hide our wrapper from oldact
		return res;
	}
#endif
	return TRUE_WRAPPER(sigaction)(sig, act, oldact);
}

// tmpfs EAGAIN exponential backoff (~500ms cap); Java NIO turns EAGAIN into a fatal exception.
static inline int OpenWithEagainRetry(const char* realFile, int flags, int mode)
{
	int kr = Real_open(errno,realFile, flags, mode);
	if (!(kr < 0 && errno == EAGAIN))
		return kr;
	struct timespec ts;
	ts.tv_sec = 0;
	long usec = 100;
	for (int retries = 0; retries < 13; ++retries)
	{
		ts.tv_nsec = usec * 1000;
		nanosleep(&ts, nullptr);
		kr = Real_open(errno,realFile, flags, mode);
		if (!(kr < 0 && errno == EAGAIN))
			return kr;
		usec *= 2;
		if (usec > 50000) usec = 50000;
	}
	return kr;
}

UBA_EXPORT int UBA_WRAPPER(open)(const char* file, int flags, ...)
{
	va_list args;
	va_start(args, flags);
	int mode = va_arg(args, int);
	va_end(args);
	UBA_INIT_DETOUR(open, file, flags, mode);
	return Shared_open("open", file, flags, mode, OpenWithEagainRetry);
}

UBA_EXPORT int UBA_WRAPPER(openat)(int dirfd, const char* pathname, int flags, ...)
{
	va_list args;
	va_start(args, flags);
	int mode = va_arg(args, int);
	va_end(args);
	UBA_INIT_DETOUR(openat, dirfd, pathname, flags, mode);

	StringBuffer<> temp;
	if (const char* resolvedPath = ResolvePath(dirfd, pathname, temp))
		return Shared_open("openat", resolvedPath, flags, mode, OpenWithEagainRetry);

	UBA_ASSERTF(!g_runningRemote, TC("openat: dirfd=%d not in g_fileHandles, path='%s' cwd='%s' flags=0x%x"), dirfd, pathname, g_virtualWorkingDir.data, flags);

	int res = Real_openat(errno,dirfd, pathname, flags, mode);
	DEBUG_LOG_TRUE("openat", "(NODETOUR) %s -> %i (%s)", pathname, res, StrError(res, errno));
	if (res)
		TrackInput(ToView(pathname));
	return res;
}

UBA_EXPORT int UBA_WRAPPER(creat)(const char *path, mode_t mode)
{
	UBA_INIT_DETOUR(creat, path, mode);
	return Shared_open("creat", path, O_WRONLY | O_CREAT | O_TRUNC, mode, [](const char* p, int f, int m) { return Real_open(errno,p, f, m); });
}

#if PLATFORM_LINUX

UBA_EXPORT int UBA_WRAPPER(__open_2)(const char* path, int flags)
{
	UBA_INIT_DETOUR(__open_2, path, flags);
	return Shared_open("__open_2", path, flags, 0, [](const char* p, int f, int m) { return Real_open(errno,p, f, m); });
}

UBA_EXPORT int UBA_WRAPPER(__openat_2)(int dirfd, const char* pathname, int flags)
{
	UBA_INIT_DETOUR(__openat_2, dirfd, pathname, flags);
	StringBuffer<> temp;
	if (const char* resolvedPath = ResolvePath(dirfd, pathname, temp))
		return Shared_open("__openat_2", resolvedPath, flags, 0, [](const char* p, int f, int m) { return Real_open(errno,p, f, m); });
	UBA_ASSERTF(!g_runningRemote, TC("__openat_2: dirfd=%d not in g_fileHandles, path='%s' cwd='%s'"), dirfd, pathname, g_virtualWorkingDir.data);
	int res = Real_openat(errno,dirfd, pathname, flags, 0);
	DEBUG_LOG_TRUE("__openat_2", "(NODETOUR unresolved-dirfd %d) %s -> %i (%s)", dirfd, pathname, res, StrError(res, errno));
	if (res)
		TrackInput(ToView(pathname));
	return res;
}

UBA_EXPORT int UBA_WRAPPER(__open64_2)(const char* path, int flags)
{
	UBA_INIT_DETOUR(__open64_2, path, flags);
	return Shared_open("__open64_2", path, flags, 0, [](const char* p, int f, int m) { return Real_open(errno,p, f, m); });
}

UBA_EXPORT int UBA_WRAPPER(__openat64_2)(int dirfd, const char* pathname, int flags)
{
	UBA_INIT_DETOUR(__openat64_2, dirfd, pathname, flags);
	StringBuffer<> temp;
	if (const char* resolvedPath = ResolvePath(dirfd, pathname, temp))
		return Shared_open("__openat64_2", resolvedPath, flags, 0, [](const char* p, int f, int m) { return Real_open(errno,p, f, m); });
	UBA_ASSERTF(!g_runningRemote, TC("__openat64_2: dirfd=%d not in g_fileHandles, path='%s' cwd='%s'"), dirfd, pathname, g_virtualWorkingDir.data);
	int res = Real_openat(errno,dirfd, pathname, flags, 0);
	DEBUG_LOG_TRUE("__openat64_2", "(NODETOUR unresolved-dirfd %d) %s -> %i (%s)", dirfd, pathname, res, StrError(res, errno));
	if (res)
		TrackInput(ToView(pathname));
	return res;
}

#endif // PLATFORM_LINUX

#if UBA_DETOUR_DEBUG && PLATFORM_LINUX
UBA_EXPORT ssize_t UBA_WRAPPER(write)(int fd, const void* buf, size_t count)
{
	UBA_INIT_DETOUR(write, fd, buf, count);
	DEBUG_LOG_TRUE("write", "(%i size: %llu)", fd, count);
	TimerScope ts(g_kernelStats.writeFile);
	return Real_write(errno,fd, buf, count);
}

UBA_EXPORT size_t UBA_WRAPPER(fwrite)(const void* ptr, size_t size, size_t nitems, FILE* stream)
{
	UBA_INIT_DETOUR(fwrite, ptr, size, nitems, stream);
	DEBUG_LOG_TRUE("fwrite", "(size: %llu)", size*nitems);
	TimerScope ts(g_kernelStats.writeFile);
	return TRUE_WRAPPER(fwrite)(ptr, size, nitems, stream);
}
#endif

UBA_EXPORT int UBA_WRAPPER(dup)(int oldfd)
{
	UBA_INIT_DETOUR(dup, oldfd);
	auto res = Real_dup(errno, oldfd);
	if (res != -1)
		DuplicateFd(oldfd, res, /*newCloexec*/false);
	DEBUG_LOG_TRUE("dup", "(%i) -> %i", oldfd, res);
	return res;
}

UBA_EXPORT int UBA_WRAPPER(dup2)(int oldfd, int newfd)
{
	UBA_INIT_DETOUR(dup2, oldfd, newfd);
	// Finalize the about-to-be-overwritten write fd with its current fstat BEFORE the kernel implicit-close — otherwise Deinit fstats the new fd (stdout/pipe) and corrupts the writtenFile entry. Bash redirect-with-restore (dup2(file,1)→dup2(saved,1)) hits this.
	if (oldfd != newfd)
		Shared_PreFinalizeForOverwrite("dup2", newfd, LdpreloadSnapshot);
	auto res = Real_dup2(errno, oldfd, newfd);
	if (res != -1)
		DuplicateFd(oldfd, newfd, /*newCloexec*/false);
	DEBUG_LOG_TRUE("dup2", "(%i, %i) -> %i", oldfd, newfd, res);
	return res;
}

UBA_EXPORT int UBA_WRAPPER(close)(int fd)
{
	UBA_INIT_DETOUR(close, fd);
	return Shared_close("close", fd, [&]() { return Real_close(errno,fd); }, LdpreloadSnapshot);
}

UBA_EXPORT uid_t UBA_WRAPPER(getuid)(void)  { return g_virtualUid; }
UBA_EXPORT uid_t UBA_WRAPPER(geteuid)(void) { return g_virtualUid; }
UBA_EXPORT gid_t UBA_WRAPPER(getgid)(void)  { return g_virtualGid; }
UBA_EXPORT gid_t UBA_WRAPPER(getegid)(void) { return g_virtualGid; }

#if PLATFORM_LINUX
UBA_EXPORT FILE* UBA_WRAPPER(fopen64)(const char* path, const char* mode)
{
	UBA_INIT_DETOUR(fopen64, path, mode);
	return Shared_fopen("fopen64", path, mode, "open64", [](const char* realFile, int flags, int openMode) { return Real_open(errno,realFile, flags, openMode); });
}
UBA_EXPORT FILE* UBA_WRAPPER(freopen64)(const char* filename, const char* mode, FILE* stream)
{
	UBA_INIT_DETOUR(freopen64, filename, mode, stream);

	if (!filename) // glibc: freopen64(NULL, mode, stream) only mutates the stream's mode

	{
		DEBUG_LOG_TRUE("freopen64", "(null, %s, %p)", mode, stream);
		return TRUE_WRAPPER(freopen64)(filename, mode, stream);
	}

	return Shared_freopen("freopen64", filename, mode, stream,
		[](const char* p, const char* m, FILE* s) { return TRUE_WRAPPER(freopen64)(p, m, s); },
		[](const char* realFile, int flags, int openMode) { return Real_open(errno,realFile, flags, openMode); });
}
UBA_EXPORT int UBA_WRAPPER(fstat64)(int fd, struct stat64* buf)
{
	UBA_INIT_DETOUR(fstat64, fd, buf);
	return Shared_fstat("fstat64", fd, (struct stat*)buf, [](int fd, struct stat* buf) { return Real_fstat(errno, fd, buf); });
}
UBA_EXPORT int UBA_WRAPPER(__fxstat64)(int ver, int fd, struct stat64* attr)
{
	UBA_INIT_DETOUR(__fxstat64, ver, fd, attr);
	UBA_ASSERT(ver == _STAT_VER);
	return Shared_fstat("__fxstat64", fd, (struct stat*)attr, [](int fd, struct stat* buf) { return Real_fstat(errno, fd, buf); });
}
#endif

UBA_EXPORT FILE* UBA_WRAPPER(fopen)(const char* path, const char* mode)
{
	UBA_INIT_DETOUR(fopen, path, mode);
	return Shared_fopen("fopen", path, mode, "open", [](const char* realFile, int flags, int openMode) { return Real_open(errno,realFile, flags, openMode); });
}

UBA_EXPORT FILE* UBA_WRAPPER(fdopen)(int fd, const char* mode)
{
	UBA_INIT_DETOUR(fdopen, fd, mode);
	DEBUG_LOG_TRUE("fdopen", "(%i)", fd);
	TimerScope ts(g_kernelStats.createFile);
	return TRUE_WRAPPER(fdopen)(fd, mode);
}

UBA_EXPORT FILE* UBA_WRAPPER(freopen)(const char* filename, const char* mode, FILE* stream)
{
	UBA_INIT_DETOUR(freopen, filename, mode, stream);

	if (!filename) // glibc: freopen(NULL, mode, stream) only mutates the stream's mode

	{
		DEBUG_LOG_TRUE("freopen", "(null, %s, %p)", mode, stream);
		return TRUE_WRAPPER(freopen)(filename, mode, stream);
	}

	return Shared_freopen("freopen", filename, mode, stream,
		[](const char* p, const char* m, FILE* s) { return TRUE_WRAPPER(freopen)(p, m, s); },
		[](const char* realFile, int flags, int openMode) { return Real_open(errno,realFile, flags, openMode); });
}

UBA_EXPORT int UBA_WRAPPER(fchmod)(int fd, mode_t mode)
{
	UBA_INIT_DETOUR(fchmod, fd, mode);
	TimerScope ts(g_kernelStats.setFileInfo);
	auto res = Real_fchmod(errno,fd, mode);
	ts.Leave();
	DEBUG_LOG_TRUE("fchmod", "(%i) %i -> %i (%s)", fd, mode, res, StrError(res, errno));
	return res;
}

UBA_EXPORT int UBA_WRAPPER(fchmodat)(int dirfd, const char* pathname, mode_t mode, int flags)
{
	UBA_INIT_DETOUR(fchmodat, dirfd, pathname, mode, flags);

	StringBuffer<> resolvedTemp; // resolve non-AT_FDCWD dirfds via tracked fd entry so they stay inside the detour
	const char* resolved = ResolvePath(dirfd, pathname, resolvedTemp);
	if (!resolved)
	{
		int res = Real_fchmodat(errno,dirfd, pathname, mode, flags);
		DEBUG_LOG_TRUE("fchmodat", "%i %s %i %i (unresolvable) -> %i (%s)", dirfd, pathname, mode, flags, res, StrError(res, errno));
		return res;
	}

	int err = 0;
	int res = Shared_chmod("fchmodat", resolved, mode, err, [flags](const char* p, mode_t m) {
		return Real_fchmodat(errno,AT_FDCWD, p, m, flags); // p is absolute, kernel cwd irrelevant
	});
	if (res != 0) errno = err;
	return res;
}

UBA_EXPORT int UBA_WRAPPER(fstat)(int fd, struct stat* buf)
{
	UBA_INIT_DETOUR(fstat, fd, buf);
	return Shared_fstat("fstat", fd, buf, [](int fd, struct stat* buf) { return Real_fstat(errno,fd, buf); });
}

UBA_EXPORT int UBA_WRAPPER(fstatat)(int dirfd, const char* pathname, struct stat* statbuf, int flags)
{
	UBA_INIT_DETOUR(fstatat, dirfd, pathname, statbuf, flags);

	StringBuffer<> temp;
	if (const char* resolvedPath = ResolvePath(dirfd, pathname, temp))
	{
		bool followSymlink = (flags & AT_SYMLINK_NOFOLLOW) == 0; // lstat semantics
		return Shared_stat("fstatat", resolvedPath, statbuf, [](const char* file, struct stat* attr) { return Real_stat(errno,file, attr); }, followSymlink);
	}
	DEBUG_LOG_TRUE("fstatat", "%s", pathname);
	TimerScope ts(g_kernelStats.getFileInfo);
	return Real_fstatat(errno,dirfd, pathname, statbuf, flags);
}

UBA_EXPORT int UBA_WRAPPER(futimens)(int fd, const struct timespec* times)
{
	UBA_INIT_DETOUR(futimens, fd, times);
	int res = TRUE_WRAPPER(futimens)(fd, times);
	DEBUG_LOG_TRUE("futimens", "(%i) -> %i (%s)", fd, res, StrError(0, errno));
	return res;
}

UBA_EXPORT int UBA_WRAPPER(fclose)(FILE* stream)
{
	UBA_INIT_DETOUR(fclose, stream);

	if (!stream)
	{
		int res = TRUE_WRAPPER(fclose)(stream);
		DEBUG_LOG_TRUE("fclose", "(null) -> %i (%s)", res, StrError(res, errno));
		return res;
	}

	fflush(stream); // flush stdio buffer before Shared_close's dup/fstat — otherwise size is page-aligned and soong_zip embeds truncated content
	
	return Shared_close("fclose", fileno(stream), [&]() { return TRUE_WRAPPER(fclose)(stream); }, LdpreloadSnapshot);
}

// Random-suffix retry loop via UBA_WRAPPER(open) so the dirtable sees the file before downstream rename (sed -i, install -p).
static int UbaMkstempCore(const char* funcName, char* templ, int suffixlen, int extraFlags)
{
	static const char alphabet[] = "abcdefghijklmnopqrstuvwxyzABCDEFGHIJKLMNOPQRSTUVWXYZ0123456789";
	const uint64_t base = (sizeof(alphabet) - 1);

	DEBUG_LOG_TRUE(funcName, "%s suffixlen=%i flags=0x%x", templ, suffixlen, extraFlags);

	auto len = strlen(templ);
	if (len < (size_t)(6 + suffixlen))
	{
		errno = EINVAL;
		return -1;
	}
	char* posToWrite = templ + len - 6 - suffixlen;
	while (true)
	{
		Guid g;
		CreateGuid(g);
		u64 v = *(u64*)&g;
		for (size_t i = 0; i < 6; ++i)
		{
			posToWrite[i] = alphabet[v % base];
			v /= base;
		}

		int flags = extraFlags | O_RDWR | O_CREAT | O_EXCL;
		int fd = UBA_WRAPPER(open)(templ, flags, 0600);
		if (fd >= 0)
			return fd;
		if (errno != EEXIST)
			return -1;
	}
}

UBA_EXPORT int UBA_WRAPPER(mkstemp)(char* templ)
{
	UBA_INIT_DETOUR(mkstemp, templ);
	return UbaMkstempCore("mkstemp", templ, 0, 0);
}

UBA_EXPORT int UBA_WRAPPER(mkostemp)(char* templ, int flags)
{
	UBA_INIT_DETOUR(mkostemp, templ, flags);
	return UbaMkstempCore("mkostemp", templ, 0, flags);
}

#if PLATFORM_LINUX
UBA_EXPORT int UBA_WRAPPER(mkstemp64)(char* templ)
{
	UBA_INIT_DETOUR(mkstemp64, templ);
	return UbaMkstempCore("mkstemp64", templ, 0, 0);
}

UBA_EXPORT int UBA_WRAPPER(mkostemp64)(char* templ, int flags)
{
	UBA_INIT_DETOUR(mkostemp64, templ, flags);
	return UbaMkstempCore("mkostemp64", templ, 0, flags);
}

#endif // PLATFORM_LINUX

UBA_EXPORT int UBA_WRAPPER(mkstemps)(char* templ, int suffixlen)
{
	UBA_INIT_DETOUR(mkstemps, templ, suffixlen);
	return UbaMkstempCore("mkstemps", templ, suffixlen, 0);
}

UBA_EXPORT int UBA_WRAPPER(mkostemps)(char* templ, int suffixlen, int flags)
{
	UBA_INIT_DETOUR(mkostemps, templ, suffixlen, flags);
	return UbaMkstempCore("mkostemps", templ, suffixlen, flags);
}

UBA_EXPORT DIR* UBA_WRAPPER(opendir)(const char* name)
{
	UBA_INIT_DETOUR(opendir, name);

	name = ResolveProcSelfFd(name); // /proc/self/fd/<N> → originalName; otherwise proc symlink leaks into dirtable as a key

	StringBuffer<> dirName;

	if (!FixPath(dirName, name) || !CanDetour2(dirName) || Equals(name, "/proc/self/fd"))
	{
		DIR* res = TRUE_WRAPPER(opendir)(dirName.data);
		DEBUG_LOG_TRUE("opendir", "(%s) -> %p", dirName.data, res);
		return res;
	}

	DevirtualizePath(dirName);

#if UBA_SUPPORT_SYMLINKS
	{
		// Follow prefix/leaf symlinks so lookup uses the real directory path.
		DirectoryTableEntryInformation einfo;
		StringBuffer<> resolved;
		StringKey resolvedKey;
		if (Rpc_GetEntryInformation(einfo, ToFilenameKey(dirName), dirName, true, true, &resolved, &resolvedKey))
		{
			if (resolved.count && !resolved.Equals(dirName))
				dirName.Clear().Append(resolved);
		}
	}
#endif

	StringBuffer<> forHash(dirName);
	if (forHash.count == 1)
		forHash.Resize(0);
	if (CaseInsensitiveFs)
		forHash.MakeLower();
	DirHash hash(forHash);

	auto& dirTable = *g_directoryTable;

	SCOPED_WRITE_LOCK(dirTable.m_lookupLock, lookLock);
	auto insres = dirTable.m_lookup.try_emplace(hash.key, g_memoryBlock);
	DirectoryTable::Directory& dir = insres.first->second;
	if (insres.second)
		if (dirTable.EntryExistsNoLock(hash.key, forHash) != DirectoryTable::Exists_No)
			Rpc_UpdateDirectory(hash.key, dirName.data, dirName.count, false);

	bool exists = false;
	DirTableOffset entryOffset;
	if (dirTable.GetLatestOffset(entryOffset, dir))
	{
		DirectoryTableEntryInformation entryInfo;
		dirTable.GetEntryInformation(entryInfo, entryOffset);
		exists = entryInfo.attributes != 0;
	}

	if (!exists)
	{
		errno = ENOENT;
		DEBUG_LOG_DETOURED("opendir", "(%s) -> %p", dirName.data, nullptr);
		return nullptr;
	}

	dirTable.PopulateDirectory(hash.open, dir);

	auto dirInfo = new DirInfo();

	SCOPED_READ_LOCK(dir.lock, lock);
	dirInfo->fileTableOffsets.resize(dir.files.size());
	u32 it = 0;
	for (auto& pair : dir.files)
		dirInfo->fileTableOffsets[it++] = pair.second;
	lock.Leave();

	// Back the virtual DIR* with a real fd so dirfd() / openat(fd,subname) stay tracked (depmod, external/kmod).
	int realFd = Real_open(errno,"/dev/null", O_RDONLY | O_CLOEXEC, 0);
	if (realFd >= 0)
	{
		auto* fi = new FileInfo();
		fi->originalName = g_memoryBlock.Strdup(dirName).data;
		fi->name = fi->originalName;
		auto* fo = new FileObject();
		fo->fileInfo = fi;
		fo->dirInfo = dirInfo;       // closes the loop: getdents64 finds dirInfo via fd
		fo->isDirectory = true;
		fo->ownsFileInfo = true;
		ScopedWriteLock fhLock(g_fileHandlesLock);
		auto& h = g_fileHandles[realFd];
		h.fileObject = fo;
		dirInfo->realFd = realFd;
	}

	DEBUG_LOG_DETOURED("opendir", "(%s) -> %p (realFd %i)", dirName.data, dirInfo, dirInfo->realFd);

	return (DIR*)(u64(dirInfo) | 0x1000'0000'0000'0000);
}

UBA_EXPORT int UBA_WRAPPER(dirfd)(DIR* dirp)
{
	UBA_INIT_DETOUR(dirfd, dirp);

	if (IsDirInfo(dirp))
	{
		auto& dirInfo = *AsDirInfo(dirp);
		if (dirInfo.realFd < 0)
		{
			DEBUG_LOG_DETOURED("dirfd", "(%p) -> -1 (EBADF: opendir failed to mint /dev/null fd)", dirp);
			errno = EBADF;
			return -1;
		}
		DEBUG_LOG_DETOURED("dirfd", "(%p) -> %i", dirp, dirInfo.realFd);
		return dirInfo.realFd;
	}

	int res = TRUE_WRAPPER(dirfd)(dirp);
	DEBUG_LOG_TRUE("dirfd", "(%p) -> %i", dirp, res);
	return res;
}

dirent* Shared_readdir(const char* func, DIR* dirp)
{
	auto& dirInfo = *AsDirInfo(dirp);
	while (true)
	{
		++dirInfo.it;
		if (dirInfo.it >= dirInfo.fileTableOffsets.size())
		{
			DEBUG_LOG_DETOURED(func, "(%p) -> nullptr (found %u entries)", dirp, dirInfo.fileTableOffsets.size());
			errno = 0; // JDK NIO turns stale errno into a fatal FileSystemException
			return nullptr;
		}
		DirTableOffset fileTableOffset = dirInfo.fileTableOffsets[dirInfo.it];

		DirectoryTableEntryInformation info;
		g_directoryTable->GetEntryInformation(info, fileTableOffset, dirInfo.ent.d_name, 256);
		if (info.attributes == 0) // File was deleted
			continue;

		auto nameLen = u16(strlen(dirInfo.ent.d_name));
		bool isDir = S_ISDIR(info.attributes);

		dirInfo.ent.d_ino = info.fileIndex;
		dirInfo.ent.d_reclen = offsetof(struct dirent, d_name) + nameLen + 1;
		// d_type matters: callers (toybox rm, AOSP) trust it; misreporting S_IFLNK as DT_REG breaks `rm -rf` on dangling-symlink dirs.
		dirInfo.ent.d_type = isDir ? DT_DIR : (S_ISLNK(info.attributes) ? DT_LNK : DT_REG);

		#if PLATFORM_LINUX
		dirInfo.ent.d_off = 0;
		#else
		dirInfo.ent.d_namlen = nameLen;
		#endif

		return &dirInfo.ent;
	}
}

UBA_EXPORT dirent* UBA_WRAPPER(readdir)(DIR* dirp)
{
	UBA_INIT_DETOUR(readdir, dirp);
	if (IsDirInfo(dirp))
		return Shared_readdir("readdir", dirp);
	auto res = TRUE_WRAPPER(readdir)(dirp);
	DEBUG_LOG_TRUE("readdir", "(%p) -> %p", dirp, res);
	return res;
}

UBA_EXPORT int UBA_WRAPPER(closedir)(DIR* dirp)
{
	UBA_INIT_DETOUR(closedir, dirp);

	if (IsDirInfo(dirp))
	{
		auto* dirInfo = (DirInfo*)AsDirInfo(dirp);
		// Real-fd path: close() runs canonical FileObject cleanup; plain delete only when /dev/null open failed.
		if (dirInfo->realFd >= 0)
		{
			int fd = dirInfo->realFd;
			DEBUG_LOG_DETOURED("closedir", "(%p) -> close(%i)", dirp, fd);
			return UBA_WRAPPER(close)(fd);
		}
		delete dirInfo;
		DEBUG_LOG_DETOURED("closedir", "(%p)", dirp);
		return 0;
	}

	DEBUG_LOG_TRUE("closedir", "(%p)", dirp);
	return TRUE_WRAPPER(closedir)(dirp);
}

template<typename Dirent, typename ReadDir>
int Shared_scandir(const char* func, const char* dirp, Dirent*** namelist, int (*filter)(const Dirent*), int (*compar)(const Dirent**, const Dirent**), ReadDir readDir)
{
	DIR* dir = UBA_WRAPPER(opendir)(dirp);
	if (!dir)
	{
		DEBUG_LOG_DETOURED(func, "(%s) -> -1 (%s)", dirp, strerror(errno));
		return -1;
	}

	Vector<Dirent*> entries;
	Dirent* ent;
	while ((ent = readDir(dir)) != nullptr)
	{
		if (filter && !filter(ent))
			continue;
		auto copy = (Dirent*)malloc(ent->d_reclen);
		memcpy(copy, ent, ent->d_reclen);
		entries.push_back(copy);
	}
	UBA_WRAPPER(closedir)(dir);

	if (compar && entries.size() > 1)
		qsort(entries.data(), entries.size(), sizeof(Dirent*), (int(*)(const void*, const void*))compar);

	*namelist = (Dirent**)malloc(entries.size() * sizeof(Dirent*));
	for (size_t i = 0; i < entries.size(); i++)
		(*namelist)[i] = entries[i];

	DEBUG_LOG_DETOURED(func, "(%s) -> %zu", dirp, entries.size());
	return (int)entries.size();
}

#if PLATFORM_LINUX

UBA_EXPORT dirent64* UBA_WRAPPER(readdir64)(DIR* dirp)
{
	UBA_INIT_DETOUR(readdir64, dirp);
	if (IsDirInfo(dirp))
		return (dirent64*)Shared_readdir("readdir64", dirp);
	auto res = TRUE_WRAPPER(readdir64)(dirp);
	DEBUG_LOG_TRUE("readdir64", "(%p) -> %p", dirp, res);
	return res;
}

UBA_EXPORT int UBA_WRAPPER(scandir64)(const char* dirp, dirent64*** namelist, int (*filter)(const dirent64*), int (*compar)(const dirent64**, const dirent64**))
{
	UBA_INIT_DETOUR(scandir64, dirp, namelist, filter, compar);
	return Shared_scandir("scandir64", dirp, namelist, filter, compar, [](DIR* d) { return UBA_WRAPPER(readdir64)(d); });
}

// Shared_getdents64 lives in UbaDetoursSharedPosix.cpp (shared with stub).
#endif

UBA_EXPORT void UBA_WRAPPER(rewinddir)(DIR* dirp)
{
	UBA_INIT_DETOUR(rewinddir, dirp);
	if (IsDirInfo(dirp))
	{
		auto& dirInfo = *AsDirInfo(dirp);
		dirInfo.it = -1;
		DEBUG_LOG_DETOURED("rewinddir", "(%p)", dirp);
		return;
	}

	DEBUG_LOG_TRUE("rewinddir", "(%p)", dirp);
	TRUE_WRAPPER(rewinddir)(dirp);
}

UBA_EXPORT int UBA_WRAPPER(scandir)(const char* dirp, dirent*** namelist, int (*filter)(const dirent*), int (*compar)(const dirent**, const dirent**))
{
	UBA_INIT_DETOUR(scandir, dirp, namelist, filter, compar);
	return Shared_scandir("scandir", dirp, namelist, filter, compar, [](DIR* d) { return UBA_WRAPPER(readdir)(d); });
}
UBA_EXPORT void UBA_WRAPPER(seekdir)(DIR* dirp, long loc)
{
	UBA_INIT_DETOUR(seekdir, dirp, loc);
	UBA_ASSERTF(!IsDirInfo(dirp), "seekdir");
	DEBUG_LOG_TRUE("seekdir", "(%p)", dirp);
	return TRUE_WRAPPER(seekdir)(dirp, loc);
}

UBA_EXPORT long UBA_WRAPPER(telldir)(DIR* dirp)
{
	UBA_INIT_DETOUR(telldir, dirp);
	UBA_ASSERTF(!IsDirInfo(dirp), "telldir");
	DEBUG_LOG_TRUE("telldir", "(%p)", dirp);
	return TRUE_WRAPPER(telldir)(dirp);
}

UBA_EXPORT DIR* UBA_WRAPPER(fdopendir)(int fd)
{
	UBA_INIT_DETOUR(fdopendir, fd);

	SCOPED_READ_LOCK(g_fileHandlesLock, lock);
	auto findIt = g_fileHandles.find(fd);
	if (findIt == g_fileHandles.end())
	{
		DIR* res = TRUE_WRAPPER(fdopendir)(fd);
		DEBUG_LOG_TRUE("fdopendir", "(%i) -> %p (%s)", fd, res, StrError(res != nullptr, errno));
		return res;
	}
	DetouredHandle& h = findIt->second;
	lock.Leave();
	FileObject* fo = h.fileObject;
	FileInfo& info = *fo->fileInfo;
	if (!fo->isDirectory)
	{
		DEBUG_LOG_DETOURED("fdopendir", "(%i) -> nullptr (ENOTDIR)", fd);
		errno = ENOTDIR;
		return nullptr;
	}

	DEBUG_LOG_DETOURED("fdopendir", "(%i)", fd);
	return opendir(info.originalName);
}

UBA_EXPORT int UBA_WRAPPER(glob)(const char* pattern, int flags, int (*errfunc)(const char* epath, int eerrno), glob_t* pglob)
{
	UBA_INIT_DETOUR(glob, pattern, flags, errfunc, pglob);
	int res = TRUE_WRAPPER(glob)(pattern, flags, errfunc, pglob);
	DEBUG_LOG_TRUE("glob", "%s -> %i", pattern, res);
	return res;
}

#if UBA_DEBUG && PLATFORM_LINUX

UBA_EXPORT int UBA_WRAPPER(fstatat64)(int dirfd, const char* pathname, struct stat64* buf, int flags)
{
	UBA_INIT_DETOUR(fstatat64, dirfd, pathname, buf, flags);

	StringBuffer<> temp;
	if (const char* resolvedPath = ResolvePath(dirfd, pathname, temp))
	{
		bool followSymlink = (flags & AT_SYMLINK_NOFOLLOW) == 0;
		return Shared_stat("fstatat64", resolvedPath, (struct stat*)buf, [](const char* file, struct stat* attr) { return Real_stat(errno, file, attr); }, followSymlink);
	}
	UBA_ASSERTF(!g_runningRemote, TC("fstatat64: dirfd=%d not in g_fileHandles, path='%s' cwd='%s' flags=0x%x"), dirfd, pathname, g_virtualWorkingDir.data, flags);
	int res = Real_fstatat(errno, dirfd, pathname, (struct stat*)buf, flags);
	DEBUG_LOG_TRUE("fstatat64", "%i %s -> %i (%s)", dirfd, pathname, res, StrError(res, errno));
	return res;
}

UBA_EXPORT long UBA_WRAPPER(fpathconf)(int fd, int name)
{
	UBA_INIT_DETOUR(fpathconf, fd, name);
	DEBUG_LOG_TRUE("fpathconf", "");
	UBA_ASSERT(false);
	return TRUE_WRAPPER(fpathconf)(fd, name);
}
UBA_EXPORT long UBA_WRAPPER(pathconf)(char *path, int name)
{
	UBA_INIT_DETOUR(pathconf, path, name);
	DEBUG_LOG_TRUE("pathconf", "");
	UBA_ASSERT(false);
	return TRUE_WRAPPER(pathconf)(path, name);
}

#endif

UBA_EXPORT int UBA_WRAPPER(lstat)(const char *path, struct stat *buf)
{
	UBA_INIT_DETOUR(lstat, path, buf);
	return Shared_stat("lstat", path, buf, [](const char* path, struct stat* buf) { return Real_lstat(errno,path, buf); }, false);
}

UBA_EXPORT off_t  UBA_WRAPPER(lseek)(int fd, off_t offset, int whence)
{
	UBA_INIT_DETOUR(lseek, fd, offset, whence);
	int res = Real_lseek(errno, fd, offset, whence);
	DEBUG_LOG_TRUE("lseek", "%i %i %i -> %i", fd, offset, whence, res);
	return res;
}

UBA_EXPORT int UBA_WRAPPER(stat)(const char* file, struct stat* attr)
{
	UBA_INIT_DETOUR(stat, file, attr);
	return Shared_stat("stat", file, attr, [](const char* file, struct stat* attr) { return Real_stat(errno,file, attr); });
}

#if PLATFORM_LINUX

// Return 0 (no xattrs) instead of ENOENT — mkfs.erofs treats ENOENT as fatal.
UBA_EXPORT ssize_t UBA_WRAPPER(listxattr)(const char *path, char *_Nullable list, size_t size)
{
	UBA_INIT_DETOUR(listxattr, path, list, size);

	StringBuffer<> fixedPath;
	FixPath(fixedPath, path);
	DevirtualizePath(fixedPath);

	if (!CanDetour2(fixedPath))
	{
		auto res = TRUE_WRAPPER(listxattr)(path, list, size);
		DEBUG_LOG_TRUE("listxattr", "(NODETOUR) %s -> %ll (%s)", path, res, StrError(res, errno));
		return res;
	}

	DEBUG_LOG_TRUE("listxattr", "%s -> 0 (no xattrs)", path);
	return 0;
}

UBA_EXPORT ssize_t UBA_WRAPPER(llistxattr)(const char *path, char *_Nullable list, size_t size)
{
	UBA_INIT_DETOUR(llistxattr, path, list, size);

	StringBuffer<> fixedPath;
	FixPath(fixedPath, path);
	DevirtualizePath(fixedPath);

	if (!CanDetour2(fixedPath))
	{
		auto res = TRUE_WRAPPER(llistxattr)(path, list, size);
		DEBUG_LOG_TRUE("llistxattr", "(NODETOUR) %s -> %ll (%s)", path, res, StrError(res, errno));
		return res;
	}

	DEBUG_LOG_TRUE("llistxattr", "%s -> 0 (no xattrs)", path);
	return 0;
}

UBA_EXPORT ssize_t UBA_WRAPPER(flistxattr)(int fd, char *_Nullable list, size_t size)
{
	UBA_INIT_DETOUR(flistxattr, fd, list, size);

	bool tracked;
	{
		ScopedReadLock lock(g_fileHandlesLock);
		tracked = (g_fileHandles.find(fd) != g_fileHandles.end());
	}

	if (!tracked)
	{
		auto res = TRUE_WRAPPER(flistxattr)(fd, list, size);
		DEBUG_LOG_TRUE("flistxattr", "(NODETOUR) fd=%i -> %ll (%s)", fd, res, StrError(res, errno));
		return res;
	}

	DEBUG_LOG_TRUE("flistxattr", "fd=%i -> 0 (no xattrs)", fd);
	return 0;
}

UBA_EXPORT int UBA_WRAPPER(stat64)(const char* file, struct stat64* attr)
{
	UBA_INIT_DETOUR(stat64, file, attr);
	return Shared_stat("stat64", file, (struct stat*)attr, [](const char* file, struct stat* attr) { return Real_stat(errno, file, attr); });
}

UBA_EXPORT int UBA_WRAPPER(__xstat64)(int ver, const char* file, struct stat64* attr)
{
	UBA_INIT_DETOUR(__xstat64, ver, file, attr);
	UBA_ASSERT(ver == _STAT_VER);
	return Shared_stat("__xstat64", file, (struct stat*)attr, [](const char* file, struct stat* attr) { return Real_stat(errno, file, attr); });
}

UBA_EXPORT int UBA_WRAPPER(lstat64)(const char* file, struct stat64* attr)
{
	UBA_INIT_DETOUR(lstat64, file, attr);
	return Shared_stat("lstat64", file, (struct stat*)attr, [](const char* file, struct stat* attr) { return Real_lstat(errno, file, attr); }, false);
}
#endif

UBA_EXPORT int UBA_WRAPPER(truncate)(const char* path, off_t length)
{
	UBA_INIT_DETOUR(truncate, path, length);
	UBA_ASSERTF(!g_runningRemote, "truncate not implemented for remote execution (path: %s)", path); // TODO: Implement this if it is ever called
	UBA_ASSERTF(false, "truncate(%s, %lld) — dirtable not updated; concurrent mmap readers will SIGBUS. Route through UBA RPC or use rename instead.", path, (long long)length);
	return Real_truncate(errno,path, length);
}

UBA_EXPORT int UBA_WRAPPER(access)(const char* pathname, int mode)
{
	UBA_INIT_DETOUR(access, pathname, mode);
	return Shared_access("access", pathname, mode);
}

#if PLATFORM_LINUX
UBA_EXPORT int UBA_WRAPPER(eaccess)(const char* pathname, int mode)
{
	UBA_INIT_DETOUR(eaccess, pathname, mode);
	return Shared_access("eaccess", pathname, mode, /*strict=*/true);
}

// Bash and others call euidaccess directly; without this wrapper the call goes kernel-direct on remote helpers.
UBA_EXPORT int UBA_WRAPPER(euidaccess)(const char* pathname, int mode)
{
	if (!g_isDetouring || DisallowDetourDepth())
		return eaccess(pathname, mode); // older glibc treats euidaccess as a weak alias for eaccess
	return Shared_access("euidaccess", pathname, mode, /*strict=*/true);
}
#endif // PLATFORM_LINUX

UBA_EXPORT int UBA_WRAPPER(faccessat)(int dirfd, const char *pathname, int mode, int flags)
{
	UBA_INIT_DETOUR(faccessat, dirfd, pathname, mode, flags);
	StringBuffer<> temp;
	if (const char* resolvedPath = ResolvePath(dirfd, pathname, temp))
		return Shared_access("faccessat", resolvedPath, mode);
	UBA_ASSERTF(!g_runningRemote, TC("faccessat: dirfd=%d not in g_fileHandles, path='%s' cwd='%s' flags=0x%x"), dirfd, pathname, g_virtualWorkingDir.data, flags);
	return Real_faccessat(errno, dirfd, pathname, mode, flags);
}

// faccessat2 (Linux 5.8+) — not in DETOURED_FUNCTIONS (needs glibc 2.34+ decltype); checked manually. extern "C" so LD_PRELOAD sees the unmangled symbol.
extern "C" UBA_EXPORT int UBA_WRAPPER(faccessat2)(int dirfd, const char *pathname, int mode, int flags)
{
	if (!g_isDetouring || DisallowDetourDepth())
	{
#ifdef SYS_faccessat2
		return (int)syscall(SYS_faccessat2, dirfd, pathname, mode, flags);
#else
		return faccessat(dirfd, pathname, mode, flags & ~AT_EACCESS); // under detour uid==euid, plain faccessat suffices
#endif
	}
	StringBuffer<> temp;
	if (const char* resolvedPath = ResolvePath(dirfd, pathname, temp))
		return Shared_access("faccessat2", resolvedPath, mode);
	UBA_ASSERTF(!g_runningRemote, TC("faccessat2: dirfd=%d not in g_fileHandles, path='%s' cwd='%s' flags=0x%x"), dirfd, pathname, g_virtualWorkingDir.data, flags);
#ifdef SYS_faccessat2
	return (int)syscall(SYS_faccessat2, dirfd, pathname, mode, flags);
#else
	return faccessat(dirfd, pathname, mode, flags & ~AT_EACCESS);
#endif
}

#if PLATFORM_LINUX
UBA_EXPORT int UBA_WRAPPER(__fxstatat)(int ver, int dirfd, const char* pathname, struct stat* buf, int flags)
{
	UBA_INIT_DETOUR(__fxstatat, ver, dirfd, pathname, buf, flags);
	StringBuffer<> temp;
	if (const char* resolvedPath = ResolvePath(dirfd, pathname, temp))
	{
		bool followSymlink = (flags & AT_SYMLINK_NOFOLLOW) == 0;
		return Shared_stat("__fxstatat", resolvedPath, buf, [](const char* file, struct stat* attr) { return Real_stat(errno, file, attr); }, followSymlink);
	}
	DEBUG_LOG_TRUE("__fxstatat", "%i %s", dirfd, pathname);
	return Real_fstatat(errno, dirfd, pathname, buf, flags);
}

UBA_EXPORT int UBA_WRAPPER(__fxstatat64)(int ver, int dirfd, const char* pathname, struct stat64* buf, int flags)
{
	static_assert(sizeof(struct stat64) == sizeof(struct stat));
	UBA_INIT_DETOUR(__fxstatat64, ver, dirfd, pathname, buf, flags);
	StringBuffer<> temp;
	if (const char* resolvedPath = ResolvePath(dirfd, pathname, temp))
	{
		bool followSymlink = (flags & AT_SYMLINK_NOFOLLOW) == 0;
		return Shared_stat("__fxstatat64", resolvedPath, (struct stat*)buf, [](const char* file, struct stat* attr) { return Real_stat(errno, file, attr); }, followSymlink);
	}
	DEBUG_LOG_TRUE("__fxstatat64", "%i %s", dirfd, pathname);
	return Real_fstatat(errno, dirfd, pathname, (struct stat*)buf, flags);
}

UBA_EXPORT int UBA_WRAPPER(__xstat)(int ver, const char* file, struct stat* attr)
{
	UBA_INIT_DETOUR(__xstat, ver, file, attr);
	return Shared_stat("__xstat", file, attr, [](const char* file, struct stat* attr) { return Real_stat(errno, file, attr); });
}

UBA_EXPORT int UBA_WRAPPER(__lxstat)(int ver, const char* file, struct stat* attr)
{
	UBA_INIT_DETOUR(__lxstat, ver, file, attr);
	return Shared_stat("__lxstat", file, attr, [](const char* file, struct stat* attr) { return Real_lstat(errno, file, attr); }, false);
}

UBA_EXPORT int UBA_WRAPPER(__fxstat)(int ver, int fd, struct stat* attr)
{
	UBA_INIT_DETOUR(__fxstat, ver, fd, attr);
	return Shared_fstat("__fxstat", fd, attr, [](int fd, struct stat* attr) { return Real_fstat(errno, fd, attr); });
}

UBA_EXPORT int UBA_WRAPPER(__lxstat64)(int ver, const char* file, struct stat64* attr)
{
	static_assert(sizeof(struct stat64) == sizeof(struct stat));
	UBA_INIT_DETOUR(__lxstat64, ver, file, attr);
	return Shared_stat("__lxstat64", file, (struct stat*)attr, [](const char* file, struct stat* attr) { return Real_lstat(errno, file, attr); }, false);
}

UBA_EXPORT int UBA_WRAPPER(statx)(int dirfd, const char* pathname, int flags, unsigned int mask, struct statx* statxbuf)
{
	UBA_INIT_DETOUR(statx, dirfd, pathname, flags, mask, statxbuf);
	return LdpreloadStatx("statx", dirfd, pathname, flags, mask, statxbuf);
}
#endif

namespace uba
{
	inline int LdpreloadRename(const char* o, const char* n)
	{
		return Real_rename(errno,o, n);
	}
}

UBA_EXPORT int UBA_WRAPPER(rename)(const char* oldpath, const char* newpath)
{
	UBA_INIT_DETOUR(rename, oldpath, newpath);
	return Shared_rename("rename", oldpath, newpath, errno, LdpreloadRename);
}

UBA_EXPORT int UBA_WRAPPER(chmod)(const char* pathname, mode_t mode)
{
	UBA_INIT_DETOUR(chmod, pathname, mode);
	int err = 0;
	int res = Shared_chmod("chmod", pathname, mode, err,
		[](const char* p, mode_t m) { return Real_chmod(errno,p, m); });
	if (res != 0) errno = err;
	return res;
}

UBA_EXPORT int UBA_WRAPPER(renameat)(int olddirfd, const char* oldpath, int newdirfd, const char* newpath)
{
	UBA_INIT_DETOUR(renameat, olddirfd, oldpath, newdirfd, newpath);

	StringBuffer<> temp1;
	const char* resolvedOldPath = ResolvePath(olddirfd, oldpath, temp1);
	UBA_ASSERT(resolvedOldPath);
	StringBuffer<> temp2;
	const char* resolvedNewPath = ResolvePath(newdirfd, newpath, temp2);
	UBA_ASSERT(resolvedNewPath);
	return Shared_rename("renameat", resolvedOldPath, resolvedNewPath, errno, LdpreloadRename);
}

#if PLATFORM_LINUX
UBA_EXPORT int UBA_WRAPPER(renameat2)(int olddirfd, const char* oldpath, int newdirfd, const char* newpath, unsigned int flags)
{
	UBA_INIT_DETOUR(renameat2, olddirfd, oldpath, newdirfd, newpath, flags);
	(void)flags;

	StringBuffer<> temp1;
	const char* resolvedOldPath = ResolvePath(olddirfd, oldpath, temp1);
	UBA_ASSERT(resolvedOldPath);
	StringBuffer<> temp2;
	const char* resolvedNewPath = ResolvePath(newdirfd, newpath, temp2);
	UBA_ASSERT(resolvedNewPath);
	return Shared_rename("renameat2", resolvedOldPath, resolvedNewPath, errno, uba::LdpreloadRename);
}
#endif

UBA_EXPORT int UBA_WRAPPER(utimensat)(int dirfd, const char* pathname, const struct timespec* times, int flags)
{
	UBA_INIT_DETOUR(utimensat, dirfd, pathname, times, flags);

	StringBuffer<> temp;
	if (const char* resolvedPath = ResolvePath(dirfd, pathname, temp))
	{
		StringBuffer<> fixedPath;
		FixPath(fixedPath, resolvedPath);
		DevirtualizePath(fixedPath);
		FileAttributes fileAttr;
		Shared_GetFileAttributes(fileAttr, fixedPath);
		if (!fileAttr.exists)
		{
			DEBUG_LOG_DETOURED("utimensat", "%s -> -1 (ENOENT)", fixedPath.data);
			errno = ENOENT;
			return -1;
		}

		UBA_ASSERT(flags == 0);

		if (fileAttr.data.st_size == InvalidValue) // Means there is a file handle for it
		{
			SCOPED_READ_LOCK(g_fileHandlesLock, lock);
			for (auto& kv : g_fileHandles)
			{
				int fd = kv.first;
				DetouredHandle& h = kv.second;
				if (!fixedPath.Equals(h.fileObject->fileInfo->originalName))
					continue;
				DEBUG_LOG_TRUE("utimensat", "(futimens) %s", fixedPath.data);
				return futimens(fd, times);
			}
			UBA_ASSERTF(false, "Expected to find %s in file handles list", fixedPath.data);
		}

		// TODO: This should be added to the overlay and applied if process finish successfully
		Rpc_WriteLogf("Skipping utimensat on %s. Needs fix!", resolvedPath);
		return 0;
	}

	UBA_ASSERT(g_runningRemote);
	int res = TRUE_WRAPPER(utimensat)(dirfd, pathname, times, flags);
	DEBUG_LOG_TRUE("utimensat", "(%s) -> %i (%s)", pathname, res, StrError(res, errno));
	return res;
}

UBA_EXPORT int UBA_WRAPPER(symlink)(const char* symlinkContent, const char* symlinkFile)
{
	UBA_INIT_DETOUR(symlink, symlinkContent, symlinkFile);

	StringBuffer<> symlinkFilePath;
	FixPath(symlinkFilePath, symlinkFile);
	DevirtualizePath(symlinkFilePath);

	StringBuffer<MaxPath> symlinkContentPath; // MaxPath: targets routinely exceed StringBuffer<>'s 512 default
	if (IsAbsolutePath(symlinkContent))
		FixPath(symlinkContentPath, symlinkContent);
	else
		FixPath(symlinkContentPath, ToView(symlinkContent), ToView(symlinkFilePath).GetPath(true));

	DevirtualizePath(symlinkContentPath);
	if (IsAbsolutePath(symlinkContent))
		symlinkContent = symlinkContentPath.data;

	if (!CanDetour2(symlinkContentPath) && !CanDetour2(symlinkFilePath))
	{
		DEBUG_LOG_TRUE("symlink", "(file: %s content: %s)", symlinkFile, symlinkContent);
		return Real_symlink(errno,symlinkContent, symlinkFile);
	}

	StringKey contentPathKey = ToFilenameKey(symlinkContentPath);
	StringKey filePathKey = ToFilenameKey(symlinkFilePath);

	{
		DirectoryTableEntryInformation existing;
		if (Rpc_GetEntryInformation(existing, filePathKey, symlinkFilePath, false, /*followSymlink=*/false))
		{
			errno = EEXIST; // POSIX: linkpath already exists
			DEBUG_LOG_DETOURED("symlink", "(file: %s content: %s) -> -1 (EEXIST)", symlinkFilePath.data, symlinkContent);
			return -1;
		}
	}

	// Register the SYMLINK in the dirtable, not its target. POSIX: mode bits ignored — mode=S_IFLNK|0777, size=target length.
	DirectoryTableEntryInformation info;
	info.attributes = S_IFLNK | 0777;
	info.size = u64(strlen(symlinkContent));
	info.lastWrite = 0; // host's symlink() doesn't let us set mtime; refresh later if needed

	RPC_MESSAGE(Symlink, symlink)
	writer.WriteStringKey(filePathKey);
	writer.WriteString(symlinkFilePath);
	writer.WriteStringKey(contentPathKey);
	writer.WriteString(symlinkContentPath);
	writer.WriteString(symlinkContent);
	writer.WriteFileSize(info.size);
	writer.WriteFileTime(info.lastWrite);
	writer.WriteFileAttributes(info.attributes);
	BinaryReader reader = writer.Flush();
	bool success = reader.ReadBool();
	u32 error = reader.ReadU32();
	DirTableSize directoryTableSize = FromU64(reader.ReadU64());
	RPC_MESSAGE_END

	g_directoryTable->ParseDirectoryTable(directoryTableSize);

	errno = error;
	int res = success ? 0 : -1;
	DEBUG_LOG_TRUE("symlink", "(file: %s content: %s) -> %i (%s)", symlinkFilePath.data, symlinkContent, res, StrError(res, error));
	return res;
}

UBA_EXPORT ssize_t UBA_WRAPPER(pread)(int __fd, void * __buf, size_t __nbyte, off_t __offset)
{
	return Real_pread(errno,__fd, __buf, __nbyte, __offset);
}

UBA_EXPORT ssize_t UBA_WRAPPER(read)(int fd, void *buf, size_t nbyte)
{
	UBA_INIT_DETOUR(read, fd, buf, nbyte)
	ssize_t res = Real_read(errno,fd, buf, nbyte);
	DEBUG_LOG_TRUE("read", "%u %llu -> %llu", fd, nbyte, res);
	return res;
}

int Shared_DeleteFile(const char* funcName, const char* pathname)
{
	return Shared_deleteFile(funcName, pathname, errno, [&](const char* p) {
		int res = Real_unlink(errno,p);
		DEBUG_LOG_TRUE(funcName, "(%s) -> %i (%s)", p, res, StrError(res, errno));
		return res;
	});
}

UBA_EXPORT int UBA_WRAPPER(remove)(const char* pathname)
{
	UBA_INIT_DETOUR(remove, pathname);
	// TODO: Should check if pathname is a dir, in that case call rmdir
	return Shared_DeleteFile("remove", pathname);
}

int Shared_link(const char* func, const char* oldpath, const char* newpath)
{
	oldpath = ResolveProcSelfFd(oldpath);
	newpath = ResolveProcSelfFd(newpath);
	StringBuffer<> fromPath;
	FixPath(fromPath, oldpath);
	DevirtualizePath(fromPath);
	StringBuffer<> toPath;
	FixPath(toPath, newpath);
	DevirtualizePath(toPath);

	if (!CanDetour2(toPath))
	{
		UBA_ASSERT(!CanDetour2(fromPath));
		int res = Real_link(errno,fromPath.data, toPath.data); // FixPath'd absolute, never raw user paths
		DEBUG_LOG_TRUE(func, "(NODETOUR) %s to %s -> %i (%s)", fromPath.data, toPath.data, res, StrError(res, errno));
		return res;
	}

	StringKey fromKey = ToFilenameKey(fromPath);

	DirectoryTableEntryInformation fromEntryInfo;
	if (!Rpc_GetEntryInformation(fromEntryInfo, fromKey, fromPath, false))
	{
		DEBUG_LOG_DETOURED(func, "(DIRTABLE) %s to %s -> -1 (ENOENT)", oldpath, newpath);
		errno = ENOENT;
		return -1;
	}
	UBA_ASSERT(!IsDirectory(fromEntryInfo.attributes));

	StringKey toKey = ToFilenameKey(toPath);

	DirectoryTableEntryInformation toEntryInfo;
	if (Rpc_GetEntryInformation(toEntryInfo, toKey, toPath, false))
	{
		DEBUG_LOG_DETOURED(func, "(DIRTABLE) %s to %s -> -1 (EEXIST)", oldpath, newpath);
		errno = EEXIST;
		return -1;
	}

	StringBuffer<> newFromPath;
	StringBuffer<> newToPath;

	RPC_MESSAGE(CopyFile, copyFile);
	writer.WriteStringKey(fromKey);
	writer.WriteString(fromPath);
	writer.WriteU64(fromEntryInfo.size);
	writer.WriteU64(fromEntryInfo.lastWrite);
	writer.WriteU32(fromEntryInfo.attributes);
	writer.WriteStringKey(toKey);
	writer.WriteString(toPath);
	BinaryReader reader = writer.Flush();
	reader.ReadString(newFromPath);
	reader.ReadString(newToPath);
	u32 closeId = reader.ReadU32();
	u32 lastError = reader.ReadU32();
	DirTableSize directoryTableSize = FromU64(reader.ReadU64());
	RPC_MESSAGE_END;

	g_directoryTable->ParseDirectoryTable(directoryTableSize);

	UBA_ASSERT(closeId != ~0u);
	UBA_ASSERT(closeId != 0);

	int res = Real_link(errno,newFromPath.data, newToPath.data);
	DEBUG_LOG_TRUE(func, "%s to %s (%s to %s) -> %i (%s)", oldpath, newpath, newFromPath.data, newToPath.data, res, StrError(res, errno));

	bool deleteOnClose = res != 0; // If failing to copy we set deleteOnClose
	Rpc_UpdateCloseHandle(newToPath.data, closeId, deleteOnClose, "", {}, fromEntryInfo.size, fromEntryInfo.lastWrite, fromEntryInfo.attributes, true);

	return res;
}

UBA_EXPORT int UBA_WRAPPER(link)(const char* oldpath, const char* newpath)
{
	UBA_INIT_DETOUR(link, oldpath, newpath);
	return Shared_link("link", oldpath, newpath);
}

UBA_EXPORT int UBA_WRAPPER(linkat)(int olddirfd, const char *oldpath, int newdirfd, const char *newpath, int flags)
{
	UBA_INIT_DETOUR(linkat, olddirfd, oldpath, newdirfd, newpath, flags);
	StringBuffer<> temp1;
	const char* resolvedOldPath = ResolvePath(olddirfd, oldpath, temp1);
	UBA_ASSERT(resolvedOldPath);
	StringBuffer<> temp2;
	const char* resolvedNewPath = ResolvePath(newdirfd, newpath, temp2);
	UBA_ASSERT(resolvedNewPath);
	UBA_ASSERT(!flags);
	return Shared_link("linkat", resolvedOldPath, resolvedNewPath);
}

UBA_EXPORT int UBA_WRAPPER(unlink)(const char* pathname)
{
	UBA_INIT_DETOUR(unlink, pathname);
	return Shared_DeleteFile("unlink", pathname);
}

UBA_EXPORT int UBA_WRAPPER(unlinkat)(int dirfd, const char *pathname, int flags)
{
	UBA_INIT_DETOUR(unlinkat, dirfd, pathname, flags);

	StringBuffer<> temp;
	if (const char* resolvedPath = ResolvePath(dirfd, pathname, temp))
	{
		if (flags == AT_REMOVEDIR)
		{
			int err = 0;
			int rc = Shared_rmdir("unlinkat/rmdir", resolvedPath, err, [](const char* p) { return Real_rmdir(errno,p); });
			if (rc == -1 && err)
				errno = err;
			return rc;
		}
		return Shared_DeleteFile("unlinkat", resolvedPath);
	}
	// dirfd not in g_fileHandles (raw-syscall open / flag bypass) — kernel does the delete; dirtable best-effort.
	DEBUG_LOG_TRUE("unlinkat", "(%i) %s (untracked dirfd, kernel passthrough)", dirfd, pathname);
	return Real_unlinkat(errno,dirfd, pathname, flags);
}

void FlattenArgs(StringBufferBase& out, const char* const argv[])
{
	if (!argv)
		return;
	for (u32 i = 0; argv[i]; ++i)
	{
		if (i != 0)
			out.Append(' ');
		out.Append(argv[i]);
	}
}

bool ExecuteHostRun(StringBufferBase& out, const char* const* argv, bool removeLineFeed = true)
{
	{
		StringBuffer<4096> command;
		FlattenArgs(command, argv);
		DEBUG_LOG_DETOURED("HostRun", "%s", command.data);
	}
	RPC_MESSAGE(HostRun, getFullFileName);
	u16& size = *(u16*)writer.AllocWrite(2);
	u64 pos = writer.GetPosition();
	for (u32 i = 0; argv[i]; ++i)
		writer.WriteString(argv[i]);
	size = u16(writer.GetPosition() - pos);

	BinaryReader reader = writer.Flush();
	bool success = reader.ReadBool();
	reader.ReadString(out);
	
	if (removeLineFeed && out.count && out.data[out.count-1] == '\n')
		out.Resize(out.count-1);

	if (!success)
		DEBUG_LOG("HOSTRUN FAILED: %s", out.data);
	return success;
}

int SpawnEcho(char* str, pid_t* pid, const posix_spawn_file_actions_t* file_actions, const posix_spawnattr_t* attrp, char* const envp[])
{
#if 0
	static Atomic<u32> counter;
	StringBuffer<> tempFile;
	tempFile.Append(g_systemTemp).EnsureEndsWithSlash().Append("UbaTempFile").AppendValue(getpid()).Append('_').AppendValue(counter++);
	int fd = Real_open(errno,tempFile.data, O_CREAT|O_TRUNC|O_WRONLY, S_IRUSR | S_IWUSR);
	Real_write(errno,fd, str, strlen(str));
	Real_close(errno,fd);

	DEBUG_LOG("Created %s for cat", tempFile.data);
	char cmd[] = "/bin/cat"; 
	char* const argv2[] = { cmd, tempFile.data, nullptr };
	int res = TRUE_WRAPPER(posix_spawn)(pid, cmd, file_actions, attrp, argv2, envp);
	DEBUG_LOG_TRUE("posix_spawn", "(CAT) %s (pid: %u) -> %i", str, *pid, res);
#else
	char* const env[] = { nullptr };
	char cmd[] = "/bin/echo"; 
	char* const argv2[] = { cmd, str, nullptr };
	int res = TRUE_WRAPPER(posix_spawn)(pid, cmd, file_actions, attrp, argv2, env);
	DEBUG_LOG_TRUE("posix_spawn", "(ECHO) %s (pid: %u) -> %i", str, *pid, res);
#endif
	return res;
}

void UnsupportedHostRun(char* const argv[], const char* msg)
{
	StringBuffer<4096> command;
	command.Append(msg).Append(": ");
	FlattenArgs(command, argv);
	UbaAssert(command.data, __FILE__, __LINE__, nullptr, true, 1999, nullptr, 0);
}


int Shared_posix_spawn(const char* func, pid_t* pid, const char* path, const posix_spawn_file_actions_t* file_actions, const posix_spawnattr_t* attrp, char* const argv[], char* const envp[])
{
	UBA_INIT_DETOUR(posix_spawn, pid, path, file_actions, attrp, argv, envp);

	const char* tempArgv[1024];
	StringBuffer<> result;
	StringBuffer<> additionalArg;

	bool fixupPath = true;

	if (strstr(path, "xcode-select"))
	{
		if (strcmp(argv[1], "--print-path") != 0)
			UbaAssert("xcode-select only supported with --print-path", __FILE__, __LINE__, "", true, 1999, nullptr, 0);
		if (!ExecuteHostRun(result, argv))
			return -1;
		return SpawnEcho(result.data, pid,  file_actions, attrp, envp);
	}
	
	if(strstr(path, "xcrun"))
	{
		if (strcmp(argv[1], "--sdk") != 0)
			UnsupportedHostRun(argv, "xcrun unsupported first param");

		if (strcmp(argv[3], "--find") == 0)
		{
			if (!ExecuteHostRun(result, argv))
				return -1;
			return SpawnEcho(result.data, pid,  file_actions, attrp, envp);
		}
		
		if (strcmp(argv[3], "metal") != 0 && strcmp(argv[3], "metallib") != 0 && strcmp(argv[3], "metal-strip") != 0)
			UnsupportedHostRun(argv, "xcrun unsupported third param");

		const char* argv2[] = { argv[0], argv[1], argv[2], "--find", argv[3], 0 };
		if (!ExecuteHostRun(result, argv2))
			return -1;

#if 0 // No longer needed; if it returns, switch to a host realpath query (this block doesn't work with newer SDKs).
		if (const char* usrbin = strstr(result.data, "/bin/metal"))
		{
			result.Resize(usrbin - result.data);
			result.Append("/metal/");
			if (strcmp(argv[2], "macosx") == 0)
				result.Append("macos");
			else
				result.Append("ios");
			result.Append("/bin/").Append(argv[3]);
		}
#endif

		path = result.data;

		u32 argc3 = 0;
		tempArgv[argc3++] = result.data;
		for (int i=4;argv[i]; ++i)
			tempArgv[argc3++] = argv[i];
		if (Equals(argv[3], "metal"))
		{
			// Clang cache must be machine-local (host's temp may not match remote helper's).
			additionalArg.Append("-fmodules-cache-path=").Append(g_systemTemp).EnsureEndsWithSlash().Append("clangcache");
			tempArgv[argc3++] = additionalArg.data;
		}
		tempArgv[argc3] = 0;
		argv = (char*const*)tempArgv;
		fixupPath = false;
	}

	ProcessCreator pc;
	int res = pc.SendMessage(MessageType_CreateProcess, "CreateProcess", g_stats.createProcess, path, argv, envp, fixupPath, false);
	if (res != 0)
	{
		DEBUG_LOG_DETOURED(func, "SendMessage failed");
		return res;
	}

	// Propagate UBA_INHERIT_HANDLES / UBA_VFS_OVERLAYS / UBA_PASSTHROUGH_ROOTS (parity with Shared_Internal_exec). GNU make sub-makes use posix_spawn.
	StringBuffer<64*1024> inheritEnv;
	Shared_BuildInheritedHandlesEnv(inheritEnv);
	if (inheritEnv.count > 0)
	{
		if (!pc.envvars.empty() && pc.envvars.back() == nullptr)
			pc.envvars.pop_back();
		pc.envvars.push_back(inheritEnv.data);
		pc.envvars.push_back(nullptr);
		DEBUG_LOG_DETOURED(func, "(handed off) %s", inheritEnv.data);
	}
	StringBuffer<64*1024> vfsEnv;
	Shared_BuildVfsEnv(vfsEnv);
	if (vfsEnv.count > 0)
	{
		if (!pc.envvars.empty() && pc.envvars.back() == nullptr)
			pc.envvars.pop_back();
		pc.envvars.push_back(vfsEnv.data);
		pc.envvars.push_back(nullptr);
		DEBUG_LOG_DETOURED(func, "(vfs-overlays) %s", vfsEnv.data);
	}
	StringBuffer<4*1024> passEnv;
	Shared_BuildPassThroughEnv(passEnv);
	if (passEnv.count > 0)
	{
		if (!pc.envvars.empty() && pc.envvars.back() == nullptr)
			pc.envvars.pop_back();
		pc.envvars.push_back(passEnv.data);
		pc.envvars.push_back(nullptr);
		DEBUG_LOG_DETOURED(func, "(pass-through) %s", passEnv.data);
	}

	#if UBA_DEBUG_LOG_ENABLED
	DEBUG_LOG_TRUE(func, "%s (%s)", pc.realApplication.data(), pc.logFile.data());
	for (u32 i = 0; pc.fixedArgv[i]; ++i)
		DEBUG_LOG("            %s", pc.fixedArgv[i]);
	#endif

	// Refcount handles open for write.
	u32 handlesWithCloseId = 0;
	for (auto& kv : g_fileHandles)
		if (kv.second.fileObject->closeId)
			++handlesWithCloseId;
	if (handlesWithCloseId)
	{
		RPC_MESSAGE(InheritFiles, inheritFiles);
		for (auto& kv : g_fileHandles)
			if (!kv.second.hasCloexec)
				if (u32 closeId = kv.second.fileObject->closeId)
				{
					writer.WriteU32(closeId);
					DEBUG_LOG("AddRef closeId: %u", closeId);
				}
		writer.WriteU32(0);
		writer.Flush();
	}

	constexpr u32 maxRetries = 30; // ~5s ETXTBSY budget for WSL2's long task_work_run windows under parallel-write startup storms
	for (u32 attempt = 0; ; ++attempt)
	{
		res = TRUE_WRAPPER(posix_spawn)(pid, pc.realApplication.data(), file_actions, attrp, pc.fixedArgv, (char**)pc.envvars.data());
		if (res == 0 || res != ETXTBSY || attempt >= maxRetries)
			break;
		u32 sleepMs = 20*(attempt+1);
		if (sleepMs > 200) sleepMs = 200;
		Sleep(sleepMs);
		DEBUG_LOG_TRUE(func, "ETXTBSY retry %u for %s (slept %ums)", attempt + 1, pc.realApplication.data(), sleepMs);
	}
	bool success = res == 0;
	UBA_ASSERT(success || !handlesWithCloseId); // We need to implement this if we ever hit failing to spawn process while having inherited handles

	SendStartProcess(pc.processId, *pid, success, res);
	DEBUG_LOG("         Child process started %s -> %i (pid: %i)", path, res, *pid);
	return res;
}

UBA_EXPORT int UBA_WRAPPER(posix_spawn)(pid_t* pid, const char* path, const posix_spawn_file_actions_t* file_actions, const posix_spawnattr_t* attrp, char* const argv[], char* const envp[])
{
	return Shared_posix_spawn("posix_spawn", pid, path, file_actions, attrp, argv, envp);
}

UBA_EXPORT int UBA_WRAPPER(posix_spawnp)(pid_t* pid, const char* file, const posix_spawn_file_actions_t* file_actions, const posix_spawnattr_t* attrp, char* const argv[], char* const envp[])
{
	return Shared_posix_spawn("posix_spawnp", pid, file, file_actions, attrp, argv, envp);
}

UBA_EXPORT pid_t UBA_WRAPPER(wait)(int* status)
{
	UBA_INIT_DETOUR(wait, status);
	pid_t res = TRUE_WRAPPER(wait)(status);
	DEBUG_LOG_TRUE("wait", "%i -> %i", status ? *status : 0, res);
	return res;
}

UBA_EXPORT pid_t UBA_WRAPPER(waitpid)(pid_t pid, int* status, int options)
{
	UBA_INIT_DETOUR(waitpid, pid, status, options);
	int resStatus = 0; // always pass status so we can inspect WIFEXITED
	pid_t res = TRUE_WRAPPER(waitpid)(pid, &resStatus, options);
	DEBUG_LOG_TRUE("waitpid", "(%i) -> %i (%i)", pid, res, resStatus);
	// WNOHANG returns 0 with status untouched and WIFEXITED(0)=true; gate on res>0 or every poll floods Rpc_UpdateTables.
	if (res > 0 && WIFEXITED(resStatus))
		Rpc_UpdateTables();
	if (status)
		*status = resStatus;
	return res;
}

const char* GetResult(siginfo_t* info)
{
	if (!info)
		return "null";
	int code = info->si_code;
	if (code == CLD_EXITED)
		return "Exited";
	if (code == CLD_KILLED)
		return "Killed";
	if (code == CLD_STOPPED)
		return "Stopped";
	if (code == CLD_CONTINUED)
		return "Continued";
	if (code == CLD_TRAPPED)
		return "Trapped";
	return "Running";
}

UBA_EXPORT int UBA_WRAPPER(waitid)(idtype_t idtype, id_t id, siginfo_t* infop, int options)
{
	UBA_INIT_DETOUR(waitid, idtype, id, infop, options);
	auto res = TRUE_WRAPPER(waitid)(idtype, id, infop, options);
	DEBUG_LOG_TRUE("waitid", "%i -> %i (%s)", id, res, GetResult(infop));
	// WNOHANG without a reap leaves si_pid=0 (and si_code stale) — gate both before firing Rpc_UpdateTables.
	if (res == 0 && infop && infop->si_pid != 0 && infop->si_code == CLD_EXITED)
		Rpc_UpdateTables();
	return res;
}

UBA_EXPORT pid_t UBA_WRAPPER(wait3)(int* status, int options, struct rusage* rusage)
{
	UBA_INIT_DETOUR(wait3, status, options, rusage);
	int resStatus = 0;
	pid_t res = TRUE_WRAPPER(wait3)(&resStatus, options, rusage);
	DEBUG_LOG_TRUE("wait3", "-> %i (%i)", res, resStatus);
	if (res > 0 && WIFEXITED(resStatus)) // see waitpid
		Rpc_UpdateTables();
	if (status)
		*status = resStatus;
	return res;
}

UBA_EXPORT pid_t UBA_WRAPPER(wait4)(pid_t pid, int* status, int options, struct rusage* rusage)
{
	UBA_INIT_DETOUR(wait4, pid, status, options, rusage);
	int resStatus = 0;
	pid_t res = TRUE_WRAPPER(wait4)(pid, &resStatus, options, rusage);
	if (res > 0 && WIFEXITED(resStatus)) // see waitpid
		Rpc_UpdateTables();
	DEBUG_LOG_TRUE("wait4", "(pid %i) -> %i (%i)", pid, res, resStatus);
	if (status)
		*status = resStatus;
	return res;
}

UnorderedSet<TString> g_handledLibraries;
UnorderedMap<TString, TString> g_dlopenStagedToVirtual; // staged-on-disk path → virtual path; dladdr undoes the substitution
ReaderWriterLock g_dlopenStagedToVirtualLock;

void Shared_LoadLibrary(const char*& path, const char* const* loaderPaths, StringBufferBase& tempBuf)
{
	StringBuffer<512> virtualPath;
	Rpc_GetFullFileName2(path, tempBuf, virtualPath, loaderPaths);
	StringView originalPath = StringView(virtualPath).GetPath();
	StringBuffer<> error;
	BinaryInfo info;
	ParseBinary(tempBuf, originalPath, info, [&](const tchar* import, bool isKnown, const char* const* importLoaderPaths)
	{
		if (!g_handledLibraries.insert(import).second)
			return;
		StringBuffer<> temp;
		Shared_LoadLibrary(import, importLoaderPaths, temp);
	}, error);
	if (error.count)
		DEBUG_LOG(error.data);
}

UBA_EXPORT void* UBA_WRAPPER(dlopen)(const char* path, int mode)
{
	UBA_INIT_DETOUR(dlopen, path, mode);

	auto runTrue = [&]()
		{
			void* res = TRUE_WRAPPER(dlopen)(path, mode);
			DEBUG_LOG_TRUE("dlopen", "%s (%i) -> 0x%x", path, mode, res);
			return res;
		};

	StringBuffer<> tempBuf;
	if (g_isUsingLocalBinary || !path || !*path)
	{
		if (path && *path == '/' && IsVfsEnabled()) // absolute paths still need VFS devirt or NS-pinned dlopen returns NULL
		{
			StringBuffer<> devirt(path);
			if (DevirtualizePath(devirt))
			{
				const char* oldPath = path;
				path = devirt.data;
				{
					SCOPED_WRITE_LOCK(g_dlopenStagedToVirtualLock, lock);
					g_dlopenStagedToVirtual.try_emplace(path, oldPath);
				}
				void* res = TRUE_WRAPPER(dlopen)(path, mode);
				DEBUG_LOG_TRUE("dlopen", "%s (devirt %s) (%i) -> 0x%x", oldPath, path, mode, res);
				return res;
			}
		}
		return runTrue();
	}

#if PLATFORM_MAC
	if (StartsWith(path, "@rpath/"))
	{
		path += 7;
		const char* loaderPaths[] = { "/", 0 };
		if (g_handledLibraries.insert(path).second)
			Shared_LoadLibrary(path, loaderPaths, tempBuf);
	}
	else if (!StartsWith(path, "/System") && !StartsWith(path, "/usr/lib"))
	{
		u64 nameLen = 0;
		Rpc_GetFullFileName(path, nameLen, tempBuf, false);
	}
#else

	if (!IsKnownSystemFile(ToView(path).GetFileName().data))
	{
		const tchar* virtualPath = path;
		const char* loaderPaths[] = { "/", 0 };
		bool fetched = false;
		if (g_handledLibraries.insert(path).second)
		{
			Shared_LoadLibrary(path, loaderPaths, tempBuf);
			fetched = tempBuf.count > 0 && tempBuf.data[0] == '/';
		}
		if (fetched)
		{
			// Record staged→virtual so dladdr returns virtual path (HotSpot os::jvm_path derives JAVA_HOME/lib from it).
			SCOPED_WRITE_LOCK(g_dlopenStagedToVirtualLock, lock);
			g_dlopenStagedToVirtual.try_emplace(tempBuf.data, virtualPath);
			lock.Leave();
			path = tempBuf.data;
		}
		else if (const char* name = TStrrchr(path, '/'))
			path = name + 1;
	}
#endif

	return runTrue();
}

UBA_EXPORT int UBA_WRAPPER(dladdr)(const void *addr, Dl_info *info)
{
	UBA_INIT_DETOUR(dladdr, addr, info);
	int res = TRUE_WRAPPER(dladdr)(addr, info);
	if (res && info->dli_fname && *info->dli_fname)
	{
		// Undo dlopen's staged-path substitution; detoured callers see virtual paths everywhere.
		SCOPED_READ_LOCK(g_dlopenStagedToVirtualLock, lock);
		auto findIt = g_dlopenStagedToVirtual.find(info->dli_fname);
		if (findIt != g_dlopenStagedToVirtual.end())
			info->dli_fname = g_memoryBlock.Strdup(findIt->second.c_str());
		else if (StartsWith(info->dli_fname, g_exeDir.data)) // restored to fix a chrome build
		{
			StringBuffer<PATH_MAX> newPath;
			newPath.AppendDir(g_virtualApplication).Append(info->dli_fname + g_exeDir.count - 1);
			info->dli_fname = g_memoryBlock.Strdup(newPath).data;
		}
	}
	DEBUG_LOG_TRUE("dladdr", "%s -> %i", (res && info->dli_fname && *info->dli_fname) ? info->dli_fname : "", res);
	return res;
}

int Internal_exec(const char* func, const char* pathname, char* const _Nullable argv[], char* const _Nullable envp[], bool fixupPath)
{
	UBA_INIT_DETOUR(execve, pathname, argv, envp);

	DEBUG_LOG_DETOURED(func, "ENTRY path=%s argv0=%s fixupPath=%i", pathname ? pathname : "<null>", (argv && argv[0]) ? argv[0] : "<null>", fixupPath ? 1 : 0); // log raw path before FixPath/Devirt/RPC

	// vfork commonly closes all fds; we re-open g_debugFile for logging. O_CLOEXEC handles successful execve.
	#if UBA_DEBUG_LOG_ENABLED
	if (Real_fcntl(errno, g_debugFile, F_GETFD, 0) == -1 && errno == EBADF)
		g_debugFile = (FileHandle)Real_open(errno,g_debugLogPath.data, O_APPEND | O_WRONLY | O_CLOEXEC, S_IRUSR | S_IWUSR);
	#endif

	auto trueExecve = [](const char* p, char* const av[], char* const ev[]) -> int
	{
		if (TRUE_WRAPPER(execve)(p, av, ev) == 0) return 0;
		return errno;
	};
	auto trueNanosleepUs = [](u32 us)
	{
		struct timespec ts { 0, long(us) * 1000 };
		nanosleep(&ts, nullptr);
	};

	int r = Shared_Internal_exec(func, pathname, argv, envp, fixupPath, trueExecve, trueNanosleepUs, LdpreloadSnapshot);
	if (r < 0)
	{
		errno = -r;
		return -1;
	}
	return r;
}

UBA_EXPORT int UBA_WRAPPER(execv)(const char* path, char* const argv[])
{
	return Internal_exec("execv", path, argv, environ, true);
}

UBA_EXPORT int UBA_WRAPPER(execve)(const char* pathname, char* const _Nullable argv[], char* const _Nullable envp[])
{
	return Internal_exec("execve", pathname, argv, envp, true);
}

UBA_EXPORT int UBA_WRAPPER(execvp)(const char* file, char* const argv[])
{
	// TODO: devirt shouldn't be needed here, but fixes Mac ld — find and fix the deeper issue.
	StringBuffer<> temp(file);
	DevirtualizePath(temp);
	file = temp.data;
	return Internal_exec("execvp", file, argv, environ, false);
}

UBA_EXPORT int UBA_WRAPPER(execl)(const char *path, const char *arg0, ...)
{
	DEBUG_LOG_TRUE("execl", "");
	UBA_ASSERTF(false, "execl NotImplemented");
	return -1;
}

UBA_EXPORT int UBA_WRAPPER(execle)(const char *path, const char *arg0, ...)
{
	DEBUG_LOG_TRUE("execle", "");
	UBA_ASSERTF(false, "execle NotImplemented");
	return -1;
}

UBA_EXPORT int UBA_WRAPPER(execlp)(const char *file, const char *arg0, ...)
{
	DEBUG_LOG_TRUE("execlp", "");
	UBA_ASSERTF(false, "execlp NotImplemented");
	return -1;
}

#if PLATFORM_LINUX
UBA_EXPORT int UBA_WRAPPER(fexecve)(int fd, char *const argv[], char *const envp[])
{
	DEBUG_LOG_TRUE("fexecve", "");
	char path[PATH_MAX];
	char procLink[64];
	snprintf(procLink, sizeof(procLink), "/proc/self/fd/%d", fd);
	ssize_t n = readlink(procLink, path, sizeof(path) - 1);
	if (n < 0) {
		DEBUG_LOG_TRUE("fexecve", "readlink failed fd=%d", fd);
		return TRUE_WRAPPER(fexecve)(fd, argv, envp);
	}
	path[n] = 0;
	DEBUG_LOG_DETOURED("fexecve", "fd=%d -> %s", fd, path);
	return Internal_exec("fexecve", path, argv, envp, false);
}
#endif

#if PLATFORM_MAC
UBA_EXPORT int UBA_WRAPPER(execvP)(const char *file, const char *search_path, char *const argv[])
{
	DEBUG_LOG_TRUE("execvP", "");
	UBA_ASSERTF(false, "execvP NotImplemented");
	return -1;
}
#endif


UBA_EXPORT pid_t UBA_WRAPPER(fork)(void)
{
	UBA_INIT_DETOUR(fork);
	DEBUG_LOG_TRUE("fork", "");
	return Shared_fork();
}

#if PLATFORM_MAC
#pragma clang diagnostic ignored "-Wdeprecated-declarations"
#endif

UBA_EXPORT pid_t UBA_WRAPPER(vfork)(void)
{
	UBA_INIT_DETOUR(fork);
	DEBUG_LOG_DETOURED("vfork", "");
	return Shared_fork();
}

#if PLATFORM_LINUX
UBA_EXPORT long UBA_WRAPPER(syscall)(long number, ...)
{
	va_list args;
	va_start(args, number);
	void* a1 = va_arg(args, void*);
	void* a2 = va_arg(args, void*);
	void* a3 = va_arg(args, void*);
	void* a4 = va_arg(args, void*);
	void* a5 = va_arg(args, void*);
	void* a6 = va_arg(args, void*);
	va_end(args);

	UBA_INIT_DETOUR(syscall, number, a1, a2, a3, a4, a5, a6);

	switch (number)
	{
	case SYS_write:
	{
		DEBUG_LOG_TRUE("syscall.write", "fd=%ld", (long)(uintptr_t)a1);
		return TRUE_WRAPPER(syscall)(number, a1, a2, a3, a4, a5, a6);
	}

	case SYS_close:
	{
		int fd = (int)(long)(uintptr_t)a1;
		return Shared_close("syscall.close", fd, [fd]() { return Real_close(errno,fd); }, LdpreloadSnapshot);
	}

	case SYS_gettid:
	case SYS_capget:
	case SYS_futex:
		break;

	case SYS_getuid:
	case SYS_geteuid:
		{ DEBUG_LOG_DETOURED("syscall.getuid", "-> %u", g_virtualUid);
		return g_virtualUid; }
	case SYS_getgid:
	case SYS_getegid:
		{ DEBUG_LOG_DETOURED("syscall.getgid", "-> %u", g_virtualGid);
		return g_virtualGid; }

#if !PLATFORM_LINUX || !PLATFORM_CPU_ARM_FAMILY
	case SYS_vfork:
		{ DEBUG_LOG_DETOURED("syscall.vfork", "");
		return Shared_fork(); }
	case SYS_fork:
		{ DEBUG_LOG_DETOURED("syscall.fork", "");
		return Shared_fork(); }
#endif

	case SYS_clone:
		{ DEBUG_LOG_DETOURED("syscall.clone", "");
		return Shared_fork(); }

	case 217: // SYS_getdents64
	{
		int fd = (int)(long)(uintptr_t)a1;
		void* buf = a2;
		unsigned int cnt = (unsigned int)(uintptr_t)a3;
		int capturedErr = 0;
		long r = uba::Shared_getdents64("syscall.getdents64", fd, buf, cnt, &capturedErr);
		if (r < 0 && capturedErr == 0)
		{
			long kr = TRUE_WRAPPER(syscall)(number, a1, a2, a3, a4, a5, a6); // not a UBA fd — kernel with tmpfs EAGAIN backoff
			if (!(kr < 0 && errno == EAGAIN))
				return kr;
			struct timespec ts;
			ts.tv_sec = 0;
			long usec = 100;
			for (int retries = 0; retries < 13; ++retries)
			{
				ts.tv_nsec = usec * 1000;
				nanosleep(&ts, nullptr);
				kr = TRUE_WRAPPER(syscall)(number, a1, a2, a3, a4, a5, a6);
				if (!(kr < 0 && errno == EAGAIN))
					return kr;
				usec *= 2;
				if (usec > 50000) usec = 50000;
			}
			return kr;
		}
		if (r < 0)
		{
			errno = capturedErr;
			return -1;
		}
		return r;
	}

	case SYS_statx:
		return LdpreloadStatx("syscall.statx", (int)(long)(uintptr_t)a1, (const char*)a2, (int)(long)(uintptr_t)a3, (unsigned int)(uintptr_t)a4, (struct statx*)a5);

#ifdef SYS_access
	case SYS_access:
		return Shared_access("syscall.access", (const char*)a1, (int)(long)(uintptr_t)a2);
#endif

	case SYS_faccessat:
	{
		int dirfd = (int)(long)(uintptr_t)a1;
		const char* pathname = (const char*)a2;
		int mode = (int)(long)(uintptr_t)a3;
		StringBuffer<> temp;
		if (const char* resolved = ResolvePath(dirfd, pathname, temp))
			return Shared_access("syscall.faccessat", resolved, mode);
		DEBUG_LOG_TRUE("syscall.faccessat", "%s", pathname);
		return TRUE_WRAPPER(syscall)(number, a1, a2, a3, a4, a5, a6);
	}

#ifdef SYS_faccessat2
	case SYS_faccessat2:
	{
		int dirfd = (int)(long)(uintptr_t)a1;
		const char* pathname = (const char*)a2;
		int mode = (int)(long)(uintptr_t)a3;
		StringBuffer<> temp;
		if (const char* resolved = ResolvePath(dirfd, pathname, temp))
			return Shared_access("syscall.faccessat2", resolved, mode);
		DEBUG_LOG_TRUE("syscall.faccessat2", "%s", pathname);
		return TRUE_WRAPPER(syscall)(number, a1, a2, a3, a4, a5, a6);
	}
#endif

	case SYS_mkdirat:
	{
		int dirfd = (int)(long)(uintptr_t)a1;
		const char* pathname = (const char*)a2;
		mode_t mode = (mode_t)(uintptr_t)a3;
		StringBuffer<> temp;
		if (const char* resolved = ResolvePath(dirfd, pathname, temp))
		{
			int err = 0;
			int rc = Shared_mkdir("syscall.mkdirat", resolved, mode, err, [](const char* p, mode_t m) { return Real_mkdir(errno,p, m); });
			if (rc == -1 && err)
				errno = err;
			return rc;
		}
		return TRUE_WRAPPER(syscall)(number, a1, a2, a3, a4, a5, a6);
	}

	case SYS_fcntl:
	{
		int fd  = (int)(long)(uintptr_t)a1;
		int cmd = (int)(long)(uintptr_t)a2;
		long res = TRUE_WRAPPER(syscall)(number, a1, a2, a3, a4, a5, a6);
		if (res != -1 && (cmd == F_DUPFD || cmd == F_DUPFD_CLOEXEC))
			DuplicateFd(fd, (int)res, /*newCloexec*/cmd == F_DUPFD_CLOEXEC);
		if (res != -1 && cmd == F_SETFD)
		{
			SCOPED_WRITE_LOCK(g_fileHandlesLock, lock);
			auto it = g_fileHandles.find(fd);
			if (it != g_fileHandles.end())
				it->second.hasCloexec = ((u64)(uintptr_t)a3 & FD_CLOEXEC) != 0;
		}
		DEBUG_LOG_TRUE("syscall.fcntl", "fd=%d cmd=%d -> %ld", fd, cmd, res);
		return res;
	}

	case 425: // SYS_io_uring_setup
		errno = ENOSYS;
		return -1;

	case 262: // SYS_newfstatat — AL2025 glibc bypasses __fxstatat; need the tmpfs EAGAIN backoff here
	{
		long res = TRUE_WRAPPER(syscall)(number, a1, a2, a3, a4, a5, a6);
		if (res < 0 && errno == EAGAIN)
		{
			struct timespec ts;
			ts.tv_sec = 0;
			long usec = 100;
			for (int retries = 0; retries < 25 &&res < 0 && errno == EAGAIN; ++retries)
			{
				ts.tv_nsec = usec * 1000;
				nanosleep(&ts, nullptr);
				res = TRUE_WRAPPER(syscall)(number, a1, a2, a3, a4, a5, a6);
				usec *= 2;
				if (usec > 50000) usec = 50000;
			}
		}
		return res;
	}

	// nsjail uses raw syscall() for NS setup; route to Shared_*.
	case 165: // SYS_mount  (source, target, fstype, flags, data)
		return Shared_mount((const char*)a1, (const char*)a2,
			(const char*)a3, (unsigned long)(uintptr_t)a4, (const void*)a5);
	case 155: // SYS_pivot_root  (new_root, put_old)
		return Shared_pivot_root((const char*)a1, (const char*)a2);
	case 166: // SYS_umount2  (target, flags)
		return Shared_umount2((const char*)a1, (int)(uintptr_t)a2);
	case 161: // SYS_chroot  (path)
		return Shared_chroot((const char*)a1);

	case 435: // SYS_clone3 — a1=`struct clone_args*`, flags=first u64. Inlined (no template — stub's linker can't capture instantiations).
		{
			constexpr uint64_t kCloneNsMask =
				0x00020000ULL | // CLONE_NEWNS
				0x02000000ULL | // CLONE_NEWCGROUP
				0x04000000ULL | // CLONE_NEWUTS
				0x08000000ULL | // CLONE_NEWIPC
				0x10000000ULL | // CLONE_NEWUSER
				0x20000000ULL | // CLONE_NEWPID
				0x40000000ULL;  // CLONE_NEWNET
			uint64_t flags = a1 ? *(const uint64_t*)a1 : 0;
			DEBUG_LOG_DETOURED("syscall.clone3", "flags=0x%llx", (unsigned long long)flags);
			if (!(flags & kCloneNsMask))
				return Shared_fork();

			u32 childProcessId = ~0u; // NS case — inline Shared_fork-equivalent with real clone3

			StringBuffer<256> comId;
			StringBuffer<512> logFile;
			{
				RPC_MESSAGE(ForkProcess, forkProcess);
				for (auto& kv : g_fileHandles)
				{
					if (u32 closeId = kv.second.fileObject->closeId)
					{
						writer.WriteU32(closeId);
						DEBUG_LOG("AddRef closeId: %u", closeId);
					}
				}
				writer.WriteU32(0);
				BinaryReader reader = writer.Flush();
				childProcessId = reader.ReadU32();
				comId.AppendValue(reader.ReadU64()).Append('+').AppendValue(reader.ReadU32());
				reader.ReadString(logFile);
			}

			// Non-init PID NS: OR CLONE_PIDFD so we can recover host-NS pid from /proc/self/fdinfo/<pidfd>.
			int childPidfd = -1;
			long pid;
			size_t userSize = (size_t)(uintptr_t)a2;
			if (g_parentInNonInitNs && a1 && userSize >= 16 && userSize <= 256)
			{
				alignas(8) char argsBuf[256];
				memset(argsBuf, 0, sizeof(argsBuf));
				memcpy(argsBuf, a1, userSize);
				*(uint64_t*)(argsBuf + 0) |= 0x1000ULL; // CLONE_PIDFD
				*(uint64_t*)(argsBuf + 8) = (uint64_t)(uintptr_t)&childPidfd;
				pid = TRUE_WRAPPER(syscall)(number, argsBuf, (long)userSize, a3, a4, a5, a6);
			}
			else
				pid = TRUE_WRAPPER(syscall)(number, a1, a2, a3, a4, a5, a6);

			__asm__ volatile("" ::: "memory"); // without this barrier bash sometimes fails to start downstream after fork

			if (pid == 0)
			{
				Real_prctl_pdeathsig();

				#if UBA_DEBUG_LOG_ENABLED
				g_debugLogPath.Clear().Append(logFile);
				int err = 0;
				g_debugFile = (FileHandle)Real_open(err, logFile.data, O_CREAT | O_TRUNC | O_WRONLY | O_CLOEXEC, S_IRUSR | S_IWUSR);
				#endif

				DEBUG_LOG(TC("FORKED pid: %i (clone3-NS)"), Real_getpid());

				// Parse "<u64-uid>+<u32-offset>" — same as Shared_fork, inline.
				const char* plusIndex = comId.First('+');
				u64 comIdUid = 0;
				for (const char* p = comId.data; p < plusIndex; ++p)
				{
					if (*p < '0' || *p > '9')
					{
						DEBUG_LOG(TC("clone3-NS: bad uid digit in comId: %s"), comId.data);
						return -1;
					}
					comIdUid = comIdUid * 10u + (u64)(*p - '0');
				}
				u32 comIdOffset = 0;
				for (const char* p = plusIndex + 1; *p >= '0' && *p <= '9'; ++p)
					comIdOffset = comIdOffset * 10u + (u32)(*p - '0');

				char comIdName[64]; // build "/uba_<hex_uid>"

				{
					const char prefix[] = "/uba_";
					const char hexChars[] = "0123456789abcdef";
					u32 idx = 0;
					for (u32 k = 0; k < sizeof(prefix) - 1; ++k)
						comIdName[idx++] = prefix[k];
					u64 v = comIdUid;
					for (int i = 0; i < 8; ++i)
					{
						comIdName[idx++] = hexChars[(v >> 4) & 0xf];
						comIdName[idx++] = hexChars[v & 0xf];
						v = v >> 8;
						if (v == 0)
							break;
					}
					comIdName[idx] = 0;
				}

				u8* rptr = FileMapping_MapFromHost(FileMappingHandle::FromU64(comIdUid), CommunicationMemSize, comIdOffset, true, TC("ComHandle"));
				if (!rptr)
				{
					DEBUG_LOG(TC("clone3-NS: mmap failed for %llu"), comIdUid);
					return -1;
				}

				g_messageMappingMem = rptr;
				g_cancelEvent = ((SharedEvent*)rptr);
				g_readEvent   = ((SharedEvent*)rptr) + 1;
				g_writeEvent  = ((SharedEvent*)rptr) + 2;
				g_messageMappingMem += sizeof(SharedEvent) * 3;

				for (auto it=g_fileHandles.begin(); it!=g_fileHandles.end();)
				{
					if (it->second.fileObject->closeId == 0)
						it = g_fileHandles.erase(it);
					else
						++it;
				}

				{
					u64 emptyset = 0; // reset signal mask in the child (same as Shared_fork)
					#if defined(__x86_64__)
					register long r10 __asm__("r10") = 8;
					register long _r asm("rax");
					__asm__ volatile(
						"syscall"
						: "=a"(_r)
						: "0"((long)14), "D"((long)2), "S"((long)&emptyset), "d"((long)0), "r"(r10)
						: "rcx", "r11", "memory");
					#endif
				}

				return 0;
			}

			if (pid > 0)
			{
				u32 hostPid = (u32)pid;
				if (childPidfd >= 0)
				{
					if (u32 hp = GetHostPidFromPidfd(childPidfd))
						hostPid = hp;
					int err = 0; Real_close(err, childPidfd);
				}
				SendStartProcess(childProcessId, hostPid, true, 0);
				DEBUG_LOG("         Process forked (child pid: %lld, hostPid: %u) (clone3-NS)", (long long)pid, hostPid);
			}
			return pid;
		}

	case 436: // SYS_close_range
		UBA_NOT_IMPLEMENTED("syscall.close_range")
		break;

#if UBA_DEBUG_LOG_ENABLED
	default:
		DEBUG_LOG_TRUE("syscall", "%ld", number);
		break;
#endif
	}

	return TRUE_WRAPPER(syscall)(number, a1, a2, a3, a4, a5, a6);
}
#endif

// TODO: Allow multiple popen.. hacky!
FILE* g_activePopen;
StringBuffer<64*1024>* g_activePopenResult;
u32 g_activePopenReadPos = 0;

UBA_EXPORT FILE* UBA_WRAPPER(popen)(const char* command, const char* type)
{
	UBA_INIT_DETOUR(popen, command, type);
	DEBUG_LOG_DETOURED("popen", "%s", command);

	UBA_ASSERT(!g_activePopen);
	const char* argv[] = { command, nullptr };
	g_activePopenResult = new StringBuffer<64*1024>();

	if (!ExecuteHostRun(*g_activePopenResult, argv, false))
		return nullptr;
	g_activePopenReadPos = 0;
	g_activePopen = (FILE*)1337;
	return g_activePopen;
}

UBA_EXPORT char* UBA_WRAPPER(fgets)(char* str, int count, FILE* stream)
{
	UBA_INIT_DETOUR(fgets, str, count, stream);
	DEBUG_LOG_TRUE("fgets", "(%p)", stream);
	if (stream == g_activePopen)
	{
		u32 toWrite;
		if (const char* endl = g_activePopenResult->First('\n', g_activePopenReadPos))
		{
			u32 lineLen = u32(endl - g_activePopenResult->data) - g_activePopenReadPos + 1;
			toWrite = Min(u32(count - 2), lineLen);
		}
		else
			toWrite = Min(u32(count - 2), g_activePopenResult->count - g_activePopenReadPos);
		memcpy(str, g_activePopenResult->data + g_activePopenReadPos, toWrite);
		str[toWrite] = 0;
		g_activePopenReadPos += toWrite;
		return str;
	}

	auto res = TRUE_WRAPPER(fgets)(str, count, stream);
	return res;
}

UBA_EXPORT int UBA_WRAPPER(pclose)(FILE* stream)
{
	UBA_INIT_DETOUR(pclose, stream);
	if (stream == g_activePopen)
	{
		DEBUG_LOG_DETOURED("pclose", "%p", stream);
		delete g_activePopenResult;
		g_activePopenResult = nullptr;
		g_activePopen = nullptr;
		return 0;
	}

	DEBUG_LOG_TRUE("pclose", "%p", stream);
	return TRUE_WRAPPER(pclose)(stream);
}

UBA_EXPORT int UBA_WRAPPER(kill)(pid_t pid, int sig)
{
	UBA_INIT_DETOUR(kill, pid, sig);
	int res = TRUE_WRAPPER(kill)(pid, sig);
	DEBUG_LOG_TRUE("kill", "%i %i -> %i", pid, sig, res);
	return res;
}

UBA_EXPORT void UBA_WRAPPER(exit)(int status)
{
	DEBUG_LOG_TRUE("exit", "(%i)", status);
	TRUE_WRAPPER(exit)(status);
}

UBA_EXPORT void UBA_WRAPPER(_exit)(int status)
{
	DEBUG_LOG_TRUE("_exit", "(%i)", status);
	Deinit();
	CloseCom();
	TRUE_WRAPPER(_exit)(status);
}

UBA_EXPORT void UBA_WRAPPER(_Exit)(int status)
{
	DEBUG_LOG_TRUE("_Exit", "(%i)", status);
	Deinit();
	CloseCom();
	TRUE_WRAPPER(_Exit)(status);
}

UBA_EXPORT int UBA_WRAPPER(system)(const char* command)
{
	UBA_INIT_DETOUR(system, command);
	DEBUG_LOG_TRUE("system", "");
	return TRUE_WRAPPER(system)(command);
}

#if PLATFORM_LINUX

UBA_EXPORT int UBA_WRAPPER(openat64)(int dirfd, const char* pathname, int flags, ...)
{
	va_list args;
	va_start(args, flags);
	int mode = va_arg(args, int);
	va_end(args);
	UBA_INIT_DETOUR(openat64, dirfd, pathname, flags, mode);

	StringBuffer<> temp;
	if (const char* resolvedPath = ResolvePath(dirfd, pathname, temp))
		return Shared_open("openat64", resolvedPath, flags, mode, [](const char* realFile, int flags, int mode) { return Real_open(errno,realFile, flags, mode); });

	UBA_ASSERTF(!g_runningRemote, TC("openat64: dirfd=%d not in g_fileHandles, path='%s' cwd='%s' flags=0x%x"), dirfd, pathname, g_virtualWorkingDir.data, flags);

	int res = Real_openat(errno,dirfd, pathname, flags, mode);
	DEBUG_LOG_TRUE("openat64", "(NODETOUR) %s -> %i", pathname, res);
	if (res)
		TrackInput(ToView(pathname));
	return res;
}

UBA_EXPORT int UBA_WRAPPER(dup3)(int oldfd, int newfd, int flags)
{
	UBA_INIT_DETOUR(dup3, oldfd, newfd, flags);
	if (oldfd != newfd)
		Shared_PreFinalizeForOverwrite("dup3", newfd, LdpreloadSnapshot);
	auto res = Real_dup3(errno, oldfd, newfd, flags);
	if (res != -1)
		DuplicateFd(oldfd, newfd, /*newCloexec*/(flags & O_CLOEXEC) != 0);
	return res;
}

UBA_EXPORT pid_t UBA_WRAPPER(clone)(int (*__fn) (void *__arg), void *__child_stack, int __flags, void *__arg, ...)
{
	DEBUG_LOG_TRUE("clone", "");
	UBA_ASSERT(false);
	return -1;
}

UBA_EXPORT char* UBA_WRAPPER(get_current_dir_name)(void)
{
	UBA_INIT_DETOUR(get_current_dir_name);

	// GNU extension: malloc'd cwd string. Return virtual cwd (kernel getcwd is wrong remote — see getcwd above).
	u32 len = g_virtualWorkingDir.count;
	if (len > 1 && g_virtualWorkingDir.data[len - 1] == '/')
		--len;
	char* buf = (char*)malloc(len + 1);
	memcpy(buf, g_virtualWorkingDir.data, len);
	buf[len] = 0;
	DEBUG_LOG_DETOURED("get_current_dir_name", "-> %s", buf);
	return buf;
}

#endif

#if PLATFORM_MAC
#define DETOURED_FUNCTION(func) \
	DYLD_INTERPOSE(UBA_WRAPPER(func), func);
DETOURED_FUNCTIONS
#undef DETOURED_FUNCTION
#endif

namespace uba
{
	int GetProcessExecutablePath(tchar* Path, u32 PathSize)
	{
#if PLATFORM_LINUX
		auto res = Real_readlink(errno, "/proc/self/exe", Path, PathSize);
		if (res != -1)
			Path[res] = 0;
		return res;
#elif PLATFORM_MAC
		if (_NSGetExecutablePath(Path, &PathSize) == 0)
			return strlen(Path);
		return -1;
#endif
	}

	void PreInit(const char* logFile, bool fromExec)
	{
		InitFileHandles();

		g_pid = getpid();
		g_isUsingLocalBinary = !g_runningRemote;

		SuppressDetourScope s;

		const char* tmpdir = getenv("TMPDIR");
		UBA_ASSERT(tmpdir);
		g_systemTemp.Append(tmpdir);

		#if PLATFORM_LINUX
			#define DETOURED_FUNCTION(func) \
			if (!(TRUE_WRAPPER(func) = (Symbol_##func*)dlsym(RTLD_NEXT, #func))) {}
			DETOURED_FUNCTIONS
			#undef DETOURED_FUNCTION
		#endif

		g_parentInNonInitNs = CheckParentInNonInitPidNs(); // helper uses TRUE_WRAPPER(open/read/close); must run after population

		InstallCrashHandlers(); // Crash handler must be installed after TRUE_WRAPPER population, before user code, so non-sigaction processes still get crash logs.

		#if UBA_DEBUG_LOG_ENABLED
		if (g_logToScreen)
			g_debugFile = (FileHandle)open("/dev/tty", O_WRONLY);
		else if (logFile)
		{
			g_debugLogPath.Clear().Append(logFile);
			g_debugFile = (FileHandle)Real_open(errno,logFile, (fromExec?O_APPEND:O_CREAT|O_TRUNC) | O_WRONLY | O_CLOEXEC, S_IRUSR | S_IWUSR);
		}
		#endif

		StringBuffer<> exePath;
		exePath.count = GetProcessExecutablePath(exePath.data, exePath.capacity);
		UBA_ASSERTF(exePath.count > 0, "exePath.count == 0");
		char* lastSlash = strrchr(exePath.data, '/');
		UBA_ASSERTF(lastSlash, "no slash found in %s", exePath.data);
		exePath.Resize(lastSlash - exePath.data);
		FixPath(g_exeDir, exePath.data);
		g_exeDir.EnsureEndsWithSlash();

#if UBA_SUPPORTS_GO
		if (IsGoBinary("/proc/self/exe"))
		{
			DEBUG_LOG(TC("Detected Go binary — patching internal/runtime/syscall.Syscall6"));
			PatchGoSyscalls();
			PatchGoExit();
		}
#endif
	}

// GoHandlesPath + GoHandle* bodies now live in UbaDetoursGo.cpp alongside their only callers in the syscall6 switch.

	void LogHeader()
	{
		#if UBA_DEBUG_LOG_ENABLED
		static StringBuffer<128 * 1024> buf;
		buf.Clear();
		#if PLATFORM_LINUX
		int fd = Real_open(errno, "/proc/self/cmdline", O_RDONLY, 0);
		if (fd != -1)
		{
			auto bufSize = read(fd, buf.data, buf.capacity);
			if (bufSize != -1)
			{
				char* it = buf.data;
				while (*it)
				{
					it += strlen(it);
					*it = ' ';
					++it;
				}
				buf.Resize(it - buf.data - 1);
			}
			Real_close(errno,fd);
		}
		#else
		char** argv = *_NSGetArgv();
		for (int i=0, e=*_NSGetArgc(); i<e; ++i)
		{
			if (i != 0)
				buf.Append(' ');
			buf.Append(argv[i]);
		}
		#endif
		LogHeader(buf);
		#endif
	}

	void Init() { Shared_Init(environ); }

	void Deinit()
	{
		if (!g_isInitialized)
			return;

		u64 startTime = GetTime();

		g_isInitialized = false;
		g_isDetouring = false;

		if (!g_isCancelled)
		{
			// Flush stdio before fstat/close — freopen'd stdout/stderr (wayland_scanner) never fclose; libc's atexit flush runs after us and the trailing buffer is lost.
			fflush(nullptr);

			SCOPED_WRITE_LOCK(g_fileHandlesLock, lock);
			for (auto& kv : g_fileHandles)
			{
				int fd = kv.first;
				DetouredHandle& h = kv.second;
				FileObject* fo = h.fileObject;
				if (fo->closeId)
				{
					FileInfo& fi = *fo->fileInfo;
					SharedMemoryHandle memoryHandle;
					struct stat attr;
					int res = Real_fstat(errno,fd, &attr);
					UBA_ASSERTF(res == 0, "fstat failed for file %s", fi.name);(void)res;
					u64 fileSize = attr.st_size;
					u64 fileLastWriteTime = FromTimeSpec(attr.st_mtimespec);
					u32 fileAttributes = attr.st_mode;
					const tchar* path = fi.name;
					Rpc_UpdateCloseHandle(path, fo->closeId, fo->deleteOnClose, fo->newName.c_str(), memoryHandle, fileSize, fileLastWriteTime, fileAttributes, true);
				}
				Real_close(errno,fd);
			}
		}

		SendInput();

		g_stats.detach.time += GetTime() - startTime;
		g_stats.detach.count = 1;

		// Don't wait on Exit: session may move on and reuse shared memory. If Exit ever writes files, revisit — parent could race the files.
		Shared_SendExit(0);

		#if UBA_DEBUG_LOG_ENABLED
		if (isLogging())
		{
			DEBUG_LOG("Finished");
			int debugFile = (int)g_debugFile;
			g_debugFile = InvalidFileHandle;
			Real_close(errno,debugFile);
		}
		#endif
	}

	#if UBA_DEBUG_LOG_ENABLED
	void WriteDebugInner(const void* data, u32 dataLen)
	{
		int t = errno;

		#if UBA_DETOUR_DEBUG && PLATFORM_LINUX
		Real_write(errno,g_debugFile, data, dataLen);
		#else
		write(g_debugFile, data, dataLen);
		#endif
		errno = t;
	}
	void FlushDebugLog()
	{
		if (isLogging())
			fsync(g_debugFile);
	}
	#endif

	ANALYSIS_NORETURN void UbaAssert(const tchar* text, const char* file, u32 line, const char* expr, bool allowTerminate, u32 terminateCode, void* context, u32 skipCallstackCount)
	{
		SuppressDetourScope s;
		static CriticalSection cs;
		ScopedCriticalSection scs(cs);

		#if UBA_DEBUG_LOG_ENABLED
		FlushDebugLog();
		#endif

		static auto& sb =  *new StringBuffer<8*1024>;
		WriteAssertInfo(sb.Clear(), text, file, line, expr, context);

		// Forked-not-exec'd child shares comm channel with parent — RPC here corrupts parent's stream and marks it exited host-side. Stderr only.
		bool isForkedChild = (getpid() != g_pid);

		if (isForkedChild)
		{
			int fd = 1; // Unfortunately unit tests on farm reports fail (even though test exists) if output on stderr. and tests always run in shipping
			const char prefix[] = "[UBA-FORK-CHILD-CRASH] ";
			write(fd, prefix, sizeof(prefix) - 1);
			write(fd, sb.data, (size_t)sb.count);
			write(fd, "\n", 1);
		}

		if (g_isInitialized && !isForkedChild)
		{
			Rpc_ResolveCallstack(sb, 3 + skipCallstackCount, context);
			Rpc_WriteLog(sb.data, sb.count, true, true);

			if (!allowTerminate)
				return;

			g_stats.detoursMemory = g_memoryBlock.writtenSize;

			if (TakeLockForRpc())
			{
				RPC_MESSAGE_NO_LOCK(Exit, log);
				writer.WriteU32(terminateCode); // Exit code
				writer.WriteString(""); // Log name
				g_stats.Write(writer);
				g_kernelStats.Write(writer);
				writer.Flush(false);
				g_communicationLock.Leave();
			}
		}
		else
		{
			#if UBA_DEBUG_LOG_ENABLED
			g_debugFile = (FileHandle)open("/dev/tty", O_WRONLY);
			sb.Append('\n');
			WriteDebug(sb.data, sb.count);
			#endif
		}

		CloseCom();
		TRUE_WRAPPER(_exit)(int(terminateCode));
	}

	LastErrorToText::LastErrorToText(u32 lastError)
	{
		Append(strerror(int(lastError)));
	}
}

extern "C" UBA_EXPORT bool UbaRequestNextProcess2(u32 prevExitCode, char* outArguments, u32 outArgumentsCapacity, u32 timeOutMs, bool* outShouldExit)
{
	#if UBA_DEBUG_LOG_ENABLED
	FlushDebugLog();
	#endif

	*outArguments = 0;
	bool newProcess;
	DirTableSize directoryTableSize;
	{
		RPC_MESSAGE(GetNextProcess, log);
		writer.WriteU32(prevExitCode);
		writer.WriteU32(timeOutMs);
		g_stats.Write(writer);
		g_kernelStats.Write(writer);

		BinaryReader reader = writer.Flush();
		newProcess = reader.ReadBool();
		*outShouldExit = reader.ReadBool();
		if (newProcess)
		{
			reader.ReadString(outArguments, outArgumentsCapacity);
			reader.SkipString(); // workingDir
			reader.SkipString(); // description
			reader.ReadString(g_logName.Clear());
			directoryTableSize = FromU64(reader.ReadU64());
		}
	}

	g_kernelStats = {};
	g_stats = {};

	if (newProcess)
	{
		#if UBA_DEBUG_LOG_ENABLED
		SuppressDetourScope scope;
		int debugFile = g_debugFile;
		g_debugFile = InvalidFileHandle;
		close(debugFile);
		debugFile = Real_open(errno,g_logName.data, O_CREAT | O_TRUNC | O_WRONLY | O_CLOEXEC, S_IRUSR | S_IWUSR);
		g_debugFile = (FileHandle)debugFile;
		LogHeader(ToView(outArguments));
		#endif

		auto& dirTable = *g_directoryTable;
		const u8* overlayMemory = dirTable.m_overlayMemory;
		dirTable.DeinitOverlay();
		dirTable.m_overlayMemory = overlayMemory;

		if (u32 overlaySize = directoryTableSize.overlay)
		{
			directoryTableSize.overlay = 0; // Want to just parse normal directory first
			dirTable.ParseDirectoryTableNoLock(directoryTableSize);
			dirTable.InitOverlay(overlayMemory, overlaySize);
		}
	}

	Rpc_UpdateTables();
	return newProcess;
}

extern "C" UBA_EXPORT bool UbaRequestNextProcess(u32 prevExitCode, char* outArguments, u32 outArgumentsCapacity)
{
	bool shouldExit;
	return UbaRequestNextProcess2(prevExitCode, outArguments, outArgumentsCapacity, 0, &shouldExit);
}
