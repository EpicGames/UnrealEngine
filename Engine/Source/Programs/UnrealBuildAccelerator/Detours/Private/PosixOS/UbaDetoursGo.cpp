// Copyright Epic Games, Inc. All Rights Reserved.

// UbaDetoursGo.cpp
//
// In-memory patching of Go binaries on Linux.
//
// Go's runtime makes syscalls directly via the SYSCALL instruction, bypassing
// glibc entirely, so LD_PRELOAD injection never reaches it.  Instead we patch
// internal/runtime/syscall.Syscall6 — the single naked assembly stub that all
// regular (non-raw) Go syscalls bottom out in:
//
//   syscall.Syscall  → runtime.entersyscall → internal/runtime/syscall.Syscall6
//   syscall.Syscall6 → runtime.entersyscall → internal/runtime/syscall.Syscall6
//
// We do NOT patch syscall.RawSyscall / syscall.RawSyscall6.  Those bypass
// entersyscall, so our C bridge would run on the goroutine stack without the
// goroutine being in _Gsyscall state.  The Go GC is still allowed to run
// (asynchronous preemption via SIGURG), and when it scans the goroutine stack
// it cannot understand our C frame, leading to data corruption and crashes.
//
// File I/O (openat, newfstatat, statx, unlinkat) is always routed through
// syscall.Syscall / syscall.Syscall6 because file operations can block;
// syscall.RawSyscall is only used for non-blocking queries like getpid,
// gettimeofday, fcntl(F_GETFL), etc., which we do not need to intercept.
//
// Because internal/runtime/syscall.Syscall6 is called AFTER entersyscall, the
// goroutine is in _Gsyscall state when our bridge runs.  The GC's stack scanner
// only scans frames above the syscallsp saved by entersyscall, so our bridge
// frame (which lives below syscallsp) is never scanned — GC-safe.
//
// Known gaps (syscalls that bypass our patch):
//   syscall.RawSyscall / RawSyscall6
//     — getpid, gettimeofday, fcntl(F_GETFL), etc.  Not file I/O.
//   runtime.open / runtime.read / runtime.write1 / runtime.closefd
//     — runtime-internal file access (e.g. /proc/self/maps, trace output);
//       use direct SYSCALL instructions.
//   rawVforkSyscall
//     — clone() for child processes; separate assembly, not Syscall6.
//   rawSyscallNoError
//     — getpid/getppid style; separate stub.
//
// Calling convention (Go 1.17+ register ABI on amd64):
//
//   RawSyscall (trap, a1, a2, a3 uintptr) -> (r1, r2, errno)
//     In:   AX=trap  BX=a1  CX=a2  DI=a3
//     Out:  AX=r1    BX=r2  CX=errno
//
//   RawSyscall6 (trap, a1..a6 uintptr) -> (r1, r2, errno)
//     In:   AX=trap  BX=a1  CX=a2  DI=a3  SI=a4  R8=a5  R9=a6
//     Out:  AX=r1    BX=r2  CX=errno
//
// This differs from the C System-V AMD64 ABI (args in RDI, RSI, RDX, RCX,
// R8, R9), so assembly bridges translate between the two conventions.
//
// Trampoline layout:
//   The first HOOK_PATCH_SIZE (14) bytes of the target function are replaced
//   with an indirect JMP embedding the absolute 64-bit address of our bridge:
//
//     FF 25 00 00 00 00   jmp [rip+0]
//     XX XX XX XX XX XX XX XX  <- 64-bit bridge address
//
//   This instruction clobbers no registers.  A separate executable page holds
//   the displaced original bytes followed by a jump back to original+14, so
//   calling the trampoline stub is equivalent to calling the original function.
//
//   RawSyscall and RawSyscall6 are hand-written assembly whose prologues
//   consist of register-to-register MOVs with no RIP-relative operands, so
//   the displaced bytes are safe to relocate.  If hooks are ever added to
//   other functions whose prologues contain CALL or RIP-relative LEA
//   instructions, a proper x86-64 disassembler (e.g. Zydis) will be needed
//   to fixup the displaced offsets.


#include "UbaPlatform.h"
#include "UbaDetoursGoPosix.h"

#if UBA_SUPPORTS_GO

#include "UbaDetoursShared.h"
#include "UbaDetoursSharedPosix.h"
#include "UbaLinuxRawSyscall.h"
#include <atomic>
#include <elf.h>
#include <errno.h>
#include <fcntl.h>
#include <pthread.h>
#include <signal.h>
#include <sys/mman.h>
#include <sys/syscall.h>
#include <unistd.h>

// Raw futex op constants (avoids including <linux/futex.h> which can conflict).
#ifndef FUTEX_WAIT
#define FUTEX_WAIT 0
#define FUTEX_WAKE 1
#endif

// CloseCom is defined in global scope in UbaDetoursMainPosix.cpp.
void CloseCom();

namespace uba
{

void Deinit();

extern u8* g_messageMappingMem;
extern bool g_runningRemote;
extern bool g_isDetouring;
extern u32 g_virtualUid;
extern u32 g_virtualGid;

// ─────────────────────────────────────────────────────────────────────────────
// Detection
// ─────────────────────────────────────────────────────────────────────────────

// Returns true if the ELF binary at `path` is a Go binary.
// Go binaries always contain .gopclntab (needed for goroutine stack unwinding
// at runtime); it is present even when stripped with -s -w.
bool IsGoBinary(const char* path)
{
	int fd = open(path, O_RDONLY | O_CLOEXEC);
	if (fd < 0)
		return false;

	bool isGo = false;

	Elf64_Ehdr ehdr;
	if (pread(fd, &ehdr, sizeof(ehdr), 0) == sizeof(ehdr) &&
		memcmp(ehdr.e_ident, ELFMAG, SELFMAG) == 0 &&
		ehdr.e_machine == EM_X86_64)
	{
		size_t shsize = (size_t)ehdr.e_shentsize * ehdr.e_shnum;
		Elf64_Shdr* shdrs = (Elf64_Shdr*)malloc(shsize);
		if (shdrs && pread(fd, shdrs, shsize, (off_t)ehdr.e_shoff) == (ssize_t)shsize)
		{
			Elf64_Shdr& shstrtab = shdrs[ehdr.e_shstrndx];
			char* names = (char*)malloc(shstrtab.sh_size);
			if (names && pread(fd, names, shstrtab.sh_size, (off_t)shstrtab.sh_offset) == (ssize_t)shstrtab.sh_size)
			{
				for (int i = 0; i < ehdr.e_shnum && !isGo; ++i)
					if (strcmp(names + shdrs[i].sh_name, ".gopclntab") == 0)
						isGo = true;
			}
			free(names);
		}
		free(shdrs);
	}

	close(fd);
	return isGo;
}

// ─────────────────────────────────────────────────────────────────────────────
// Symbol lookup — ELF symbol table (.symtab)
// ─────────────────────────────────────────────────────────────────────────────

// Returns the in-memory virtual address of the named Go function, or nullptr.
// Reads /proc/self/exe (must be called from within the target process).
// Works for non-stripped Go binaries (compiled without -ldflags="-s").
// Go ELF symbol names use the full package path, e.g. "syscall.RawSyscall".
void* FindGoSymbol(const char* name)
{
	int fd = open("/proc/self/exe", O_RDONLY | O_CLOEXEC);
	if (fd < 0)
		return nullptr;

	void* result = nullptr;

	Elf64_Ehdr ehdr;
	if (pread(fd, &ehdr, sizeof(ehdr), 0) != sizeof(ehdr) ||
		memcmp(ehdr.e_ident, ELFMAG, SELFMAG) != 0)
	{
		close(fd);
		return nullptr;
	}

	size_t shsize = (size_t)ehdr.e_shentsize * ehdr.e_shnum;
	Elf64_Shdr* shdrs = (Elf64_Shdr*)malloc(shsize);
	if (!shdrs) { close(fd); return nullptr; }
	pread(fd, shdrs, shsize, (off_t)ehdr.e_shoff);

	Elf64_Shdr& shstrtab = shdrs[ehdr.e_shstrndx];
	char* shnames = (char*)malloc(shstrtab.sh_size);
	if (shnames)
		pread(fd, shnames, shstrtab.sh_size, (off_t)shstrtab.sh_offset);

	Elf64_Shdr* symHdr = nullptr;
	Elf64_Shdr* strHdr = nullptr;
	for (int i = 0; i < ehdr.e_shnum && shnames; ++i)
	{
		if (shdrs[i].sh_type == SHT_SYMTAB)
			symHdr = &shdrs[i];
		if (shdrs[i].sh_type == SHT_STRTAB &&
			strcmp(shnames + shdrs[i].sh_name, ".strtab") == 0)
			strHdr = &shdrs[i];
	}

	if (symHdr && strHdr)
	{
		size_t nsyms = symHdr->sh_size / sizeof(Elf64_Sym);
		Elf64_Sym* syms = (Elf64_Sym*)malloc(symHdr->sh_size);
		char*      strs = (char*)malloc(strHdr->sh_size);

		if (syms && strs)
		{
			pread(fd, syms, symHdr->sh_size, (off_t)symHdr->sh_offset);
			pread(fd, strs, strHdr->sh_size, (off_t)strHdr->sh_offset);

			for (size_t i = 0; i < nsyms && !result; ++i)
			{
				auto typ = ELF64_ST_TYPE(syms[i].st_info);
				if ((typ == STT_FUNC || typ == STT_NOTYPE) &&
					syms[i].st_value != 0 &&
					strcmp(strs + syms[i].st_name, name) == 0)
					result = (void*)(uintptr_t)syms[i].st_value;
			}
		}
		free(syms);
		free(strs);
	}

	free(shnames);
	free(shdrs);
	close(fd);
	return result;
}

// ─────────────────────────────────────────────────────────────────────────────
// Symbol lookup — .gopclntab (stripped binary fallback)
// ─────────────────────────────────────────────────────────────────────────────
//
// .gopclntab is retained even under -s -w stripping because the Go runtime
// uses it for goroutine stack unwinding.
//
// Header layout for Go 1.18+ (magic 0xFFFFFBFF or 0xFFFFFAFF):
//   u32 magic
//   u8  pad[2]
//   u8  minLC       (instruction size quantum; 1 on x86-64)
//   u8  ptrSize     (8 on 64-bit)
//   u64 nfunc
//   u64 nfiles
//   u64 textStart
//   u64 funcnameOffset   offset within pclntab of the function-name string table
//   u64 cuOffset
//   u64 filetabOffset
//   u64 pctabOffset
//   u64 pclnOffset       offset of the func-table entries within pclntab
//
// Each func-table entry at (pclntab + pclnOffset + i*16):
//   u64 entryPC    virtual address of the function
//   u64 funcOff    offset within pclntab of the _func descriptor
//
// _func descriptor starts with:
//   u32 entryOff   (relative to textStart for 1.20+)
//   i32 nameOff    offset within funcname table of the null-terminated name

#pragma pack(push, 1)
struct GoPclntab118Header
{
	u32 magic;
	u8  pad[2];
	u8  minLC;
	u8  ptrSize;
	u64 nfunc;
	u64 nfiles;
	u64 textStart;
	u64 funcnameOffset;
	u64 cuOffset;
	u64 filetabOffset;
	u64 pctabOffset;
	u64 pclnOffset;
};
#pragma pack(pop)
static_assert(sizeof(GoPclntab118Header) == 72, "GoPclntab118Header size mismatch");

void* FindGoSymbolInPclntab(const char* name)
{
	int fd = open("/proc/self/exe", O_RDONLY | O_CLOEXEC);
	if (fd < 0) return nullptr;

	void* result = nullptr;

	Elf64_Ehdr ehdr;
	if (pread(fd, &ehdr, sizeof(ehdr), 0) != sizeof(ehdr) ||
		memcmp(ehdr.e_ident, ELFMAG, SELFMAG) != 0)
	{ close(fd); return nullptr; }

	size_t shsize = (size_t)ehdr.e_shentsize * ehdr.e_shnum;
	Elf64_Shdr* shdrs = (Elf64_Shdr*)malloc(shsize);
	if (!shdrs) { close(fd); return nullptr; }
	pread(fd, shdrs, shsize, (off_t)ehdr.e_shoff);

	Elf64_Shdr& shstrtab = shdrs[ehdr.e_shstrndx];
	char* shnames = (char*)malloc(shstrtab.sh_size);
	if (shnames)
		pread(fd, shnames, shstrtab.sh_size, (off_t)shstrtab.sh_offset);

	for (int i = 0; i < ehdr.e_shnum && !result && shnames; ++i)
	{
		if (strcmp(shnames + shdrs[i].sh_name, ".gopclntab") != 0)
			continue;

		u8* tab = (u8*)malloc(shdrs[i].sh_size);
		if (!tab)
			break;
		if (pread(fd, tab, shdrs[i].sh_size, (off_t)shdrs[i].sh_offset) != (ssize_t)shdrs[i].sh_size)
		{
			free(tab);
			break;
		}

		u32 magic;
		memcpy(&magic, tab, 4);

		// Go pclntab magic: first byte determines version, next 3 bytes are 0xFF.
		// As stored in the file (little-endian read): 0xFFFFFF<ver>.
		//   ver=0xFB → Go 1.18: entries are (u64 entryPC, u64 funcOff), abs PC.
		//   ver=0xFA → Go 1.20: entries are (u32 entryOff, u32 funcOff), relative.
		//   ver=0xF1 → Go 1.22/1.24: same entry layout as 1.20.
		//   (1.16=0xFC, 1.12=0xFD differ structurally and are not handled.)
		//
		// In all handled formats the _func descriptor starts with:
		//   u32 entryOff (skip), i32 nameOff
		const u8 ver = (u8)(magic & 0xFF); // first byte in file = version byte
		if (ver == 0xFBu)
		{
			GoPclntab118Header hdr;
			memcpy(&hdr, tab, sizeof(hdr));

			const char* funcnames = (const char*)(tab + hdr.funcnameOffset);
			const u8*   entries   = tab + hdr.pclnOffset;

			for (u64 j = 0; j < hdr.nfunc && !result; ++j)
			{
				u64 entryPC;
				u64 funcOff;
				memcpy(&entryPC, entries + j * 16,     8);
				memcpy(&funcOff, entries + j * 16 + 8, 8);

				u32 nameOff;
				memcpy(&nameOff, tab + funcOff + 4, 4);

				if (strcmp(funcnames + nameOff, name) == 0)
					result = (void*)(uintptr_t)entryPC;
			}
		}
		else if (ver == 0xFAu || ver == 0xF1u || ver == 0xF2u || ver == 0xF3u)
		{
			// Go 1.20+: 8-byte entries, PC relative to textStart.
			GoPclntab118Header hdr;
			memcpy(&hdr, tab, sizeof(hdr));

			const char* funcnames = (const char*)(tab + hdr.funcnameOffset);
			const u8*   entries   = tab + hdr.pclnOffset;

			for (u64 j = 0; j < hdr.nfunc && !result; ++j)
			{
				u32 entryOff;
				u32 funcOff;
				memcpy(&entryOff, entries + j * 8,     4);
				memcpy(&funcOff,  entries + j * 8 + 4, 4);

				u32 nameOff;
				memcpy(&nameOff, tab + funcOff + 4, 4);

				if (strcmp(funcnames + nameOff, name) == 0)
					result = (void*)(uintptr_t)(hdr.textStart + entryOff);
			}
		}

		free(tab);
	}

	free(shnames);
	free(shdrs);
	close(fd);
	return result;
}

// ─────────────────────────────────────────────────────────────────────────────
// Inline-hook (no trampoline — the bridge makes the SYSCALL itself)
// ─────────────────────────────────────────────────────────────────────────────
//
// We do NOT allocate a separate executable (RWX) page for a trampoline stub.
// On systems that enforce W^X (e.g. via SELinux execmem policy), mmap with
// PROT_EXEC on an anonymous mapping fails or silently drops the exec bit,
// which would cause a SEGV_ACCERR when the bridge tried to jump to the stub.
//
// Instead, hook_go_rawsyscall6 handles every syscall itself:
//   - UBA-intercepted syscalls  → GoSyscallHandlerInline → GoHandle* functions
//   - Everything else           → RawSyscallDirect (inline SYSCALL instruction)
// The bridge never jumps back to the original RawSyscall code.

static constexpr u32 HookPatchSize = 14; // size of the indirect-JMP patch

// Writes: FF 25 00 00 00 00 <8-byte-target>
// "jmp [rip+0]" — loads the 64-bit address in the following 8 bytes.
// Total: 14 bytes.  Clobbers no registers.
static void WriteAbsJmp(u8* dst, void* target)
{
	dst[0] = 0xFF; dst[1] = 0x25;
	dst[2] = dst[3] = dst[4] = dst[5] = 0; // disp32 = 0
	memcpy(dst + 6, &target, 8);
}

// Temporarily make `len` bytes at `addr` writable (RW, not RWX) so we can
// write the patch.  Caller must call MakeExecutable afterwards.
static int MakeWritable(void* addr, u32 len)
{
	long ps    = sysconf(_SC_PAGESIZE);
	void* page = (void*)((uintptr_t)addr & ~(uintptr_t)(ps - 1));
	size_t span = (u8*)addr + len - (u8*)page;
	return mprotect(page, span, PROT_READ | PROT_WRITE); // no EXEC — W^X safe
}

static int MakeExecutable(void* addr, u32 len)
{
	long ps    = sysconf(_SC_PAGESIZE);
	void* page = (void*)((uintptr_t)addr & ~(uintptr_t)(ps - 1));
	size_t span = (u8*)addr + len - (u8*)page;
	return mprotect(page, span, PROT_READ | PROT_EXEC);
}

// Installs a 14-byte indirect-JMP hook at targetFn redirecting to replacement.
// No trampoline is created — the replacement is responsible for emulating
// whatever the original did. Mechanism is generic (not Go-specific); only
// caller today is PatchGoSyscalls, which redirects Go's Syscall6 stubs.
static bool HookFunction(void* targetFn, void* replacement)
{
	u8 patch[HookPatchSize];
	WriteAbsJmp(patch, replacement);

	if (MakeWritable(targetFn, HookPatchSize) < 0)
		return false;

	memcpy(targetFn, patch, HookPatchSize);
	__builtin___clear_cache((char*)targetFn, (char*)targetFn + HookPatchSize);
	MakeExecutable(targetFn, HookPatchSize);
	return true;
}

// Raw syscall helper RawSyscallDirect / RawSyscallResult — see UbaDetoursGoPosix.h.
// Defined inline in the header so it can also be used from UbaDetoursFunctionsPosix.cpp.

// ─────────────────────────────────────────────────────────────────────────────
// C handler — called from the assembly bridge below
// ─────────────────────────────────────────────────────────────────────────────
//
// GoSyscallHandlerInline is called on goroutine OS threads, which may be created
// by Go's newosproc() via clone() rather than pthread_create().  Such threads
// do NOT have the glibc DTV (dynamic thread vector) set up for DSOs loaded
// after the initial set, so __tls_get_addr (used by thread_local variables
// such as glibc's errno) would SIGSEGV.
//
// UBA's own state is now TLS-free (DisallowDetourDepth uses a tid-keyed atomic
// array), so SuppressDetourScope, DEBUG_LOG, and the Shared_* handlers can run
// on goroutine threads.  We still avoid touching glibc errno / __tls_get_addr —
// raw SYSCALLs and capturedErr out-params are used in GoHandle* wrappers.

// SuppressDetourScope is now TLS-free (see UbaDetoursShared.cpp): the disallow
// depth lives in a tiny tid-keyed atomic array, so the same scope type works on
// both glibc-TCB threads and goroutine OS threads.  Previously this file had a
// parallel GoSuppressDetourScope + IsGoThreadSuppressed() hack to avoid touching
// __tls_get_addr on goroutine threads; that is gone now that the root TLS
// access is gone.

// Write a null-terminated string to the debug log using a raw SYS_write.
// Safe on any OS thread — no glibc, no TLS.
static inline void RawWriteLog(const char* str)
{
#if UBA_DEBUG_LOG_ENABLED
	if (g_debugFile == InvalidFileHandle || !str) return;
	size_t len = 0;
	while (str[len]) ++len;
	if (!len) return;
	register long rax asm("rax") = 1; // SYS_write
	register long rdi asm("rdi") = (long)g_debugFile;
	register const char* rsi asm("rsi") = str;
	register long rdx asm("rdx") = (long)len;
	asm volatile("syscall" : "+a"(rax) : "D"(rdi), "S"(rsi), "d"(rdx) : "rcx", "r11", "memory");
#endif
}

// Write an unsigned long as decimal to the debug log. No glibc, no TLS.
static inline void RawWriteLogUlong(unsigned long v)
{
#if UBA_DEBUG_LOG_ENABLED
	if (g_debugFile == InvalidFileHandle) return;
	char buf[24];
	int len = 0;
	if (v == 0)
	{
		buf[0] = '0'; len = 1;
	}
	else
	{
		char tmp[24]; int n = 0;
		while (v) { tmp[n++] = '0' + (char)(v % 10); v /= 10; }
		for (int i = 0; i < n; ++i) buf[i] = tmp[n - 1 - i];
		len = n;
	}
	register long rax asm("rax") = 1;
	register long rdi asm("rdi") = (long)g_debugFile;
	register const char* rsi asm("rsi") = buf;
	register long rdx asm("rdx") = len;
	asm volatile("syscall" : "+a"(rax) : "D"(rdi), "S"(rsi), "d"(rdx) : "rcx", "r11", "memory");
#endif
}

// ─────────────────────────────────────────────────────────────────────────────
// Inline-bridge primitives — mirror DetoursStatic/UbaStaticStub.S step-by-step.
// ─────────────────────────────────────────────────────────────────────────────
//
// We run UBA handlers DIRECTLY on the goroutine OS thread.  No worker pthread,
// no futex handshake.  That requires three things, all of which the static
// detour stub proved out on AOSP soong_zip / merge_zips / link / compile etc.:
//
//   1. 256 KB alt-stack — UBA's Shared_* descent (StringBuffer<MaxPath>,
//      BLAKE3, ToStringKey, PopulateDirectoryRecursive's 192 KB u32 array)
//      blows past Go's 8 KB system stack.
//   2. All-signals-blocked window around the bridge — Go's SIGURG preemption
//      handler synthesizes a SIGSEGV if it inspects an interrupted rsp that's
//      outside any goroutine stack (which is what an alt-stack rsp looks like).
//   3. A spinlock (g_goBridgeLock) serialising concurrent Ms onto the single
//      shared alt-stack buffer.  Acquired AFTER signals are blocked so a
//      signal handler can't re-enter the bridge and self-deadlock.
//
// Replacing the futex round-trip + worker context switch with sigprocmask +
// spinlock + alt-stack switch is a 3-5× win on the uncontended path, and the
// contended path is bounded by UBA RPC latency anyway.
//
// Raw futex helpers (RawFutexWait/Wake) below are still used by the unified
// exit path (cleanup pthread + signal handler + runtime.exit trampoline).

// 256 KB: matches kAltStackBytes in DetoursStatic/UbaStaticStubCore.cpp.
// PopulateDirectoryRecursive alone uses 192 KB on a directory-table miss.
static constexpr u64 kGoAltStackBytes = 256 * 1024;

extern "C" u8* g_goAltStackTop __attribute__((used)) = nullptr;
extern "C" u64 g_goFullMask     __attribute__((used)) = ~0ull;
extern "C" u32 g_goBridgeLock   __attribute__((used)) = 0;

// Raw futex helpers — use RawSyscallDirect so goroutine threads need no TLS.
// Used by the cleanup-thread / signal / runtime.exit path below.
static inline void RawFutexWait(std::atomic<int>* addr, int expected)
{
	RawSyscallDirect(SYS_futex, (long)(void*)addr, FUTEX_WAIT,
	                   (long)expected, 0, 0, 0);
}
static inline void RawFutexWake(std::atomic<int>* addr)
{
	RawSyscallDirect(SYS_futex, (long)(void*)addr, FUTEX_WAKE, 1, 0, 0, 0);
}

// ─────────────────────────────────────────────────────────────────────────────
// Unified exit path — serialise runtime.exit trampoline and fatal signals
// through a single cleanup pthread so Deinit+CloseCom run exactly once with
// proper glibc TLS.
// ─────────────────────────────────────────────────────────────────────────────
//
// Two independent events can end a Go process:
//
//   A. runtime.exit(code) — patched to jump to a small trampoline that calls
//      GoTrampolineExit(code).  This happens on whatever thread called exit.
//   B. Fatal signal (SIGHUP / SIGTERM / SIGINT) — Go's default disposition is
//      dieFromSignal, which resets the handler and re-raises, killing the
//      process without going through runtime.exit.  We install our own handler
//      so UBA still gets a chance to send the Exit RPC.
//
// Both paths converge on an encoded exit request posted via a single atomic;
// a dedicated cleanup pthread (spawned with full glibc TLS) wakes on the
// futex, runs Deinit + CloseCom, and _exits the whole process.  Whichever
// path fires first wins the CAS; the other just pauses forever waiting for
// _exit to kill it.  This guarantees:
//
//   • Deinit (which flips g_isInitialized=false) runs exactly once, on a
//     thread with glibc TLS — so RPC_MESSAGE / DEBUG_LOG / SCOPED_WRITE_LOCK
//     all work.
//   • No race where signal interrupts trampoline's Deinit mid-flight and the
//     cleanup thread then short-circuits on g_isInitialized==false.
//   • Signal handler is async-signal-safe: only atomics, raw syscalls, and
//     RawWriteLog (which is a single SYS_write on a constant fd).
//
// Repro scenario for the signal path: esbuild spawned via python → node
// pipe.  When node closes its end of esbuild's stdout pipe, parent-death
// notification delivers SIGHUP to esbuild; without this handling, UBA never
// saw the child finish.
//
// sigaction is a thin rt_sigaction wrapper and is TLS-free, so installing
// the handler from a goroutine thread (first GoSyscallHandlerInline dispatch) is OK.

// Exit request encoding in a single 32-bit atomic:
//   0                         — idle
//   EXIT_KIND_SIGNAL | sig    — signal exit; exit code = 128 + sig
//   EXIT_KIND_NORMAL | code   — runtime.exit(code); exit code = code
static constexpr int EXIT_KIND_SIGNAL = 0x100;
static constexpr int EXIT_KIND_NORMAL = 0x200;
static constexpr int EXIT_PAYLOAD_MASK = 0xFF;

static std::atomic<bool> g_signalHandlersInstalled{false};
static std::atomic<int>  g_goExitRequest{0};         // 0 = idle, non-zero = encoded request
static std::atomic<bool> g_signalCleanupThreadStarted{false};

// Cleanup thread: blocks until a non-zero exit request is posted, then runs
// Deinit+CloseCom and terminates the process with the appropriate exit code.
// Runs with full glibc TLS (normal pthread) so Deinit's RPC path works.
static void* GoSignalCleanupThreadFunc(void*)
{
	// Block our fatal signals on this thread so a signal delivery can't
	// interrupt Deinit mid-flight (it would call our handler, which would
	// CAS-fail and then pause forever — deadlocking Deinit).  Process-directed
	// signals will route to some other thread instead.
	sigset_t mask;
	sigemptyset(&mask);
	sigaddset(&mask, SIGHUP);
	sigaddset(&mask, SIGTERM);
	sigaddset(&mask, SIGINT);
	pthread_sigmask(SIG_BLOCK, &mask, nullptr);

	for (;;)
	{
		int v = g_goExitRequest.load(std::memory_order_acquire);
		if (v != 0) break;
		RawFutexWait(&g_goExitRequest, 0);
	}

	int v = g_goExitRequest.load(std::memory_order_acquire);
	bool isSignal = (v & EXIT_KIND_SIGNAL) != 0;
	int payload = v & EXIT_PAYLOAD_MASK;

	DEBUG_LOG(TC("GoSignalCleanupThread: exit request=0x%x (isSignal=%d payload=%d)"),
		v, (int)isSignal, payload);

	// Deinit sends the Exit RPC (UBA session server marks this child done);
	// CloseCom unmaps the comm shared memory and closes the socket.
	Deinit();
	::CloseCom();

	int exitCode = isSignal ? (128 + payload) : payload;
	DEBUG_LOG(TC("GoSignalCleanupThread: cleanup done, _exit(%d)"), exitCode);
	// Terminate the whole process.  _exit() calls SYS_exit_group, which kills
	// every thread including anything currently pause()-ing below.  Never returns.
	_exit(exitCode);
}

// Called from the runtime.exit trampoline with the original exit code.
// Posts a normal-exit request and parks forever — the cleanup thread handles
// actual shutdown and _exit.  Declared extern "C" so the assembly trampoline
// can CALL it by absolute address.
extern "C" [[noreturn]] void GoTrampolineExit(int code)
{
	RawWriteLog("GoTrampolineExit: runtime.exit code=");
	RawWriteLogUlong((unsigned long)(unsigned int)code);
	RawWriteLog("\n");

	int posted = EXIT_KIND_NORMAL | (code & EXIT_PAYLOAD_MASK);
	int expected = 0;
	if (g_goExitRequest.compare_exchange_strong(expected, posted,
	        std::memory_order_acq_rel, std::memory_order_relaxed))
	{
		RawFutexWake(&g_goExitRequest);
	}
	// Whether we won the CAS or a signal beat us to it, the cleanup thread
	// will _exit the process.  Park until then.
	for (;;)
		RawSyscallDirect(SYS_pause, 0, 0, 0);
}

// Signal handler — runs on whatever thread receives the signal (typically a
// goroutine thread).  Touches no glibc TLS: only atomics, RawSyscallDirect,
// and RawFutexWake (which are all TLS-free).
extern "C" void UbaGoSignalHandler(int sig)
{
	int posted = EXIT_KIND_SIGNAL | (sig & EXIT_PAYLOAD_MASK);
	int expected = 0;
	if (g_goExitRequest.compare_exchange_strong(expected, posted,
	        std::memory_order_acq_rel, std::memory_order_relaxed))
	{
		RawWriteLog("UbaGoSignalHandler: signal=");
		RawWriteLogUlong((unsigned long)sig);
		RawWriteLog(", waking cleanup thread\n");
		RawFutexWake(&g_goExitRequest);
	}

	// Park this thread until the cleanup thread _exits the whole process.
	// pause() returns only on signal delivery, so loop.  Using the raw
	// pause() syscall keeps us off glibc TLS paths even when re-entered.
	for (;;)
		RawSyscallDirect(SYS_pause, 0, 0, 0);
}

static void InstallGoSignalHandlers()
{
	// Spawn the cleanup thread first — it must exist before the handler can
	// usefully wake it.
	pthread_attr_t attr;
	pthread_attr_init(&attr);
	pthread_attr_setdetachstate(&attr, PTHREAD_CREATE_DETACHED);
	pthread_attr_setstacksize(&attr, 8 * 1024 * 1024);
	pthread_t cleanupTid;
	if (pthread_create(&cleanupTid, &attr, GoSignalCleanupThreadFunc, nullptr) == 0)
	{
		g_signalCleanupThreadStarted.store(true, std::memory_order_release);
		DEBUG_LOG(TC("InstallGoSignalHandlers: cleanup thread started"));
	}
	else
	{
		DEBUG_LOG(TC("InstallGoSignalHandlers: pthread_create for cleanup thread failed"));
		pthread_attr_destroy(&attr);
		return;
	}
	pthread_attr_destroy(&attr);

	struct sigaction sa{};
	sa.sa_handler = UbaGoSignalHandler;
	sigemptyset(&sa.sa_mask);
	// Block our three fatal signals during handler so re-entry stays simple.
	sigaddset(&sa.sa_mask, SIGHUP);
	sigaddset(&sa.sa_mask, SIGTERM);
	sigaddset(&sa.sa_mask, SIGINT);
	sa.sa_flags = 0; // no SA_RESTART — we're not returning anyway

	sigaction(SIGHUP,  &sa, nullptr);
	sigaction(SIGTERM, &sa, nullptr);
	sigaction(SIGINT,  &sa, nullptr);
	// SIGPIPE is intentionally NOT installed: Go tracks SIGPIPE explicitly per
	// fd (EPIPE on non-stdio, die on stdio) and our override would break that.
	// SIGURG is used by Go for async preemption — do not touch.

	DEBUG_LOG(TC("InstallGoSignalHandlers: handlers installed for SIGHUP/SIGTERM/SIGINT"));
}

// Called from hook_go_rawsyscall6's slow path on the goroutine thread, after
// the asm bridge switched to the alt-stack and blocked all signals.  Reads
// trap/args from saved[] (layout matches DetoursStatic/UbaStaticStub.S step 7:
// pushq rax,rbx,rcx,rdx,rdi,rsi,rbp,r8,r9,r10,r11,r12,r13,r14,r15 with r15
// closest to rsp and rax furthest, plus a saved[0] alignment pad).
//
// Go's syscall.Syscall6 internal ABI: AX=trap, BX=a1, CX=a2, DI=a3, SI=a4,
} // namespace uba

// File-local externs into UbaDetoursFunctionsPosix.cpp (defined at global scope after `using namespace uba`). Used only by the GoHandle* bodies below.
int  LdpreloadStatx(const char* funcName, int dirfd, const char* pathname, int flags, unsigned int mask, struct statx* statxbuf);
int  Shared_DeleteFile(const char* funcName, const char* pathname);
bool LdpreloadSnapshot(int fd, uba::u64& outSize, uba::u64& outLastWritten, uba::u32& outAttributes);

namespace uba {

// Real dirfds (not AT_FDCWD) fall through to raw syscall — no virtual path recoverable from a fd-backed dir.
static inline bool GoHandlesPath(int dirfd, const char* path)
{
	if (!path || !*path) return false;
	if (path[0] == '/')  return true;
	return dirfd == AT_FDCWD;
}

static bool GoHandleOpenat(int dirfd, const char* path, int flags, int mode, long* result, int* errOut)
{
	if (!GoHandlesPath(dirfd, path))
		return false;
	int capturedErr = 0;
	*result = Shared_open("openat(go)", path, flags, mode,
		[&capturedErr](const char* realFile, int flg, int mod) -> int {
			RawSyscallResult r = RawSyscallDirect(SYS_openat, AT_FDCWD, (long)realFile, flg, mod);
			if (r.r1 < 0 && r.r1 >= -4095) { capturedErr = (int)(-r.r1); return -1; }
			return (int)r.r1;
		});
	*errOut = (*result < 0) ? capturedErr : 0;
	return true;
}

static bool GoHandleNewfstatat(int dirfd, const char* path, struct stat* buf, int flags, long* result, int* errOut)
{
	if (!GoHandlesPath(dirfd, path))
		return false;
	// Raw syscall in the lambda — on goroutine threads FS points at Go's M, __errno_location() corrupts Go runtime.
	int capturedErr = 0;
	*result = Shared_stat("newfstatat(go)", path, buf,
		[&capturedErr](const char* realPath, struct stat* outBuf) -> int {
			RawSyscallResult r = RawSyscallDirect(SYS_newfstatat, AT_FDCWD, (long)realPath, (long)outBuf, 0);
			if (r.r1 < 0 && r.r1 >= -4095) { capturedErr = (int)(-r.r1); return -1; }
			return (int)r.r1;
		});
	*errOut = (*result < 0) ? capturedErr : 0;
	return true;
}

static bool GoHandleStatx(int dirfd, const char* path, int flags, unsigned int mask, struct statx* statxbuf, long* result, int* errOut)
{
	if (!GoHandlesPath(dirfd, path))
		return false;
	// Force AT_FDCWD so LdpreloadStatx's FixPath resolves relatives against g_virtualWorkingDir.
	*result = LdpreloadStatx("statx(go)", AT_FDCWD, path, flags, mask, statxbuf);
	*errOut = (*result < 0) ? errno : 0;
	return true;
}

static bool GoHandleUnlinkat(int dirfd, const char* path, int flags, long* result, int* errOut)
{
	if (flags & AT_REMOVEDIR)
		return false; // directory removal — not a tracked file operation
	if (!GoHandlesPath(dirfd, path))
		return false;
	*result = Shared_DeleteFile("unlinkat(go)", path);
	*errOut = (*result < 0) ? errno : 0;
	return true;
}

static bool GoHandleClose(int fd, long* result, int* errOut)
{
	// Only intercept tracked fds — Go opens its own sockets/pipes/stdio that we never registered.
	{
		SCOPED_READ_LOCK(g_fileHandlesLock, lock);
		if (g_fileHandles.find(fd) == g_fileHandles.end())
			return false;
	}
	*result = Shared_close("close(go)", fd, [fd]() {
		RawSyscallResult r = RawSyscallDirect(SYS_close, fd, 0, 0);
		if (r.r1 < 0 && r.r1 >= -4095) { errno = (int)(-r.r1); return -1; }
		return (int)r.r1;
	}, LdpreloadSnapshot);
	*errOut = (*result < 0) ? errno : 0;
	return true;
}

static bool GoHandleGetdents64(int fd, void* buf, unsigned int count, long* result, int* errOut)
{
	// Only intercept UBA-tracked dir fds (Shared_open's $d placeholder branch); others pass through to kernel getdents64.
	int capturedErr = 0;
	long r = Shared_getdents64("getdents64(go)", fd, buf, count, &capturedErr);
	if (r < 0 && capturedErr == 0)
		return false; // fd not a UBA-tracked dir; caller falls through
	if (r < 0) { *result = -1; *errOut = capturedErr; }
	else       { *result = r;  *errOut = 0;           }
	return true;
}

// R8=a5, R9=a6.  Return: AX=r1, BX=r2, CX=err.  We write synthesized returns
// back into saved[15]/saved[14]/saved[13] when handled and return 1; the asm
// bridge restores them via the pop sequence and rets to Go's caller.  Returning
// 0 makes the bridge fall through to the existing direct-SYSCALL passthrough.
extern "C" u64 GoSyscallHandlerInline(u64* saved)
{
	// Install signal handlers on first dispatch.  Go's initsig has run by now
	// (we're servicing a syscall from user code), so our sigaction overwrites
	// Go's handler rather than being overwritten by it.  Safe on any thread,
	// including goroutines — sigaction is a thin syscall wrapper with no TLS.
	{
		bool expected = false;
		if (g_signalHandlersInstalled.compare_exchange_strong(expected, true))
			InstallGoSignalHandlers();
	}

	const u64 trap = saved[15];
	const u64 a1   = saved[14];
	const u64 a2   = saved[13];
	const u64 a3   = saved[11];
	const u64 a4   = saved[10];
	const u64 a5   = saved[ 8];
	// a6 (saved[7]) is unused by any current handler.

	// Log interesting syscalls using raw SYS_write (no glibc, no TLS).
	// logPath=true  → a2 is a path string (openat-family)
	// logPath=false → a1 is an fd (read/write/close/fstat/getdents64)
	const char* prefix = nullptr;
	bool logPath = false;
	switch (trap)
	{
	case SYS_read:       prefix = "read(go) fd=";         break;
	case SYS_write:      prefix = "write(go) fd=";        break;
	case SYS_close:      prefix = "close(go) fd=";        break;
	case SYS_fstat:      prefix = "fstat(go) fd=";        break;
	case SYS_openat:     prefix = "openat(go) ";          logPath = true; break;
	case SYS_newfstatat: prefix = "newfstatat(go) ";      logPath = true; break;
	case SYS_getdents64: prefix = "getdents64(go) fd=";   break;
	case SYS_statx:      prefix = "statx(go) ";           logPath = true; break;
	case SYS_unlinkat:   prefix = "unlinkat(go) ";        logPath = true; break;
	default:             break;
	}
	if (prefix)
	{
		RawWriteLog(prefix);
		if (logPath)
		{
			const char* path = (const char*)a2;
			if (path && path[0]) RawWriteLog(path);
		}
		else
		{
			RawWriteLogUlong((unsigned long)a1);
		}
		RawWriteLog("\n");
	}

	// SYS_getcwd must return the virtualised working dir so Go resolves relative
	// paths against UBA's view of CWD.  The process's physical CWD can differ
	// (notably in agent mode, where the process is spawned in a session temp
	// dir), so the raw syscall would give Go the wrong base for ../../foo.
	// Served inline: just a memcpy from g_virtualWorkingDir, no TLS, no worker.
	if (trap == SYS_getcwd)
	{
		char* buf = (char*)a1;
		size_t size = (size_t)a2;
		u32 len = g_virtualWorkingDir.count;
		if (len > 1 && g_virtualWorkingDir.data[len - 1] == '/')
			--len;
		if (!buf || size < (size_t)len + 1)
		{
			saved[15] = (u64)(long)(-1);
			saved[14] = 0;
			saved[13] = (u64)(unsigned int)ERANGE;
			return 1;
		}
		memcpy(buf, g_virtualWorkingDir.data, len);
		buf[len] = 0;
		RawWriteLog("getcwd(go) -> ");
		RawWriteLog(buf);
		RawWriteLog("\n");
		saved[15] = (u64)(long)(len + 1);
		saved[14] = 0;
		saved[13] = 0;
		return 1;
	}

	// Skip UBA dispatch if Init() hasn't completed (g_isDetouring is set last).
	// In that case the asm bridge passes through to a direct SYSCALL.
	if (!g_isDetouring || !g_messageMappingMem)
		return 0;

	// Run UBA handlers INLINE on the goroutine OS thread.  We're on a 256 KB
	// alt-stack with all signals blocked; g_communicationLock's uncontended
	// path is pure atomics.  Contended-path TLS access (ParkingLot) is the
	// known risk; mitigated by the bridge spinlock serialising goroutines.
	long result = 0;
	int  errOut = 0;
	bool handled = false;
	switch (trap)
	{
	case SYS_openat:
		handled = GoHandleOpenat((int)a1, (const char*)a2, (int)a3, (int)a4, &result, &errOut);
		break;
	case SYS_newfstatat:
		handled = GoHandleNewfstatat((int)a1, (const char*)a2, (struct stat*)a3, (int)a4, &result, &errOut);
		break;
	case SYS_statx:
		handled = GoHandleStatx((int)a1, (const char*)a2, (int)a3, (unsigned int)a4, (struct statx*)a5, &result, &errOut);
		break;
	case SYS_unlinkat:
		handled = GoHandleUnlinkat((int)a1, (const char*)a2, (int)a3, &result, &errOut);
		break;
	case SYS_close:
		handled = GoHandleClose((int)a1, &result, &errOut);
		break;
	case SYS_getdents64:
		handled = GoHandleGetdents64((int)a1, (void*)a2, (unsigned int)a3, &result, &errOut);
		break;
	// Same uid/gid local + remote keeps the dirtable paths consistent (nsjail expects /run/user/<euid>).
	case SYS_getuid:
	case SYS_geteuid:
		result = g_virtualUid;
		handled = true;
		break;
	case SYS_getgid:
	case SYS_getegid:
		result = g_virtualGid;
		handled = true;
		break;
	default:
		return 0; // asm bridge passes through
	}
	if (!handled)
		return 0;

	if (result < 0)
	{
		saved[15] = (u64)(long)(-1);
		saved[14] = 0;
		saved[13] = (u64)(unsigned int)errOut;
	}
	else
	{
		saved[15] = (u64)(long)result;
		saved[14] = 0;
		saved[13] = 0;
	}
	return 1;
}

// ─────────────────────────────────────────────────────────────────────────────
// Assembly bridge — Go register ABI → fast-path SYSCALL or C handler
// ─────────────────────────────────────────────────────────────────────────────
//
// hook_go_rawsyscall6 is installed at internal/runtime/syscall.Syscall6.
// It is reached from two callers:
//
//   syscall.RawSyscall / RawSyscall6 (no entersyscall):
//     Direct path; the goroutine is NOT in _Gsyscall state.  Calling C code
//     here is unsafe: Go's async preemption (SIGURG) can run while our C frame
//     sits on the goroutine stack, and the runtime cannot understand that frame.
//     → FAST PATH: execute SYSCALL directly in assembly, no C code.
//
//   syscall.Syscall / Syscall6 (with entersyscall):
//     entersyscall was called before reaching us.  The goroutine is in
//     _Gsyscall state; the GC scans the stack only up to syscallsp, which was
//     saved before the CALL to us.  Our C bridge frame is below syscallsp and
//     is never scanned.  Calling C is safe.
//     → SLOW PATH: alt-stack switch + spinlock + GoSyscallHandlerInline.
//
// File I/O syscalls (openat=257, newfstatat=262, unlinkat=263, statx=332) are
// always routed through syscall.Syscall / Syscall6 because they can block.
// So when we reach the slow path, entersyscall is always in effect.
//
// Entry (Go 1.17+ register ABI, matches internal/runtime/syscall.Syscall6):
//   AX=trap  BX=a1  CX=a2  DI=a3  SI=a4  R8=a5  R9=a6
//
// Fast-path register translation (Go → Linux syscall):
//   RAX=trap  RDI=BX  RSI=CX  RDX=DI  R10=SI  R8=R8  R9=R9
//   (SYSCALL clobbers RCX→next-RIP and R11→RFLAGS; both caller-saved in Go.)
//
// Return (Go register ABI):
//   AX=r1  BX=r2  CX=errno   (errno=0 on success, positive on error)
//
// Slow path: mirrors DetoursStatic/UbaStaticStub.S (steps 1-14).  The bridge
// blocks all signals, acquires g_goBridgeLock, switches to a 256 KB alt-stack,
// saves all 15 GPRs, calls GoSyscallHandlerInline(saved), pops back, releases
// the lock, restores signals, and either rets (handled) or jumps to the same
// direct-SYSCALL path the fast lane uses (passthrough).

__asm__(
".global hook_go_rawsyscall6\n"
"hook_go_rawsyscall6:\n"
"    # Dispatch on syscall number.\n"
"    # Slow path: ONLY syscalls UBA actually intercepts in C\n"
"    # (GoSyscallHandlerInline switch). Pure passthroughs (read,\n"
"    # write, fstat) go through the fast path — running 64 B of\n"
"    # Go-stack push/pop + sigprocmask + spinlock + alt-stack switch\n"
"    # just to fall through to a direct SYSCALL is wasted machinery\n"
"    # and a real bug surface (a stale rsp delta after a fall-through\n"
"    # would corrupt parent-goroutine stack vars; observed as garbage\n"
"    # lazybuf slice headers in esbuild --service mode under load).\n"
"    cmpq  $3,   %rax\n"          // SYS_close
"    je    .Lhrs6_slow\n"
"    cmpq  $217, %rax\n"          // SYS_getdents64
"    je    .Lhrs6_slow\n"
"    cmpq  $231, %rax\n"          // SYS_exit_group
"    je    .Lhrs6_slow\n"
"    cmpq  $257, %rax\n"          // SYS_openat
"    je    .Lhrs6_slow\n"
"    cmpq  $262, %rax\n"          // SYS_newfstatat
"    je    .Lhrs6_slow\n"
"    cmpq  $263, %rax\n"          // SYS_unlinkat
"    je    .Lhrs6_slow\n"
"    cmpq  $332, %rax\n"          // SYS_statx
"    je    .Lhrs6_slow\n"
"\n"
"    # Fast path: translate Go register ABI → Linux syscall ABI and execute.\n"
"    # No C calls, no TLS access — safe even without entersyscall protection.\n"
"    # Also reached from .Lhrs6_slow when the goroutine stack is too small for C.\n"
".Lhrs6_direct_syscall:\n"
"    movq  %rsi, %r10\n"          // r10 = a4  (Go: SI)
"    movq  %rdi, %rdx\n"          // rdx = a3  (Go: DI)
"    movq  %rcx, %rsi\n"          // rsi = a2  (Go: CX)
"    movq  %rbx, %rdi\n"          // rdi = a1  (Go: BX)
"    syscall\n"
"    # Convert Linux result → Go ABI.  Match original internal/runtime/syscall.Syscall6:\n"
"    #   error if rax > 0xfffffffffffff001  (i.e. rax in [-4094,-1])\n"
"    cmpq  $-4095, %rax\n"        // -4095 = 0xfffffffffffff001
"    jbe   .Lhrs6_ok\n"           // unsigned <=: no error
"    negq  %rax\n"
"    movq  %rax, %rcx\n"          // CX = errno
"    movq  $-1,  %rax\n"          // AX = -1  (r1 on error)
"    xorl  %ebx, %ebx\n"          // BX = 0   (r2)
"    # Go ABI: XMM15=0 + clean upper YMMs (AVX libc leakage → AL2023 sysmon/netpoll stack-split panic).\n"
"    vzeroupper\n"
"    pxor  %xmm15, %xmm15\n"
"    ret\n"
".Lhrs6_ok:\n"
"    movq  %rdx, %rbx\n"          // BX = r2 (rdx from syscall, usually 0)
"    xorl  %ecx, %ecx\n"          // CX = 0  (no error)
"    vzeroupper\n"
"    pxor  %xmm15, %xmm15\n"
"    ret\n"
"\n"
"    # Slow path — file-I/O syscalls reach here from the fan-out above.\n"
"    # Mirrors DetoursStatic/UbaStaticStub.S steps 1-14.\n"
".Lhrs6_slow:\n"
"    # Goroutine stack guard: keep the existing safety net.  Even with the\n"
"    # alt-stack we use ~64 bytes of Go stack for the sigprocmask save area\n"
"    # before the switch, so a near-empty goroutine stack still falls back\n"
"    # to a direct SYSCALL.  R14 = current g; g.stack.lo is at offset 0.\n"
"    movq  0(%r14), %r11\n"
"    addq  $1024, %r11\n"
"    cmpq  %r11, %rsp\n"
"    jb    .Lhrs6_direct_syscall\n"
"\n"
"    # ──────────────────────────────────────────────────────────────────\n"
"    # 1. Save the regs the first sigprocmask will use/clobber (7 × 8 = 56 B).\n"
"    pushq %rax\n"                  // Go trap
"    pushq %rcx\n"                  // Go a2
"    pushq %rdx\n"
"    pushq %rdi\n"                  // Go a3
"    pushq %rsi\n"                  // Go a4
"    pushq %r10\n"
"    pushq %r11\n"                  // syscall clobbers r11
"\n"
"    # 2. Reserve an 8-byte slot on the Go stack for the saved signal mask.\n"
"    subq $8, %rsp\n"
"\n"
"    # 3. rt_sigprocmask(SIG_SETMASK, &g_goFullMask, oldset_on_stack, 8).\n"
"    movq $14, %rax\n"               // SYS_rt_sigprocmask
"    movl $2, %edi\n"                // SIG_SETMASK
"    leaq g_goFullMask(%rip), %rsi\n"
"    movq %rsp, %rdx\n"              // oldset → Go stack slot at [rsp]
"    movl $8, %r10d\n"               // sigsetsize
"    syscall\n"
"\n"
"    # 4. Acquire bridge spinlock.  cmpxchgl uses %eax implicitly; the rax\n"
"    #    save above means we don't lose the Go trap value.\n"
".Lhrs6_spin:\n"
"    xorl %eax, %eax\n"
"    movl $1, %ecx\n"
"    lock cmpxchgl %ecx, g_goBridgeLock(%rip)\n"
"    je   .Lhrs6_got_lock\n"
"    pause\n"
"    jmp  .Lhrs6_spin\n"
".Lhrs6_got_lock:\n"
"\n"
"    # 5. Switch to alt-stack.  Stash the Go rsp (which currently points at\n"
"    #    the saved mask slot) on the alt-stack, then reserve a handled-flag\n"
"    #    slot just below it.  rax is scratch — its real value is in the Go\n"
"    #    stack save area we'll reload from in step 6.\n"
"    movq %rsp, %rax\n"               // rax = Go rsp (= &saved_mask_slot)
"    movq g_goAltStackTop(%rip), %rsp\n"
"    pushq %rax\n"                    // alt-stack: saved Go rsp
"    subq $8, %rsp\n"                 // alt-stack: handled-flag slot
"\n"
"    # 6. Reload Go ABI registers from the Go stack save area.  Go stack\n"
"    #    layout at [rax]:  +0 mask  +8 r11  +16 r10  +24 rsi  +32 rdi\n"
"    #                      +40 rdx  +48 rcx  +56 rax (Go trap)\n"
"    movq  8(%rax), %r11\n"
"    movq 16(%rax), %r10\n"
"    movq 24(%rax), %rsi\n"
"    movq 32(%rax), %rdi\n"
"    movq 40(%rax), %rdx\n"
"    movq 48(%rax), %rcx\n"
"    movq 56(%rax), %rax\n"           // rax = Go trap
"\n"
"    # 7. Push all 15 GPRs on the alt-stack.  GoSyscallHandlerInline reads\n"
"    #    them via the saved[] pointer below.  r15 ends up closest to rsp.\n"
"    pushq %rax\n"
"    pushq %rbx\n"
"    pushq %rcx\n"
"    pushq %rdx\n"
"    pushq %rdi\n"
"    pushq %rsi\n"
"    pushq %rbp\n"
"    pushq %r8\n"
"    pushq %r9\n"
"    pushq %r10\n"
"    pushq %r11\n"
"    pushq %r12\n"
"    pushq %r13\n"
"    pushq %r14\n"
"    pushq %r15\n"
"    subq $8, %rsp\n"                 // 16-byte align for the C call (saved[0] pad)
"\n"
"    # 8. Call GoSyscallHandlerInline(saved).  saved points at the alignment\n"
"    #    pad; saved[1] = r15 … saved[15] = rax (Go trap).\n"
"    movq %rsp, %rdi\n"
"    call GoSyscallHandlerInline\n"
"    # rax = handled flag (0 = passthrough, 1 = synthesized return).\n"
"\n"
"    # 9. Stash handled into the dedicated slot at rsp+128 (above the 15 regs\n"
"    #    + alignment pad).  Handled flag travels back via %r8 in step 11.\n"
"    movq %rax, 128(%rsp)\n"
"\n"
"    # 10. Discard pad and pop the 15 saved regs.\n"
"    addq $8, %rsp\n"
"    popq %r15\n"
"    popq %r14\n"
"    popq %r13\n"
"    popq %r12\n"
"    popq %r11\n"
"    popq %r10\n"
"    popq %r9\n"
"    popq %r8\n"
"    popq %rbp\n"
"    popq %rsi\n"
"    popq %rdi\n"
"    popq %rdx\n"
"    popq %rcx\n"
"    popq %rbx\n"
"    popq %rax\n"                     // rax = Go trap, or synthesized r1 if handled
"\n"
"    # 11. Pop handled flag into %r8 (caller-saved in Go ABI; clobbering it\n"
"    #     across syscall.Syscall6 is safe).  Then pop the saved Go rsp,\n"
"    #     switching back to the Go goroutine stack.  Release the lock now\n"
"    #     so the Go-stack second sigprocmask doesn't hold it.\n"
"    popq %r8\n"
"    popq %rsp\n"
"    movl $0, g_goBridgeLock(%rip)\n"
"\n"
"    # 12. rt_sigprocmask(SIG_SETMASK, &saved_mask_on_go_stack, NULL, 8).\n"
"    pushq %r8\n"
"    pushq %rax\n"
"    pushq %rcx\n"
"    pushq %rdx\n"
"    pushq %rdi\n"
"    pushq %rsi\n"
"    pushq %r10\n"
"    pushq %r11\n"
"    movq $14, %rax\n"
"    movl $2, %edi\n"
"    leaq 64(%rsp), %rsi\n"           // saved mask is 64 bytes above post-push rsp
"    xorl %edx, %edx\n"
"    movl $8, %r10d\n"
"    syscall\n"
"    popq %r11\n"
"    popq %r10\n"
"    popq %rsi\n"
"    popq %rdi\n"
"    popq %rdx\n"
"    popq %rcx\n"
"    popq %rax\n"
"    popq %r8\n"
"\n"
"    # 13. Free the Go-stack 64 B (7 saved regs + saved-mask slot).\n"
"    addq $64, %rsp\n"
"\n"
"    # 14. Dispatch on r8.  Handled (r8 != 0): rax/rbx/rcx already hold the\n"
"    #     synthesized return values; zero XMM15 then ret to Go's caller.\n"
"    #     Passthrough: jump into the existing direct-SYSCALL fast lane\n"
"    #     (which translates Go ABI to Linux syscall ABI, runs SYSCALL,\n"
"    #     converts the result back, and zeroes XMM15 there).\n"
"    #\n"
"    # Go ABI: XMM15=0 at call boundaries (Go emits MOVUPS XMM15, [mem] as zero-store).\n"
"    # C slow path (BLAKE3 SIMD) clobbers it — without reset, Go writes garbage into slice/map\n"
"    # headers (chromium devtools-frontend 'index out of range' lazybuf panic).\n"
"    vzeroupper\n"
"    pxor  %xmm15, %xmm15\n"
"    testq %r8, %r8\n"
"    jnz   .Lhrs6_handled\n"
"    jmp   .Lhrs6_direct_syscall\n"
".Lhrs6_handled:\n"
"    ret\n"
);

// ─────────────────────────────────────────────────────────────────────────────
// Entry point
// ─────────────────────────────────────────────────────────────────────────────

extern "C" void hook_go_rawsyscall6();

// Patches syscall.RawSyscall and syscall.RawSyscall6 in the loaded Go binary.
// Must be called from process startup (PreInit) before any goroutines run.
// No trampoline is needed — hook_go_rawsyscall6 performs all syscalls itself.
void PatchGoSyscalls()
{
	// (diagnostic early-return removed)

	struct Target { const char* name; void* bridge; };

	Target targets[] =
	{
		// internal/runtime/syscall.Syscall6: the naked SYSCALL stub reached after
		// runtime.entersyscall.  Calling convention (Go 1.17+ register ABI):
		//   AX=trap  BX=a1  CX=a2  DI=a3  SI=a4  R8=a5  R9=a6
		//   → AX=r1  BX=r2  CX=errno
		// Same convention as hook_go_rawsyscall6.
		{ "internal/runtime/syscall.Syscall6", (void*)hook_go_rawsyscall6 },
	};

	for (auto& t : targets)
	{
		void* fn = FindGoSymbol(t.name);
		if (!fn)
			fn = FindGoSymbolInPclntab(t.name); // stripped binary fallback

		if (!fn)
		{
			DEBUG_LOG(TC("PatchGoSyscalls: symbol not found: %s"), t.name);
			continue;
		}

		DEBUG_LOG(TC("PatchGoSyscalls: patching %s at %p"), t.name, fn);

		if (!HookFunction(fn, t.bridge))
		{
			DEBUG_LOG(TC("PatchGoSyscalls: HookFunction failed for %s"), t.name);
		}
		else
		{
			DEBUG_LOG(TC("PatchGoSyscalls: patched %s OK"), t.name);
		}
	}

	// Allocate the 256 KB alt-stack for the inline bridge.  Must be done on
	// the main pthread (PreInit's caller) before any goroutine spawns, so the
	// bridge sees a valid g_goAltStackTop on its first dispatch.  The bridge
	// spinlock serialises concurrent Ms onto this single shared buffer; same
	// pattern + budget as DetoursStatic/UbaStaticStubCore.cpp.
	void* m = mmap(nullptr, kGoAltStackBytes, PROT_READ | PROT_WRITE,
		MAP_PRIVATE | MAP_ANONYMOUS, -1, 0);
	if (m == MAP_FAILED)
	{
		DEBUG_LOG(TC("PatchGoSyscalls: alt-stack mmap failed"));
		return;
	}
	g_goAltStackTop = (u8*)m + kGoAltStackBytes;
	DEBUG_LOG(TC("PatchGoSyscalls: alt-stack at %p (top %p)"), m, g_goAltStackTop);
}

// ─────────────────────────────────────────────────────────────────────────────
// Patch runtime.exit to route through our unified exit path.
//
// Go's runtime.exit(code int32) executes SYS_exit_group directly via a naked
// SYSCALL instruction in assembly.  This bypasses our LD_PRELOAD hook and the
// Syscall6 patch, so UBA never receives the exit notification.
//
// We fix this by patching the first 5 bytes of runtime.exit with a 5-byte
// relative JMP (E9 rel32) to a trampoline stub.  The trampoline:
//   1. Loads the exit code from the ABI0 stack slot (int32 at [RBP+16]) into %edi
//   2. Aligns RSP to 16 bytes
//   3. Calls GoTrampolineExit(code) — a noreturn C function that posts the
//      exit request and parks the thread.  The cleanup pthread then runs
//      Deinit+CloseCom+_exit on a proper glibc thread.
//
// The previous design inlined Deinit/CloseCom/exit_group directly here, but
// that raced against fatal signals: if SIGHUP arrived mid-Deinit, the signal
// handler's cleanup thread would then find g_isInitialized already false and
// skip the Exit RPC.  Routing both paths through a single CAS-serialized
// request makes the cleanup run exactly once on a known-safe thread.
//
// The trampoline must be within ±2 GB of the Go binary (which is non-PIE,
// loaded at ~0x400000) so that the 32-bit relative offset fits.  We allocate
// the page via mmap MAP_32BIT which guarantees placement in the low 2 GB.
// ─────────────────────────────────────────────────────────────────────────────

// Machine code template for the trampoline.
// Address of GoTrampolineExit is patched in at the XX positions.
//   55                                    push   %rbp
//   48 89 E5                              mov    %rsp, %rbp
//   8B 7D 10                              movl   16(%rbp), %edi     ; exit code → C arg1
//   48 83 E4 F0                           andq   $-16, %rsp         ; 16-byte RSP align
//   48 B8 XX XX XX XX XX XX XX XX         movabs $GoTrampolineExit, %rax
//   FF D0                                 call   *%rax
//   0F 0B                                 ud2                       ; noreturn guard
static const u8 kExitTrampolineTemplate[] = {
	0x55,                                               // push %rbp
	0x48, 0x89, 0xE5,                                   // mov %rsp, %rbp
	0x8B, 0x7D, 0x10,                                   // movl 16(%rbp), %edi
	0x48, 0x83, 0xE4, 0xF0,                             // andq $-16, %rsp
	0x48, 0xB8, 0,0,0,0,0,0,0,0,                       // movabs $GoTrampolineExit, %rax
	0xFF, 0xD0,                                         // call *%rax
	0x0F, 0x0B,                                         // ud2
};
static constexpr u32 kGoTrampolineExitAddrOffset = 13; // offset of address in template

void PatchGoExit()
{
	// Find runtime.exit in the Go binary.
	// Go 1.17+ exposes the ABI0 assembly stub as "runtime.exit.abi0" in the
	// ELF symbol table; older versions use "runtime.exit".
	void* fn = FindGoSymbol("runtime.exit.abi0");
	if (!fn) fn = FindGoSymbol("runtime.exit");
	if (!fn) fn = FindGoSymbolInPclntab("runtime.exit");
	if (!fn)
	{
		DEBUG_LOG(TC("PatchGoExit: runtime.exit not found"));
		return;
	}
	DEBUG_LOG(TC("PatchGoExit: runtime.exit at %p"), fn);

	// Allocate a trampoline page in the low 2 GB so a 5-byte relative JMP fits.
	void* tramp = mmap(nullptr, 4096,
		PROT_READ | PROT_WRITE,
		MAP_PRIVATE | MAP_ANONYMOUS | MAP_32BIT,
		-1, 0);
	if (tramp == MAP_FAILED)
	{
		DEBUG_LOG(TC("PatchGoExit: mmap trampoline failed"));
		return;
	}

	// Fill in the trampoline code.
	u8* code = (u8*)tramp;
	memcpy(code, kExitTrampolineTemplate, sizeof(kExitTrampolineTemplate));

	// Patch in the absolute address of GoTrampolineExit.  That C function
	// posts a normal-exit request to the cleanup pthread and parks forever;
	// the cleanup thread runs Deinit+CloseCom+_exit exactly once.
	uintptr_t trampexit_addr = (uintptr_t)(void*)&GoTrampolineExit;
	memcpy(code + kGoTrampolineExitAddrOffset, &trampexit_addr, 8);

	// Make the trampoline page executable (W^X: drop write, add exec).
	if (mprotect(tramp, 4096, PROT_READ | PROT_EXEC) < 0)
	{
		DEBUG_LOG(TC("PatchGoExit: mprotect trampoline failed"));
		munmap(tramp, 4096);
		return;
	}

	// Compute the 32-bit relative offset for the 5-byte JMP at runtime.exit.
	// JMP target = tramp, instruction after JMP = (u8*)fn + 5.
	intptr_t rel = (intptr_t)tramp - ((intptr_t)fn + 5);
	if (rel < -(intptr_t)0x80000000LL || rel > 0x7fffffffLL)
	{
		DEBUG_LOG(TC("PatchGoExit: trampoline %p too far from runtime.exit %p"), tramp, fn);
		munmap(tramp, 4096);
		return;
	}

	// Patch runtime.exit: write E9 rel32 (5-byte relative JMP).
	u8 patch[5] = { 0xE9, 0,0,0,0 };
	int rel32 = (int)rel;
	memcpy(patch + 1, &rel32, 4);

	if (MakeWritable(fn, 5) < 0)
	{
		DEBUG_LOG(TC("PatchGoExit: MakeWritable failed"));
		munmap(tramp, 4096);
		return;
	}
	memcpy(fn, patch, 5);
	__builtin___clear_cache((char*)fn, (char*)fn + 5);
	MakeExecutable(fn, 5);

	DEBUG_LOG(TC("PatchGoExit: patched runtime.exit → trampoline at %p"), tramp);
}

} // namespace uba

#endif // UBA_SUPPORTS_GO
