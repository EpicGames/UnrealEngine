// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaDefinitions.h"
#include "UbaPlatform.h"
#include <sys/stat.h>

namespace uba
{
#if UBA_SUPPORTS_GO

	bool IsGoBinary(const char* path);
	void PatchGoSyscalls();
	void PatchGoExit();
	// GoHandle*/GoHandlesPath are now file-local statics inside UbaDetoursGo.cpp (only callers live in the same TU).

#endif
}
