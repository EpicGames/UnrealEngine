// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaBinaryReaderWriter.h"
#include "UbaDefinitions.h"
#include "UbaDetoursShared.h"
#include "UbaEvent.h"
#include "UbaProcessStats.h"
#include "UbaFileMapping.h"

namespace uba { extern bool g_parentInNonInitNs; }

#include <dlfcn.h>
#if PLATFORM_LINUX
#include <sys/prctl.h>
#include <sys/syscall.h>
#include <signal.h>
#include "UbaLinuxRawSyscall.h"
#endif
#if PLATFORM_MAC
#include <pthread.h>
#include <sys/event.h>
#include <unistd.h>
#endif
#if UBA_USE_NATIVE_MAC_SEMAPHORES
#include <mach/mach.h>
#include <servers/bootstrap.h>
#endif
using namespace uba;

namespace uba
{
	int g_sessionPid = -1;
	SharedEvent* g_cancelEvent;
	SharedEvent* g_readEvent;
	SharedEvent* g_writeEvent;
	u8* g_messageMappingMem;
	extern bool g_isCancelled;

	void PreInit(const char* logFile, bool fromExec);
	void Init();
	void Deinit();
	void AddExceptionHandler();
}

static void __attribute__((constructor(102))) PreInitCtor()
{
	using namespace uba;
	SuppressDetourScope s;

	#if PLATFORM_LINUX
	// Reset sigmask: Go vfork bridge leaves all signals blocked on child side; without this
	// SIGTERM stays blocked across execve. Raw syscall avoids ctor-time libc/detour reentrancy.
	{
		u64 emptyset = 0;
		RawSyscallDirect(SYS_rt_sigprocmask, SIG_SETMASK, (long)&emptyset, 0, 8);
	}

	prctl(PR_SET_PDEATHSIG, SIGHUP, 0, 0, 0); // Catchable so child runs UBA's exit handshake; UbaNinja's reaper handles the orphan case.
	#elif PLATFORM_MAC
	// macOS has no PR_SET_PDEATHSIG equivalent — kqueue NOTE_EXIT on getppid() then _exit so a killed UbaHost cleans up the spawned compile tree instead of orphaning to launchd.
	{
		int kq = kqueue();
		if (kq != -1)
		{
			struct kevent ke;
			EV_SET(&ke, getppid(), EVFILT_PROC, EV_ADD | EV_ONESHOT, NOTE_EXIT, 0, NULL);
			if (kevent(kq, &ke, 1, NULL, 0, NULL) == 0)
			{
				pthread_t t;
				if (pthread_create(&t, NULL, [](void* arg) -> void*
					{
						int kqLocal = (int)(intptr_t)arg;
						struct kevent ev;
						kevent(kqLocal, NULL, 0, &ev, 1, NULL);
						close(kqLocal);
						_exit(1340);
						return NULL;
					}, (void*)(intptr_t)kq) == 0)
					pthread_detach(t);
				else
					close(kq);
			}
			else
				close(kq);
		}
	}
	#endif

	InitSharedVariables();

	u64 startTime = GetTime();

	g_runningRemote = getenv("UBA_REMOTE") != nullptr;
	
	bool fromExec = getenv("UBA_EXEC") != nullptr;
	unsetenv("UBA_EXEC");

	const char* logFile = getenv("UBA_LOGFILE");
	PreInit(logFile, fromExec);
	unsetenv("UBA_LOGFILE");

	const char* comId = getenv("UBA_COMID");
	//printf("Starting up %s... (comid: %s) %u\n", __progname, (comId && *comId) ? comId : "NOTSET", getpid());

	if (!comId || !*comId)
		return;

	StringBuffer<256> comIdName;
	const char* plusIndex = strchr(comId, '+');
	comIdName.Append(comId, u64(plusIndex - comId));
	u64 comIdUid;
	if (!comIdName.Parse(comIdUid))
		UBA_ASSERT(false);

	u32 comIdOffset = strtoul(plusIndex + 1, nullptr, 10);
	unsetenv("UBA_COMID");

	u8* rptr = FileMapping_MapFromHost(FileMappingHandle::FromU64(comIdUid), CommunicationMemSize, comIdOffset, true, TC("ComHandle"));
	if (!rptr)
	{
		u32 err = GetLastError();
		const char* hint = "";
		if (IsMemfdHandle(FileMappingHandle::FromU64(comIdUid)) && err == 1) // EPERM
			hint = " - pidfd_getfd blocked (container seccomp?); give the container CAP_SYS_PTRACE or run the host with UBA_FILE_MAPPING_MEMFD=0";
		else if (err == 2) // ENOENT
			hint = " - shm file missing; host and this process disagree on UBA_FILE_MAPPING_DIR/UBA_FILE_MAPPING_MEMFD";
		fprintf(stderr, "UbaDetours[%d]: Failed to attach comm mapping uid=%llu (err %u)%s\n", getpid(), comIdUid, err, hint);
		return;
	}

	g_messageMappingMem = rptr;

	if (UseSemaphoreForSharedEvents())
	{
		g_readEvent = new SharedEvent();
		g_writeEvent = new SharedEvent();
		g_cancelEvent = new SharedEvent();

		#if UBA_USE_NATIVE_MAC_SEMAPHORES

		const char* pidStr = getenv("UBA_PID");
		UBA_ASSERT(pidStr);
		u64 pid;
		Parse(pid, pidStr, TStrlen(pidStr));
		unsetenv("UBA_PID");

		StringBuffer<> service;
		service.Append("UbaSemaphoreService");
		service.Append(getenv("UBA_SESSION_ID"));
		mach_port_t server_send = MACH_PORT_NULL;
		kern_return_t kr = bootstrap_look_up(bootstrap_port, service.data, &server_send);
		UBA_ASSERTF(kr == KERN_SUCCESS, "bootstrap_look_up failed to lookup %s (%u)", service.data, kr);
		
		mach_port_t reply_port = MACH_PORT_NULL;
		kr = mach_port_allocate(mach_task_self(), MACH_PORT_RIGHT_RECEIVE, &reply_port);
		UBA_ASSERT(kr == KERN_SUCCESS);

		struct { mach_msg_header_t header; u32 pid; } req;
		memset(&req, 0, sizeof(req));
		req.header.msgh_bits = MACH_MSGH_BITS(MACH_MSG_TYPE_COPY_SEND, MACH_MSG_TYPE_MAKE_SEND_ONCE);
		req.header.msgh_size = sizeof(req);
		req.header.msgh_remote_port = server_send;
		req.header.msgh_local_port  = reply_port;
		req.pid = u32(pid);
		kr = mach_msg(&req.header, MACH_SEND_MSG, req.header.msgh_size, 0, MACH_PORT_NULL, MACH_MSG_TIMEOUT_NONE, MACH_PORT_NULL);
		UBA_ASSERTF(kr == KERN_SUCCESS, "Failed to send mach msg (%u)", kr);

		struct { mach_msg_header_t header; mach_msg_body_t body; mach_msg_port_descriptor_t sem_desc[3]; uint8_t extra[512]; } rep;
		memset(&rep, 0, sizeof(rep));
		kr = mach_msg(&rep.header, MACH_RCV_MSG, 0, sizeof(rep), reply_port, MACH_MSG_TIMEOUT_NONE, MACH_PORT_NULL);
		UBA_ASSERTF(kr == KERN_SUCCESS, "Failed to receive mach msg (%u)", kr);

		StringBuffer<32> b;
		b.AppendValue(rep.sem_desc[0].name);
		g_cancelEvent->Create(b, false);
		b.Clear().AppendValue(rep.sem_desc[1].name);
		g_readEvent->Create(b, false);
		b.Clear().AppendValue(rep.sem_desc[2].name);
		g_writeEvent->Create(b, false);

		#else

		StringBuffer<256> name;
		name.Append(TCV("uba")).AppendValue(comIdUid).Append('_').AppendValue(comIdOffset);
		u64 len = name.count;
		name.Append("_cancel");
		g_cancelEvent->Create(name, false);
		name.Resize(len).Append("_read");
		g_readEvent->Create(name, false);
		name.Resize(len).Append("_write");
		g_writeEvent->Create(name, false);

		#endif
	}
	else
	{
		g_cancelEvent = ((SharedEvent*)rptr);
		g_readEvent = ((SharedEvent*)rptr) + 1;
		g_writeEvent = ((SharedEvent*)rptr) + 2;
		g_messageMappingMem += sizeof(SharedEvent) * 3;
	}

	AddExceptionHandler();
	InitMemory();

	const char* sessionPidStr = getenv("UBA_SESSION_PROCESS");
	UBA_ASSERT(sessionPidStr);
	g_sessionPid = atoi(sessionPidStr);

	g_rulesIndex = strtoul(getenv("UBA_RULES"), nullptr, 10);
	g_rules = GetApplicationRules()[g_rulesIndex].rules;
	unsetenv("UBA_RULES");

	const char* realCwd = getenv("UBA_CWD");
	UBA_ASSERT(realCwd);
	chdir(realCwd);
	unsetenv("UBA_CWD");

	g_stats.attach.time += GetTime() - startTime;
	g_stats.attach.count = 1;


	// TODO: Don't know if we want this
	//int old;
	//pthread_setcancelstate(PTHREAD_CANCEL_DISABLE, &old);

	// Diagnostic: dump interrupted PC/RSP/callee-saved regs to /tmp/uba_pc.<pid> on SIGURG.
	// Async-signal-safe — raw syscalls, no malloc, no printf.
	#if PLATFORM_LINUX
	{
		struct sigaction sa = {};
		sa.sa_flags = SA_SIGINFO;
		sa.sa_sigaction = [](int /*sig*/, siginfo_t* /*info*/, void* uctx) {
			#if defined(__x86_64__)
			ucontext_t* uc = (ucontext_t*)uctx;
			unsigned long long rip = uc->uc_mcontext.gregs[REG_RIP];
			unsigned long long rsp = uc->uc_mcontext.gregs[REG_RSP];
			unsigned long long rbp = uc->uc_mcontext.gregs[REG_RBP];
			unsigned long long rbx = uc->uc_mcontext.gregs[REG_RBX];
			unsigned long long r12 = uc->uc_mcontext.gregs[REG_R12];
			unsigned long long r13 = uc->uc_mcontext.gregs[REG_R13];
			unsigned long long r14 = uc->uc_mcontext.gregs[REG_R14];
			unsigned long long r15 = uc->uc_mcontext.gregs[REG_R15];
			unsigned int pid = (unsigned int)getpid();
			char path[64]; char* p = path;
			const char prefix[] = "/tmp/uba_pc.";
			for (size_t i = 0; i < sizeof(prefix)-1; ++i) *p++ = prefix[i];
			char digs[16]; int dl = 0; unsigned int v = pid; if (!v) digs[dl++] = '0';
			while (v) { digs[dl++] = '0' + (char)(v % 10); v /= 10; }
			while (dl--) *p++ = digs[dl]; *p = 0;
			int fd = ::open(path, O_WRONLY|O_CREAT|O_TRUNC, 0644);
			if (fd >= 0) {
				char buf[512]; char* b = buf;
				auto hex = [](unsigned long long v, char* out) -> int {
					const char* hd = "0123456789abcdef";
					int n = 0; char tmp[24]; int tl = 0;
					if (!v) tmp[tl++] = '0';
					while (v) { tmp[tl++] = hd[v & 0xf]; v >>= 4; }
					out[n++] = '0'; out[n++] = 'x';
					while (tl--) out[n++] = tmp[tl];
					return n;
				};
				auto emit = [&](const char* lbl, unsigned long long v) {
					while (*lbl) *b++ = *lbl++;
					*b++ = '=';
					b += hex(v, b);
					*b++ = '\n';
				};
				emit("rip", rip);
				emit("rsp", rsp);
				emit("rbp", rbp);
				emit("rbx", rbx);
				emit("r12", r12);
				emit("r13", r13);
				emit("r14", r14);
				emit("r15", r15);
				(void)!::write(fd, buf, b - buf);
				::close(fd);
			}
			#else
			(void)uctx;
			#endif
		};
		sigemptyset(&sa.sa_mask);
		// SIGURG: rare in practice, and our sigaction wrapper chains if anyone overrides it.
		sigaction(SIGURG, &sa, nullptr);
	}
	#endif

	Init();
}

void CloseCom()
{
	if (!g_messageMappingMem)
		return;
	void* mem = g_messageMappingMem;
	if (UseSemaphoreForSharedEvents())
	{
		delete g_cancelEvent;
		delete g_readEvent;
		delete g_writeEvent;
	}
	else
	{
		mem = g_cancelEvent;
	}
	g_cancelEvent = nullptr;
	g_readEvent = nullptr;
	g_writeEvent = nullptr;
	g_messageMappingMem = nullptr;
	munmap(mem, CommunicationMemSize);
}

static void __attribute__((destructor(102))) InitDtor()
{
	using namespace uba;

	Deinit();
	CloseCom();
	//printf("Exiting... %s   %u \n", __progname, getpid());
}

