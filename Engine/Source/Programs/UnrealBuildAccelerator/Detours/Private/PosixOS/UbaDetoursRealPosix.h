// Copyright Epic Games, Inc. All Rights Reserved.
//
// Real_<libcname>: thin path-direct callers used by both the LD_PRELOAD detour and the static stub.
// Linux invokes the kernel via raw syscalls (no glibc, no errno TLS); macOS goes through libc via
// TRUE_WRAPPER because Apple does not commit to a stable syscall ABI. outError receives errno on
// failure and is left untouched on success — same contract as libc, but explicit.

#pragma once

#include "UbaPlatform.h"

#if PLATFORM_LINUX
	#include "UbaLinuxRawSyscall.h"
	#include <fcntl.h>
	#include <sys/stat.h>
	#include <sys/syscall.h>
#endif

#if PLATFORM_MAC
	#include "UbaDetoursTrueWrappersPosix.h"
	#include <errno.h>
	#include <unistd.h>
#endif

namespace uba
{

inline long Real_readlink(int& outError, const char* path, char* buf, unsigned long bufSize)
{
#if PLATFORM_LINUX
	long r = RawReadlinkat(path, buf, bufSize);
	if (r < 0) { outError = (int)-r; return -1; }
	return r;
#else
	long r = TRUE_WRAPPER(readlink)(path, buf, bufSize);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline long Real_readlinkat(int& outError, int dirfd, const char* path, char* buf, unsigned long bufSize)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_readlinkat, dirfd, (long)path, (long)buf, (long)bufSize);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return r.r1;
#else
	long r = TRUE_WRAPPER(readlinkat)(dirfd, path, buf, bufSize);
	if (r < 0) outError = errno;
	return r;
#endif
}

// access/dup2 syscalls are gone on aarch64 — use the *at variants with AT_FDCWD / SYS_dup3 with flags=0.

inline int Real_faccessat(int& outError, int dirfd, const char* path, int mode, int flags)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_faccessat, dirfd, (long)path, mode, flags);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return 0;
#else
	int r = TRUE_WRAPPER(faccessat)(dirfd, path, mode, flags);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_access(int& outError, const char* path, int mode)
{
#if PLATFORM_LINUX
	return Real_faccessat(outError, AT_FDCWD, path, mode, 0);
#else
	int r = TRUE_WRAPPER(access)(path, mode);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline long Real_lseek(int& outError, int fd, long offset, int whence)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_lseek, fd, offset, whence, 0);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return r.r1;
#else
	long r = TRUE_WRAPPER(lseek)(fd, offset, whence);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_dup(int& outError, int oldfd)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_dup, oldfd, 0, 0, 0);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return (int)r.r1;
#else
	int r = TRUE_WRAPPER(dup)(oldfd);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_dup3(int& outError, int oldfd, int newfd, int flags)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_dup3, oldfd, newfd, flags, 0);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return (int)r.r1;
#else
	// macOS has no dup3; emulate via dup2 + optional FD_CLOEXEC.
	int r = TRUE_WRAPPER(dup2)(oldfd, newfd);
	if (r < 0) { outError = errno; return r; }
	if (flags & O_CLOEXEC)
	{
		if (TRUE_WRAPPER(fcntl)(newfd, F_SETFD, FD_CLOEXEC) < 0) { outError = errno; return -1; }
	}
	return r;
#endif
}

inline int Real_dup2(int& outError, int oldfd, int newfd)
{
#if PLATFORM_LINUX
	return Real_dup3(outError, oldfd, newfd, 0);
#else
	int r = TRUE_WRAPPER(dup2)(oldfd, newfd);
	if (r < 0) outError = errno;
	return r;
#endif
}

// open/openat: on 64-bit Linux SYS_openat covers open/open64/openat/openat64 (O_LARGEFILE is a no-op).

inline int Real_openat(int& outError, int dirfd, const char* path, int flags, int mode)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_openat, dirfd, (long)path, flags, mode);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return (int)r.r1;
#else
	int r = TRUE_WRAPPER(openat)(dirfd, path, flags, mode);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_open(int& outError, const char* path, int flags, int mode)
{
#if PLATFORM_LINUX
	return Real_openat(outError, AT_FDCWD, path, flags, mode);
#else
	int r = TRUE_WRAPPER(open)(path, flags, mode);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_close(int& outError, int fd)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_close, fd, 0, 0, 0);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return 0;
#else
	int r = TRUE_WRAPPER(close)(fd);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline long Real_read(int& outError, int fd, void* buf, unsigned long count)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_read, fd, (long)buf, (long)count, 0);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return r.r1;
#else
	long r = TRUE_WRAPPER(read)(fd, buf, count);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline long Real_pread(int& outError, int fd, void* buf, unsigned long count, long offset)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_pread64, fd, (long)buf, (long)count, offset);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return r.r1;
#else
	long r = TRUE_WRAPPER(pread)(fd, buf, count, offset);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline long Real_write(int& outError, int fd, const void* buf, unsigned long count)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_write, fd, (long)buf, (long)count, 0);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return r.r1;
#else
	long r = TRUE_WRAPPER(write)(fd, buf, count);
	if (r < 0) outError = errno;
	return r;
#endif
}

// stat family: on x86_64/aarch64 with modern glibc, struct stat matches the kernel's stat layout
// from SYS_newfstatat / SYS_fstat — direct syscall fills the caller's buffer without translation.
// aarch64 has no SYS_fstat; fstat goes through SYS_fstatat with AT_EMPTY_PATH.

inline int Real_fstatat(int& outError, int dirfd, const char* path, struct stat* buf, int flags)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_newfstatat, dirfd, (long)path, (long)buf, flags);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return 0;
#else
	int r = TRUE_WRAPPER(fstatat)(dirfd, path, buf, flags);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_fstat(int& outError, int fd, struct stat* buf)
{
#if PLATFORM_LINUX
#if PLATFORM_CPU_ARM_FAMILY
	return Real_fstatat(outError, fd, "", buf, AT_EMPTY_PATH);
#else
	auto r = RawSyscallDirect(SYS_fstat, fd, (long)buf, 0, 0);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return 0;
#endif
#else
	int r = TRUE_WRAPPER(fstat)(fd, buf);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_stat(int& outError, const char* path, struct stat* buf)
{
#if PLATFORM_LINUX
	return Real_fstatat(outError, AT_FDCWD, path, buf, 0);
#else
	int r = TRUE_WRAPPER(stat)(path, buf);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_lstat(int& outError, const char* path, struct stat* buf)
{
#if PLATFORM_LINUX
	return Real_fstatat(outError, AT_FDCWD, path, buf, AT_SYMLINK_NOFOLLOW);
#else
	int r = TRUE_WRAPPER(lstat)(path, buf);
	if (r < 0) outError = errno;
	return r;
#endif
}

#if PLATFORM_LINUX
inline int Real_statx(int& outError, int dirfd, const char* path, int flags, unsigned int mask, struct ::statx* buf)
{
	auto r = RawSyscallDirect(SYS_statx, dirfd, (long)path, flags, mask, (long)buf);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return 0;
}
#endif

// mkdir/unlink/rename/link/symlink: aarch64 only has the *at variants — always go through them.

inline int Real_mkdirat(int& outError, int dirfd, const char* path, unsigned int mode)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_mkdirat, dirfd, (long)path, mode, 0);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return 0;
#else
	int r = TRUE_WRAPPER(mkdirat)(dirfd, path, (mode_t)mode);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_mkdir(int& outError, const char* path, unsigned int mode)
{
#if PLATFORM_LINUX
	return Real_mkdirat(outError, AT_FDCWD, path, mode);
#else
	int r = TRUE_WRAPPER(mkdir)(path, (mode_t)mode);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_unlinkat(int& outError, int dirfd, const char* path, int flags)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_unlinkat, dirfd, (long)path, flags, 0);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return 0;
#else
	int r = TRUE_WRAPPER(unlinkat)(dirfd, path, flags);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_unlink(int& outError, const char* path)
{
#if PLATFORM_LINUX
	return Real_unlinkat(outError, AT_FDCWD, path, 0);
#else
	int r = TRUE_WRAPPER(unlink)(path);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_rmdir(int& outError, const char* path)
{
#if PLATFORM_LINUX
	return Real_unlinkat(outError, AT_FDCWD, path, AT_REMOVEDIR);
#else
	int r = TRUE_WRAPPER(rmdir)(path);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_renameat(int& outError, int olddirfd, const char* oldpath, int newdirfd, const char* newpath)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_renameat, olddirfd, (long)oldpath, newdirfd, (long)newpath);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return 0;
#else
	int r = TRUE_WRAPPER(renameat)(olddirfd, oldpath, newdirfd, newpath);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_rename(int& outError, const char* oldpath, const char* newpath)
{
#if PLATFORM_LINUX
	return Real_renameat(outError, AT_FDCWD, oldpath, AT_FDCWD, newpath);
#else
	int r = TRUE_WRAPPER(rename)(oldpath, newpath);
	if (r < 0) outError = errno;
	return r;
#endif
}

#if PLATFORM_LINUX
inline int Real_renameat2(int& outError, int olddirfd, const char* oldpath, int newdirfd, const char* newpath, unsigned int flags)
{
	auto r = RawSyscallDirect(SYS_renameat2, olddirfd, (long)oldpath, newdirfd, (long)newpath, flags);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return 0;
}
#endif

inline int Real_linkat(int& outError, int olddirfd, const char* oldpath, int newdirfd, const char* newpath, int flags)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_linkat, olddirfd, (long)oldpath, newdirfd, (long)newpath, flags);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return 0;
#else
	int r = TRUE_WRAPPER(linkat)(olddirfd, oldpath, newdirfd, newpath, flags);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_link(int& outError, const char* oldpath, const char* newpath)
{
#if PLATFORM_LINUX
	return Real_linkat(outError, AT_FDCWD, oldpath, AT_FDCWD, newpath, 0);
#else
	int r = TRUE_WRAPPER(link)(oldpath, newpath);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_symlinkat(int& outError, const char* target, int newdirfd, const char* linkpath)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_symlinkat, (long)target, newdirfd, (long)linkpath, 0);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return 0;
#else
	int r = TRUE_WRAPPER(symlinkat)(target, newdirfd, linkpath);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_symlink(int& outError, const char* target, const char* linkpath)
{
#if PLATFORM_LINUX
	return Real_symlinkat(outError, target, AT_FDCWD, linkpath);
#else
	int r = TRUE_WRAPPER(symlink)(target, linkpath);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_fchmodat(int& outError, int dirfd, const char* path, unsigned int mode, int flags)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_fchmodat, dirfd, (long)path, mode, flags);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return 0;
#else
	int r = TRUE_WRAPPER(fchmodat)(dirfd, path, (mode_t)mode, flags);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_chmod(int& outError, const char* path, unsigned int mode)
{
#if PLATFORM_LINUX
	return Real_fchmodat(outError, AT_FDCWD, path, mode, 0);
#else
	int r = TRUE_WRAPPER(chmod)(path, (mode_t)mode);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_fchmod(int& outError, int fd, unsigned int mode)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_fchmod, fd, mode, 0, 0);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return 0;
#else
	int r = TRUE_WRAPPER(fchmod)(fd, (mode_t)mode);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_truncate(int& outError, const char* path, long length)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_truncate, (long)path, length, 0, 0);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return 0;
#else
	int r = TRUE_WRAPPER(truncate)(path, length);
	if (r < 0) outError = errno;
	return r;
#endif
}

// fcntl's third arg is variadic and command-dependent. Caller passes the resolved arg as `long`.
inline int Real_fcntl(int& outError, int fd, int cmd, long arg)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_fcntl, fd, cmd, arg, 0);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return (int)r.r1;
#else
	int r = TRUE_WRAPPER(fcntl)(fd, cmd, arg);
	if (r < 0) outError = errno;
	return r;
#endif
}

inline int Real_kill(int& outError, int pid, int sig)
{
#if PLATFORM_LINUX
	auto r = RawSyscallDirect(SYS_kill, pid, sig, 0);
	if (r.r1 < 0) { outError = (int)-r.r1; return -1; }
	return 0;
#else
	int r = TRUE_WRAPPER(kill)(pid, sig);
	if (r < 0) outError = errno;
	return r;
#endif
}

// SYS_exit_group on Linux (terminates the whole process); _exit on macOS (no atexit handlers either way).
[[noreturn]] inline void Real_exit_group(int code)
{
#if PLATFORM_LINUX
	RawSyscallDirect(SYS_exit_group, code, 0, 0);
#else
	TRUE_WRAPPER(_exit)(code);
#endif
	__builtin_unreachable();
}

// Fire-and-forget primitives — used at fork/exec/init boundaries where errno isn't consulted.

inline int Real_fork()
{
#if PLATFORM_LINUX
	#if PLATFORM_CPU_ARM_FAMILY
	// aarch64 dropped SYS_fork; emulate via clone(SIGCHLD).
	return (int)RawSyscallDirect(SYS_clone, /*SIGCHLD*/ 17, 0, 0, 0, 0).r1;
	#else
	return (int)RawSyscallDirect(SYS_fork, 0, 0, 0).r1;
	#endif
#else
	return TRUE_WRAPPER(fork)();
#endif
}

inline int Real_getpid()
{
#if PLATFORM_LINUX
	return (int)RawSyscallDirect(SYS_getpid, 0, 0, 0).r1;
#else
	return TRUE_WRAPPER(getpid)();
#endif
}

inline int Real_prctl_pdeathsig()
{
#if PLATFORM_LINUX
	return (int)RawSyscallDirect(SYS_prctl, /*PR_SET_PDEATHSIG*/ 1, /*SIGHUP*/ 1, 0).r1;
#else
	return 0; // Mac has no PR_SET_PDEATHSIG analogue
#endif
}

// CLONE_PIDFD clone3 for host-pid tracking (see UbaDetoursSharedPosix.h). Linux 5.5+; -1 on Mac.
inline long Real_clone3_pidfd(int* outPidfd)
{
#if PLATFORM_LINUX
	struct uba_clone_args { unsigned long flags, pidfd, child_tid, parent_tid, exit_signal, stack, stack_size, tls; };
	uba_clone_args args = {};
	args.flags       = 0x1000ULL;            // CLONE_PIDFD
	args.pidfd       = (unsigned long)outPidfd;
	args.exit_signal = 17;                   // SIGCHLD
	return RawSyscallDirect(/*SYS_clone3*/ 435, (long)&args, (long)sizeof(args), 0).r1;
#else
	(void)outPidfd;
	return -1;
#endif
}

}
