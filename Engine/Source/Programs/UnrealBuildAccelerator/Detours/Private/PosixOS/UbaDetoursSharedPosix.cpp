// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaDetoursSharedPosix.h"
#include "UbaDetoursShared.h"
#include "UbaDetoursFileMappingTable.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaEvent.h"
#include "UbaFileMapping.h"
#include "UbaProcessStats.h"
#include "UbaProtocol.h"
#include "UbaStringBuffer.h"
#include "UbaPathUtils.h"
#include "UbaDetoursRealPosix.h"
#if PLATFORM_LINUX
#include "UbaLinuxRawSyscall.h"
#include <sys/syscall.h>
#endif
#if PLATFORM_MAC
#include <signal.h>
#endif

#include <errno.h>
#include <fcntl.h>
#include <stdio.h>
#include <string.h>
#include <sys/resource.h>
#include <sys/stat.h>
#include <unistd.h>

#if !UBA_STUB_BUILD
#include <stdlib.h>
#endif

#if PLATFORM_MAC
#include <mach-o/dyld.h>
#include <mach/mach.h>
#endif

// =========================================================================
// CanDetour2 — global-scope by historical convention; the LD_PRELOAD detour
// originally placed it at file scope and many call sites in
// UbaDetoursFunctionsPosix.cpp rely on the unqualified name.  Moved here so
// the static-stub build can link the same definition.
// =========================================================================
bool CanDetour2(uba::StringView file)
{
	if (!uba::g_isDetouring || uba::DisallowDetourDepth())
		return false;

	// Per-process pass-through subtrees (nsjail tmpfs / ephemeral mounts). Must run BEFORE
	// /run/ — callers pass devirtualized paths, which often start with /run/.../root/...
	if (uba::IsUnderPassThroughRoot(file))
		return false;

	if (file.StartsWith("/dev/") || file.StartsWith("/etc/"))
		return false;

	#if PLATFORM_LINUX
	if (file.StartsWith("/sys/") || file.StartsWith("/proc/") || file.Equals("/proc"))
		return false; // /proc + /sys are kernel-virtual + per-process — can't share via dirtable

	// /run/: deny only known-non-detourable subtrees. /run/user/<uid>/<scratch> is regular
	// files (nsjail bind-mount scratch under XDG_RUNTIME_DIR); blanket-denying broke it.
	if (file.StartsWith("/run/"))
	{
		file = file.Skip(5);
		return !file.StartsWith("systemd/") && !file.StartsWith("lock/") && !file.StartsWith("log/")
			&& !file.StartsWith("dbus/") && !file.StartsWith("udev/") && !file.StartsWith("mount/");
	}
	#endif

	if (uba::g_isUsingLocalBinary)
		if (file.StartsWith("/usr/lib64") || file.StartsWith("/lib64"))
			return false;

	// UBA's own session staging dirs (where stub-patched binaries live —
	// path pattern <build-root>/.uba/sessions/<timestamp>/bin/.../<hash>)
	// are NOT build artifacts. UBA's dirtable doesn't track them, so a
	// detoured openat of the staged binary itself returns ENOENT and Go
	// programs that resolve os.Executable() to look at sibling files
	// fail (observed: licenseMetadataRule reading rsp file ends up with
	// "flag needs an argument: -o"). Match the substring anywhere in
	// the path because users can put their build root anywhere.
	//
	// IMPORTANT: only exclude the bin/ subtree, NOT the entire .uba/
	// session. Sibling paths under the same session — most notably
	// `<session>/temp/<id>/` (UBA sets TMPDIR there for child
	// processes) — MUST be detoured. Otherwise opens in TMPDIR aren't
	// tracked, the file's closeId is never registered with the host's
	// dirtable, and a subsequent rename(2) on it returns ENOENT in
	// Shared_rename's Rpc_GetEntryInformation lookup. Python's
	// `_write_atomic` (used by py_compile in every precompilePython
	// edge) fails this way: it writes a temp file to TMPDIR, closes,
	// and renames over the destination — the rename can't find the
	// just-closed source if it wasn't tracked.
	if (file.Contains("/.uba/sessions/") && file.Contains("/bin/"))
		return false;

	return true;
}

namespace uba
{
	// =========================================================================
	// Globals shared between LD_PRELOAD and the static stub.
	// =========================================================================
	// On the LD_PRELOAD path UbaDetoursFunctionsPosix sets these during init.
	// On the static-stub path they're set true in UbaStaticStubCore::uba_detour_init
	// (the stub is always the only detour layer in its process).
	bool g_isDetouring;
	bool g_isUsingLocalBinary;

	u32 g_virtualUid = 1000;
	u32 g_virtualGid = 1000;

	VARIABLE_MEM(FileHandles, g_fileHandles);
	VARIABLE_MEM(ReaderWriterLock, g_fileHandlesLock);

#if UBA_STUB_BUILD
	void* g_fileHandlesStorage()     { return g_fileHandlesMem.data; }
	void* g_fileHandlesLockStorage() { return g_fileHandlesLockMem.data; }
#endif

	void InitFileHandles()
	{
		g_fileHandlesMem.Create();
		g_fileHandlesLockMem.Create();
	}

	const char* ResolveProcSelfFd(const char* pathname)
	{
		if (!pathname || !StartsWith(pathname, "/proc/self/fd/"))
			return pathname;
		const char* digits = pathname + 14;
		StringBuffer<16> fdStr;
		fdStr.Append(digits);
		u32 fd;
		if (!fdStr.Parse(fd))
			return pathname;
		ScopedReadLock lock(g_fileHandlesLock);
		auto findIt = g_fileHandles.find(int(fd));
		if (findIt == g_fileHandles.end())
			return pathname;
		FileObject* fo = findIt->second.fileObject;
		if (!fo || !fo->fileInfo || !fo->fileInfo->originalName)
			return pathname;
		return fo->fileInfo->originalName;
	}

	const char* ResolvePath(int dirfd, const char* pathname, StringBufferBase& temp)
	{
		if (pathname && *pathname == '/')
			return pathname;
		if (dirfd == AT_FDCWD)
			return pathname;
		{
			ScopedReadLock lock(g_fileHandlesLock);
			auto findIt = g_fileHandles.find(dirfd);
			if (findIt != g_fileHandles.end())
			{
				DetouredHandle& h = findIt->second;
				FileObject* fo = h.fileObject;
				if (fo && fo->fileInfo)
				{
					FileInfo& info = *fo->fileInfo;
					temp.Append(info.originalName);
					if (pathname)
						temp.EnsureEndsWithSlash().Append(pathname);
					return temp.data;
				}
			}
		}
		// Tracking-miss recovery: read /proc/self/fd/<dirfd> via direct readlink (avoids
		// recursion through Shared_prepareReadlink). nsjail child uses raw SYS_openat
		// post-pivot_root so dirfds escape our wrappers; kernel still knows the path.
		char procPath[64];
		int n = snprintf(procPath, sizeof(procPath), "/proc/self/fd/%d", dirfd);
		if (n <= 0 || (size_t)n >= sizeof(procPath))
			return nullptr;
		char linkBuf[1024];
		int err = 0;
		long rl = Real_readlink(err, procPath, linkBuf, sizeof(linkBuf) - 1);
		if (rl <= 0)
			return nullptr;
		linkBuf[rl] = 0;
		if (rl > 10 && memcmp(linkBuf + rl - 10, " (deleted)", 10) == 0) // unlinked dir: treat as miss
			return nullptr;
		temp.Append(linkBuf);
		if (pathname)
			temp.EnsureEndsWithSlash().Append(pathname);
		return temp.data;
	}

	// =========================================================================
	// GetFileAccessFlags / ToFilenameKey — pure helpers, no globals touched.
	// =========================================================================
	u8 GetFileAccessFlags(int flags)
	{
		u8 access = 0;
		int accmode = flags & O_ACCMODE;
		if (accmode == O_RDWR)         access |= AccessFlag_Read | AccessFlag_Write;
		else if (accmode == O_WRONLY)  access |= AccessFlag_Write;
		else                           access |= AccessFlag_Read;
		// O_CREAT/O_TRUNC mutate even with O_RDONLY (coreutils touch); without Write here
		// Session::CreateFile takes the read path and the agent opens the virtual path → ENOENT.
		if (flags & (O_CREAT | O_TRUNC))
			access |= AccessFlag_Write;
		return access;
	}

	StringKey ToFilenameKey(const StringBufferBase& b) { return CaseInsensitiveFs ? ToStringKeyLower(b) : ToStringKey(b); }
	StringKey ToFilenameKey(StringView b)              { return CaseInsensitiveFs ? ToStringKeyLower(b) : ToStringKey(b); }

	// Posix files aren't compressed by UBA — always false. Defined here so the
	// stub can link Shared_GetFileAttributes (which references it).
	bool CouldBeCompressedFile(const StringView&) { return false; }

	SharedReadlinkAction Shared_prepareReadlink(const char* funcName, const char* pathname, char* buf, u64 bufsiz, u64& outLen, int& error)
	{
		if (!pathname || !buf || !bufsiz)
			return SharedReadlinkAction::Passthrough;

		auto copyOut = [&](const char* src, u64 srcLen) -> SharedReadlinkAction
		{
			// POSIX readlink: silently truncate to bufsiz and DO NOT append a null byte.
			// erofs-utils (and others) size the buffer to exactly stat.st_size, so bufsiz == srcLen is normal.
			u64 n = srcLen < bufsiz ? srcLen : bufsiz;
			memcpy(buf, src, n);
			outLen = n;
			DEBUG_LOG_DETOURED(funcName, "(%s) (%.*s) -> %llu", pathname, (int)n, buf, outLen);
			return SharedReadlinkAction::Synthesized;
		};

		if (Equals(pathname, "/proc/self/exe"))
		{
			// Local: kernel-resolved binary path so self-locating launchers find sibling resources.
			// Remote: agent's /proc/self/exe is staging path; use g_virtualApplication instead.
			// VFS-overlay: kernel path is host-real (bind mounts are virtual), but tools chase
			// sibling resources relative to install dir — return virtual app path so DevirtualizePath
			// then resolves them to the right host files.
			if (!g_runningRemote)
			{
#if PLATFORM_LINUX
				if (IsVfsEnabled() && g_virtualApplication.count > 0)
					return copyOut(g_virtualApplication.data, g_virtualApplication.count);
				return SharedReadlinkAction::Passthrough;
#elif PLATFORM_MAC
				uint32_t sz = u32(bufsiz);
				if (_NSGetExecutablePath(buf, &sz) != 0)
					return SharedReadlinkAction::Passthrough;
				outLen = strlen(buf);
				DEBUG_LOG_DETOURED(funcName, "(%s) (%s) -> %llu", pathname, buf, outLen);
				return SharedReadlinkAction::Synthesized;
#else
				return SharedReadlinkAction::Passthrough;
#endif
			}
			return copyOut(g_virtualApplication.data, g_virtualApplication.count);
		}

		if (Equals(pathname, "/proc/self/cwd"))
		{
			u64 len = g_virtualWorkingDir.count;
			if (len > 1 && g_virtualWorkingDir.data[len - 1] == '/')
				--len;
			return copyOut(g_virtualWorkingDir.data, len);
		}

		if (StartsWith(pathname, "/proc/self/fd/"))
		{
			const char* original = ResolveProcSelfFd(pathname);
			if (original == pathname)
				return SharedReadlinkAction::Passthrough; // fd not tracked
			return copyOut(original, strlen(original));
		}

		// /proc/* (other than the cases above): kernel-authoritative; never
		// run UBA's dirtable lookup on these — that would issue
		// Rpc_UpdateDirectory("/proc/self") which the server rejects.
		if (StartsWith(pathname, "/proc/"))
			return SharedReadlinkAction::Passthrough;

		UBA_ASSERTF(!StartsWith(pathname, "/UEVFS"), TC("Need to devirtualize %s"), pathname);

		// DIRTABLE is ground truth — serve symlinks from the dirtable in all modes. Kernel
		// readlinkat fails for UBA-virtualized dirfds (CreateDirFd backs them with /dev/null,
		// so kernel returns ENOTDIR). followSymlink=false to surface S_IFLNK; synthesize from
		// stored target, else EINVAL.
		{
			StringBuffer<> fixedPath;
			if (FixPath(fixedPath, pathname) && CanDetour2(fixedPath))
			{
				StringBuffer<> devirtualized(fixedPath);
				DevirtualizePath(devirtualized);
				FileAttributes fileAttr;
				Shared_GetFileAttributes(fileAttr, devirtualized, /*followSymlink=*/false);
				if (fileAttr.exists)
				{
				#if UBA_SUPPORT_SYMLINKS
					if (IsSymlink(fileAttr.data.st_mode))
					{
						DirTableOffset offset;
						StringKey key = ToFilenameKey(devirtualized);
						if (Rpc_GetEntryOffset(offset, key, devirtualized, false))
						{
							StringBuffer<MaxPath> target;
							if (g_directoryTable->GetEntrySymlink(target, offset))
								return copyOut(target.data, target.count);
						}
					}
				#endif
					error = EINVAL;
					DEBUG_LOG_DETOURED(funcName, "(%s) -> -1 (EINVAL, not a symlink)", pathname);
					return SharedReadlinkAction::Error;
				}
				// Unbound parent of bind mounts: real kernel synthesizes the intermediate dir;
				// without this, `readlink -f` on it returns empty → envsetup.sh gettop infinite loop.
				if (IsVfsParentDir(fixedPath))
				{
					error = EINVAL;
					DEBUG_LOG_DETOURED(funcName, "(%s) -> -1 (EINVAL, vfs-parent dir)", pathname);
					return SharedReadlinkAction::Error;
				}
			}
		}

		return SharedReadlinkAction::Passthrough;
	}

	SharedDeleteAction Shared_prepareDelete(const char* funcName, const char* pathname, int& error, StringBufferBase& fixedNameOut)
	{
		pathname = ResolveProcSelfFd(pathname);
		FixPath(fixedNameOut, pathname);
		// Snapshot NS-virtual form; pass-through targets resolve only inside the namespace
		// (devirt'd /run/.../root/tmp/... is invisible there). Hand back pre-Devirt on Passthrough.
		StringBuffer<MaxPath> preDevirt(fixedNameOut);
		DevirtualizePath(fixedNameOut);
		StringBufferBase& fixedName = fixedNameOut;

		if (!CanDetour2(fixedName))
		{
			fixedNameOut.Clear().Append(preDevirt);
			return SharedDeleteAction::Passthrough;
		}

		StringKey fileNameKey = ToFilenameKey(fixedName);

		// followSymlink=false: unlink targets the symlink itself; symlink-to-dir must NOT be EISDIR.
		DirectoryTableEntryInformation entryInfo;
		if (!Rpc_GetEntryInformation(entryInfo, fileNameKey, fixedName, false, /*followSymlink=*/false))
		{
			DEBUG_LOG_DETOURED(funcName, "(DIRTABLE) %s (%s) -> Error (ENOENT)", pathname, fixedName.data);
			error = ENOENT;
			return SharedDeleteAction::Error;
		}

		// Real directory must return EISDIR (kernel spec for unlink / unlinkat without AT_REMOVEDIR).
		if (IsDirectory(entryInfo.attributes))
		{
			DEBUG_LOG_DETOURED(funcName, "(DIRTABLE) %s (%s) -> Error (EISDIR)", pathname, fixedName.data);
			error = EISDIR;
			return SharedDeleteAction::Error;
		}

		u32 closeId = 0;
		DirTableSize directoryTableSize;
		{
			RPC_MESSAGE(DeleteFile, deleteFile);
			writer.WriteString(fixedName);
			writer.WriteStringKey(fileNameKey);
			writer.WriteU32(closeId);
			BinaryReader reader = writer.Flush();
			// Drain the response fields the host writes (kept in the wire
			// format for backward compatibility / debugging) but do NOT
			// surface result/errorCode to the caller. By the time we get
			// here, the dirtable said this path exists, so the caller has
			// already validated success. The host's actual disk delete
			// might fail internally (e.g. uba::DeleteFileW returning
			// ENOTEMPTY because of stale on-disk content the dirtable
			// doesn't know about) — that is a host-internal concern, not
			// something we expose to the detoured process. The overlay is
			// already updated by RegisterOverlayDelete on the host so the
			// dirtable view stays consistent: future stats see "deleted".
			reader.ReadBool();
			reader.ReadU32();
			directoryTableSize = FromU64(reader.ReadU64());
		}

		g_directoryTable->ParseDirectoryTable(directoryTableSize);
		g_mappedFileTable.SetDeleted(fileNameKey, fixedName.data, true);

		DEBUG_LOG_DETOURED(funcName, "%s (%s) -> 0", pathname, fixedName.data);
		return SharedDeleteAction::Success;
	}

	int Shared_fchdir(int fd, int& error)
	{
		ScopedReadLock lock(g_fileHandlesLock);
		auto it = g_fileHandles.find(fd);
		if (it == g_fileHandles.end())
		{
			error = 9; // EBADF
			DEBUG_LOG_TRUE("fchdir", "fd=%d -> -1 (EBADF, untracked)", fd);
			return -1;
		}
		FileObject* fo = it->second.fileObject;
		if (!fo || !fo->fileInfo || !fo->fileInfo->originalName)
		{
			error = 9; // EBADF
			DEBUG_LOG_TRUE("fchdir", "fd=%d -> -1 (EBADF, no fileInfo)", fd);
			return -1;
		}
		const char* path = fo->fileInfo->originalName;
		lock.Leave();
		return Shared_chdir("fchdir", path, error);
	}

	const char* StrError(int res, int error)
	{
		if (res != -1)
			return "Success";
		return strerror(error);
	}

	DetouredHandle& CreateFileTableEntry(int fd, const SharedOpenPrep& prep, bool isDirectory, const char* funcName, const char* hint)
	{
		UBA_ASSERT(prep.fileInfo);
		ScopedWriteLock lock(g_fileHandlesLock);
		auto insres = g_fileHandles.insert({ fd, DetouredHandle() });
		DetouredHandle& h = insres.first->second;
		UBA_ASSERTF(insres.second, TC("File handle %i already added. %s - %s (%s)"), fd, funcName, (h.fileObject->fileInfo ? h.fileObject->fileInfo->originalName : "<nofileinfo>"), hint);
		auto fo = new FileObject();
		fo->closeId = prep.closeId;
		fo->fileInfo = prep.fileInfo;
		fo->desiredAccess = prep.desiredAccess;
		fo->isDirectory = isDirectory;
		h.fileObject = fo;
		h.hasCloexec = (prep.trueFlags & O_CLOEXEC) != 0;
		return h;
	}

#if PLATFORM_LINUX
	// linux_dirent64 layout (matches man getdents64). d_reclen padded to 8-byte boundary.
	#pragma pack(push, 1)
	struct UbaLinuxDirent64
	{
		u64  d_ino;
		s64  d_off;
		u16  d_reclen;
		u8   d_type;
		char d_name[]; // null-terminated filename
	};
	#pragma pack(pop)

	long Shared_getdents64(const char* funcName, int fd, void* buf, unsigned int count, int* outErrno)
	{
		DirInfo* dirInfo = nullptr;
		{
			SCOPED_READ_LOCK(g_fileHandlesLock, lock);
			auto it = g_fileHandles.find(fd);
			if (it == g_fileHandles.end())
				return -1; // not tracked: fall back to kernel
			FileObject* fo = it->second.fileObject;
			if (!fo || !fo->dirInfo)
				return -1;
			dirInfo = fo->dirInfo;
		}

		u8* const base = (u8*)buf;
		u8* out = base;
		char nameBuf[256];

		while (dirInfo->it + 1 < (int)dirInfo->fileTableOffsets.size())
		{
			int next = dirInfo->it + 1;
			DirTableOffset off = dirInfo->fileTableOffsets[next];

			DirectoryTableEntryInformation info;
			g_directoryTable->GetEntryInformation(info, off, nameBuf, sizeof(nameBuf));
			if (info.attributes == 0) // deleted
			{
				dirInfo->it = next;
				continue;
			}

			u16 nameLen = (u16)strlen(nameBuf);
			u16 fixed  = (u16)offsetof(UbaLinuxDirent64, d_name);
			u16 reclen = (u16)((fixed + nameLen + 1 + 7) & ~7u);

			if ((u64)(out - base) + reclen > count)
			{
				if (out == base)
				{
					*outErrno = EINVAL; // buffer too small for even one entry
					DEBUG_LOG_DETOURED(funcName, "(fd=%i, count=%u) -> -1 (EINVAL: first entry needs %u bytes)", fd, count, reclen);
					return -1;
				}
				break; // return what we've filled so far; caller will re-issue
			}

			auto* e = (UbaLinuxDirent64*)out;
			e->d_ino    = info.fileIndex ? info.fileIndex : 1;
			e->d_off    = (s64)next + 1;
			e->d_reclen = reclen;
			// fts (GNU rm -rf, find, etc.) uses d_type as a fast-path to skip lstat. Misreporting
			// S_IFLNK as DT_REG works for unlink but trips up FTS_LOGICAL callers and some
			// directory walkers; report DT_LNK so consumers route through their symlink path.
			e->d_type   = S_ISDIR(info.attributes) ? DT_DIR : (S_ISLNK(info.attributes) ? DT_LNK : DT_REG);
			memcpy(e->d_name, nameBuf, nameLen);
			memset(e->d_name + nameLen, 0, reclen - fixed - nameLen);

			out += reclen;
			dirInfo->it = next;
		}

		long written = (long)(out - base);
		*outErrno = 0;
		DEBUG_LOG_DETOURED(funcName, "(fd=%i, count=%u) -> %ld (pos=%i/%zu)", fd, count, written, dirInfo->it + 1, dirInfo->fileTableOffsets.size());
		return written;
	}
#endif // PLATFORM_LINUX

	int Shared_chdir(const char* func, const char* path, int& error)
	{
		size_t pathlen = strlen(path);
		if (pathlen >= g_virtualWorkingDir.capacity)
		{
			error = ENAMETOOLONG;
			DEBUG_LOG_TRUE(func, "(TOO_LONG) %s -> -1", path);
			return -1;
		}
		StringBuffer<> fixedName;
		FixPath(fixedName, StringView(path, pathlen));
		UBA_ASSERT(fixedName.count < g_virtualWorkingDir.capacity);

		memcpy(g_virtualWorkingDir.data, fixedName.data, fixedName.count + 1);
		g_virtualWorkingDir.count = fixedName.count;
#if !UBA_STUB_BUILD
		// Some glibc-using tools read PWD directly instead of
		// calling getcwd. Keep it in sync on the LD_PRELOAD path.
		// The static-stub build has no glibc / setenv.
		setenv("PWD", g_virtualWorkingDir.data, 1);
#endif
		g_virtualWorkingDir.EnsureEndsWithSlash();
		DEBUG_LOG_TRUE(func, "%s -> 0", fixedName.data);
		return 0;
	}

	int ProcessCreator::SendMessage(MessageType messageType, const char* messageName, Timer& messageTimer, const char* path, char* const argv[], char* const envp[], bool fixupPath, bool isExec)
	{
		// NULL envp is valid POSIX (bash `env -i`, nsjail wrappers); treat as empty env.
		static char* const s_emptyEnvp[1] = { nullptr };
		if (!envp)
			envp = s_emptyEnvp;

		fixedArgv = argv;

		auto prefixArgv = [&](u32 prefixCount, u32 skipCount)
			{
				UBA_ASSERT(fixedArgv != tempArgv.data());
				u32 count = 0;
				for (int i = skipCount; argv[i]; ++i)
					++count;
				tempArgv.resize(count + prefixCount + 1);
				for (int i = skipCount; argv[i]; ++i)
					tempArgv[i - skipCount + prefixCount] = argv[i];
				tempArgv[count + prefixCount] = nullptr;
				fixedArgv = (char* const*)tempArgv.data();
			};

		if (fixupPath)
		{
			// Apple clang's out-of-process cc1 dispatch (taken whenever DYLD_INSERT_LIBRARIES
			// is set — i.e. always under UBA) calls posix_spawn(path="", argv[0]="",
			// argv[1]="-cc1", …): the empty path/argv[0] is the self-exec shape used by
			// LLVM's Unix Program.inc when re-spawning the current binary. Plain libSystem
			// rejects empty path with ENOENT, which is why the test fails. Substitute the
			// current process's executable (g_virtualApplication) so PrepareProcess on the
			// host can resolve it.
			if ((!path || !*path) && (!argv[0] || !*argv[0]) && g_virtualApplication.count > 0)
				FixPath(result, g_virtualApplication);
			else if (!path || !*path)
				FixPath(result, argv[0]);
			else
				FixPath(result, path);

			DevirtualizePath(result);
			path = result.data;

			// If the executable is a shell script, wrap it with an explicit
			// interpreter so the kernel doesn't need to parse the shebang
			// — and so UBA's virtual FS interception doesn't prevent the
			// script from being opened by the kernel's exec path. Use
			// /bin/bash rather than /bin/sh because Android's build
			// scripts (e.g. build/soong/scripts/strip.sh) declare
			// #!/bin/bash and use bash-only features like
			// `set -o pipefail`. On distros where /bin/sh is dash
			// (Debian/Ubuntu), running these scripts through /bin/sh
			// fails immediately with "Illegal option -o pipefail" before
			// any real work. /bin/bash is universally available on the
			// systems UBA targets, and bash respects sh-mode behavior
			// when invoked with sh-only inputs.
			u32 plen = result.count;
			if (plen >= 3 && result.data[plen-3] == '.' && result.data[plen-2] == 's' && result.data[plen-1] == 'h')
			{
				prefixArgv(2, 1);
				tempArgv[0] = "/bin/bash";
				tempArgv[1] = path; // the .sh script (still in result.data)
				path = "/bin/bash";
			}
		}
		else if (path && strchr(path, '/'))
		{
			// execvp-style (no fixup for bare filenames), but if the path contains a slash
			// it may still be a virtualized absolute/relative path — devirtualize it.
			result.Append(path);
			DevirtualizePath(result);
			path = result.data;
		}

		for (u32 i = 1; fixedArgv[i]; ++i)
		{
			if (i != 1)
				cmdLineWithoutApplication.append(" ");
			const char* arg = fixedArgv[i];
			// Quote arguments that contain spaces or double-quotes so ParseArguments
			// on the receiving end reconstructs the correct argv (e.g. sed -e scripts).
			bool needsQuoting = false;
			for (const char* p = arg; *p; ++p)
				if (*p == ' ' || *p == '\t' || *p == '"') { needsQuoting = true; break; }
			if (needsQuoting)
			{
				cmdLineWithoutApplication.append("\"");
				for (const char* p = arg; *p; ++p)
				{
					if (*p == '"') cmdLineWithoutApplication.append("\\\"");
					else           cmdLineWithoutApplication += *p;
				}
				cmdLineWithoutApplication.append("\"");
			}
			else
				cmdLineWithoutApplication.append(arg);
		}

		RPC_MESSAGE2(messageType, messageName, messageTimer);
		writer.WriteString(path); // Application
		writer.WriteLongString(cmdLineWithoutApplication, CommunicationMemSize - 4096);
		writer.WriteString(g_virtualWorkingDir); // Current dir
		writer.WriteBool(false); // Start suspended
		writer.WriteBool(true); // Is child

		BinaryReader reader = writer.Flush();
		processId = reader.ReadU32();

		if (!processId)
		{
			DEBUG_LOG("Rpc %s failed for application %s", messageName, path);
			// Two callers reach this:
			//   - Disconnected session (rare; the worker is dying anyway).
			//   - Host saw PrepareProcess fail because the binary file does
			//     not exist (Python's _posixsubprocess.fork_exec PATH walk
			//     calls execve once per candidate; most return ENOENT).
			// Returning -ENOENT (= -2) lets the LD_PRELOAD execve wrapper
			// translate it to errno=ENOENT so the caller's PATH-walk loop
			// continues to the next candidate. Stub callers receive the
			// same negative errno and translate identically.
			return -ENOENT;
		}

		StringBuffer<1024> temp;

		rulesStr = temp.Append("UBA_RULES=").AppendValue(reader.ReadU32()).ToStringAndClear();

		StringBuffer<256> detoursLibPath;
		reader.ReadString(detoursLibPath);

		virtualApplication = reader.ReadString();

		currentDir = temp.Append("UBA_CWD=").ReadString(reader).ToStringAndClear();
		realApplication = reader.ReadString();

		#if UBA_USE_NATIVE_MAC_SEMAPHORES
		pidVar = temp.Append("UBA_PID=").AppendValue(reader.ReadU32()).ToStringAndClear();
		#endif

		comIdVar = temp.Append("UBA_COMID=").AppendValue(reader.ReadU64()).Append('+').AppendValue(reader.ReadU32()).ToStringAndClear();
		logFile = temp.Append("UBA_LOGFILE=").ReadString(reader).ToStringAndClear();
		RPC_MESSAGE_END;

		// Shebang? Skip the basename-mismatch heuristic for UBA-staged binaries whose
		// realApplication basename is a hash: /1111/ (patched), /output/ + /exec_libs/ (writtenFiles).
		StringView sv(realApplication);
		if (!sv.GetFileName().Equals(ToView(path).GetFileName()) && !sv.Contains("/1111/") && !sv.Contains("/output/") && !sv.Contains("/exec_libs/")) // TODO: breaks if app is both patched and a shebang
		{
			DEBUG_LOG("Application is a shebang (%s -> %s)", path, virtualApplication.c_str());
			prefixArgv(1, 0);
			tempArgv[0] = virtualApplication.c_str();
		}

		#if PLATFORM_LINUX
		constexpr auto libPathName = TCV("LD_LIBRARY_PATH=");
		#else
		constexpr auto libPathName = TCV("DYLD_LIBRARY_PATH=");
		#endif

		StringView callerLibPath;
		bool hasLdPreload = false;
		bool hasTmpDir = false;

		for (u32 i = 0; envp[i]; ++i)
		{
			// Remove all UBA_ variables (some distros strip them, some don't — easier to just redo) EXCEPT the
			// file-mapping backing selection, which is session-constant and must reach every descendant verbatim.
			if (StartsWith(envp[i], "UBA_") && !StartsWith(envp[i], "UBA_FILE_MAPPING_"))
				continue;

			if (StartsWith(envp[i], libPathName.data, false))
			{
				callerLibPath = ToView(envp[i]).Skip(libPathName.count);
				continue;
			}
			hasLdPreload = hasLdPreload || StartsWith(envp[i], "LD_PRELOAD=", false);
			hasTmpDir = hasTmpDir || StartsWith(envp[i], "TMPDIR=", false);

			envvars.push_back(envp[i]);
		}

		{
			temp.Append(libPathName);
			bool isFirst = true;
			auto append = [&](StringView s) { if (!isFirst) temp.Append(':'); temp.Append(s); isFirst = false; };
			if (!callerLibPath.Contains(detoursLibPath))
				append(detoursLibPath);
			if (g_runningRemote && !callerLibPath.Contains(realApplication))
				append(ToView(realApplication).GetPath());
			if (callerLibPath.count) // TODO: DO WE REALLY WANT THIS? We will end up with the parent's directory path as well
				append(callerLibPath);
			ldLibraryPath = temp.ToStringAndClear();
		}

		if (g_runningRemote)
			envvars.push_back("UBA_REMOTE=1");

		// EnvLookup against envp[] instead of getenv() so this code works in
		// the static stub (no libc). LD_PRELOAD also forwards through envp,
		// so behaviour is identical.
		const char* sessionProc = EnvLookup(envp, "UBA_SESSION_PROCESS=");
		envvars.push_back((sessionProcess = temp.Append("UBA_SESSION_PROCESS=").Append(sessionProc ? sessionProc : "").ToStringAndClear()).data());

		if (!hasLdPreload)
			envvars.push_back("LD_PRELOAD=libUbaDetours.so");

		if (!hasTmpDir)
		{
			const char* tmpdirEnv = EnvLookup(envp, "TMPDIR=");
			envvars.push_back((tmpDir = temp.Append(TCV("TMPDIR=")).Append(tmpdirEnv ? tmpdirEnv : "").ToStringAndClear()).data());
		}

		UBA_ASSERT(!g_runningRemote || !ldLibraryPath.empty());

		envvars.push_back(comIdVar.data());
		envvars.push_back(currentDir.data());
		envvars.push_back(rulesStr.data());
		envvars.push_back(logFile.data());
		envvars.push_back(ldLibraryPath.data());

		if (isExec)
			envvars.push_back("UBA_EXEC=1");
		//envvars.push_back("LD_DEBUG=bindings");
		//envvars.push_back("LD_DEBUG=libs");

		#if UBA_USE_NATIVE_MAC_SEMAPHORES
		envvars.push_back(pidVar.data);
		#endif

		envvars.push_back(nullptr);

		#if 0
		DEBUG_LOG("SPAWNING %s", envvars.data()[0]);
		for (int i=1;envvars.data()[i]; ++i)
			DEBUG_LOG("env: %s", envvars.data()[i]);
		#endif

		return 0;
	}

	void SendStartProcess(u32 processId, u32 pid, bool success, u32 res)
	{
		RPC_MESSAGE(StartProcess, startProcess);
		writer.WriteU32(processId);
		writer.WriteBool(success);
		writer.WriteU32(res);
		writer.WriteU64(1); // Process handle
		writer.WriteU32(pid);
		writer.WriteU64(0); // Thread handle
		writer.Flush();
	}

	pid_t Shared_fork()
	{
		u32 childProcessId = ~0u;
		StringBuffer<256> comId;
		StringBuffer<512> logFile;
		{
			RPC_MESSAGE(ForkProcess, forkProcess);
			for (auto& kv : g_fileHandles)
				if (u32 closeId = kv.second.fileObject->closeId)
				{
					// TODO: Should kv.second.hasCloexec be checked here? The correct fix is probably to traverse g_fileHandles just before exec and close the cloexec handles manually to make sure Rpc_UpdateCloseHandle is called
					writer.WriteU32(closeId);
					DEBUG_LOG("AddRef closeId: %u", closeId);
				}
			writer.WriteU32(0);
			BinaryReader reader = writer.Flush();
			childProcessId = reader.ReadU32();
			comId.AppendValue(reader.ReadU64()).Append('+').AppendValue(reader.ReadU32());
			reader.ReadString(logFile);
		}

		// In a non-init PID NS, fork() returns NS-local pid that the host can't waitpid() on.
		// clone3(CLONE_PIDFD) + /proc/self/fdinfo readback recovers the kernel-canonical pid.
		pid_t pid;
		u32 hostPid = 0;
		if (g_parentInNonInitNs)
		{
			int childPidfd = -1;
			long forkRet = Real_clone3_pidfd(&childPidfd);
			if (forkRet < 0)
			{
				pid = (pid_t)Real_fork(); // clone3 unavailable — host waitpid will likely ECHILD
			}
			else
			{
				pid = (pid_t)forkRet;
				if (pid > 0)
				{
					hostPid = GetHostPidFromPidfd(childPidfd);
					int err = 0; Real_close(err, childPidfd);
				}
			}
		}
		else
		{
			pid = (pid_t)Real_fork();
		}

		// Without this barrier bash sometimes fails to start downstream after fork.
		__asm__ volatile("" ::: "memory");

		if (pid == 0)
		{
			Real_prctl_pdeathsig();

			#if UBA_DEBUG_LOG_ENABLED
			g_debugLogPath.Clear().Append(logFile);
			int err = 0;
			g_debugFile = (FileHandle)Real_open(err, logFile.data, O_CREAT | O_TRUNC | O_WRONLY | O_CLOEXEC, S_IRUSR | S_IWUSR);
			#endif

			DEBUG_LOG(TC("FORKED pid: %i"), Real_getpid());

			// Inline parse of "<u64-uid>+<u32-offset>" — stub has no libc strtoul.
			const char* plusIndex = comId.First('+');
			u64 comIdUid = 0;
			for (const char* p = comId.data; p < plusIndex; ++p)
			{
				if (*p < '0' || *p > '9')
				{
					DEBUG_LOG(TC("Shared_fork: bad uid digit in comId: %s"), comId.data);
					return -1;
				}
				comIdUid = comIdUid * 10u + (u64)(*p - '0');
			}
			u32 comIdOffset = 0;
			for (const char* p = plusIndex + 1; *p >= '0' && *p <= '9'; ++p)
				comIdOffset = comIdOffset * 10u + (u32)(*p - '0');

			// Inline build of "/uba_<hex_uid>" matching AppendPrefixAndValue (2 hex/byte, stops at 0).
			char comIdName[64];
			{
				const char prefix[] = "/uba_";
				const char hexChars[] = "0123456789abcdef";
				u32 idx = 0;
				for (u32 k = 0; k < sizeof(prefix) - 1; ++k)
					comIdName[idx++] = prefix[k];
				u64 v = comIdUid;
				for (int i = 0; i < 8; ++i)
				{
					comIdName[idx++] = hexChars[(v >> 4) & 0xf];
					comIdName[idx++] = hexChars[v & 0xf];
					v = v >> 8;
					if (v == 0)
						break;
				}
				comIdName[idx] = 0;
			}
			u8* rptr = FileMapping_MapFromHost(FileMappingHandle::FromU64(comIdUid), CommunicationMemSize, comIdOffset, true, TC("ComHandle"));
			if (!rptr)
			{
				DEBUG_LOG(TC("Shared_fork: mmap failed for %llu"), comIdUid);
				return -1;
			}

			g_messageMappingMem = rptr;
			g_cancelEvent = ((SharedEvent*)rptr);
			g_readEvent   = ((SharedEvent*)rptr) + 1;
			g_writeEvent  = ((SharedEvent*)rptr) + 2;
			g_messageMappingMem += sizeof(SharedEvent) * 3;

			for (auto it=g_fileHandles.begin(); it!=g_fileHandles.end();)
			{
				if (it->second.fileObject->closeId == 0)
					it = g_fileHandles.erase(it);
				else
					++it;
			}

			// Static-stub bridge blocked all signals before SYS_fork; that mask propagates
			// across execve and starves every descendant of SIGTERM (e.g. py multiprocessing.Pool).
			// Raw syscall avoids libc/detour interaction.
			{
				u64 emptyset = 0;
				#if defined(__x86_64__)
				register long r10 __asm__("r10") = 8; // sigsetsize
				register long _r asm("rax");
				__asm__ volatile(
					"syscall"
					: "=a"(_r)
					: "0"((long)14), "D"((long)2), "S"((long)&emptyset), "d"((long)0), "r"(r10)
					: "rcx", "r11", "memory");
				#endif
			}
		}
		else
		{
			u32 reportedPid = hostPid ? hostPid : (u32)pid; // hostPid set only when pidfd readback succeeded
			SendStartProcess(childProcessId, reportedPid, true, 0);
			DEBUG_LOG("         Process forked (child pid: %i, hostPid: %u)", pid, reportedPid);
		}

		return pid;
	}

	// PID-NS support. Both helpers parse /proc text via Real_open/Real_read/Real_close so they work identically in LD_PRELOAD and stub builds.

	bool g_parentInNonInitNs = false;

	u32 GetHostPidFromPidfd(int pidfd)
	{
		#if !PLATFORM_LINUX
		(void)pidfd; return 0;
		#else
		if (pidfd < 0) return 0;

		// Build "/proc/self/fdinfo/<pidfd>"
		char path[32];
		const char prefix[] = "/proc/self/fdinfo/";
		u32 idx = 0;
		for (u32 k = 0; k < sizeof(prefix) - 1; ++k) path[idx++] = prefix[k];
		char tmp[12]; u32 t = 0;
		int n = pidfd;
		if (n == 0) { tmp[t++] = '0'; }
		else { while (n > 0) { tmp[t++] = char('0' + (n % 10)); n /= 10; } }
		while (t > 0) path[idx++] = tmp[--t];
		path[idx] = 0;

		int err = 0;
		int fd = Real_open(err, path, O_RDONLY | O_CLOEXEC, 0);
		if (fd < 0) return 0;
		char buf[1024];
		long r = Real_read(err, fd, buf, sizeof(buf) - 1);
		Real_close(err, fd);
		if (r <= 0) return 0;
		buf[r] = 0;

		const char* p = buf;
		const char* end = buf + r;
		while (p < end)
		{
			if (p[0] == 'N' && p + 6 <= end && p[1] == 'S' && p[2] == 'p' && p[3] == 'i' && p[4] == 'd' && p[5] == ':')
			{
				p += 6;
				while (p < end && (*p == ' ' || *p == '\t')) ++p;
				u32 hostPid = 0;
				while (p < end && *p >= '0' && *p <= '9') { hostPid = hostPid * 10u + u32(*p - '0'); ++p; }
				return hostPid;
			}
			while (p < end && *p != '\n') ++p;
			if (p < end) ++p;
		}
		return 0;
		#endif
	}

	bool CheckParentInNonInitPidNs()
	{
		#if !PLATFORM_LINUX
		return false;
		#else
		int err = 0;
		int fd = Real_open(err, "/proc/self/status", O_RDONLY | O_CLOEXEC, 0);
		if (fd < 0) return false;
		char buf[4096];
		long r = Real_read(err, fd, buf, sizeof(buf) - 1);
		Real_close(err, fd);
		if (r <= 0) return false;
		buf[r] = 0;

		// Count tab-separated values after "NSpid:" — >1 means non-init NS.
		const char* p = buf;
		const char* end = buf + r;
		while (p < end)
		{
			if (p + 6 <= end && p[0] == 'N' && p[1] == 'S' && p[2] == 'p' && p[3] == 'i' && p[4] == 'd' && p[5] == ':')
			{
				p += 6;
				u32 valueCount = 0;
				while (p < end && *p != '\n')
				{
					if (*p == ' ' || *p == '\t') { ++p; continue; }
					if (*p >= '0' && *p <= '9') { ++valueCount; while (p < end && *p >= '0' && *p <= '9') ++p; continue; }
					++p;
				}
				return valueCount > 1;
			}
			while (p < end && *p != '\n') ++p;
			if (p < end) ++p;
		}
		return false;
		#endif
	}

	// BinaryWriter(ProcessCommunication) + Flush — single implementation shared by LD_PRELOAD and the static stub via Real_kill / Real_exit_group.
	constexpr u32 WritableMemSize = CommunicationMemSize - sizeof(SharedEvent) * 3;

	BinaryWriter::BinaryWriter(ProcessCommunication)
	{
		m_begin = g_messageMappingMem;
		m_pos   = m_begin;
		m_end   = m_begin + WritableMemSize;

		#if UBA_PROTOCOL_GUARD
		WriteU32(0xdeadbeef);
		#endif
	}

	BinaryReader BinaryWriter::Flush(bool waitOnResponse)
	{
		#if UBA_PROTOCOL_GUARD
		if (BinaryReader(g_messageMappingMem, 0, WritableMemSize).ReadU32() != 0xdeadbeef)
			Real_exit_group(2345);
		#endif

		g_writeEvent->Set();

		if (!waitOnResponse)
			return BinaryReader(nullptr, 0, 0);

		TimerScope ts(g_stats.waitOnResponse);
		while (true)
		{
			if (g_readEvent->IsSet(1000))
				break;

			// Non-init PID NS: host pid invisible, kill(_,0) always ESRCHes — skip the probe.
			if (g_sessionPid > 0 && !g_parentInNonInitNs)
			{
				int err = 0;
				if (Real_kill(err, g_sessionPid, 0) == -1 && err == ESRCH)
				{
					g_isCancelled = true;
					Real_exit_group(1337);
				}
			}

			if (g_cancelEvent && g_cancelEvent->IsSet(0))
			{
				g_isCancelled = true;
				Real_exit_group(1339);
			}
		}

		BinaryReader reader(g_messageMappingMem, 0, WritableMemSize);

		#if UBA_PROTOCOL_GUARD
		if (reader.ReadU32() != 0xbaadf00d)
			Real_exit_group(2345);
		BinaryWriter(g_messageMappingMem, 0, CommunicationMemSize).WriteU32(0xcafebabe);
		#endif

		m_begin = nullptr;
		m_pos   = 0;
		m_end   = nullptr;
		return reader;
	}

	// Init driver — wire-format parse + table mapping + DirectoryTable adopt + env adoption. Shared between LD_PRELOAD's Init() and the stub's DetourCore.
	// Mirror of HandleInit() in UbaProcess.cpp — field mismatch desynchronises the stream and every later RPC reads corrupt data.
	void Shared_Init(char** envp)
	{
		#if !UBA_STUB_BUILD
		extern bool g_isInitialized;
		UBA_ASSERTF(!g_isInitialized, "Already initialized");
		g_isInitialized = true;
		#endif

		bool trackInputs;
		FileMappingHandle directoryTableHandle, mappedFileTableHandle, overlayTableHandle, directoryTableSnapshotHandle;
		DirTableSize directoryTableSize;
		u32 directoryTableCount, mappedFileTableSize, mappedFileTableCount, directoryTableSnapshotSize;
		u64 directoryTableSnapshotAddress;
		u16 vfsSize = 0;
		const u8* vfsBytes = nullptr;
		#if UBA_STUB_BUILD
		char virtualApp[512];     u64 virtualAppLen     = 0; (void)virtualApp; (void)virtualAppLen; // g_virtualApplication intentionally NOT populated — see comment in else-branch below
		char virtualWorkDir[512]; u64 virtualWorkDirLen = 0;
		#endif

		{
			RPC_MESSAGE(Init, init);
			BinaryReader reader = writer.Flush();

			g_processId = reader.ReadU32();
			g_isChild   = reader.ReadByte() != 0;
			trackInputs = reader.ReadByte() != 0;
			#if !UBA_STUB_BUILD
			reader.ReadString(g_virtualApplication);
			reader.ReadString(g_virtualWorkingDir);
			#else
			virtualAppLen     = reader.ReadString(virtualApp,     sizeof(virtualApp));
			virtualWorkDirLen = reader.ReadString(virtualWorkDir, sizeof(virtualWorkDir));
			#endif
			u64 sharedMemoryAllocatorHandle = reader.ReadU64();
			#if !UBA_STUB_BUILD
			extern SharedMemoryAllocatorHandle g_sharedMemoryAllocator;
			g_sharedMemoryAllocator = SharedMemoryAllocatorHandle::FromU64(sharedMemoryAllocatorHandle);
			#else
			(void)sharedMemoryAllocatorHandle; // stub doesn't allocate via the host
			#endif
			reader.ReadU64(); // permanent files memory handle (discarded)
			directoryTableHandle          = FileMappingHandle::FromU64(reader.ReadU64());
			directoryTableSize            = FromU64(reader.ReadU64());
			directoryTableCount           = reader.ReadU32();
			mappedFileTableHandle         = FileMappingHandle::FromU64(reader.ReadU64());
			mappedFileTableSize           = reader.ReadU32();
			mappedFileTableCount          = reader.ReadU32();
			directoryTableSnapshotHandle  = FileMappingHandle::FromU64(reader.ReadU64());
			directoryTableSnapshotAddress = reader.ReadU64();
			directoryTableSnapshotSize    = reader.ReadU32();
			overlayTableHandle            = FileMappingHandle::FromU64(reader.ReadU64());
			vfsSize                       = reader.ReadU16();
			if (vfsSize) vfsBytes = reader.GetPositionData();

			DEBUG_LOG_PIPE(L"Init", L"");
		}

		// Unified getenv. Stub scans envp (no libc getenv); LD_PRELOAD uses libc but Shared_AdoptInheritedHandles also checks ::environ directly because libc's
		// getenv cache can be empty at detour-init time on glibc — see the .inl for the fallback logic.
		auto getenvImpl = [envp](const char* k) -> const char* {
			#if UBA_STUB_BUILD
			if (!envp || !k) return nullptr;
			u32 keyLen = 0; while (k[keyLen]) ++keyLen;
			for (u32 i = 0; envp[i]; ++i)
			{
				const char* e = envp[i];
				u32 j = 0;
				while (j < keyLen && e[j] && e[j] == k[j]) ++j;
				if (j == keyLen && e[j] == '=') return e + j + 1;
			}
			return nullptr;
			#else
			(void)envp; return getenv(k);
			#endif
		};

		// VFS overlays + pass-through roots BEFORE VirtualizePath — /proc/self/exe + dladdr + libclang resource-dir would otherwise resolve against the host tree.
		if (u32 n = Shared_AdoptVfsEntries(getenvImpl))
			DEBUG_LOG(TC("Adopted %u VFS overlay entr%s from UBA_VFS_OVERLAYS"), n, n == 1 ? TC("y") : TC("ies"));
		if (u32 n = Shared_AdoptPassThroughRoots(getenvImpl))
			DEBUG_LOG(TC("Adopted %u pass-through root(s) from UBA_PASSTHROUGH_ROOTS"), n);

		#if !UBA_STUB_BUILD
		// VFS bytes from the Init response (LD_PRELOAD-only — the stub adopts VFS entirely via UBA_VFS_OVERLAYS env).
		if (vfsSize)
		{
			BinaryReader vfsReader((u8*)vfsBytes, 0, vfsSize);
			PopulateVfs(vfsReader);
		}

		VirtualizePath(g_virtualApplication);
		VirtualizePath(g_virtualWorkingDir);
		VirtualizePath(g_exeDir);

		UBA_ASSERTF(g_virtualApplicationDir.capacity > 0, "g_virtualApplicationDir.capacity > 0");
		const char* lastSlash = strrchr(g_virtualApplication.data, '/');
		UBA_ASSERTF(lastSlash, "Need fullpath for application (%s)", g_virtualApplication.data);
		g_virtualApplicationDir.Append(g_virtualApplication.data, lastSlash - g_virtualApplication.data + 1);
		if (g_virtualWorkingDir[g_virtualWorkingDir.count - 1] == PathSeparator)
			g_virtualWorkingDir.Resize(g_virtualWorkingDir.count - 1);
		setenv("PWD", g_virtualWorkingDir.data, 1);
		g_virtualWorkingDir.EnsureEndsWithSlash();
		if (g_runningRemote && g_virtualApplicationDir.StartsWith(g_exeDir)) // local-binary mode keeps dlopen on the host tree
			g_isUsingLocalBinary = true;

		// LogHeader: inlined here. /proc/self/cmdline is NUL-separated argv → replace NULs with spaces for the CmdLine line.
		#if UBA_DEBUG_LOG_ENABLED
		{
			static StringBuffer<128 * 1024> cmdBuf;
			cmdBuf.Clear();
			int err = 0;
			int fd = Real_open(err, "/proc/self/cmdline", O_RDONLY, 0);
			if (fd != -1)
			{
				long n = Real_read(err, fd, cmdBuf.data, cmdBuf.capacity);
				if (n > 0)
				{
					char* it = cmdBuf.data;
					while (*it) { it += strlen(it); *it = ' '; ++it; }
					cmdBuf.Resize(u32(it - cmdBuf.data - 1));
				}
				Real_close(err, fd);
			}
			LogHeader(cmdBuf);
		}
		#endif
		#else
		// Stub: manual workdir populate. g_virtualApplication is intentionally NOT populated — populating it crashes early-init Go compile edges
		// (SIGSEGV in testfs.a / starlark_fmt.a / allowlists.a) even though the populate code is structurally identical to g_virtualWorkingDir's.
		(void)vfsBytes;
		{
			auto& wd = g_virtualWorkingDir;
			u32 n = (u32)virtualWorkDirLen;
			if (n >= wd.capacity - 1) n = wd.capacity - 2;
			for (u32 i = 0; i < n; ++i) wd.data[i] = virtualWorkDir[i];
			wd.data[n] = 0;
			wd.count = n;
			if (n > 0 && wd.data[n - 1] != '/') { wd.data[n] = '/'; wd.data[n + 1] = 0; wd.count = n + 1; }
		}
		g_isUsingLocalBinary = true; // stub is the only detour layer in its process; no nesting
		DEBUG_LOG(TC("ProcessId: %u"),       g_processId);
		DEBUG_LOG(TC("WorkingDir: %s"),      g_virtualWorkingDir.data);
		DEBUG_LOG(TC("ExeDir: %s"),          g_virtualApplicationDir.data);
		DEBUG_LOG(TC("ExeDir (actual): %s"), g_exeDir.data);
		DEBUG_LOG(TC("SystemTemp: %s"),      g_systemTemp.data);
		if (g_rules) DEBUG_LOG(TC("Rules: %u"), g_rules->index); // stub can't link GetApplicationRules
		DEBUG_LOG(TC("HostType: %s"),        g_runningRemote ? TC("Remote") : TC("Local"));
		#endif

		TimerScope ts2(g_stats.dirTable);
		u8* directoryTableMem = FileMapping_MapFromHost(directoryTableHandle, DirTableMaxSize, 0, false, "DirTable");
		u8* overlayTableMem   = FileMapping_MapFromHost(overlayTableHandle,   OverlayTableMaxSize, 0, false, "OverlayTable");

		if (directoryTableSnapshotHandle.IsValid())
		{
			u8* snapshotMem = FileMapping_MapFromHost2(FileMapping_DuplicateFromHost(directoryTableSnapshotHandle, "FileTable"),
													   directoryTableSnapshotSize, 0, MemoryMapType_CopyOnWrite,
													   (void*)directoryTableSnapshotAddress, "FileTable");

			RPC_MESSAGE(SnapshotDuplicated, init);
			writer.Flush();
			RPC_MESSAGE_END;

			if (u64(snapshotMem) == directoryTableSnapshotAddress)
			{
				g_directoryTable = (DirectoryTable*)snapshotMem;
				g_directoryTable->m_memory = directoryTableMem;
				g_directoryTable->ParseDirectoryTable({directoryTableSize.main, 0});
				DEBUG_LOG(TC("Using dir table snapshot with size %u"), g_directoryTable->m_memorySize);
			}
			else
			{
				DEBUG_LOG(TC("Failed to map snapshot (Needed address 0x%llx but got 0x%llx)"), directoryTableSnapshotAddress, u64(snapshotMem));
			}
		}

		if (!g_directoryTable)
		{
			#if !UBA_STUB_BUILD
			g_directoryTable = new DirectoryTable(g_memoryBlock);
			#else
			g_directoryTable = StubAllocDirectoryTable(g_memoryBlock);
			#endif
			g_directoryTable->Init(directoryTableMem, directoryTableCount, directoryTableSize.main);
		}
		g_directoryTable->InitOverlay(overlayTableMem, directoryTableSize.overlay);
		ts2.Leave();

		TimerScope ts1(g_stats.fileTable);
		u8* mappedFileTableMem = FileMapping_MapFromHost(mappedFileTableHandle, FileMappingTableMaxSize, 0, false, "FileTable");
		g_mappedFileTable.Init(mappedFileTableMem, mappedFileTableCount, mappedFileTableSize);
		ts1.Leave();

		#if !UBA_STUB_BUILD
		if (trackInputs)
			InitTrackInput();
		TrackInput(g_virtualApplication);
		LogVfsInfo();
		#else
		(void)trackInputs;
		#endif

		g_isDetouring = true;
		DEBUG_LOG(TC("Detouring enabled"));

		// UBA_INHERIT_HANDLES carries open-for-write fds the parent execve'd into us (bash-redirect fd-1-survives-execve).
		if (u32 n = Shared_AdoptInheritedHandles(getenvImpl))
			DEBUG_LOG(TC("Adopted %u inherited write handle(s) from UBA_INHERIT_HANDLES"), n);
	}

#if PLATFORM_LINUX
	static u64 Shared_ParseSmapsKbToBytes(const char* buf, u64 len, const char* key)
	{
		u64 keyLen = 0; while (key[keyLen]) ++keyLen;
		for (u64 i = 0; i + keyLen <= len; ++i)
		{
			if ((i == 0 || buf[i - 1] == '\n') && memcmp(buf + i, key, keyLen) == 0)
			{
				u64 j = i + keyLen;
				while (j < len && (buf[j] == ' ' || buf[j] == '\t')) ++j;
				u64 kb = 0;
				while (j < len && buf[j] >= '0' && buf[j] <= '9') { kb = kb * 10 + u64(buf[j] - '0'); ++j; }
				return kb * 1024;
			}
		}
		return 0;
	}
#endif

	// Private working set in bytes (excludes UBA's shared injected mappings; total RSS would over-count them).
	static u64 Shared_GetPrivateMemoryBytes()
	{
	#if PLATFORM_LINUX
		int err = 0;
		int fd = Real_open(err, "/proc/self/smaps_rollup", O_RDONLY, 0);
		if (fd < 0)
			return 0;
		char buf[8192];
		u64 total = 0;
		for (;;)
		{
			long n = Real_read(err, fd, buf + total, sizeof(buf) - 1 - total);
			if (n <= 0)
				break;
			total += u64(n);
			if (total >= sizeof(buf) - 1)
				break;
		}
		Real_close(err, fd);
		return Shared_ParseSmapsKbToBytes(buf, total, "Private_Clean:") + Shared_ParseSmapsKbToBytes(buf, total, "Private_Dirty:");
	#elif PLATFORM_MAC
		task_vm_info_data_t info;
		mach_msg_type_number_t count = TASK_VM_INFO_COUNT;
		if (task_info(mach_task_self(), TASK_VM_INFO, (task_info_t)&info, &count) == KERN_SUCCESS)
			return u64(info.phys_footprint);
		return 0;
	#else
		return 0;
	#endif
	}

	// User+system CPU time of this process in GetFrequency() ticks (the unit the host prints cpuTime in).
	static u64 Shared_GetCpuTimeTicks()
	{
		struct rusage ru;
	#if PLATFORM_LINUX
		if (RawSyscallDirect(SYS_getrusage, RUSAGE_SELF, (long)&ru, 0).r1 != 0)
			return 0;
	#else
		if (getrusage(RUSAGE_SELF, &ru) != 0)
			return 0;
	#endif
		u64 cpuUs = u64(ru.ru_utime.tv_sec) * 1000000 + u64(ru.ru_utime.tv_usec)
		          + u64(ru.ru_stime.tv_sec) * 1000000 + u64(ru.ru_stime.tv_usec);
		return cpuUs * (GetFrequency() / 1000000);
	}

	// MessageType_Exit wire send. Fire-and-forget — host's HandleExit cleans up. Caller does the FD flush / stats fixup before invoking.
	void Shared_SendExit(u32 exitCode)
	{
		g_stats.detoursMemory = g_memoryBlock.writtenSize;

		if (u64 privateBytes = Shared_GetPrivateMemoryBytes())
			AtomicMax(g_stats.peakMemory, privateBytes);

		AtomicMax(g_stats.cpuTime, Shared_GetCpuTimeTicks());

		RPC_MESSAGE(Exit, log)
		writer.WriteU32(exitCode);
		writer.WriteString(TC(""));
		g_stats.Write(writer);
		g_kernelStats.Write(writer);
		writer.Flush(false);
	}
}