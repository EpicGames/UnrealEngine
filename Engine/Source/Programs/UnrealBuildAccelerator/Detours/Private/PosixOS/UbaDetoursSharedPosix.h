// Copyright Epic Games, Inc. All Rights Reserved.
//
// Posix utilities shared between LD_PRELOAD (UbaDetoursFunctionsPosix.cpp) and the static-detour stub (DetoursStatic/UbaStaticStubCore.cpp).

#define UBA_IS_DETOURED_INCLUDE 1

#include "UbaBinaryReaderWriter.h"
#include "UbaDefinitions.h"
#include "UbaDetoursFileMappingTable.h"
#include "UbaDetoursShared.h"
#include "UbaDirectoryTable.h"
#include "UbaEvent.h"
#include "UbaProtocol.h"
#include "UbaString.h"
#include "UbaSynchronization.h"
#include "UbaUnorderedMap.h"
#include "UbaVector.h"
#include <dirent.h>

namespace uba
{
	// Defined in UbaDetoursFunctionsPosix.cpp (LD_PRELOAD) / UbaStaticStubCore.cpp (stub).
	u8 GetFileAccessFlags(int flags);
	StringKey ToFilenameKey(const StringBufferBase& b);
	StringKey ToFilenameKey(StringView b);

	extern bool g_isDetouring;
	extern bool g_isUsingLocalBinary;

	// Virtual UID/GID reported to detoured processes — constant across local + remote so dirtable paths like /run/user/<UID> resolve consistently
	// (kernel UID differs: WSL=1000, AL2025=990). Kernel-truth is fetched inline via raw syscall where actually needed (gid_map for userns setup).
	extern u32 g_virtualUid;
	extern u32 g_virtualGid;

	struct DirInfo
	{
		Vector<DirTableOffset> fileTableOffsets;
		int it = -1;
		int realFd = -1; // /dev/null fd registered in g_fileHandles so dirfd(DIR*) + openat(fd, "sub") works (depmod)
		dirent ent;
	};

	inline bool IsDirInfo(DIR* dir) { return (u64(dir) & 0x1000'0000'0000'0000) != 0; }
	inline DirInfo* AsDirInfo(DIR* dir) { return (DirInfo*)(uintptr_t(dir) & ~0x1000'0000'0000'0000); }

	struct FileObject
	{
		FileInfo* fileInfo = nullptr;
		DirInfo*  dirInfo  = nullptr;
		u32 refCount = 1;
		u32 closeId = 0;
		u32 desiredAccess = 0;
		bool deleteOnClose = false;
		bool ownsFileInfo = false;
		bool isDirectory = false;
		TString newName;
	};

	struct DetouredHandle
	{
		FileObject* fileObject = nullptr;
		bool hasCloexec = false; // Per-fd (kernel auto-closes at execve); skipped by Shared_BuildInheritedHandlesEnv so reused fd doesn't trip 'already added' assert.
	};

	using FileHandles = UnorderedMap<int, DetouredHandle>;

#if UBA_STUB_BUILD
	void* g_fileHandlesStorage();
	void* g_fileHandlesLockStorage();
	#define g_fileHandles      (*static_cast<::uba::FileHandles*>(::uba::g_fileHandlesStorage()))
	#define g_fileHandlesLock  (*static_cast<::uba::ReaderWriterLock*>(::uba::g_fileHandlesLockStorage()))
#else
	extern FileHandles& g_fileHandles;
	extern ReaderWriterLock& g_fileHandlesLock;
#endif

	void InitFileHandles();

	// Returns absolute path; nullptr if dirfd has no g_fileHandles entry (caller passthroughs).
	const char* ResolvePath(int dirfd, const char* pathname, StringBufferBase& temp);

	// "/proc/self/fd/<N>" → tracked originalName, else returns pathname unchanged.
	const char* ResolveProcSelfFd(const char* pathname);

	int Shared_chdir(const char* func, const char* path, int& error);
	int Shared_fchdir(int fd, int& error);
	const char* StrError(int res, int error);

	struct SharedOpenPrep;

	DetouredHandle& CreateFileTableEntry(int fd, const SharedOpenPrep& prep, bool isDirectory, const char* funcName, const char* hint);

	// Synthesizes a UBA-tracked directory fd via trueOpen("/dev/null"); getdents64 → Shared_getdents64; close → Shared_close.
	template<typename TrueOpen>
	int CreateDirFd(const SharedOpenPrep& prep, int flags, const char* funcName, const TrueOpen& trueOpen);

	// Returns: >0 bytes written, 0 EOS, -1 untracked-passthrough. EINVAL when buffer too small for one entry.
	long Shared_getdents64(const char* funcName, int fd, void* buf, unsigned int count, int* outErrno);

	enum class SharedDeleteAction
	{
		Passthrough,
		Error,
		Success,
	};

	// Passthrough callers MUST use fixedNameOut (FixPath'd absolute) — process cwd may not match g_virtualWorkingDir.
	SharedDeleteAction Shared_prepareDelete(const char* funcName, const char* pathname, int& error, StringBufferBase& fixedNameOut);

	template<typename TrueUnlink>
	int Shared_deleteFile(const char* funcName, const char* pathname, int& error, const TrueUnlink& trueUnlink);

	template<typename TrueRmdir>
	int Shared_rmdir(const char* funcName, const char* path, int& error, const TrueRmdir& trueRmdir);

	template<typename TrueMkdir>
	int Shared_mkdir(const char* funcName, const char* path, mode_t mode, int& error, const TrueMkdir& trueMkdir);

	// Shared code MUST NOT touch glibc errno directly (Go M-thread FS-register issue) — use the int& error out-param.
	template<typename TrueRename>
	int Shared_rename(const char* func, const char* oldpath, const char* newpath, int& error, const TrueRename& trueRename);

	// chmod / fchmodat(AT_FDCWD). Tracked writes go through SessionClient::Chmod (overlay-only); trueChmod only fires when !CanDetour2.
	template<typename TrueChmod>
	int Shared_chmod(const char* funcName, const char* pathname, mode_t mode, int& error, const TrueChmod& trueChmod);

	// Encode g_fileHandles entries with closeId != 0 into "UBA_INHERIT_HANDLES=<fd>:<closeId>:<pathLen>:<path>;…"; out.count = 0 if none.
	void Shared_BuildInheritedHandlesEnv(StringBufferBase& out);

	// Adopts UBA_INHERIT_HANDLES via getenv lambda; allocates FileObject{closeId, fileInfo, refCount=1} per entry. Returns count adopted.
	template<typename TrueGetenv>
	u32 Shared_AdoptInheritedHandles(const TrueGetenv& trueGetenv);

	// Increments FileObject::refCount so Shared_close waits for ALL fds to close before sending Rpc_UpdateCloseHandle.
	void DuplicateFd(int oldFd, int newFd, bool newCloexec);

	// Refcounted close. trueSnapshot must do dup→fstat→close so we measure the file AFTER stdio buffers flush.
	template<typename TrueClose, typename TrueSnapshot>
	int Shared_close(const char* funcName, int fd, const TrueClose& trueClose, const TrueSnapshot& trueSnapshot);

	// Pre-finalize a tracked write fd before a syscall that implicitly closes it (dup2/dup3 with newfd open). See .inl for rationale.
	template<typename TrueSnapshot>
	void Shared_PreFinalizeForOverwrite(const char* funcName, int fd, const TrueSnapshot& trueSnapshot);

	enum class SharedReadlinkAction
	{
		Passthrough,
		Synthesized,
		Error,
	};

	SharedReadlinkAction Shared_prepareReadlink(const char* funcName, const char* pathname, char* buf, u64 bufsiz, u64& outLen, int& error);

	enum class SharedOpenAction
	{
		Error,
		NoDetourFallback,
		ProcFallback,
		DirectoryFd,
		UnsupportedFail,
		NormalOpen,
	};

	struct SharedOpenPrep
	{
		SharedOpenAction action = SharedOpenAction::Error;
		StringBuffer<> fileName;
		StringKey fileNameKey;
		u32 desiredAccess = 0;
		bool isWrite = false;
		DirectoryTableEntryInformation entryInfo;
		bool fileExists = false;
		int err = 0;
		const char* tempFileName = nullptr;
		int trueFlags = 0;
		int trueMode = 0;
		Vector<u8> dataToWrite;
		bool isDirectoryEntry = false;
		FileInfo* fileInfo = nullptr;
		u32 closeId = 0;
	};

	template<typename TrueOpen>
	void Shared_prepareOpen(const char* funcName, const char* file, int flags, int mode, const TrueOpen& trueOpen, SharedOpenPrep& prep);

	template<typename TrueOpen>
	int Shared_open(const char* funcName, const char* file, int flags, int mode, const TrueOpen& trueOpen);

	// keyEquals MUST include the trailing '=' (e.g. "TMPDIR=").
	inline const char* EnvLookup(const char* const* envp, const char* keyEquals)
	{
		if (!envp || !keyEquals) return nullptr;
		u32 keyLen = 0;
		while (keyEquals[keyLen]) ++keyLen;
		for (u32 i = 0; envp[i]; ++i)
			if (StartsWith(envp[i], keyEquals, false)) return envp[i] + keyLen;
		return nullptr;
	}

	struct ProcessCreator
	{
		int SendMessage(MessageType messageType, const char* messageName, Timer& messageTimer, const char* path, char* const argv[], char* const envp[], bool fixupPath, bool isExec);

		TString virtualApplication;
		TString realApplication;
		TString cmdLineWithoutApplication;
		u32 processId = 0;
		TString currentDir;
		TString comIdVar;
		TString rulesStr;
		TString logFile;
		TString ldLibraryPath;
		TString sessionProcess;
		TString tmpDir;
		#if UBA_USE_NATIVE_MAC_SEMAPHORES
		TString pidVar;
		#endif
		StringBuffer<> result;
		Vector<const char*> tempArgv;
		char* const* fixedArgv;
		Vector<const char*> envvars;
	};

	// Returns 0 on success (unreachable in practice) or -errno on failure.
	template<typename TrueExecve, typename TrueNanosleepUs, typename TrueSnapshot>
	int Shared_Internal_exec(const char* func, const char* pathname,
		char* const argv[], char* const envp[], bool fixupPath,
		const TrueExecve& trueExecve, const TrueNanosleepUs& trueNanosleepUs,
		const TrueSnapshot& trueSnapshot);

	// PID-NS-aware fork helpers. Inside a non-init NS, getpid() returns NS-local pid the host can't waitpid() — gated by g_parentInNonInitNs.
	// Real_fork / Real_getpid / Real_clone3_pidfd / Real_prctl_pdeathsig live inline in UbaDetoursRealPosix.h.
	u32   GetHostPidFromPidfd(int pidfd);  // parse /proc/self/fdinfo/<pidfd> NSpid: line; 0 on fail
	bool  CheckParentInNonInitPidNs();     // true iff /proc/self/status NSpid: line has >1 value
	extern bool g_parentInNonInitNs;

	extern int          g_comFd;
	extern u8*          g_messageMappingMem;
	class SharedEvent;
	extern SharedEvent* g_cancelEvent;
	extern SharedEvent* g_readEvent;
	extern SharedEvent* g_writeEvent;
	extern int          g_sessionPid;
	extern bool         g_isCancelled;

	void SendStartProcess(u32 processId, u32 pid, bool success, u32 res);
	void Shared_SendExit(u32 exitCode);

	// MessageType_Init send + parse + post-init (table mappings, DirectoryTable adopt, env adoption). Driver for both LD_PRELOAD and the static stub.
	// envp is used only on stub (no libc getenv); LD_PRELOAD passes ::environ but the implementation prefers libc getenv directly.
	void Shared_Init(char** envp);

	#if UBA_STUB_BUILD
	class DirectoryTable;
	struct MemoryBlock;
	// Stub-side placement-new into VariableMem (we have no operator new). Defined in UbaStaticStubCore.cpp.
	DirectoryTable* StubAllocDirectoryTable(MemoryBlock& mb);
	#endif

	pid_t Shared_fork();
}

// Global scope: callers in UbaDetoursFunctionsPosix.cpp use unqualified name.
bool CanDetour2(uba::StringView file);

#include "UbaDetoursSharedPosix.inl"
