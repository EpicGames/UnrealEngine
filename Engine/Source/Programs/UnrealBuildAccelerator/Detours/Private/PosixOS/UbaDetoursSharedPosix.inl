// Copyright Epic Games, Inc. All Rights Reserved.
//
// UbaDetoursSharedPosix.inl — template/inline definitions for the
// posix detour helpers shared between LD_PRELOAD
// (UbaDetoursFunctionsPosix.cpp) and the static-detour stub
// (DetoursStatic/UbaStaticStubCore.cpp).
//
// Included by UbaDetoursSharedPosix.h. Do NOT include directly.

#pragma once

#include "UbaDetoursShared.h"           // DEBUG_LOG_*, g_directoryTable, g_mappedFileTable, FixPath, ...
#include "UbaDetoursFileMappingTable.h" // Rpc_CreateFileW / Rpc_GetEntryOffset / Rpc_UpdateDirectory
#include "UbaSharedMemoryView.h"        // IsMemoryHandle
#include "UbaPathUtils.h"               // CaseInsensitiveFs / PathSeparator
#include "UbaDetoursRealPosix.h"        // Real_fstatat for SharedStatPath
#include <fcntl.h>                      // O_* flags
#include <sys/stat.h>                   // struct stat for SharedStatPath
#if PLATFORM_LINUX
#include "UbaLinuxRawSyscall.h"         // RawSyscallDirect for /proc/self/{uid_map,gid_map,setgroups} userns fakery
#include <sys/syscall.h>                // SYS_getuid / SYS_getgid / SYS_write / SYS_close
#endif
#include <errno.h>                      // errno (stub has __errno_location via patcher import-slot)
#if !UBA_STUB_BUILD
#include <unistd.h>                     // read / close (preload path only)
#endif

// Shared_open lives at GLOBAL scope (forward-declared in
// UbaDetoursSharedPosix.h before this file is included). Shared_prepareOpen's
// readDataFromOldFile lambda calls it by unqualified name; from inside
// `namespace uba` lookup walks out to global and finds it.

namespace uba
{
	// Pretty-print openat flag bits. inline so both TUs can use without
	// linkage. Gated on UBA_DEBUG_LOG_ENABLED to match the LD_PRELOAD
	// behavior — the helper is only ever called from DEBUG_LOG_DETOURED
	// format strings.
#if UBA_DEBUG_LOG_ENABLED
	inline StringBuffer<256> OpenFlagsToString(int flags)
	{
		StringBuffer<256> buf;
		auto append = [&](const char* s) { if (buf.count) buf.Append('|'); buf.Append(s); };
		int acc = flags & O_ACCMODE;
		if      (acc == O_RDONLY) append("O_RDONLY");
		else if (acc == O_WRONLY) append("O_WRONLY");
		else if (acc == O_RDWR)   append("O_RDWR");
		if (flags & O_CREAT)     append("O_CREAT");
		if (flags & O_TRUNC)     append("O_TRUNC");
		if (flags & O_APPEND)    append("O_APPEND");
		if (flags & O_NONBLOCK)  append("O_NONBLOCK");
		if (flags & O_CLOEXEC)   append("O_CLOEXEC");
		if (flags & O_EXCL)      append("O_EXCL");
		if (flags & O_NOCTTY)    append("O_NOCTTY");
		if (flags & O_NOFOLLOW)  append("O_NOFOLLOW");
		if (flags & O_SYNC)      append("O_SYNC");
		#if PLATFORM_LINUX
		if (flags & O_TMPFILE)   append("O_TMPFILE");
		#endif
		if (flags & O_DIRECTORY) append("O_DIRECTORY");
		return buf;
	}
#endif

	// =========================================================================
	// Shared_prepareOpen — runs all path-decision logic before openat.
	// =========================================================================
	// See UbaDetoursSharedPosix.h for the contract. The body lives here so
	// both LD_PRELOAD and the static stub instantiate identical code (same
	// FixPath / CanDetour2 / Rpc_GetEntryOffset / Rpc_CreateFileW / dirtable
	// lookups / DEBUG_LOG_DETOURED entries).
	template<typename TrueOpen>
	void Shared_prepareOpen(const char* funcName, const char* file, int flags, int mode, const TrueOpen& trueOpen, SharedOpenPrep& prep)
	{
		UBA_ASSERTF(file && *file, "%s called with null/empty path", funcName); // fail here, not later in IsMemoryHandle

		file = ResolveProcSelfFd(file);
		FixPath(prep.fileName, file);
		DevirtualizePath(prep.fileName);

		UBA_ASSERTF(prep.fileName.count, "%s: FixPath/DevirtualizePath emptied path (input=%s)", funcName, file);

		prep.desiredAccess = GetFileAccessFlags(flags);
		prep.isWrite = (prep.desiredAccess & AccessFlag_Write) != 0;
		prep.trueFlags = flags;
		prep.trueMode = mode;

		if (!CanDetour2(prep.fileName) || prep.fileName.Equals("/"))
		{
			prep.action = SharedOpenAction::NoDetourFallback;
			return;
		}

		#if PLATFORM_LINUX
		if (prep.fileName.StartsWith("/proc/"))
		{
			if (prep.fileName.StartsWith("/proc/self/cmdline"))
				DEBUG_LOG("TODO!!! /proc/self/cmdline");
			prep.action = SharedOpenAction::ProcFallback;
			return;
		}
		#endif

		prep.fileNameKey = ToFilenameKey(prep.fileName);
		// open() always follows symlinks — use Rpc_GetEntryInformation and rewrite prep.fileName/Key
		// to the resolved target so downstream RPCs don't reference a path the agent doesn't have.
	#if UBA_SUPPORT_SYMLINKS
		StringBuffer<> resolvedName;
		StringKey resolvedKey;
		prep.fileExists = Rpc_GetEntryInformation(prep.entryInfo, prep.fileNameKey, prep.fileName, false, true, &resolvedName, &resolvedKey);
		if (prep.fileExists && resolvedName.count && !resolvedName.Equals(prep.fileName))
		{
			prep.fileName.Clear().Append(resolvedName);
			prep.fileNameKey = resolvedKey;
		}
	#else
		prep.fileExists = Rpc_GetEntryInformation(prep.entryInfo, prep.fileNameKey, prep.fileName, false);
	#endif

		if (!(flags & O_CREAT))
		{
			if (!prep.fileExists)
			{
				prep.err = ENOENT;
				DEBUG_LOG_DETOURED(funcName, "(DIRTABLE) %s -> -1 (ENOENT)", prep.fileName.data);
				prep.action = SharedOpenAction::Error;
				return;
			}
		}
		else if (const tchar* lastSeparator = prep.fileName.Last(PathSeparator))
		{
			if (prep.fileExists)
			{
				if (flags & O_EXCL)
				{
					prep.err = EEXIST;
					DEBUG_LOG_DETOURED(funcName, "(DIRTABLE) %s -> -1 (EEXIST)", prep.fileName.data);
					prep.action = SharedOpenAction::Error;
					return;
				}
			}
			else
			{
				// Resolve symlinks on the parent so we look up by real-path key, and splice
				// the resolved parent back into prep.fileName so create RPC sees the real path.
				StringView dirName(prep.fileName.data, u32(lastSeparator - prep.fileName.data));
			#if UBA_SUPPORT_SYMLINKS
				StringBuffer<> resolvedDir;
				DirectoryTableEntryInformation parentInfo;
				if (Rpc_GetEntryInformation(parentInfo, ToFilenameKey(dirName), dirName, true, true, &resolvedDir))
				{
					if (resolvedDir.count)
						dirName = resolvedDir;
					if (!resolvedDir.Equals(StringView(prep.fileName.data, u32(lastSeparator - prep.fileName.data))))
					{
						StringBuffer<> rebuilt(resolvedDir);
						rebuilt.Append(lastSeparator);
						prep.fileName.Clear().Append(rebuilt);
						prep.fileNameKey = ToFilenameKey(prep.fileName);
						lastSeparator = prep.fileName.Last(PathSeparator);
					}
				}
				else
				{
					// Parent unresolvable (dangling symlink / missing dir): surface ENOENT directly
					// so we don't send a path-through-symlink to Rpc_UpdateDirectory below.
					prep.err = ENOENT;
					DEBUG_LOG_DETOURED(funcName, "(DIRTABLE) %s -> -1 (ENOENT, parent unresolvable)", prep.fileName.data);
					prep.action = SharedOpenAction::Error;
					return;
				}
			#endif
				DirHash hash(dirName);

				auto& dirTable = *g_directoryTable;
				SCOPED_WRITE_LOCK(dirTable.m_lookupLock, lookupLock);
				auto insres = dirTable.m_lookup.try_emplace(hash.key, g_memoryBlock);
				DirectoryTable::Directory& dir = insres.first->second;
				if (insres.second)
					if (dirTable.EntryExistsNoLock(hash.key, dirName) != DirectoryTable::Exists_No)
						Rpc_UpdateDirectory(hash.key, dirName.data, dirName.count, false);
				DirTableOffset dirTableOffset2;
				if (!dirTable.GetLatestOffset(dirTableOffset2, dir))
				{
					prep.err = ENOENT;
					DEBUG_LOG_DETOURED(funcName, "(DIRTABLE) %s -> -1 (ENOENT)", prep.fileName.data);
					prep.action = SharedOpenAction::Error;
					return;
				}
			}
		}
		else
		{
			UBA_ASSERT(false); // Should never end up here
		}

		if (prep.fileExists)
		{
			if (IsDirectory(prep.entryInfo.attributes))
			{
				#if PLATFORM_LINUX
				if ((flags & O_TMPFILE) && flags & (O_WRONLY|O_RDWR))
				{
					UBA_ASSERTF(false, "Tmpfile not implemented");
					prep.action = SharedOpenAction::UnsupportedFail;
					return;
				}
				#endif

				if ((flags & (O_WRONLY | O_RDWR | O_CREAT)) != 0)
				{
					prep.err = EISDIR;
					DEBUG_LOG_DETOURED(funcName, "(DIRTABLE) %s (%s) -> -1 (EISDIR)", prep.fileName.data, OpenFlagsToString(flags).data);
					prep.action = SharedOpenAction::Error;
					return;
				}

				// Always synthesize DirectoryFd from DIRTABLE (even local) — Session::CreateDirectory
				// edits overlay before disk-mkdir, so concurrent kernel open races and ENOENTs.
				// DIRTABLE is ground truth.
				prep.fileInfo = new (g_memoryBlock.Allocate(sizeof(FileInfo), 8, "FileInfo")) FileInfo();
				prep.fileInfo->originalName = g_memoryBlock.Strdup(prep.fileName).data;
				prep.fileInfo->name = prep.fileInfo->originalName;
				prep.action = SharedOpenAction::DirectoryFd;
				return;
			}
			else
			{
				if (flags & O_DIRECTORY)
				{
					prep.err = ENOTDIR;
					DEBUG_LOG_DETOURED(funcName, "(DIRTABLE) %s (%s) -> -1 (ENOTDIR)", prep.fileName.data, OpenFlagsToString(flags).data);
					prep.action = SharedOpenAction::Error;
					return;
				}
			}
		}

		bool keepInMemory = false;
		u64 size = InvalidValue;
		const char* realFileName = prep.fileName.data;
		const char* prevFileName = nullptr;
		(void)prevFileName; // Only consumed inside `#if !UBA_STUB_BUILD` preload branches.

		SCOPED_WRITE_LOCK(g_mappedFileTable.m_lookupLock, fileTableLock);
		auto insres = g_mappedFileTable.m_lookup.try_emplace(prep.fileNameKey);
		FileInfo& info = insres.first->second;
		prep.fileInfo = &info;
		u32 lastDesiredAccess = info.lastDesiredAccess; (void)lastDesiredAccess;
		if (insres.second || info.recursiveCall)
		{
			info.lastSeenTableOffset = prep.entryInfo.dirTableOffset;
			info.originalName = g_memoryBlock.Strdup(prep.fileName).data;
			info.name = info.originalName;
			if (!keepInMemory)
			{
				char newFileName[512];
				Rpc_CreateFileW(prep.fileName, prep.fileNameKey, u8(prep.desiredAccess), newFileName, sizeof_array(newFileName), size, prep.closeId, false);
				UBA_ASSERTF(*newFileName, "%s: Rpc_CreateFileW returned empty backedName for %s", funcName, prep.fileName.data); // host wire-format invariant
				info.name = g_memoryBlock.Strdup(newFileName);
				realFileName = info.name;
			}

			info.size = size;
			info.fileNameKey = prep.fileNameKey;
			info.lastDesiredAccess = prep.desiredAccess;
		}
		else
		{
			// TODO: have Rpc_UpdateCloseHandle update info.lastSeenTableOffset to avoid this extra RPC on write-then-open.
			if (info.lastSeenTableOffset.internal != InvalidTableOffset && info.lastSeenTableOffset.internal != prep.entryInfo.dirTableOffset.internal)
			{
				StringBuffer<> backedFile;
				Rpc_CheckRemapping2(backedFile, prep.fileNameKey);
				info.lastSeenTableOffset.internal = prep.entryInfo.dirTableOffset.internal;
				if (!backedFile.IsEmpty() && !backedFile.Equals(info.name))
					info.name = g_memoryBlock.Strdup(backedFile).data;
			}

			if (!info.originalName)
				info.originalName = g_memoryBlock.Strdup(prep.fileName).data;
			if (prep.isWrite)
			{
				prevFileName = info.name;
				bool shouldReport = true;
				shouldReport = shouldReport && !keepInMemory;
				if (shouldReport)
				{
					char newFileName[1024];
					Rpc_CreateFileW(prep.fileName, prep.fileNameKey, u8(prep.desiredAccess), newFileName, sizeof_array(newFileName), size, prep.closeId, false);
					UBA_ASSERTF(*newFileName, "%s: Rpc_CreateFileW returned empty backedName for %s (write reopen)", funcName, prep.fileName.data);
					info.name = g_memoryBlock.Strdup(newFileName);
					realFileName = info.name;
				}
				if (prep.desiredAccess == 0 || info.lastDesiredAccess == 0)
					realFileName = info.name;
				info.lastDesiredAccess |= prep.desiredAccess;
			}
			else
			{
				size = info.size;
				realFileName = info.name;
			}
		}
		fileTableLock.Leave();

		// Fail here on empty realFileName so the cache-hit-on-empty case is named, not masked under $d / # / IsMemoryHandle.
		UBA_ASSERTF(realFileName && *realFileName, "%s: realFileName empty for %s (cached info.name=%s)", funcName, prep.fileName.data, info.name ? info.name : "<null>");

		if (realFileName[0] == '$')
		{
			if (realFileName[1] == 'd') // This is a directory and we will need to fake it locally..
			{
				prep.action = SharedOpenAction::DirectoryFd;
				return;
			}
			DEBUG_LOG_DETOURED(funcName, "FAILED %s (%s)", prep.fileName.data, realFileName);
			UBA_ASSERTF(false, "unsupported filename %s", realFileName);
			prep.action = SharedOpenAction::UnsupportedFail;
			return;
		}

		if (IsMemoryHandle(realFileName))
		{
			DEBUG_LOG_DETOURED(funcName, "FAILED %s (%s)", prep.fileName.data, realFileName);
			UBA_ASSERTF(false, "^ filenames not implemented");
			prep.action = SharedOpenAction::UnsupportedFail;
			return;
		}

		if (keepInMemory)
		{
			DEBUG_LOG_DETOURED(funcName, "FAILED %s (%s)", prep.fileName.data, realFileName);
			UBA_ASSERTF(false, "keepInMemory not implemented");
			prep.action = SharedOpenAction::UnsupportedFail;
			return;
		}

		const char* tempFileName = realFileName;
		if (tempFileName[0] == '#')
			tempFileName = prep.fileName.data;
		else
			tempFileName = info.name;

		prep.trueFlags = flags;
		int trueMode = mode;

#if !UBA_STUB_BUILD
		// readDataFromOldFile preloads the existing file's bytes when a
		// remote-build O_APPEND or O_RDWR open lands on a CAS-staged
		// destination path. Recurses into Shared_open (via trueOpen) and
		// uses libc read/close — all glibc-only. The static stub doesn't
		// run on remote targets that hit this branch, and it has no
		// glibc read/close, so we elide the entire path under
		// UBA_STUB_BUILD. If a stub-built process ever needs the
		// preload, port read/close to raw_syscall and drop the guard.
		auto readDataFromOldFile = [&]()
			{
				DEBUG_LOG_DETOURED(funcName, "INTERNAL READ FOR MEMORYWRITE (%s) (%s)", prep.fileName.data, OpenFlagsToString(prep.trueFlags).data);
				UBA_ASSERT(!info.recursiveCall);
				FileInfo temp = info;
				info.recursiveCall = true;
				fileTableLock.Leave();

				int readFd = Shared_open(funcName, prep.fileName.data, O_RDONLY|O_NONBLOCK, mode, trueOpen);
				UBA_ASSERTF(readFd != -1, "%s", prep.fileName.data);
				prep.dataToWrite.resize(prep.entryInfo.size);
				int readRes = read(readFd, prep.dataToWrite.data(), prep.entryInfo.size);
				close(readFd);
				UBA_ASSERT(readRes == (int)prep.entryInfo.size);

				fileTableLock.Enter();
				info = temp;
			};

		// Remote O_APPEND: src and staging dst are different files; preload existing bytes.
		if (prep.fileExists && (flags & O_APPEND) && !IsOverlayOffset(prep.entryInfo.dirTableOffset))
		{
			if (!prevFileName || !Equals(prevFileName, tempFileName))
			{
				readDataFromOldFile();
				prep.trueFlags |= O_CREAT;
				trueMode = prep.entryInfo.attributes;
			}
		}

		// Remote O_RDWR: preload existing bytes (mask O_ACCMODE — exact-equal missed
		// O_RDWR|O_CREAT|O_CLOEXEC|O_NOFOLLOW shape). O_TRUNC explicitly discards.
		if ((flags & O_ACCMODE) == O_RDWR && prep.fileExists && !(flags & O_TRUNC) && !IsOverlayOffset(prep.entryInfo.dirTableOffset))
		{
			if (!prevFileName || !Equals(prevFileName, tempFileName))
			{
				readDataFromOldFile();
				prep.trueFlags |= O_CREAT;
				trueMode = prep.entryInfo.attributes;
			}
		}
#endif

		// Remote O_WRONLY|O_TRUNC: if the staging dest doesn't exist
		// yet, force O_CREAT so the writer opens cleanly. (Stub uses
		// this branch identically to LD_PRELOAD — no preload needed
		// since O_TRUNC discards existing content.)
		if (flags == (O_WRONLY|O_TRUNC) && !prep.fileName.Equals(tempFileName))
		{
			prep.trueFlags |= O_CREAT;
			if (trueMode == 0)
				trueMode = S_IRUSR | S_IWUSR | S_IRGRP | S_IROTH;
		}

		prep.tempFileName = tempFileName;
		prep.trueMode = trueMode;
		prep.isDirectoryEntry = IsDirectory(prep.entryInfo.attributes);
		prep.action = SharedOpenAction::NormalOpen;
	}

	template<typename TrueUnlink>
	int Shared_deleteFile(const char* funcName, const char* pathname, int& error, const TrueUnlink& trueUnlink)
	{
		StringBuffer<> fixedName;
		switch (Shared_prepareDelete(funcName, pathname, error, fixedName))
		{
		case SharedDeleteAction::Passthrough:
			// Pass the FixPath'd absolute path to the kernel — never the
			// raw user `pathname`. The kernel cwd may differ from UBA's
			// tracked g_virtualWorkingDir.
			return trueUnlink(fixedName.data);
		case SharedDeleteAction::Error:
			return -1;
		case SharedDeleteAction::Success:
			return 0;
		}
		return -1;
	}

	// =========================================================================
	// Shared post-open invariants
	// =========================================================================
	// Both the LD_PRELOAD path (Shared_open below) and the static stub's
	// SYS_nr_openat handler (DetoursStatic/UbaStaticStubCore.cpp) MUST run the
	// same sequence after a successful open through UBA. Centralising these
	// helpers stops the two paths from drifting (read-fd tracking, missing
	// Rpc_RegisterFileForWrite, missing TrackInput on NoDetourFallback — each
	// of those was a separate session of debugging because the stub had
	// re-implemented Shared_open's switch and quietly missed an invariant).
	//
	// `registerFd` is the ONE thing that legitimately differs: LD_PRELOAD
	// inserts into g_fileHandles via CreateFileTableEntry (FileObject layout),
	// the stub uses its own freestanding fd_table_insert. Everything else
	// (TrackInput, dataToWrite preload gated by UBA_STUB_BUILD,
	// Rpc_RegisterFileForWrite) is identical and lives here.
	//
	// See ClaudeNotes/03-detour-invariants.md rule 1 for why every fd from a
	// path-based open MUST be tracked, even read fds.
	template<typename RegisterFd>
	inline void Shared_postOpenSuccess(int fd, const SharedOpenPrep& prep, const RegisterFd& registerFd)
	{
#if !UBA_STUB_BUILD
		// dataToWrite preload for remote O_APPEND/O_RDWR — stub never
		// populates dataToWrite (gated in Shared_prepareOpen), so the write
		// is dead on the stub side and elided here too.
		if (auto toWrite = prep.dataToWrite.size())
		{
			int wrres = write(fd, prep.dataToWrite.data(), toWrite);
			UBA_ASSERT(wrres == (int)toWrite);
		}
#endif

		TrackInput(prep.fileName);
		registerFd(fd, prep);
		// Register the file with the host overlay so its size=InvalidValue
		// entry becomes immediately visible in the local DIRTABLE. Without
		// this, a process that opens(O_CREAT) and then newfstats the same
		// path sees DIRTABLE -> ENOENT for a file it just created (Go
		// writers like soong_zip do exactly this and bail with
		// "lstat <path>: no such file or directory").
		//
		// Two distinct cases trigger registration:
		//   (1) Write opens (any O_WRONLY/O_RDWR) — `prep.isWrite` is set.
		//   (2) **Newly-created files via O_CREAT, even with O_RDONLY** —
		//       `touch` opens with O_RDONLY|O_CREAT to create a 0-byte
		//       file. `prep.isWrite` is false (read-only access flag), but
		//       the file IS created on disk by the kernel. Without this
		//       branch, the parent process's DIRTABLE never learns about
		//       the file (touch's RPC trace shows CreateFile but no
		//       RegisterFileForWrite), and downstream consumers see ENOENT
		//       — sbox's `validateOutputFiles` then panics in
		//       `findAllFilesUnder.func1` when its filepath.Walk callback
		//       gets nil-info for a path the dirtable says doesn't exist.
		if (prep.isWrite || ((prep.trueFlags & O_CREAT) && !prep.fileExists))
			Rpc_RegisterFileForWrite(prep.fileName, prep.fileNameKey);
	}

	// Shared post-NoDetourFallback helper — when a path falls outside UBA's
	// detour scope (CanDetour2 returned false), we still need to record that
	// this process touched it as input. The stub previously skipped this and
	// it was a latent divergence waiting to surface.
	inline void Shared_postNoDetourFallback(int res, const SharedOpenPrep& prep)
	{
		if (res >= 0)
			TrackInput(prep.fileName);
	}

	template<typename TrueOpen>
	int CreateDirFd(const SharedOpenPrep& prep, int flags, const char* funcName, const TrueOpen& trueOpen)
	{
		StringBuffer<> forHash(prep.fileName);
		if (forHash.count == 1)
			forHash.Resize(0);
		if (CaseInsensitiveFs)
			forHash.MakeLower();
		DirHash hash(forHash);

		DirInfo* newDirInfo = new DirInfo();
		{
			auto& dirTable = *g_directoryTable;
			SCOPED_WRITE_LOCK(dirTable.m_lookupLock, lookLock);
			auto dirInsres = dirTable.m_lookup.try_emplace(hash.key, g_memoryBlock);
			DirectoryTable::Directory& dir = dirInsres.first->second;
			if (dirInsres.second)
				if (dirTable.EntryExistsNoLock(hash.key, forHash) != DirectoryTable::Exists_No)
					Rpc_UpdateDirectory(hash.key, prep.fileName.data, prep.fileName.count, false);
			dirTable.PopulateDirectory(hash.open, dir);

			SCOPED_READ_LOCK(dir.lock, dirLock);
			newDirInfo->fileTableOffsets.resize(dir.files.size());
			u32 i = 0;
			for (auto& pair : dir.files)
				newDirInfo->fileTableOffsets[i++] = pair.second;
		}

		int fd = (int)trueOpen("/dev/null", O_RDONLY, 0);

		auto& dh = CreateFileTableEntry(fd, prep, true, funcName, prep.fileName.data);
		dh.fileObject->dirInfo = newDirInfo;
		DEBUG_LOG_DETOURED(funcName, "(DIR) %s (%s) -> %i", prep.fileName.data, OpenFlagsToString(flags).data, fd);
		return fd;
	}

	// Out-param `error` is set to the kernel-style errno value when the
	// return is -1 (Error / failed trueOpen). Caller surfaces it: LD_PRELOAD
	// passes glibc's `errno` directly so the assignment lands in the right
	// TLS slot; the stub passes a local int and translates the result back
	// into the kernel-style negative return that raw_syscall6 callers
	// expect. Shared code MUST NOT touch glibc errno directly — in patched
	// Go binaries the FS register points at Go's M-struct, not a glibc TCB,
	// and __errno_location() returns a wrong address that would corrupt the
	// runtime.
	template<typename TrueOpen, typename RegisterFd>
	int Shared_open(const char* funcName, const char* file, int flags, int mode, int& error, const TrueOpen& trueOpen, const RegisterFd& registerFd)
	{
		SharedOpenPrep prep;
		Shared_prepareOpen(funcName, file, flags, mode, trueOpen, prep);

		#if UBA_DEBUG_LOG_ENABLED
		const char* isWriteStr = prep.isWrite ? " WRITE" : ""; (void)isWriteStr;
		#endif

		// nsjail userns: caller's writes to /proc/<pid>/{uid,gid}_map use detoured g_virtualUid/Gid
		// (1000), but kernel demands "<virtual> <real-outside-uid> 1". Write the kernel-correct
		// mapping via raw syscall using the real outside uid/gid, then hand the caller a /dev/null
		// fd to write into. Inside the userns the kernel-real id is 1000; detoured tools see 1000.
		bool userNsFake = false;

		#if PLATFORM_LINUX
		if (prep.isWrite && prep.fileName.StartsWith("/proc/") && (prep.fileName.EndsWith("/setgroups") || prep.fileName.EndsWith("/uid_map") || prep.fileName.EndsWith("/gid_map")))
		{
			int realFd = (int)trueOpen(file, flags, mode);
			if (realFd >= 0)
			{
				char buf[64];
				int n;
				if (prep.fileName.EndsWith("/setgroups"))
				{
					n = (int)snprintf(buf, sizeof(buf), "deny");
				}
				else if (prep.fileName.EndsWith("/uid_map"))
				{
					long realOutside = RawSyscallDirect(SYS_getuid, 0, 0, 0).r1;
					n = (int)snprintf(buf, sizeof(buf), "%u %u 1\n", uba::g_virtualUid, (unsigned)realOutside);
				}
				else // /gid_map
				{
					long realOutside = RawSyscallDirect(SYS_getgid, 0, 0, 0).r1;
					n = (int)snprintf(buf, sizeof(buf), "%u %u 1\n", uba::g_virtualGid, (unsigned)realOutside);
				}
				RawSyscallDirect(SYS_write, realFd, (long)buf, n);
				RawSyscallDirect(SYS_close, realFd, 0, 0);
			}
			userNsFake = true;
		}
		#endif

		switch (prep.action)
		{
		case SharedOpenAction::Error:
			error = prep.err;
			return -1;

		case SharedOpenAction::NoDetourFallback:
		{
			TimerScope ts(g_kernelStats.getFileInfo);
			int res = trueOpen(userNsFake ? "/dev/null" : file, flags, mode);
			ts.Leave();
			DEBUG_LOG_TRUE(funcName, "NODETOUR%s (%s)%s -> %i", isWriteStr, file, userNsFake ? " [userns-fake -> /dev/null]" : "", res);
			Shared_postNoDetourFallback(res, prep);
			return res;
		}

		case SharedOpenAction::ProcFallback:
		{
			int res = trueOpen(userNsFake ? "/dev/null" : file, flags, mode);
			DEBUG_LOG_TRUE(funcName, "NODETOUR (%s)%s -> %i", file, userNsFake ? " [userns-fake -> /dev/null]" : "", res);
			return res;
		}

		case SharedOpenAction::DirectoryFd:
			return CreateDirFd(prep, flags, funcName, trueOpen);

		case SharedOpenAction::UnsupportedFail:
			return -1;

		case SharedOpenAction::NormalOpen:
		{
			TimerScope ts(g_kernelStats.getFileInfo);
			int fd = trueOpen(prep.tempFileName, prep.trueFlags, prep.trueMode);
			ts.Leave();

			DEBUG_LOG_TRUE(funcName, "%s%s (%s) %s %i -> %i", file, isWriteStr, prep.tempFileName, OpenFlagsToString(prep.trueFlags).data, prep.trueMode, fd);
			if (fd == -1)
				return fd;

			Shared_postOpenSuccess(fd, prep, registerFd);
			return fd;
		}
		}
		return -1;
	}

	// Convenience overload: defaults to CreateFileTableEntry (g_fileHandles + FileObject layout) and
	// uses glibc-style `errno` as the error out-param. Same on LD_PRELOAD and stub — both have
	// __errno_location (stub provides it via UbaStaticStubCore.cpp's import-slot wrapper).
	template<typename TrueOpen>
	inline int Shared_open(const char* funcName, const char* file, int flags, int mode, const TrueOpen& trueOpen)
	{
		return Shared_open(funcName, file, flags, mode, errno, trueOpen,
			[funcName](int fd, const SharedOpenPrep& prep)
			{
				CreateFileTableEntry(fd, prep, prep.isDirectoryEntry, funcName, prep.tempFileName);
			});
	}

	template<typename True_stat>
	int Shared_stat(const char* funcName, const char* file, struct stat* attr, const True_stat& trueStat, bool followSymlink = true)
	{
		file = ResolveProcSelfFd(file);
		StringBuffer<> fixedFile;
		bool fixPathOk = FixPath(fixedFile, file);
		if (!fixPathOk || fixedFile.Equals("/") || !CanDetour2(fixedFile))
		{
			TimerScope ts(g_kernelStats.getFileInfo);
			const char* kernelPath = fixPathOk ? fixedFile.data : file;
			int res = trueStat(kernelPath, attr);
			if (res < 0 && errno == EAGAIN) // tmpfs EAGAIN backoff: Java NIO turns this into a fatal exception
			{
				struct timespec ts2;
				ts2.tv_sec = 0;
				long usec = 100;
				for (int retries = 0; retries < 25 && res < 0 && errno == EAGAIN; ++retries)
				{
					ts2.tv_nsec = usec * 1000;
					nanosleep(&ts2, nullptr);
					res = trueStat(kernelPath, attr);
					usec *= 2;
					if (usec > 50000) usec = 50000;
				}
			}
			return res;
		}

		UBA_ASSERTF(fixedFile.count, "FixPath failed with %s", file);

		if (g_runningRemote && fixedFile.StartsWith(g_exeDir.data))
		{
			StringBuffer<> temp;
			temp.Append(g_virtualApplicationDir).Append(fixedFile.data + g_exeDir.count);
			fixedFile.Clear().Append(temp);
		}

		StringBuffer<> preDevirt(fixedFile); // IsVfsParentDir wants the namespace-virtual path; snapshot before DevirtualizePath rewrites
		DevirtualizePath(fixedFile);

		FileAttributes fileAttr;
		const char* realName = Shared_GetFileAttributes(fileAttr, fixedFile, followSymlink);

		int res = fileAttr.lastError == 0 ? 0 : -1;

		// Synth attrs for unbound bind-mount parent dirs so `readlink -f` can walk through.
		if (res != 0 && (g_runningRemote || IsVfsEnabled()) && IsVfsParentDir(preDevirt))
		{
			memset(&fileAttr.data, 0, sizeof(fileAttr.data));
			fileAttr.data.st_mode = S_IFDIR | S_IRUSR | S_IWUSR | S_IXUSR | S_IRGRP | S_IXGRP | S_IROTH | S_IXOTH;
			fileAttr.data.st_nlink = 2;
			// Distinct st_ino: clang FileManager dedupes by (dev,ino); collisions on 0 drop InitHeaderSearch entries.
			StringKey pathKey = ToFilenameKey(preDevirt);
			fileAttr.data.st_ino = (ino_t)pathKey.a;
			fileAttr.data.st_dev = 1;
			fileAttr.exists = true;
			fileAttr.lastError = 0;
			realName = "(vfs-parent)";
			res = 0;
		}

		DEBUG_LOG_DETOURED(funcName, "%s (%s size: %llu id: %llu dev: %u mode: %u)-> %i (%s)", file, realName, fileAttr.data.st_size, fileAttr.data.st_ino, fileAttr.data.st_dev, fileAttr.data.st_mode, res, StrError(res, fileAttr.lastError));

		if (res == 0) // If success and original path had ".." in path, we need to verify that the path that leads to the ".." actually exists.
		{
#if UBA_SUPPORT_SYMLINKS
			UBA_ASSERT(!followSymlink || !IsSymlink(fileAttr.data.st_mode)); // lstat may return S_IFLNK; stat may not
#endif

			// TODO: handle multiple ".." spread out in the path
			const tchar* dotdot;
			if (Contains(file, "..", false, &dotdot))
			{
				StringBuffer<> tempPath;
				tempPath.Append(file, dotdot - file);
				struct stat tempAttr;
				int tempRes = Shared_stat("stat(dotdot)", tempPath.data, &tempAttr, trueStat);
				if (tempRes != 0)
					return tempRes;
			}
		}

		errno = fileAttr.lastError;

		if (res == 0) // This check is important.. dont write to attr unless success. (some programs rely on that)
			memcpy(attr, &fileAttr.data, sizeof(fileAttr.data));

		// InvalidValue ⇒ a writer fd is open; Sqlite fileHasMoved compares stat(path).st_ino to fstat(fd).st_ino — copy the full kernel attr so they agree.
		if (res == 0 && attr->st_size == (off_t)InvalidValue)
		{
			bool fellBackToFd = false;
			{
				SCOPED_READ_LOCK(g_fileHandlesLock, fhLock);
				for (auto& kv : g_fileHandles)
				{
					DetouredHandle& h = kv.second;
					if (!h.fileObject || !h.fileObject->fileInfo)
						continue;
					if (!fixedFile.Equals(h.fileObject->fileInfo->originalName))
						continue;
					struct stat fdAttr;
					if (Real_fstat(errno, kv.first, &fdAttr) == 0)
					{
						memcpy(attr, &fdAttr, sizeof(*attr));
						DEBUG_LOG_DETOURED(funcName, "%s overlay-size=InvalidValue, writer-fd %i size=%lld ino=%llu", file, kv.first, (long long)attr->st_size, (unsigned long long)attr->st_ino);
						fellBackToFd = true;
					}
					break;
				}
			}
			if (!fellBackToFd)
			{
				// Cross-process writer: kernel path-stat is best-effort (remote mode loses st_size).
				struct stat realAttr;
				if (trueStat(file, &realAttr) == 0)
				{
					memcpy(attr, &realAttr, sizeof(*attr));
					DEBUG_LOG_DETOURED(funcName, "%s overlay-size=InvalidValue (no writer-fd), kernel size=%lld ino=%llu", file, (long long)attr->st_size, (unsigned long long)attr->st_ino);
				}
				else
				{
					attr->st_size = 0;
					DEBUG_LOG_DETOURED(funcName, "%s overlay-size=InvalidValue, no writer-fd, kernel stat failed -> 0", file);
				}
			}
		}

#if UBA_DEBUG_VALIDATE
		if (!g_runningRemote && !IsVfsEnabled() && *file)
		{
			struct stat attr2;
			int res2 = trueStat(file, &attr2);
			UBA_ASSERTF(res == res2, "%s: return value differs for %s (cached %i vs actual %i) [fixed: %s]", funcName, file, res, res2, fixedFile.data, StrError(res2, errno));
			if (res == 0)
			{
				bool isDir = S_ISDIR(attr->st_mode);
				UBA_ASSERT(isDir == S_ISDIR(attr2.st_mode));
				UBA_ASSERTF(attr->st_ino == attr2.st_ino, "stat: st_ino mismatch for %s (%llu vs %llu)", file, attr->st_ino, attr2.st_ino);
				UBA_ASSERT(isDir || attr->st_size == attr2.st_size);
				UBA_ASSERTF(isDir || FromTimeSpec(attr->st_mtimespec) == FromTimeSpec(attr2.st_mtimespec), "stat: st_mtim mismatch for %s (%llu vs %llu)", file, FromTimeSpec(attr->st_mtimespec), FromTimeSpec(attr2.st_mtimespec));
			}
			else
			{
				UBA_ASSERTF(fileAttr.lastError == errno || ((fileAttr.lastError == ENOTDIR || fileAttr.lastError == ENOENT) && (errno == ENOTDIR || errno == ENOENT))
					, "Detoured stat returned a different error. Returned %i (%s) but should return %i (%s)", fileAttr.lastError, strerror(fileAttr.lastError), errno, strerror(errno));
			}
		}
#endif
		return res;
	}

	// Shared_rmdir — see header for the contract.
	// Out-param `error` is set to the kernel-style errno value on -1 returns.
	// Caller is responsible for surfacing it (via glibc errno on the LD_PRELOAD
	// side, or by writing it into saved[]/argregs on the static-stub side
	// where glibc errno TLS may not be available in patched Go binaries).
	template<typename TrueRmdir>
	int Shared_rmdir(const char* funcName, const char* path, int& error, const TrueRmdir& trueRmdir)
	{
		StringBuffer<> fixedPath;
		if (!FixPath(fixedPath, path) || !CanDetour2(fixedPath))
		{
			int res = trueRmdir(path);
			DEBUG_LOG_TRUE(funcName, "%s -> %i", path, res);
			return res;
		}

		DevirtualizePath(fixedPath);

		StringKey fixedPathKey = ToFilenameKey(fixedPath);

		// Probe the dirtable first — the dirtable is ground truth for
		// tracked paths. If the dirtable doesn't know about the target,
		// the dir doesn't exist (any on-disk content is illegitimate
		// leftover that UBA never knew about). Mirror what
		// Shared_prepareDelete does for unlink and Shared_mkdir does for
		// mkdir. If we see the bug "Go's RemoveAll thinks the dir was
		// removed when it wasn't", look at why the dirtable thinks the
		// dir is gone — almost certainly an upstream code path that's
		// reporting a failed disk op as a successful overlay delete (e.g.
		// Session::DeleteFile / Session::RemoveDirectory unconditionally
		// calling RegisterOverlayDelete even when uba::DeleteFileW /
		// uba::RemoveDirectoryW failed).
		DirectoryTableEntryInformation dirEntryInfo;
		// followSymlink=false: rmdir doesn't follow leaf symlinks; symlink-to-dir → ENOTDIR.
		if (!Rpc_GetEntryInformation(dirEntryInfo, fixedPathKey, fixedPath, false, /*followSymlink=*/false))
		{
			error = ENOENT;
			DEBUG_LOG_DETOURED(funcName, "(DIRTABLE) %s -> -1 (ENOENT)", fixedPath.data);
			return -1;
		}
	#if UBA_SUPPORT_SYMLINKS
		if (IsSymlink(dirEntryInfo.attributes))
		{
			error = ENOTDIR;
			DEBUG_LOG_DETOURED(funcName, "(DIRTABLE) %s -> -1 (ENOTDIR, symlink)", fixedPath.data);
			return -1;
		}
	#endif
		if (!IsDirectory(dirEntryInfo.attributes))
		{
			error = ENOTDIR;
			DEBUG_LOG_DETOURED(funcName, "(DIRTABLE) %s -> -1 (ENOTDIR)", fixedPath.data);
			return -1;
		}

		auto& dirTable = *g_directoryTable;

		// Empty-dir pre-check: only send the RemoveDirectory RPC when we're
		// confident the operation will succeed. Mirror Detoured_RemoveDirectoryW
		// (Windows: UbaDetoursFunctionsKernelBase.inl): look up the directory
		// in g_directoryTable.m_lookup, populate its children if not yet up
		// to date, and walk dir.files for any entry with non-zero attributes
		// (an entry exists when its attributes are non-zero; deleted entries
		// stay in the table but with attributes=0). If any child still exists
		// the dir isn't empty — return -1/ENOTEMPTY without an RPC.
		//
		// Why this matters: the host-side RemoveDirectory unconditionally
		// fires RegisterOverlayDelete (DIRTABLE says deleted) even if the
		// underlying uba::RemoveDirectoryW fails on disk. Pre-checking here
		// keeps DIRTABLE and the kernel's view in sync — Go's os.RemoveAll
		// gets the ENOTEMPTY signal and falls into its recursive branch
		// (issuing per-child unlinks that go through DIRTABLE-aware paths)
		// instead of receiving a phantom "deleted" success on a dir UBA
		// never enumerated.
		{
			StringBuffer<> dirNameForHash(fixedPath);
			if (CaseInsensitiveFs)
				dirNameForHash.MakeLower();
			DirHash hash(dirNameForHash);

			SCOPED_WRITE_LOCK(dirTable.m_lookupLock, lookLock);
			auto insres = dirTable.m_lookup.try_emplace(hash.key, g_memoryBlock);
			DirectoryTable::Directory& dir = insres.first->second;
			if (insres.second)
				if (dirTable.EntryExistsNoLock(hash.key, fixedPath) != DirectoryTable::Exists_No)
					Rpc_UpdateDirectory(hash.key, fixedPath.data, fixedPath.count, false);
			lookLock.Leave();

			if (!dirTable.IsDirectoryUpToDate(dir))
				dirTable.PopulateDirectory(hash.open, dir);

			SCOPED_READ_LOCK(dir.lock, fileLock);
			for (auto& f : dir.files)
			{
				u32 attrs = dirTable.GetAttributes(f.second);
				if (attrs == 0)
					continue;
				error = ENOTEMPTY;
				DEBUG_LOG_DETOURED(funcName, "(DIRTABLE) %s -> -1 (ENOTEMPTY)", fixedPath.data);
				return -1;
			}
		}

		RPC_MESSAGE(RemoveDirectory, deleteFile);
		writer.WriteStringKey(fixedPathKey);
		writer.WriteString(fixedPath);
		BinaryReader reader = writer.Flush();
		DirTableSize directoryTableSize = FromU64(reader.ReadU64());
		RPC_MESSAGE_END;

		dirTable.ParseDirectoryTable(directoryTableSize);

		DEBUG_LOG_DETOURED(funcName, "%s -> 0", fixedPath.data);
		return 0;
	}

	// Shared_mkdir — see header for the contract.
	// Out-param `error` is set to the kernel-style errno value on -1 returns.
	// Same rationale as Shared_rmdir: the static stub's caller can't rely on
	// the glibc errno TLS in patched Go binaries, so we surface the error
	// explicitly via the out-param. Caller writes it into glibc's errno
	// (LD_PRELOAD path) or saved[]/argregs (stub path).
	template<typename TrueMkdir>
	int Shared_mkdir(const char* funcName, const char* path, mode_t mode, int& error, const TrueMkdir& trueMkdir)
	{
		StringBuffer<> dirPath;
		if (!FixPath(dirPath, path))
		{
			int res = trueMkdir(path, mode);
			DEBUG_LOG_TRUE(funcName, "(NODETOUR) %s -> %i", path, res);
			return res;
		}
		StringBuffer<> virtPath(dirPath); // virtual form needed by IsVfsParentDir + recursive recovery below
		// DevirtualizePath BEFORE CanDetour2 — IsUnderPassThroughRoot expects host form.
		DevirtualizePath(dirPath);
		if (!CanDetour2(dirPath) || (dirPath.count == 1 && dirPath[0] == '/'))
		{
			int res = trueMkdir(path, mode);
			DEBUG_LOG_TRUE(funcName, "(NODETOUR) %s -> %i", path, res);
			return res;
		}
		StringKey dirPathKey = ToFilenameKey(dirPath);

		DirectoryTableEntryInformation dirEntryInfo;
		if (Rpc_GetEntryInformation(dirEntryInfo, dirPathKey, dirPath, false))
		{
			error = EEXIST;
			DEBUG_LOG_DETOURED(funcName, "(DIRTABLE) %s -> -1 (EEXIST)", dirPath.data);
			return -1;
		}

		const tchar* lastSeparator = dirPath.Last(PathSeparator);
		UBA_ASSERT(lastSeparator);

		bool checkDir = true;
		const tchar* separatorIt = lastSeparator;
		while (separatorIt > dirPath.data)
		{
			--separatorIt;
			if (*separatorIt != PathSeparator)
				continue;
			checkDir = separatorIt == dirPath.data;
			break;
		}

		StringView parentDirPath(dirPath.data, u32(lastSeparator - dirPath.data));
		DirectoryTableEntryInformation parentDirEntryInfo;
		bool parentExists;
	#if UBA_SUPPORT_SYMLINKS
		StringBuffer<> resolvedParent;
		StringKey resolvedParentKey;
		parentExists = Rpc_GetEntryInformation(parentDirEntryInfo, ToFilenameKey(parentDirPath), parentDirPath, checkDir, true, &resolvedParent, &resolvedParentKey);
	#else
		parentExists = Rpc_GetEntryInformation(parentDirEntryInfo, ToFilenameKey(parentDirPath), parentDirPath, checkDir);
	#endif

		// Bind-mount implicit parent: kernel synthesizes intermediate dirs in a real NS, dirtable
		// doesn't. Recurse Shared_mkdir on virtual parent if it's a strict ancestor of any vfs entry.
		if (!parentExists)
		{
			const tchar* virtLastSep = virtPath.Last(PathSeparator);
			if (virtLastSep && virtLastSep > virtPath.data)
			{
				StringView virtParent(virtPath.data, u32(virtLastSep - virtPath.data));
				if (IsVfsParentDir(virtParent))
				{
					StringBuffer<> virtParentBuf;
					virtParentBuf.Append(virtParent);
					int subErr = 0;
					int subRes = Shared_mkdir(funcName, virtParentBuf.data, mode, subErr, trueMkdir);
					if (subRes == 0 || subErr == EEXIST)
					{
					#if UBA_SUPPORT_SYMLINKS
						parentExists = Rpc_GetEntryInformation(parentDirEntryInfo, ToFilenameKey(parentDirPath), parentDirPath, checkDir, true, &resolvedParent, &resolvedParentKey);
					#else
						parentExists = Rpc_GetEntryInformation(parentDirEntryInfo, ToFilenameKey(parentDirPath), parentDirPath, checkDir);
					#endif
					}
				}
			}
		}

		if (!parentExists)
		{
			error = ENOENT;
			DEBUG_LOG_DETOURED(funcName, "(DIRTABLE) %s -> -1 (ENOENT)", dirPath.data);
			return -1;
		}

	#if UBA_SUPPORT_SYMLINKS
		// Use resolved parent so dirtable never sees a path-through-symlink.
		if (resolvedParent.count && !resolvedParent.Equals(parentDirPath))
		{
			StringBuffer<> rebuilt(resolvedParent);
			rebuilt.Append(lastSeparator);
			dirPath.Clear().Append(rebuilt);
			dirPathKey = ToFilenameKey(dirPath);
		}
	#endif

		RPC_MESSAGE(CreateDirectory, createFile);
		writer.WriteStringKey(dirPathKey);
		writer.WriteString(dirPath);
		BinaryReader reader = writer.Flush();
		DirTableSize directoryTableSize = FromU64(reader.ReadU64());
		RPC_MESSAGE_END

		g_directoryTable->ParseDirectoryTable(directoryTableSize);
		DEBUG_LOG_DETOURED(funcName, "%s -> 0", path);
		return 0;
	}

	// Shared_rename — see header for the contract. Body lifted verbatim from
	// the LD_PRELOAD original (UbaDetoursFunctionsPosix.cpp's Shared_rename).
	// Only difference: TRUE_WRAPPER(rename)(o, n) becomes trueRename(o, n)
	// so the static stub can plug in raw_syscall6(SYS_nr_renameat, AT_FDCWD,
	// o, AT_FDCWD, n, …).
	//
	// The original body had an inner `UBA_INIT_DETOUR(rename, oldpath, newpath)`
	// guard at the top. That guard was redundant: every caller of
	// Shared_rename (UBA_WRAPPER(rename), renameat, renameat2) already does
	// UBA_INIT_DETOUR for ITS OWN syscall name at entry, so by the time
	// control reaches here we already know g_isDetouring && !DisallowDetourDepth().
	// Removed because the .inl is also instantiated from
	// UbaDetoursSharedPosix.cpp (a separate TU that doesn't include the
	// LD_PRELOAD-only UBA_INIT_DETOUR macro). Behavior is preserved — the
	// guard never fired in practice.
	template<typename TrueRename>
	int Shared_rename(const char* func, const char* oldpath, const char* newpath, int& error, const TrueRename& trueRename)
	{
		StringBuffer<> fixedOldPath;
		FixPath(fixedOldPath, oldpath);
		DevirtualizePath(fixedOldPath);
		StringKey oldKey = ToFilenameKey(fixedOldPath);

		StringBuffer<> fixedNewPath;
		FixPath(fixedNewPath, newpath);
		DevirtualizePath(fixedNewPath);
		StringKey newKey = ToFilenameKey(fixedNewPath);

		// Both endpoints in a pass-through subtree (e.g. nsjail tmpfs): kernel owns the rename.
		if (!CanDetour2(fixedOldPath) && !CanDetour2(fixedNewPath))
		{
			int res = trueRename(oldpath, newpath);
			DEBUG_LOG_TRUE(func, "(NODETOUR) from %s to %s -> %i", oldpath, newpath, res);
			return res;
		}

		// followSymlink=false: keep S_IFLNK + content so host's MoveFile records dest as symlink.
		DirectoryTableEntryInformation info;
		StringBuffer<MaxPath> symlinkContent;
		if (!Rpc_GetEntryInformation(info, oldKey, fixedOldPath, false, false))
		{
			DEBUG_LOG_DETOURED(func, "(DIRTABLE miss) from %s to %s — falling through with default attrs", fixedOldPath.data, fixedNewPath.data);
			info.size = 0;
			info.lastWrite = 0;
			info.attributes = S_IFREG | S_IRUSR | S_IWUSR | S_IRGRP | S_IROTH;
		}
		#if UBA_SUPPORT_SYMLINKS
		else if (IsSymlink(info.attributes))
		{
			DirTableOffset offset;
			if (Rpc_GetEntryOffset(offset, oldKey, fixedOldPath, false))
				g_directoryTable->GetEntrySymlink(symlinkContent, offset);
		}
		#endif

		// TODO: This might be really slow but it seems you can rename files on linux while they are open and they won't be properly renamed until closed
		// We could check if info.size == InvalidValue, this way we know if it is open
		{
			SCOPED_READ_LOCK(g_fileHandlesLock, lock);
			for (auto& kv : g_fileHandles)
			{
				FileObject& fo = *kv.second.fileObject;
				FileInfo& fi = *fo.fileInfo;
				if (fi.fileNameKey == oldKey)
				{
					UBA_ASSERTF(fo.desiredAccess & AccessFlag_Write, "Unsupported access flags");
					fo.newName = fixedNewPath.data;
					if (!fo.closeId)
					{
						char temp[1024];
						u64 size;
						Rpc_CreateFileW(fixedNewPath, newKey, AccessFlag_Write, temp, sizeof_array(temp), size, fo.closeId, true);
					}

					// Cache rekey so same-process read of newpath resolves to fi.name (open file's backed path).
					{
						SCOPED_WRITE_LOCK(g_mappedFileTable.m_lookupLock, ftLock);
						auto insres = g_mappedFileTable.m_lookup.try_emplace(newKey);
						FileInfo& toInfo = insres.first->second;
						if (insres.second)
						{
							toInfo.fileNameKey = newKey;
							toInfo.originalName = g_mappedFileTable.m_memoryBlock.Strdup(fixedNewPath).data;
						}
						toInfo.name = fi.name;
						toInfo.size = fi.size;
						toInfo.lastDesiredAccess = 0;
						fi.lastDesiredAccess = 0;
					}

					bool wasTempFile = fixedNewPath.StartsWith(g_systemTemp.data);
					if (g_runningRemote && !wasTempFile)
					{
						// TODO: We need to make sure this actually worked
						error = 0;
						DEBUG_LOG_DETOURED(func, "IS_OPEN (%i) (from %s to %s) -> 0", kv.first, fixedOldPath.data, fixedNewPath.data);
						return 0;
					}

					// Staged: kernel has the file at fi.name (hash-named), not at virtual src.
					// Virtual rename already recorded; Shared_close finalizes under dst. Skip trueRename.
					// (rustc archive-build: open O_EXCL temp → write → rename WHILE OPEN → close.)
					if (TStrcmp(fi.name, fixedOldPath.data) != 0)
					{
						error = 0;
						DEBUG_LOG_DETOURED(func, "IS_OPEN (%i) STAGED (from %s to %s, backed=%s) -> 0", kv.first, fixedOldPath.data, fixedNewPath.data, fi.name);
						return 0;
					}

					bool isTempFile = fixedOldPath.StartsWith(g_systemTemp.data);
					UBA_ASSERTF(wasTempFile == isTempFile, "File changing from temp to not or vice versa not implemented");
					(void)isTempFile; // UBA_ASSERTF may compile out (non-debug stub build)

					int res = trueRename(fixedOldPath.data, fixedNewPath.data);
					DEBUG_LOG_DETOURED(func, "IS_OPEN (%i) (from %s to %s) -> %i", kv.first, fixedOldPath.data, fixedNewPath.data, res);
					return res;
				}
			}
			// No local fd matched oldKey. info.size==InvalidValue means a fd in ANOTHER process holds it open —
			// the cross-process rename-while-open case the IS_OPEN repath can't handle.
			UBA_SHIPPING_ASSERTF(info.size != InvalidValue,
				"%s: cross-process rename-while-open: oldpath=%s newpath=%s — info.size=InvalidValue but no fd in this process owns oldKey.",
				func, fixedOldPath.data, fixedNewPath.data);
		}

		bool canDetourOld = CanDetour2(fixedOldPath);
		bool canDetourNew = CanDetour2(fixedNewPath);
		if (!canDetourOld)
		{
			if (!canDetourNew)
			{
				// Pass FixPath'd absolute paths to the kernel — never the raw
				// user oldpath/newpath. Kernel cwd may not match
				// g_virtualWorkingDir.
				int res = trueRename(fixedOldPath.data, fixedNewPath.data);
				DEBUG_LOG_TRUE(func, "(from %s to %s) -> %i", fixedOldPath.data, fixedNewPath.data, res);
				return res;
			}
		}
		else
		{
			//UBA_ASSERT(!canDetourNew);
		}


		RPC_MESSAGE(MoveFile, moveFile);
		writer.WriteStringKey(oldKey);
		writer.WriteString(fixedOldPath);
		writer.WriteU64(info.size);
		writer.WriteU64(info.lastWrite);
		writer.WriteU32(info.attributes);
		writer.WriteStringKey(newKey);
		writer.WriteString(fixedNewPath);
		writer.WriteU32(0);// dwFlags);
		#if UBA_SUPPORT_SYMLINKS
		if (IsSymlink(info.attributes))
			writer.WriteString(symlinkContent);
		#endif
		BinaryReader reader = writer.Flush();
		bool result = reader.ReadBool();
		u32 errorCode = reader.ReadU32();
		DirTableSize directoryTableSize = FromU64(reader.ReadU64());
		RPC_MESSAGE_END

		g_directoryTable->ParseDirectoryTable(directoryTableSize);

		// Copy oldKey's backed name into newKey so same-process read of newpath sees the renamed
		// bytes. Repro: Python NamedTemporaryFile + os.replace — stale info.name → empty .pyc.
		if (g_runningRemote)
		{
			SCOPED_WRITE_LOCK(g_mappedFileTable.m_lookupLock, ftLock);
			auto fromIt = g_mappedFileTable.m_lookup.find(oldKey);
			if (fromIt != g_mappedFileTable.m_lookup.end())
			{
				FileInfo& fromInfo = fromIt->second;
				auto insres = g_mappedFileTable.m_lookup.try_emplace(newKey);
				FileInfo& toInfo = insres.first->second;
				if (insres.second)
				{
					toInfo.fileNameKey = newKey;
					toInfo.originalName = g_mappedFileTable.m_memoryBlock.Strdup(fixedNewPath).data;
				}
				toInfo.name = fromInfo.name;
				toInfo.size = fromInfo.size;
				toInfo.lastDesiredAccess = 0;
				fromInfo.lastDesiredAccess = 0;
			}
			else
			{
				// Old untracked locally — fall back to SetDeleted-pair behavior.
				auto it = g_mappedFileTable.m_lookup.find(newKey);
				if (it != g_mappedFileTable.m_lookup.end())
					it->second.lastDesiredAccess = 0;
			}
		}

		int res = result ? 0 : -1;
		DEBUG_LOG_DETOURED(func, "(from %s to %s) -> %i (%s)", fixedOldPath.data, fixedNewPath.data, res, StrError(res, errorCode));

		error = errorCode;
		return res;
	}

	template<typename TrueChmod>
	int Shared_chmod(const char* funcName, const char* pathname, mode_t mode, int& error, const TrueChmod& trueChmod)
	{
		StringBuffer<> fixedName;
		FixPath(fixedName, pathname);
		DevirtualizePath(fixedName);

		if (!CanDetour2(fixedName))
		{
			int res = trueChmod(fixedName.data, mode);
			if (res != 0) error = errno;
			DEBUG_LOG_TRUE(funcName, "%s %i -> %i (%s)", pathname, mode, res, StrError(res, error));
			return res;
		}

		StringKey key = ToFilenameKey(fixedName);

		DirectoryTableEntryInformation entryInfo;
		if (!Rpc_GetEntryInformation(entryInfo, key, fixedName, false))
		{
			error = ENOENT;
			DEBUG_LOG_TRUE(funcName, "%s %i -> -1 (ENOENT)", pathname, mode);
			return -1;
		}

		if (entryInfo.attributes == mode)
		{
			DEBUG_LOG_DETOURED(funcName, "(NOCHANGE) %s %i -> 0", pathname, mode);
			return 0;
		}

		u32 errorCode;
		{
			RPC_MESSAGE(Chmod, chmod);
			writer.WriteStringKey(key);
			writer.WriteString(fixedName);
			writer.WriteU32(mode);
			BinaryReader reader = writer.Flush();
			errorCode = reader.ReadU32();
			RPC_MESSAGE_END
		}

		int res = errorCode == 0 ? 0 : -1;
		DEBUG_LOG_DETOURED(funcName, "%s %i -> %i (%s)", pathname, mode, res, StrError(res, errorCode));

		error = errorCode;
		return res;
	}

	// FlushWriteHandlesBeforeExec — called from Shared_Internal_exec.
	//
	// Without this pass, an fd opened for write in this process (typical bash
	// redirect: child opens /sandbox/out/foo.java O_WRONLY|O_CREAT before
	// execve'ing the producer) leaves m_activeFiles[closeId] live on the
	// host with contentSize=InvalidValue (= 0x7ffffffff = 34359738367).
	// The execve replaces our memory and the new program's fresh stub has
	// no record of these fds, so Rpc_UpdateCloseHandle is never sent and
	// the InvalidValue placeholder leaks into DIRTABLE forever.
	//
	// Walk g_fileHandles, snapshot each write-tracked fd's current size via
	// the caller's trueSnapshot lambda, and fire Rpc_UpdateCloseHandle so
	// the host clears the placeholder. The kernel fd remains open across
	// execve (we don't close it here); whatever the new program writes goes
	// to disk, but UBA's tracking is "the file was closed at this size".
	// That's still wrong for redirects (snapshot is taken before the
	// producer runs, so size will typically be 0), but it eliminates the
	// 34GB ghosts that confuse downstream directory walks. Stage 2 will
	// transfer ownership across execve so post-exec writes are accounted.
	template<typename TrueSnapshot>
	void FlushWriteHandlesBeforeExec(const char* funcName, const TrueSnapshot& trueSnapshot)
	{
		SCOPED_WRITE_LOCK(g_fileHandlesLock, lock);
		for (auto& kv : g_fileHandles)
		{
			DetouredHandle& h = kv.second;
			FileObject* fo = h.fileObject;
			if (!fo || !fo->closeId)
				continue;
			if (!(fo->desiredAccess & AccessFlag_Write))
				continue;
			if (!fo->fileInfo)
				continue;

			int fd = kv.first;
			u64 fileSize = 0;
			u64 lastWritten = 0;
			u32 attributes = 0;
			if (!trueSnapshot(fd, fileSize, lastWritten, attributes))
			{
				// fstat failed — fd may already be closed (e.g. dup'd-and-
				// closed before this loop). Skip the RPC: Rpc_UpdateCloseHandle
				// asserts attributes!=0, and we'd be lying about the size
				// anyway. Leave the closeId for the host to clean up at
				// process-exit time. Don't clear fo->closeId either — let the
				// existing ref-drop path try again if it gets a chance.
				DEBUG_LOG_DETOURED(funcName, "(flush fd=%i closeId=%u SNAPSHOT_FAILED — skipped)",
					fd, fo->closeId);
				continue;
			}

			if (attributes == 0)
				attributes = S_IFREG | 0644; // unlinked inode: avoid Rpc_UpdateCloseHandle attrs!=0 assert

			FileInfo& fi = *fo->fileInfo;
			const tchar* path = fi.name;
			fi.created = true;
			fi.size = fileSize;

			DEBUG_LOG_DETOURED(funcName, "(flush fd=%i closeId=%u size=%llu) %s",
				fd, fo->closeId, (unsigned long long)fileSize, path);

			Rpc_UpdateCloseHandle(path, fo->closeId, fo->deleteOnClose,
				fo->newName.c_str(), SharedMemoryHandle{},
				fileSize, lastWritten, attributes, true);

			fo->closeId = 0; // sentinel: Shared_close skips RPC for already-finalized handle
		}
	}

	// Finalize tracked write fd before a syscall that implicitly closes it (dup2/dup3 with newfd
	// open). Snapshots while fd still points at the tracked file, then Shared_close drops the
	// entry. Repro: bash redirect-with-restore (`dup2(file,1); fork; wait; dup2(saved,1)`).
	template<typename TrueSnapshot>
	inline void Shared_PreFinalizeForOverwrite(const char* funcName, int fd, const TrueSnapshot& trueSnapshot)
	{
		// Read-side check first: most dup2/dup3 don't hit a tracked write fd.
		bool needFinalize = false;
		{
			SCOPED_READ_LOCK(g_fileHandlesLock, l);
			auto it = g_fileHandles.find(fd);
			if (it != g_fileHandles.end())
			{
				FileObject* fo = it->second.fileObject;
				if (fo && (fo->desiredAccess & AccessFlag_Write) && fo->closeId)
					needFinalize = true;
			}
		}
		if (!needFinalize)
			return;
		Shared_close(funcName, fd, []() { return 0; }, trueSnapshot);
	}

	// =========================================================================
	// UBA_INHERIT_HANDLES env var — bash-redirect-fork-execve handle handoff.
	//
	// Problem: shell does
	//   open(file, O_WRONLY) → fd_orig
	//   dup2(fd_orig, 1)             // fd 1 (stdout) now points to file
	//   close(fd_orig)                 // fd_orig gone; fd 1 still tracked, refCount=1
	//   execve(cmd)                    // execve kills userspace tracker
	//   <new program writes to fd 1, exits, kernel auto-closes>
	//
	// fd 1 survives execve (kernel-level), but our g_fileHandles is wiped with
	// the program image. The new program (sed/echo/whatever) writes to fd 1
	// as stdout in the normal way; the kernel finalizes the file when the new
	// program exits. UBA never sees a CloseFile RPC for that closeId — the
	// host's m_activeFiles[X] keeps contentSize=InvalidValue forever, surfacing
	// as size=34359738367 ghosts in DIRTABLE.
	//
	// Fix: at execve time, encode every (fd, closeId, path) in g_fileHandles
	// into UBA_INHERIT_HANDLES env var. Env vars survive execve. New program's
	// detour init parses the env var and repopulates g_fileHandles[fd] with a
	// fresh FileObject carrying the closeId. The existing exit-time flush
	// (stub_flush_handles_at_exit / LD_PRELOAD Deinit) then finds the entry,
	// fstats the (now-finalized-on-disk) fd, and fires Rpc_UpdateCloseHandle
	// with the real size.
	//
	// Format: <fd>:<closeId>:<pathLen>:<pathBytes>;<fd>:<closeId>:…
	// pathLen prefix lets paths contain colons/semicolons safely.
	//
	// FD_CLOEXEC fds are filtered out — kernel auto-closes at execve, so emitting them
	// would register stale entries and the next open() reusing that fd number trips the
	// "File handle N already added" assert. Bash redirects (`exec 5>file`) don't set CLOEXEC.
	// =========================================================================
	inline void Shared_BuildInheritedHandlesEnv(StringBufferBase& out)
	{
		// Reserve "UBA_INHERIT_HANDLES=" prefix; caller checks count > prefixLen
		// to decide whether to emit at all.
		out.Append(TCV("UBA_INHERIT_HANDLES="));
		u32 prefixLen = out.count;

		SCOPED_READ_LOCK(g_fileHandlesLock, lock);
		for (auto& kv : g_fileHandles)
		{
			if (kv.second.hasCloexec)
				continue; // kernel will close at execve; do not hand to child
			FileObject* fo = kv.second.fileObject;
			if (!fo || !fo->closeId || !fo->fileInfo)
				continue;
			if (!(fo->desiredAccess & AccessFlag_Write))
				continue;

			const tchar* path = fo->fileInfo->name ? fo->fileInfo->name : TC("");
			u64 pathLen = TStrlen(path);
			out.AppendValue((u64)kv.first).Append(':')
			   .AppendValue((u64)fo->closeId).Append(':')
			   .AppendValue(pathLen).Append(':')
			   .Append(path, pathLen).Append(';');
		}

		if (out.count == prefixLen)
			out.Resize(0); // nothing to emit; signal empty
	}

	// Adopt UBA_INHERIT_HANDLES from env on detour init. Caller passes a
	// libc-style getenv lambda; LD_PRELOAD uses ::getenv, the stub uses an
	// envp-scan helper. Returns the number of entries adopted.
	template<typename TrueGetenv>
	u32 Shared_AdoptInheritedHandles(const TrueGetenv& trueGetenv)
	{
#if PLATFORM_LINUX
		// LD_PRELOAD: scan ::environ directly — getenv returns null at detour init time
		// (early libc cache / RTLD_NEXT timing). Stub has no ::environ — skip; trueGetenv
		// is the envp-scan helper.
		const char* env = nullptr;
		#if !UBA_STUB_BUILD
		static const char kKey[] = "UBA_INHERIT_HANDLES=";
		constexpr u32 kKeyLen = sizeof(kKey) - 1;
		if (::environ)
		{
			for (char** e = ::environ; *e; ++e)
			{
				const char* s = *e;
				bool match = true;
				for (u32 i = 0; i < kKeyLen; ++i)
					if (s[i] != kKey[i]) { match = false; break; }
				if (match) { env = s + kKeyLen; break; }
			}
		}
		#endif
		if (!env)
			env = trueGetenv("UBA_INHERIT_HANDLES");
		if (!env || !*env)
			return 0;

		auto parseDecimal = [](const char*& p, char terminator) -> u64
		{
			u64 v = 0;
			while (*p >= '0' && *p <= '9') { v = v * 10u + (u64)(*p - '0'); ++p; }
			if (*p != terminator) return ~u64(0);
			++p;
			return v;
		};

		UBA_ASSERT(g_fileHandles.empty());

		u32 adopted = 0;
		const char* p = env;
		while (*p)
		{
			u64 fd      = parseDecimal(p, ':'); if (fd      == ~u64(0)) break;
			u64 closeId = parseDecimal(p, ':'); if (closeId == ~u64(0)) break;
			u64 pathLen = parseDecimal(p, ':'); if (pathLen == ~u64(0)) break;
			const char* path = p;
			// Verify enough bytes + trailing ';' separator.
			u64 i = 0;
			for (; i < pathLen && p[i] != 0; ++i) {}
			if (i != pathLen || p[pathLen] != ';')
				break;
			p += pathLen + 1;

			// Register the FileInfo in g_mappedFileTable so Shared_close /
			// Deinit can resolve fileInfo->name → handleName for the
			// Rpc_UpdateCloseHandle RPC.
			StringBuffer<MaxPath> nameBuf;
			nameBuf.Append(path, (u32)pathLen);
			StringKey fileNameKey = ToFilenameKey(nameBuf);

			FileInfo* infoPtr = nullptr;
			{
				SCOPED_WRITE_LOCK(g_mappedFileTable.m_lookupLock, fileTableLock);
				auto insres = g_mappedFileTable.m_lookup.try_emplace(fileNameKey);
				FileInfo& info = insres.first->second;
				if (insres.second)
				{
					info.originalName = g_mappedFileTable.m_memoryBlock.Strdup(nameBuf).data;
					info.name = info.originalName;
					info.fileNameKey = fileNameKey;
				}
				info.lastDesiredAccess |= (u32)AccessFlag_Write;
				infoPtr = &info;
			}

			// Insert into g_fileHandles[fd] with a fresh FileObject. The
			// closeId carries the host's m_activeFiles slot; the eventual
			// exit-time flush in this process will Rpc_UpdateCloseHandle it
			// with the real fstat size.
			{
				SCOPED_WRITE_LOCK(g_fileHandlesLock, fhLock);
				auto insres = g_fileHandles.insert({(int)fd, DetouredHandle()});
				if (!insres.second)
				{
					// Slot already taken — typically impossible at init time
					// but be defensive: skip (the new program will have its
					// own handle for this fd already).
					DEBUG_LOG(TC("[inherit] fd=%llu already tracked, skipping closeId=%llu"),
						(unsigned long long)fd, (unsigned long long)closeId);
					continue;
				}
				DetouredHandle& h = insres.first->second;
				auto fo = new FileObject();
				fo->closeId = (u32)closeId;
				fo->fileInfo = infoPtr;
				fo->desiredAccess = (u32)AccessFlag_Write;
				fo->refCount = 1;
				h.fileObject = fo;
			}

			DEBUG_LOG(TC("[inherit] fd=%llu closeId=%llu path=%s"),
				(unsigned long long)fd, (unsigned long long)closeId, nameBuf.data);
			++adopted;
		}
		return adopted;
#else
		return 0;
#endif
	}

	// UBA_VFS_OVERLAYS: propagate g_vfsEntries across execve (bind mounts + chroot/pivot_root).
	// Format: <vfsLen>:<vfs>:<localLen>:<local>;... (length-prefixed).
	void Shared_BuildVfsEnv(StringBufferBase& out); // defined in UbaDetoursShared.cpp (g_vfsEntries scope)

	template<typename TrueGetenv>
	u32 Shared_AdoptVfsEntries(const TrueGetenv& trueGetenv)
	{
#if PLATFORM_LINUX
		// Same getenv-vs-environ caveat as Shared_AdoptInheritedHandles.
		const char* env = nullptr;
		#if !UBA_STUB_BUILD
		static const char kKey[] = "UBA_VFS_OVERLAYS=";
		constexpr u32 kKeyLen = sizeof(kKey) - 1;
		if (::environ)
		{
			for (char** e = ::environ; *e; ++e)
			{
				const char* s = *e;
				bool match = true;
				for (u32 i = 0; i < kKeyLen; ++i)
					if (s[i] != kKey[i]) { match = false; break; }
				if (match) { env = s + kKeyLen; break; }
			}
		}
		#endif
		if (!env)
			env = trueGetenv("UBA_VFS_OVERLAYS");
		if (!env || !*env)
			return 0;

		auto parseDecimal = [](const char*& p, char terminator) -> u64
		{
			u64 v = 0;
			while (*p >= '0' && *p <= '9') { v = v * 10u + (u64)(*p - '0'); ++p; }
			if (*p != terminator) return ~u64(0);
			++p;
			return v;
		};

		u32 adopted = 0;
		const char* p = env;
		while (*p)
		{
			u64 vfsLen = parseDecimal(p, ':'); if (vfsLen == ~u64(0)) break;
			const char* vfs = p;
			u64 i = 0;
			for (; i < vfsLen && p[i] != 0; ++i) {}
			if (i != vfsLen || p[vfsLen] != ':') break;
			p += vfsLen + 1;

			u64 localLen = parseDecimal(p, ':'); if (localLen == ~u64(0)) break;
			const char* local = p;
			i = 0;
			for (; i < localLen && p[i] != 0; ++i) {}
			if (i != localLen || p[localLen] != ';') break;
			p += localLen + 1;

			// Parent already deduped; skip dedup loop + defer bucket-index rebuild (372us → 30us @ 140 entries).
			if (AddVfsEntryBulkNoDedup(StringView(vfs, (u32)vfsLen), StringView(local, (u32)localLen)))
				++adopted;
		}
		if (adopted)
			FinalizeVfsEntriesAfterBulk();
		return adopted;
#else
		return 0;
#endif
	}

	// Adopt UBA_PASSTHROUGH_ROOTS — paths registered by parent's Shared_mount fstype branch.
	template<typename TrueGetenv>
	u32 Shared_AdoptPassThroughRoots(const TrueGetenv& trueGetenv)
	{
#if PLATFORM_LINUX
		const char* env = nullptr;
		#if !UBA_STUB_BUILD
		static const char kKey[] = "UBA_PASSTHROUGH_ROOTS=";
		constexpr u32 kKeyLen = sizeof(kKey) - 1;
		if (::environ)
		{
			for (char** e = ::environ; *e; ++e)
			{
				const char* s = *e;
				bool match = true;
				for (u32 i = 0; i < kKeyLen; ++i)
					if (s[i] != kKey[i]) { match = false; break; }
				if (match) { env = s + kKeyLen; break; }
			}
		}
		#endif
		if (!env)
			env = trueGetenv("UBA_PASSTHROUGH_ROOTS");
		if (!env || !*env)
			return 0;

		auto parseDecimal = [](const char*& p, char terminator) -> u64
		{
			u64 v = 0;
			while (*p >= '0' && *p <= '9') { v = v * 10u + (u64)(*p - '0'); ++p; }
			if (*p != terminator) return ~u64(0);
			++p;
			return v;
		};

		u32 adopted = 0;
		const char* p = env;
		while (*p)
		{
			u64 len = parseDecimal(p, ':'); if (len == ~u64(0)) break;
			const char* root = p;
			u64 i = 0;
			for (; i < len && p[i] != 0; ++i) {}
			if (i != len || p[len] != ';') break;
			p += len + 1;
			if (AddPassThroughRoot(StringView(root, (u32)len)))
				++adopted;
		}
		return adopted;
#else
		return 0;
#endif
	}

	// Shared_Internal_exec — see header for the contract. Both LD_PRELOAD's
	// Internal_exec and the stub's SYS_execve handler instantiate this with
	// their own execve / nanosleep / snapshot lambdas. Builds the
	// UBA_INHERIT_HANDLES env var (see Shared_BuildInheritedHandlesEnv above)
	// just before trueExecve so the post-execve program can adopt our open-
	// for-write fds.
	template<typename TrueExecve, typename TrueNanosleepUs, typename TrueSnapshot>
	int Shared_Internal_exec(const char* func, const char* pathname,
		char* const argv[], char* const envp[], bool fixupPath,
		const TrueExecve& trueExecve, const TrueNanosleepUs& trueNanosleepUs,
		const TrueSnapshot& trueSnapshot)
	{
		(void)trueSnapshot;

		ProcessCreator pc;
		int res = pc.SendMessage(MessageType_ExecProcess, "ExecProcess", g_stats.execProcess, pathname, argv, envp, fixupPath, true);
		if (res != 0)
		{
			DEBUG_LOG_DETOURED(func, "SendMessage failed");
			return res;
		}

		// Build UBA_INHERIT_HANDLES so post-execve init can adopt our open-for-write fds.
		// Storage on this frame; pointer survives trueExecve retry. 64K fits trusty/nsjail edges.
		StringBuffer<64*1024> inheritEnv;
		Shared_BuildInheritedHandlesEnv(inheritEnv);
		if (inheritEnv.count > 0)
		{
			// pc.envvars is nullptr-terminated; insert our entry before the null.
			if (!pc.envvars.empty() && pc.envvars.back() == nullptr)
				pc.envvars.pop_back();
			pc.envvars.push_back(inheritEnv.data);
			pc.envvars.push_back(nullptr);
			DEBUG_LOG_DETOURED(func, "(handed off) %s", inheritEnv.data);
		}

		// Propagate VFS overlays (bind mounts + chroot/pivot_root). nsjail trees can be hundreds of entries.
		StringBuffer<64*1024> vfsEnv;
		Shared_BuildVfsEnv(vfsEnv);
		if (vfsEnv.count > 0)
		{
			if (!pc.envvars.empty() && pc.envvars.back() == nullptr)
				pc.envvars.pop_back();
			pc.envvars.push_back(vfsEnv.data);
			pc.envvars.push_back(nullptr);
			DEBUG_LOG_DETOURED(func, "(vfs-overlays) %s", vfsEnv.data);
		}

		// Propagate pass-through subtree roots.
		StringBuffer<4096> passEnv;
		Shared_BuildPassThroughEnv(passEnv);
		if (passEnv.count > 0)
		{
			if (!pc.envvars.empty() && pc.envvars.back() == nullptr)
				pc.envvars.pop_back();
			pc.envvars.push_back(passEnv.data);
			pc.envvars.push_back(nullptr);
			DEBUG_LOG_DETOURED(func, "(pass-through) %s", passEnv.data);
		}

		DEBUG_LOG_TRUE(func, "%s %s", pc.realApplication.data(), pc.cmdLineWithoutApplication.data());

		// WSL2 i_writecount decrements in task_work_run() AFTER close(2) returns, so the
		// ETXTBSY window during 300+ parallel-write startups can exceed 100ms. Budget ~5s, capped.
		constexpr u32 maxRetries = 30;
		for (u32 attempt = 0; ; ++attempt)
		{
			int err = trueExecve(pc.realApplication.data(), pc.fixedArgv, (char**)pc.envvars.data());
			if (err == 0)
				return 0; // Unreachable on success — kernel replaced our memory.

			constexpr int kETXTBSY = 26;
			if (err != kETXTBSY || attempt >= maxRetries)
			{
				DEBUG_LOG_TRUE(func, "FAILED!! errno=%d attempts=%u", err, attempt + 1);
				return -err; // negative errno; LD_PRELOAD wrapper translates back.
			}
			u32 sleepUs = 20000u * (attempt + 1);
			if (sleepUs > 200000u) sleepUs = 200000u;
			trueNanosleepUs(sleepUs);
			DEBUG_LOG_TRUE(func, "ETXTBSY retry %u for %s (slept %uus)", attempt + 1, pc.realApplication.data(), sleepUs);
		}
	}

	// DuplicateFd — see header for the contract. newCloexec reflects the FD_CLOEXEC bit on the *new* fd:
	//   dup / dup2 / F_DUPFD  -> false (kernel clears CLOEXEC); dup3(O_CLOEXEC) / F_DUPFD_CLOEXEC -> true.
	inline void DuplicateFd(int oldFd, int newFd, bool newCloexec)
	{
		SCOPED_WRITE_LOCK(g_fileHandlesLock, lock);
		auto findIt = g_fileHandles.find(oldFd);
		if (findIt == g_fileHandles.end())
			return;
		DetouredHandle& h = findIt->second;
		FileObject* fo = h.fileObject;
		++fo->refCount;
		DetouredHandle& dest = g_fileHandles[newFd];
		dest.fileObject = fo;
		dest.hasCloexec = newCloexec;
	}

	// Shared_close — see header for the contract. The body lives here so
	// LD_PRELOAD and the static stub instantiate the same refcounting +
	// Rpc_UpdateCloseHandle logic; only the syscall mechanics (close + the
	// dup/fstat/close-of-dup snapshot) come from the caller's lambdas.
	template<typename TrueClose, typename TrueSnapshot>
	int Shared_close(const char* funcName, int fd, const TrueClose& trueClose, const TrueSnapshot& trueSnapshot)
	{
		auto runClose = [&]()
			{
				TimerScope ts(g_kernelStats.closeFile);
				int res = trueClose();
				DEBUG_LOG_TRUE(funcName, "(%i) -> %i", fd, res);
				return res;
			};

		if (!g_isDetouring)
			return runClose();

		SCOPED_WRITE_LOCK(g_fileHandlesLock, lock);
		auto findIt = g_fileHandles.find(fd);
		if (findIt == g_fileHandles.end())
			return runClose();

		DetouredHandle& h = findIt->second;
		FileObject* fo = h.fileObject;
		UBA_ASSERTF(fo->refCount >= 1, "FileObject needs to have ref count when closed");
		g_fileHandles.erase(findIt);
		lock.Leave();

		// Multiple fds (e.g. via dup) reference the same FileObject. Only
		// finalize the file (and notify the host) when the last fd is gone.
		if (--fo->refCount)
			return runClose();

		auto dg = MakeGuard([fo]() { delete fo->dirInfo; delete fo; });

		// Silent closeId=0 on a write file means the staging file we created
		// via Rpc_CreateFileW never gets finalized by the agent — the build
		// sees a truncated/stale output. If you hit this, track back to the
		// Shared_open / Shared_freopen path that produced this FileObject.
		if (fo->fileInfo)
		{
			UBA_ASSERTF(!(fo->desiredAccess & AccessFlag_Write) || fo->closeId != 0,
				"%s: write-access fd=%i reached refCount=0 with closeId=0 — Rpc_UpdateCloseHandle will be skipped for %s",
				funcName, fd, fo->fileInfo->name ? fo->fileInfo->name : "<null>");
		}

		if (!fo->closeId)
			return runClose();

		// Snapshot the file metadata BEFORE closing the original fd so any
		// stdio buffering has a chance to flush. The lambda dups the fd
		// internally and captures size/mtime/mode; if it fails we still
		// send Rpc_UpdateCloseHandle with the fallback values.
		u64 fileSize = 0;
		u64 lastWritten = 0;
		u32 attributes = 0;
		bool snapOk = trueSnapshot(fd, fileSize, lastWritten, attributes);

		int res = trueClose();
		int closeErrno = (res < 0) ? errno : 0;

		FileInfo& fi = *fo->fileInfo;
		const tchar* path = fi.name;
		fi.created = true; // TODO: Should be set earlier
		fi.size = fileSize;

		DEBUG_LOG_TRUE(funcName, "(%i) Size: %llu -> %i", fd, fileSize, res);
		(void)snapOk;

		Rpc_UpdateCloseHandle(path, fo->closeId, fo->deleteOnClose, fo->newName.c_str(), SharedMemoryHandle{}, fileSize, lastWritten, attributes, true);
		if (res < 0)
			errno = closeErrno; // preserve close()'s errno across Rpc_UpdateCloseHandle
		return res;
	}

	// SynthStatFromEntry / SharedStatPath — fill caller's `struct stat` from the dirtable.
	// Shared by GlibcStat/Lstat/Fstatat (UbaGlibc.cpp) and the SYS_newfstatat/statx dispatcher arms
	// (UbaStaticStubCore.cpp). Real_fstatat is used for kernel passthrough on the pre-init / NODETOUR
	// branches; the dirtable hit synthesizes locally without a syscall.
	inline void SynthStatFromEntry(struct stat* sbOut, const DirectoryTable::EntryInformation& info)
	{
		u64 reportedSize = (info.size == InvalidValue) ? 0 : info.size;
		__builtin_memset(sbOut, 0, sizeof(*sbOut));
		sbOut->st_dev     = info.volumeSerial;
		sbOut->st_ino     = info.fileIndex;
		sbOut->st_nlink   = 1;
		sbOut->st_mode    = (mode_t)info.attributes;
		sbOut->st_size    = reportedSize;
		sbOut->st_blksize = 4096;
		sbOut->st_blocks  = (reportedSize + 511) / 512;
		u64 sec  = info.lastWrite / 10000000ull;
		u64 nsec = (info.lastWrite % 10000000ull) * 100;
		sbOut->st_atimespec.tv_sec = sec; sbOut->st_atimespec.tv_nsec = nsec;
		sbOut->st_mtimespec.tv_sec = sec; sbOut->st_mtimespec.tv_nsec = nsec;
		sbOut->st_ctimespec.tv_sec = sec; sbOut->st_ctimespec.tv_nsec = nsec;
	}

	// Shared_unlinkat / Shared_renameat / Shared_mkdirat — high-level entry wrappers that
	// ResolvePath dirfd-relative paths and delegate to the resolved-path Shared_* templates.
	// Shared between UBA_WRAPPER(unlinkat) (LD_PRELOAD), Glibc<Unlinkat|Rename|Mkdir>at
	// (stub libc patch), and the SYS_nr_unlinkat/renameat/mkdirat dispatcher arms (stub Go bridge).
	// Returns libc-style (-1 + errno on failure); callers convert to kernel-style as needed.

	inline int Shared_unlinkat(const char* funcName, int dirfd, const char* path, int flags)
	{
		if (!path)
			return Real_unlinkat(errno, dirfd, path, flags);
		StringBuffer<> resolvedTemp;
		const char* resolved = ResolvePath(dirfd, path, resolvedTemp);
		if (!resolved)
			return Real_unlinkat(errno, dirfd, path, flags);
		int err = 0;
		int rc = (flags & AT_REMOVEDIR)
			? Shared_rmdir     (funcName, resolved, err, [](const char* p) { return Real_rmdir (errno, p); })
			: Shared_deleteFile(funcName, resolved, err, [](const char* p) { return Real_unlink(errno, p); });
		if (rc != 0 && err) errno = err;
		return rc;
	}

	inline int Shared_renameat(const char* funcName, int olddirfd, const char* oldpath, int newdirfd, const char* newpath)
	{
		StringBuffer<> oldTemp, newTemp;
		const char* resolvedOld = ResolvePath(olddirfd, oldpath, oldTemp);
		const char* resolvedNew = ResolvePath(newdirfd, newpath, newTemp);
		if (!resolvedOld || !resolvedNew)
			return Real_renameat(errno, olddirfd, oldpath, newdirfd, newpath);
		int err = 0;
		int rc = Shared_rename(funcName, resolvedOld, resolvedNew, err,
			[](const char* o, const char* n) { return Real_rename(errno, o, n); });
		if (rc != 0 && err) errno = err;
		return rc;
	}

	inline int Shared_mkdirat(const char* funcName, int dirfd, const char* path, mode_t mode)
	{
		if (!path)
			return Real_mkdirat(errno, dirfd, path, mode);
		StringBuffer<> resolvedTemp;
		const char* resolved = ResolvePath(dirfd, path, resolvedTemp);
		if (!resolved)
			return Real_mkdirat(errno, dirfd, path, mode);
		int err = 0;
		int rc = Shared_mkdir(funcName, resolved, mode, err,
			[](const char* p, unsigned int m) { return Real_mkdir(errno, p, m); });
		if (rc != 0 && err) errno = err;
		return rc;
	}

	// Shared_faccessat — DIRTABLE-driven existence check. Returns 0 if dirtable says exists,
	// -1+ENOENT if dirtable says no, or kernel passthrough for unresolvable/non-detourable paths.
	inline int Shared_faccessat(const char* funcName, int dirfd, const char* path, int mode, int flags)
	{
		if (!path)
			return Real_faccessat(errno, dirfd, path, mode, flags);
		StringBuffer<> resolvedTemp;
		const char* resolved = ResolvePath(dirfd, path, resolvedTemp);
		if (!resolved)
			return Real_faccessat(errno, dirfd, path, mode, flags);
		resolved = ResolveProcSelfFd(resolved);
		StringBuffer<> fixedFile;
		if (!FixPath(fixedFile, resolved) || !CanDetour2(fixedFile))
			return Real_faccessat(errno, dirfd, path, mode, flags);
		DevirtualizePath(fixedFile);
		StringKey key = ToFilenameKey(fixedFile);
		DirectoryTableEntryInformation info;
		if (Rpc_GetEntryInformation(info, key, fixedFile, false))
		{
			DEBUG_LOG_DETOURED(funcName, "%s (-> %s) -> 0 (DIRTABLE exists)", path, fixedFile.data);
			return 0;
		}
		DEBUG_LOG_DETOURED(funcName, "%s (-> %s) DIRTABLE -> -1 (ENOENT)", path, fixedFile.data);
		errno = ENOENT;
		return -1;
	}

#if PLATFORM_LINUX
	// Mirror of SynthStatFromEntry for struct statx. Used by SYS_nr_statx dispatcher arm.
	// Same DIRTABLE→struct conversion path keeps stat() and statx() consistent for the same entry.
	inline void SharedStatxFromEntry(struct ::statx* sxOut, const DirectoryTable::EntryInformation& info)
	{
		u64 reportedSize = (info.size == InvalidValue) ? 0 : info.size;
		__builtin_memset(sxOut, 0, sizeof(*sxOut));
		sxOut->stx_mask    = STATX_BASIC_STATS;
		sxOut->stx_blksize = 4096;
		sxOut->stx_nlink   = 1;
		sxOut->stx_mode    = (u16)info.attributes;
		sxOut->stx_ino     = info.fileIndex;
		sxOut->stx_size    = reportedSize;
		sxOut->stx_blocks  = (reportedSize + 511) / 512;
		s64 sec  = (s64)(info.lastWrite / 10000000ull);
		u32 nsec = (u32)((info.lastWrite % 10000000ull) * 100);
		sxOut->stx_atime.tv_sec = sec; sxOut->stx_atime.tv_nsec = nsec;
		sxOut->stx_ctime.tv_sec = sec; sxOut->stx_ctime.tv_nsec = nsec;
		sxOut->stx_mtime.tv_sec = sec; sxOut->stx_mtime.tv_nsec = nsec;
		sxOut->stx_dev_major = (u32)info.volumeSerial;
	}

	inline int Shared_statx(const char* funcName, int dirfd, const char* path, int flags, unsigned int mask, struct ::statx* sxOut)
	{
		if (!path || !sxOut)
			return Real_statx(errno, dirfd, path, flags, mask, sxOut);
		StringBuffer<> resolvedTemp;
		const char* resolved = ResolvePath(dirfd, path, resolvedTemp);
		if (!resolved)
			return Real_statx(errno, dirfd, path, flags, mask, sxOut);
		resolved = ResolveProcSelfFd(resolved);
		StringBuffer<> fixedFile;
		if (!FixPath(fixedFile, resolved) || !CanDetour2(fixedFile))
			return Real_statx(errno, dirfd, path, flags, mask, sxOut);
		DevirtualizePath(fixedFile);
		StringKey key = ToFilenameKey(fixedFile);
		DirectoryTableEntryInformation info;
		if (Rpc_GetEntryInformation(info, key, fixedFile, false))
		{
			SharedStatxFromEntry(sxOut, info);
			DEBUG_LOG_DETOURED(funcName, "%s (-> %s size=%llu attr=0x%x) -> 0 (synthesized)",
				path, fixedFile.data,
				(u64)((info.size == InvalidValue) ? 0 : info.size), (u32)info.attributes);
			return 0;
		}
		DEBUG_LOG_DETOURED(funcName, "%s (-> %s) DIRTABLE -> -1 (ENOENT)", path, fixedFile.data);
		errno = ENOENT;
		return -1;
	}
#endif

	// followSymlink unused — Rpc_GetEntryInformation resolves chains. Kept for symmetry with libc API.
	inline int SharedStatPath(const char* funcName, const char* path, struct stat* sbOut, bool followSymlink)
	{
		int flags = followSymlink ? 0 : AT_SYMLINK_NOFOLLOW;
		if (!path || !sbOut)
			return Real_fstatat(errno, AT_FDCWD, path, sbOut, flags);

		path = ResolveProcSelfFd(path);

		StringBuffer<> fixedFile;
		if (!FixPath(fixedFile, path) || !CanDetour2(fixedFile))
		{
			int r = Real_fstatat(errno, AT_FDCWD, path, sbOut, flags);
			DEBUG_LOG_TRUE(funcName, "%s NODETOUR -> %d", path, r);
			return r;
		}
		StringBuffer<> preDevirt(fixedFile); // pre-Devirt form for IsVfsParentDir

		DevirtualizePath(fixedFile);

		StringKey key = ToFilenameKey(fixedFile);
		DirectoryTableEntryInformation info;
		if (Rpc_GetEntryInformation(info, key, fixedFile, false))
		{
			SynthStatFromEntry(sbOut, info);
			DEBUG_LOG_DETOURED(funcName, "%s (-> %s size=%llu attr=0x%x) -> 0 (synthesized)",
				path, fixedFile.data,
				(u64)((info.size == InvalidValue) ? 0 : info.size), (u32)info.attributes);
			return 0;
		}

		// Unbound bind-mount parent: synth dir attrs so Go's os.Getwd→stat(".") doesn't ENOENT.
		if (IsVfsParentDir(preDevirt))
		{
			__builtin_memset(sbOut, 0, sizeof(*sbOut));
			sbOut->st_nlink = 2;
			sbOut->st_mode  = S_IFDIR | 0755;
			DEBUG_LOG_DETOURED(funcName, "%s (-> %s) DIRTABLE -> 0 (vfs-parent synth)", path, fixedFile.data);
			return 0;
		}

		DEBUG_LOG_DETOURED(funcName, "%s (-> %s) DIRTABLE -> -1 (ENOENT)", path, fixedFile.data);
		errno = ENOENT;
		return -1;
	}
}
