// Copyright Epic Games, Inc. All Rights Reserved.
//
// LD_PRELOAD dlsym-based "True_<func>" infrastructure: storage decls, type aliases,
// init template, and the macro shorthands callers use (UBA_WRAPPER / TRUE_WRAPPER /
// UBA_INIT_DETOUR). Storage is defined exactly once in UbaDetoursFunctionsPosix.cpp;
// every other detour-side TU sees externs through this header.

#pragma once

#include "UbaPlatform.h"

// System headers the DETOURED_FUNCTIONS X-macro's decltype lines depend on.
#if PLATFORM_LINUX
#include <dlfcn.h>
#include <dirent.h>
#include <fcntl.h>
#include <glob.h>
#include <signal.h>
#include <spawn.h>
#include <stdio.h>
#include <stdlib.h>
#include <sys/mount.h>
#include <sys/stat.h>
#include <sys/statvfs.h>
#include <sys/syscall.h>
#include <sys/wait.h>
#include <sys/xattr.h>
#include <unistd.h>
#endif

#if PLATFORM_MAC
	#define UBA_WRAPPER(func) uba_##func
	#define TRUE_WRAPPER(func) func

	#ifndef DYLD_INTERPOSE
	#define DYLD_INTERPOSE(_replacement,_replacee) \
		__attribute__((used)) static struct{ const void* replacement; const void* replacee; } _interpose_##_replacee \
			__attribute__ ((section ("__DATA,__interpose"))) = { (const void*)(unsigned long)&_replacement, (const void*)(unsigned long)&_replacee };
	#endif
#else
	#define UBA_WRAPPER(func) func
	#define TRUE_WRAPPER(func) True_##func

	// _FORTIFY_SOURCE-rewrite targets: no public header declares __open_2 et al., but DETOURED_FUNCTION's decltype needs them.
	extern "C" int __open_2(const char* pathname, int flags);
	extern "C" int __openat_2(int dirfd, const char* pathname, int flags);
	extern "C" int __open64_2(const char* pathname, int flags);
	extern "C" int __openat64_2(int dirfd, const char* pathname, int flags);
	extern "C" int pivot_root(const char* new_root, const char* put_old); // no glibc header declares it
#endif

#ifndef UBA_DETOUR_DEBUG
#define UBA_DETOUR_DEBUG 0
#endif

#if UBA_DETOUR_DEBUG && PLATFORM_LINUX
#define DETOURED_FUNCTION_DEBUG \
	DETOURED_FUNCTION(write) \
	DETOURED_FUNCTION(fwrite) \

#else
#define DETOURED_FUNCTION_DEBUG
#endif

#if PLATFORM_LINUX
#define DETOURED_FUNCTION_MACOS
#define DETOURED_FUNCTION_LINUX \
	DETOURED_FUNCTION(dup3) \
	DETOURED_FUNCTION(clone) \
	DETOURED_FUNCTION(fexecve) \
	DETOURED_FUNCTION(get_current_dir_name) \
	DETOURED_FUNCTION(fopen64) \
	DETOURED_FUNCTION(freopen64) \
	DETOURED_FUNCTION(secure_getenv) \
	DETOURED_FUNCTION(eaccess) \
	DETOURED_FUNCTION(mkstemp64) \
	DETOURED_FUNCTION(mkostemp64) \
	DETOURED_FUNCTION(fcntl) \
	DETOURED_FUNCTION(fcntl64) \
	DETOURED_FUNCTION(__xstat) \
	DETOURED_FUNCTION(__xstat64) \
	DETOURED_FUNCTION(__lxstat) \
	DETOURED_FUNCTION(__lxstat64) \
	DETOURED_FUNCTION(__fxstat) \
	DETOURED_FUNCTION(__fxstat64) \
	DETOURED_FUNCTION(__fxstatat) \
	DETOURED_FUNCTION(__fxstatat64) \
	DETOURED_FUNCTION(pread64) \
	DETOURED_FUNCTION(open64) \
	DETOURED_FUNCTION(openat64) \
	DETOURED_FUNCTION(creat64) \
	DETOURED_FUNCTION(__open_2) \
	DETOURED_FUNCTION(__openat_2) \
	DETOURED_FUNCTION(__open64_2) \
	DETOURED_FUNCTION(__openat64_2) \
	DETOURED_FUNCTION(listxattr) \
	DETOURED_FUNCTION(llistxattr) \
	DETOURED_FUNCTION(flistxattr) \
	DETOURED_FUNCTION(stat64) \
	DETOURED_FUNCTION(lstat64) \
	DETOURED_FUNCTION(fstat64) \
	DETOURED_FUNCTION(scandir64) \
	DETOURED_FUNCTION(readdir64) \
	DETOURED_FUNCTION(fstatat64) \
	DETOURED_FUNCTION(fpathconf) \
	DETOURED_FUNCTION(pathconf) \
	DETOURED_FUNCTION(renameat2) \
	DETOURED_FUNCTION(syscall) \
	DETOURED_FUNCTION(statx) \
	DETOURED_FUNCTION(mount) \
	DETOURED_FUNCTION(umount) \
	DETOURED_FUNCTION(umount2) \
	DETOURED_FUNCTION(pivot_root) \
	DETOURED_FUNCTION(statvfs) \
	DETOURED_FUNCTION(fstatvfs) \

#else
#define DETOURED_FUNCTION_LINUX
#define DETOURED_FUNCTION_MACOS \
	DETOURED_FUNCTION(_NSGetExecutablePath) \
	DETOURED_FUNCTION(execvP) \

#endif

#define DETOURED_FUNCTIONS \
	DETOURED_FUNCTION(chdir) \
	DETOURED_FUNCTION(fchdir) \
	DETOURED_FUNCTION(mkdir) \
	DETOURED_FUNCTION(mkdirat) \
	DETOURED_FUNCTION(rmdir) \
	DETOURED_FUNCTION(chroot) \
	DETOURED_FUNCTION(getcwd) \
	DETOURED_FUNCTION(getenv) \
	DETOURED_FUNCTION(setenv) \
	DETOURED_FUNCTION(unsetenv) \
	DETOURED_FUNCTION(realpath) \
	DETOURED_FUNCTION(readlink) \
	DETOURED_FUNCTION(readlinkat) \
	DETOURED_FUNCTION(read) \
	DETOURED_FUNCTION(pread) \
	DETOURED_FUNCTION(open) \
	DETOURED_FUNCTION(openat) \
	DETOURED_FUNCTION(creat) \
	DETOURED_FUNCTION(dup) \
	DETOURED_FUNCTION(dup2) \
	DETOURED_FUNCTION(close) \
	DETOURED_FUNCTION(fopen) \
	DETOURED_FUNCTION(fdopen) \
	DETOURED_FUNCTION(freopen) \
	DETOURED_FUNCTION(fchmod) \
	DETOURED_FUNCTION(fchmodat) \
	DETOURED_FUNCTION(fstat) \
	DETOURED_FUNCTION(faccessat) \
	DETOURED_FUNCTION(fstatat) \
	DETOURED_FUNCTION(futimens) \
	DETOURED_FUNCTION(fclose) \
	DETOURED_FUNCTION(mkstemp) \
	DETOURED_FUNCTION(mkostemp) \
	DETOURED_FUNCTION(mkstemps) \
	DETOURED_FUNCTION(mkostemps) \
	DETOURED_FUNCTION(opendir) \
	DETOURED_FUNCTION(fdopendir) \
	DETOURED_FUNCTION(dirfd) \
	DETOURED_FUNCTION(readdir) \
	DETOURED_FUNCTION(rewinddir) \
	DETOURED_FUNCTION(scandir) \
	DETOURED_FUNCTION(seekdir) \
	DETOURED_FUNCTION(telldir) \
	DETOURED_FUNCTION(closedir) \
	DETOURED_FUNCTION(stat) \
	DETOURED_FUNCTION(truncate) \
	DETOURED_FUNCTION(lstat) \
	DETOURED_FUNCTION(lseek) \
	DETOURED_FUNCTION(glob) \
	DETOURED_FUNCTION(chmod) \
	DETOURED_FUNCTION(rename) \
	DETOURED_FUNCTION(renameat) \
	DETOURED_FUNCTION(utimensat) \
	DETOURED_FUNCTION(remove) \
	DETOURED_FUNCTION(link) \
	DETOURED_FUNCTION(linkat) \
	DETOURED_FUNCTION(unlink) \
	DETOURED_FUNCTION(unlinkat) \
	DETOURED_FUNCTION(symlink) \
	DETOURED_FUNCTION(access) \
	DETOURED_FUNCTION(posix_spawn) \
	DETOURED_FUNCTION(posix_spawnp) \
	DETOURED_FUNCTION(wait) \
	DETOURED_FUNCTION(waitpid) \
	DETOURED_FUNCTION(waitid) \
	DETOURED_FUNCTION(wait3) \
	DETOURED_FUNCTION(wait4) \
	DETOURED_FUNCTION(system) \
	DETOURED_FUNCTION(dlopen) \
	DETOURED_FUNCTION(dladdr) \
	DETOURED_FUNCTION(execv) \
	DETOURED_FUNCTION(execve) \
	DETOURED_FUNCTION(execvp) \
	DETOURED_FUNCTION(execl) \
	DETOURED_FUNCTION(execle) \
	DETOURED_FUNCTION(execlp) \
	DETOURED_FUNCTION(fork) \
	DETOURED_FUNCTION(vfork) \
	DETOURED_FUNCTION(popen) \
	DETOURED_FUNCTION(fgets) \
	DETOURED_FUNCTION(pclose) \
	DETOURED_FUNCTION(kill) \
	DETOURED_FUNCTION(exit) \
	DETOURED_FUNCTION(_exit) \
	DETOURED_FUNCTION(_Exit) \
	DETOURED_FUNCTION(sigaction) \
	DETOURED_FUNCTION_DEBUG \
	DETOURED_FUNCTION_LINUX \
	DETOURED_FUNCTION_MACOS \

#if PLATFORM_LINUX

// Type aliases + extern storage decls. Storage itself is defined once in UbaDetoursFunctionsPosix.cpp.
#define DETOURED_FUNCTION(func) \
	using Symbol_##func = decltype(func); \
	extern Symbol_##func* True_##func;
DETOURED_FUNCTIONS
#undef DETOURED_FUNCTION

namespace uba
{
	template<typename TrueFunc>
	inline void InitDetour(TrueFunc*& trueFunc, const char* func)
	{
		if (trueFunc)
			return;
		trueFunc = (TrueFunc*)dlsym(RTLD_NEXT, func);
		if (!trueFunc)
			printf("dlsym failed on %s: %s\n", func, dlerror());
	}
}

#define UBA_INIT_DETOUR(func, ...) \
	if (!g_isDetouring || DisallowDetourDepth()) \
	{ \
		::uba::InitDetour(True_##func, #func); \
		return TRUE_WRAPPER(func)(__VA_ARGS__); \
	}

#elif PLATFORM_MAC

#define UBA_INIT_DETOUR(func, ...) \
	if (!g_isDetouring || DisallowDetourDepth()) \
	{ \
		return TRUE_WRAPPER(func)(__VA_ARGS__); \
	}

#endif
