// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaDetoursFileMappingTable.h"
#include "UbaDirectoryTable.h"
#include "UbaPathUtils.h"

#if PLATFORM_WINDOWS
#include "Windows/UbaDetoursUtilsWin.h"
#endif

namespace uba
{
	MappedFileTable::MappedFileTable(MemoryBlock& memoryBlock) : m_memoryBlock(memoryBlock), m_lookup(memoryBlock)
	{
	}

	void MappedFileTable::Init(const u8* mem, u32 tableCount, u32 tableSize)
	{
		m_mem = mem;
		m_lookup.reserve(tableCount + 100);
		m_memoryBlock.CommitNoLock(tableCount*(sizeof(GrowingUnorderedMap<StringKey, FileInfo>::value_type)+16), TC(""));
		ParseNoLock(tableSize);
	}

	void MappedFileTable::ParseNoLock(u32 tableSize)
	{
		u32 startPosition = m_memPosition;
		if (tableSize <= startPosition)
			return;

		BinaryReader reader(m_mem, startPosition);
		while (reader.GetPosition() != tableSize)
		{
			UBA_ASSERTF(reader.GetPosition() < tableSize, TC("Table mismatch. Size is %u and read position is %llu. Started at %u"), tableSize, reader.GetPosition(), startPosition);
			StringKey g = reader.ReadStringKey();
			StringBuffer<1024> mappedFileName;
			reader.ReadString(mappedFileName);
			u64 size = reader.Read7BitEncoded();
			auto insres = m_lookup.try_emplace(g);
			FileInfo& info = insres.first->second;
			if (!insres.second)
			{
				if (!info.name || mappedFileName.Equals(info.name, false))
					continue;

				// TODO: should revisit this code... code has changed to use dirtable for all files
				// .. so I _think_ that we can just always update info.name

				if (IsMemoryHandle(info.name)) // Mapped file has been re-mapped.
				{
					if (!IsMemoryHandle(mappedFileName.data))
					{
						info.name = m_memoryBlock.Strdup(mappedFileName).data;
						info.size = size;
						//UBA_ASSERT(!info.isFileMap);
						info.isFileMap = false;
					}
					else
					{
						UBA_ASSERTF(!info.memoryFile, TC("Mapped file %s has changed mapping (%s to %s) while being in use"), info.originalName, info.name, mappedFileName.data);
						info.name = m_memoryBlock.Strdup(mappedFileName).data;
						UBA_ASSERTF(size != InvalidValue, TC("File mapping of %s (%s)"), info.name, mappedFileName.data);
						info.size = size;
					}
				}
				else if (g_runningRemote && !mappedFileName.IsEmpty())
				{
					// For example a header could be read up as a file map but is now an output file.. and we need to use output file
					if (IsMemoryHandle(mappedFileName.data))
					{
						info.isFileMap = false;
						info.size = InvalidValue;
					}
					info.name = m_memoryBlock.Strdup(mappedFileName).data;
				}
				continue;
			}
			info.fileNameKey = g;
			info.name = m_memoryBlock.Strdup(mappedFileName).data;
			info.size = size;
		}
		m_memPosition = tableSize;
	}

	void MappedFileTable::Parse(u32 tableSize)
	{
		SCOPED_WRITE_LOCK(m_lookupLock, lock);
		ParseNoLock(tableSize);
	}

	void MappedFileTable::SetDeleted(const StringKey& key, const tchar* name, bool deleted)
	{
		SCOPED_WRITE_LOCK(m_lookupLock, lock);
		auto it = m_lookup.find(key);
		if (it == m_lookup.end())
			return;
		FileInfo& sourceInfo = it->second;
		sourceInfo.lastDesiredAccess = 0;
	}

	void Rpc_CreateFileW(const StringView& fileName, const StringKey& fileNameKey, u8 access, tchar* outNewName, u64 newNameCapacity, u64& outSize, u32& outCloseId, bool lock)
	{
		RPC_MESSAGE(CreateFile, createFile)
		writer.WriteString(fileName);
		writer.WriteStringKey(fileNameKey);
		writer.WriteByte(access);
		BinaryReader reader = writer.Flush();
		reader.ReadString(outNewName, newNameCapacity);
		UBA_ASSERTF(!*outNewName || IsAbsolutePath(outNewName) || outNewName[0] == '#' || outNewName[0] == '$' || outNewName[0] == MemoryHandleChar || outNewName[0] == FreeableMemoryHandleChar || outNewName[0] == WrittenMemoryHandleChar, TC("Rpc_CreateFileW: response path must be absolute or marker, got %s (sent %s)"), outNewName, fileName.data);
		outSize = reader.ReadU64();
		outCloseId = reader.ReadU32();
		u32 mappedFileTableSize = reader.ReadU32();
		DirTableSize directoryTableSize = FromU64(reader.ReadU64());
		pcs.Leave();
		DEBUG_LOG_PIPE(L"CreateFile", L"%ls (%ls)", (access == 0 ? L"ATTRIB" : ((access & AccessFlag_Write) ? L"WRITE" : L"READ")), fileName.data);

		if (lock)
			g_mappedFileTable.Parse(mappedFileTableSize);
		else
			g_mappedFileTable.ParseNoLock(mappedFileTableSize);
		g_directoryTable->ParseDirectoryTable(directoryTableSize);
	}

	void Rpc_RegisterFileForWrite(StringView fileName, const StringKey& fileNameKey)
	{
		RPC_MESSAGE(RegisterFileForWrite, registerCreateWrite)
		writer.WriteString(fileName);
		writer.WriteStringKey(fileNameKey);
		BinaryReader reader = writer.Flush();
		DirTableOffset offset = {reader.ReadU32() | OverlayTableFlag};
		DirTableSize directoryTableSize = FromU64(reader.ReadU64());
		pcs.Leave();
		DEBUG_LOG_PIPE(L"RegisterFileForWrite", L"(%ls)", fileName.data);
		g_directoryTable->ParseDirectoryTable(directoryTableSize);

		SCOPED_WRITE_LOCK(g_mappedFileTable.m_lookupLock, lock);
		auto it = g_mappedFileTable.m_lookup.find(fileNameKey);
		UBA_ASSERT(it != g_mappedFileTable.m_lookup.end());
		it->second.lastSeenTableOffset = offset;
	}

	void Rpc_CheckRemapping(const StringView& fileName, const StringKey& fileNameKey)
	{
		RPC_MESSAGE(CheckRemapping, checkRemapping)
		writer.WriteString(fileName);
		writer.WriteStringKey(fileNameKey);
		BinaryReader reader = writer.Flush();
		u32 mappedFileTableSize = reader.ReadU32();
		pcs.Leave();
		g_mappedFileTable.ParseNoLock(mappedFileTableSize);
	}

	void Rpc_CheckRemapping2(StringBufferBase& out, StringKey fileNameKey)
	{
		RPC_MESSAGE(CheckRemapping2, checkRemapping)
		writer.WriteStringKey(fileNameKey);
		BinaryReader reader = writer.Flush();
		reader.ReadString(out);
		pcs.Leave();
	}

	// UBA_STUB_BUILD note: Rpc_UpdateDirectory, Rpc_GetEntryOffset,
	// Rpc_GetEntryInformation and DirHash below are compiled into the
	// freestanding static-detour stub too — they give the stub's hook
	// observation path a real "does UBA know about this?" probe backed
	// by the directory-table cache. Their only heavy deps are operator
	// new[] (PopulateDirectoryRecursive's fallback) and a 192 KB on-
	// stack buffer, both of which the stub handles (operator new[]
	// routed to StubAllocator; alt-stack grown to 256 KB). Everything
	// downwards from Rpc_UpdateCloseHandle — UpdateWrittenFiles, the
	// SharedMemoryView helpers, Rpc_GetFullFileName* — still pulls in
	// SharedMemoryView / g_systemTemp / FixPath / WriteCallstackInfo
	// which the stub does not want to resolve, so they stay guarded.
	bool Rpc_UpdateDirectory(const StringKey& dirKey, const tchar* dirName, u64 dirNameLen, bool lockDirTable)
	{
		UBA_ASSERTF(dirNameLen != 1, TC("Rpc_UpdateDirectory called with single-char dirName '%c' (len 1)"), dirName ? dirName[0] : '?');
		if constexpr (IsLinux)
			UBA_ASSERTF(!StartsWith(dirName, TC("/proc/self")), TC("Invalid dir name: %s"), dirName);

		DirTableSize directoryTableSize;
		u32 tableOffset;
		{
			RPC_MESSAGE(ListDirectory, listDirectory)
			writer.WriteString(dirName, dirNameLen);
			writer.WriteStringKey(dirKey);
			BinaryReader reader = writer.Flush();
			directoryTableSize = FromU64(reader.ReadU64());
			tableOffset = reader.ReadU32();
			pcs.Leave();
			DEBUG_LOG_PIPE(L"ListDirectory", L"(%ls)", dirName);
		}
		if (lockDirTable)
			g_directoryTable->ParseDirectoryTable(directoryTableSize);
		else
			g_directoryTable->ParseDirectoryTableNoLock(directoryTableSize);
		return tableOffset != InvalidTableOffset;
	}

	bool Rpc_GetEntryOffset(DirTableOffset& out, const StringKey& entryNameKey, StringView entryName, bool checkIfDir)
	{
		StringBuffer<MaxPath> entryNameForKey;
		entryNameForKey.Append(entryName);
		if (CaseInsensitiveFs)
			entryNameForKey.MakeLower();
		else if (entryNameForKey.count == 1 && entryNameForKey[0] == '/')
			checkIfDir = true;

		CHECK_PATH(entryNameForKey);
		DirectoryTable::Exists exists = g_directoryTable->EntryExists(entryNameKey, entryNameForKey, checkIfDir, &out);
		if (exists == DirectoryTable::Exists_No)
			return false;
		if (exists == DirectoryTable::Exists_Yes)
			return true;

		u32 dirNameLen = entryName.count;
		if (const tchar* lastPathSeparator = TStrrchr(entryName.data, PathSeparator))
			dirNameLen = u32(lastPathSeparator - entryName.data);
		else
		{
			UBA_ASSERTF(false, TC("No path separator in entry %s (count=%u)"), entryName.count > 0 ? entryName.data : TC("(NULL)"), entryName.count);
			return false;
		}

		DirHash hash(StringView(entryNameForKey.data, dirNameLen));
		auto& dirTable = *g_directoryTable;

#if UBA_SUPPORT_SYMLINKS
		// Symlinked parent: return false so caller falls through to GetRealPath. Host's
		// WriteDirectoryEntries asserts on realpath != path. Lookup is purely local (no RPC).
		if (!(dirNameLen == 1 && entryNameForKey[0] == PathSeparator) && dirNameLen != 0)
		{
			StringView parentView(entryNameForKey.data, dirNameLen);
			// GetPath returns empty for "/sbin"-style parents; that maps to root's files map.
			StringKey grandparentKey = ToStringKey(parentView.GetPath());
			SCOPED_READ_LOCK(dirTable.m_lookupLock, gpLookLock);
			auto gpIt = dirTable.m_lookup.find(grandparentKey);
			if (gpIt != dirTable.m_lookup.end())
			{
				DirectoryTable::Directory& gpDir = gpIt->second;
				gpLookLock.Leave();
				SCOPED_READ_LOCK(gpDir.lock, gpDirLock);
				auto pIt = gpDir.files.find(hash.key);
				if (pIt != gpDir.files.end() && IsValidEntry(pIt->second))
				{
					BinaryReader reader = dirTable.GetReader(pIt->second.internal);
					reader.SkipString();
					if (IsSymlink(reader.ReadFileAttributes()))
						return false;
				}
			}
		}
#endif

		if (!Rpc_UpdateDirectory(hash.key, entryName.data, dirNameLen))
			return false;

		SCOPED_WRITE_LOCK(dirTable.m_lookupLock, lookLock);
		auto dirFindIt = dirTable.m_lookup.find(hash.key);
		if (dirFindIt == dirTable.m_lookup.end())
			return false;
		auto& dir = dirFindIt->second;

		if (checkIfDir)
		{
			dirTable.GetLatestOffset(out, dir);
			return true;
		}
		dirTable.PopulateDirectory(hash.open, dir);

		SCOPED_READ_LOCK(dir.lock, lock);
		auto findIt = dir.files.find(entryNameKey);
		if (findIt == dir.files.end())
			return false;
		out = findIt->second;
		return true;
	}

	bool Rpc_GetEntryInformation(DirectoryTableEntryInformation& out, StringKey entryNameKey, StringView entryName, bool checkIfDir, bool followSymlink, StringBufferBase* outResolvedPath, StringKey* outResolvedKey)
	{
		auto& dirTable = *g_directoryTable;
#if !UBA_SUPPORT_SYMLINKS
		DirTableOffset dirTableOffset;
		if (!Rpc_GetEntryOffset(dirTableOffset, entryNameKey, entryName, checkIfDir))
			return false;
		out.dirTableOffset = dirTableOffset;

		dirTable.GetEntryInformation(out, dirTableOffset);
		if (outResolvedPath) outResolvedPath->Clear().Append(entryName);
		if (outResolvedKey) *outResolvedKey = entryNameKey;
		return out.attributes != 0;
#else

		StringBuffer<MaxPath> path;
		StringBuffer<MaxPath> originalEntryName(entryName);
		StringBuffer<MaxPath> lastResolved;
		u32 realpathRetries = 8;
		u32 hops = 40; // POSIX ELOOP guard (SYMLOOP_MAX)

		while (true)
		{
			if (!Rpc_GetEntryOffset(out.dirTableOffset, entryNameKey, entryName, checkIfDir))
			{
				// Each failed Rpc_UpdateDirectory still populates ancestor dirs as a side effect,
				// so a subsequent GetRealPath may resolve more symlinks. Keep retrying until either
				// GetRealPath returns the same path twice (no progress) or we hit the bound.
				if (realpathRetries-- == 0)
					return false;
				StringBuffer<MaxPath> raw;
				if (!dirTable.GetRealPath(raw, originalEntryName.data))
					return false;
				path.Clear();
				FixPath(path, raw); // GetRealPath leaves `..` unresolved; normalize for the key

				if (lastResolved.count && path.Equals(lastResolved))
					return false;
				lastResolved.Clear().Append(path);

				entryName = path;
				entryNameKey = ToStringKey(entryName);
				continue;
			}
			dirTable.GetEntryInformation(out, out.dirTableOffset);
			if (!IsSymlink(out.attributes) || !followSymlink)
			{
				if (outResolvedPath) outResolvedPath->Clear().Append(entryName);
				if (outResolvedKey) *outResolvedKey = entryNameKey;
				return out.attributes != 0;
			}
			if (hops-- == 0)
				return false; // ELOOP

			StringBuffer<MaxPath> workDir(entryName.GetPath(true));
			if (!workDir.count)
				workDir.Append(PathSeparator);

			StringBuffer<MaxPath> symlinkContent;
			UBA_FOR_ASSERT(bool gotContent =) dirTable.GetEntrySymlink(symlinkContent, out.dirTableOffset);
			UBA_ASSERT(gotContent);
			UBA_FOR_ASSERT(bool fixed =) FixPath(path.Clear(), symlinkContent, workDir);
			UBA_ASSERT(fixed);

			entryName = path;
			entryNameKey = ToStringKey(entryName);
		}
#endif
	}

	DirHash::DirHash(const StringView& str)
	{
		CHECK_PATH(str);
		open.Update(str);
		key = ToStringKey(open);
	}

	// Rpc_UpdateCloseHandle is callable from the stub too — it's how the
	// static detour notifies the server that an openat'd file has been
	// closed. The `if (*newName)` branch (FixPath + ToStringKey) is the
	// only heavy dep; stub callers must pass an empty newName to skip it.
	void Rpc_UpdateCloseHandle(const tchar* handleName, u32 closeId, bool deleteOnClose, const tchar* newName, SharedMemoryHandle memoryHandle, u64 fileSize, u64 lastWriteTime, u32 attributes, bool success)
	{
		UBA_ASSERT(handleName);
		UBA_ASSERT(fileSize != InvalidValue);
		UBA_ASSERTF((!*handleName && !success) || attributes || deleteOnClose, TC("UpdateCloseHandle for file %s with attributes that are 0"), handleName);

		DirTableSize directoryTableSize;
		{
			RPC_MESSAGE(CloseFile, closeFile)
			writer.WriteString(handleName);
			writer.WriteU32(closeId);
			writer.WriteBool(deleteOnClose);
			writer.WriteBool(success);
			writer.WriteU64(memoryHandle.ToU64());
			writer.WriteU64(fileSize);
			writer.WriteU64(lastWriteTime);
			writer.WriteU32(attributes);

			if (*newName)
			{
#if UBA_STUB_BUILD
				// Stub callers must not pass a non-empty newName — they don't
				// link FixPath / ToStringKey. Assert and bail.
				UBA_ASSERTF(false, TC("UBA_STUB_BUILD passed non-empty newName to Rpc_UpdateCloseHandle"));
				writer.WriteStringKey(StringKeyZero);
#else
				StringBuffer<> fixedName;
				FixPath(fixedName, newName);
				StringBuffer<> forKey(fixedName);
				if (CaseInsensitiveFs)
					forKey.MakeLower();
				StringKey newNameKey = ToStringKey(forKey);
				writer.WriteStringKey(newNameKey);
				writer.WriteString(fixedName);
#endif
			}
			else
				writer.WriteStringKey(StringKeyZero);
			BinaryReader reader = writer.Flush();
			directoryTableSize = FromU64(reader.ReadU64());
			pcs.Leave();
			DEBUG_LOG_PIPE(L"CloseFile", L"(%ls%ls%ls)", handleName, (newName && *newName) ? L" -> " : L"", (newName && *newName) ? newName : L"");
		}
		g_directoryTable->ParseDirectoryTable(directoryTableSize);
	}

	// Returns true if done (no overflow)
	bool UpdateWrittenFiles(BinaryReader& reader)
	{
		u32 count = reader.ReadU32();
		u8 overflow = reader.ReadByte();
		while (count--)
		{
			StringKey key = reader.ReadStringKey();
			auto insres = g_mappedFileTable.m_lookup.try_emplace(key);
			FileInfo& info = insres.first->second;

			bool isInTemp = reader.ReadBool();
			StringBuffer<> originalName;
			if (isInTemp)
				originalName.Append(g_systemTemp).EnsureEndsWithSlash();
			reader.ReadString(originalName);
			if (!info.originalName || !originalName.Equals(info.originalName))
				info.originalName = g_mappedFileTable.m_memoryBlock.Strdup(originalName).data;

			StringBufferBase& backedName = originalName.Clear();
			reader.ReadString(backedName);
			UBA_ASSERTF(backedName.count == 0 || IsAbsolutePath(backedName.data) || backedName.data[0] == '#' || backedName.data[0] == '$' || backedName.data[0] == MemoryHandleChar || backedName.data[0] == FreeableMemoryHandleChar || backedName.data[0] == WrittenMemoryHandleChar, TC("UpdateWrittenFiles: backedName must be absolute or marker, got %s"), backedName.data);

			SharedMemoryHandle memoryHandle = SharedMemoryHandle::FromU64(reader.Read7BitEncoded());
			u64 fileSize = reader.Read7BitEncoded();
			info.fileNameKey = key;
			info.size = fileSize;
			info.created = true;

			if (memoryHandle.IsValid())
				backedName.Clear().Append(WrittenMemoryHandleChar).AppendBase62(memoryHandle.ToU64());

			if (!info.name || !backedName.Equals(info.name))
				info.name = g_mappedFileTable.m_memoryBlock.Strdup(backedName).data;

			DEBUG_LOG(TC("GOT WRITTEN FILE: %s (BackedFile: %s Size: %llu)"), info.originalName, info.name, info.size);

#if PLATFORM_WINDOWS
			if (auto mf = info.memoryFile) // Memory file is now wrong.. we need to delete it (or should we create a new one with the new mapping?)
			{
				if (info.refCount)
					DEBUG_LOG(TC("File has memory file and refcount %u. Will drop memory file and use received file. %s (BackedFile: %s Size: %llu)"), info.refCount, info.originalName, info.name, info.size);
				if (!mf->isLocalOnly)
				{
					DEBUG_LOG(TC("Unmapping old memoryfile for: %s"), info.originalName);

					#if UBA_DEBUG
					// Happens when linking server exe in merged modules on win64
					//UBA_ASSERTF(false, TC("REPORT THIS TO UBA DEVS"));
					//UnmapViewOfFile(mf->baseAddress);
					//CloseHandle(mf->memoryHandle.mh);
					//CloseHandle(mf->memoryHandle.fh);
					#endif
				}
				// delete mf; // Let them leak
				info.memoryFile = nullptr;
			}
#endif
		}
		return overflow == 0;
	}

	void Rpc_GetWrittenFilesNoLock(bool isInit)
	{
		while (true)
		{
			RPC_MESSAGE_NO_LOCK(GetWrittenFiles, updateTables)
			writer.WriteBool(isInit);
			BinaryReader reader = writer.Flush();
			if (UpdateWrittenFiles(reader))
				break;
		}
	}

	void Rpc_UpdateTables()
	{
		SCOPED_FUTEX(g_communicationLock, pcs);
		DirTableSize directoryTableSize;
		u32 fileMappingTableSize;
		bool done = true;
		{
			RPC_MESSAGE_NO_LOCK(UpdateTables, updateTables)
			writer.WriteBool(false);
			BinaryReader reader = writer.Flush();
			directoryTableSize = FromU64(reader.ReadU64());
			fileMappingTableSize = reader.ReadU32();

#if !PLATFORM_LINUX
			done = UpdateWrittenFiles(reader);
#endif

			DEBUG_LOG_PIPE(L"UpdateTables", L"");
		}
		if (!done)
			Rpc_GetWrittenFilesNoLock(false);

		pcs.Leave();

		g_directoryTable->ParseDirectoryTable(directoryTableSize);
		g_mappedFileTable.Parse(fileMappingTableSize);
	}

	void Rpc_GetWrittenFiles()
	{
		SCOPED_FUTEX(g_communicationLock, pcs);
		Rpc_GetWrittenFilesNoLock(true);
	}

#if !UBA_STUB_BUILD

	SharedMemoryHandle Rpc_CreateSharedMemory(SharedMemoryView& out, SharedMemoryAllocatorHandle allocatorHandle, u64 capacity, u64 initialSize, const tchar* hint)
	{
		RPC_MESSAGE(CreateSharedMemory, createSharedMemory)
		writer.WriteU64(initialSize);
		writer.WriteString(hint);
		BinaryReader reader = writer.Flush();
		SharedMemoryHandle handle{ reader.Read7BitEncoded() };

		out.Init(allocatorHandle, capacity);
		if (!initialSize)
			return handle;
		out.AddRequestedMemory(allocatorHandle, reader);
		memset(out.GetMemory(), 0, out.GetMappedSize());
		return handle;
	}

	void Rpc_CommitSharedMemory(SharedMemoryView& view, SharedMemoryAllocatorHandle allocatorHandle, SharedMemoryHandle handle, u64 size)
	{
		if (!handle.IsValid())
			return;
		RPC_MESSAGE(CommitSharedMemory, commitSharedMemory)
		writer.WriteU64(handle.ToU64());
		writer.WriteU64(size);
		BinaryReader reader = writer.Flush();
		u64 start = view.GetMappedSize();
		view.AddRequestedMemory(allocatorHandle, reader);
		memset(view.GetMemory() + start, 0, view.GetMappedSize() - start);
	}

	void Rpc_GetSharedMemory(SharedMemoryView& view, SharedMemoryAllocatorHandle allocatorHandle, u8* allocatorMem, SharedMemoryHandle handle, u64 offset, u64 size, const tchar* hint, SharedMemoryMapType type)
	{
		UBA_ASSERTF(AlignUp(offset, PageSize) == offset, TC("Offset %llu unaligned. Can't get unaligned shared memory (%s)"), offset, hint);
		//UBA_ASSERTF(AlignUp(size, PageSize) == size, TC("Size %llu unaligned. Can't get unaligned shared memory (%s)"), size, hint);

		RPC_MESSAGE(GetSharedMemory, getSharedMemory)
		writer.WriteU64(handle.ToU64());
		writer.WriteU64(offset);
		writer.WriteU64(size);
		BinaryReader reader = writer.Flush();
		u64 committed = size;
		if (!committed)
			committed = reader.Read7BitEncoded();
		bool res = view.Init(allocatorHandle, allocatorMem, reader, committed, hint, type);
		UBA_ASSERT(res);(void)res;
		//UBA_ASSERTF(u64(view.GetMemory()) != 0xFFFFFFFFFFFFFFFFllu, TC("%llu %llu %llu"), offset, size, committed);
		//view.AddRequestedMemory(allocatorHandle, reader, SharedMemoryMapType_ReadOnly);
	}

	void Rpc_UpdateSharedMemory(SharedMemoryView& view, SharedMemoryAllocatorHandle allocatorHandle, SharedMemoryHandle handle, u64 capacity)
	{
		RPC_MESSAGE(GetSharedMemory, getSharedMemory)
		writer.WriteU64(handle.ToU64());
		writer.WriteU64(0);
		writer.WriteU64(0);
		BinaryReader reader = writer.Flush();
		u64 committed = reader.Read7BitEncoded(); (void)committed;
		UBA_ASSERT(committed <= capacity);
		view.Init(allocatorHandle, capacity);
		view.AddRequestedMemory(allocatorHandle, reader);
	}

	void Rpc_GetFullFileName(const tchar*& path, u64& pathLen, StringBufferBase& tempBuf, bool useVirtualName, const tchar* const* loaderPaths)
	{
		StringKey fileNameKey;
		StringBuffer<> temp2;
		if (IsAbsolutePath(path))
		{
			FixPath(tempBuf, path);
			temp2.Append(tempBuf);
			path = temp2.data;

			if (CaseInsensitiveFs)
				tempBuf.MakeLower();
			fileNameKey = ToStringKey(tempBuf);
			tempBuf.Clear();
		}

		u32 mappedFileTableSize;

		#if UBA_DEBUG
		StringBuffer<> virtualName;
		#endif

		{
			RPC_MESSAGE(GetFullFileName, getFullFileName)
			writer.WriteString(path);
			writer.WriteStringKey(fileNameKey);
			u16& bytes = *(u16*)writer.AllocWrite(2);
			auto pos = writer.GetPosition();
			if (loaderPaths)
				for (auto i=loaderPaths; *i; ++i)
					writer.WriteString(*i);
			bytes = u16(writer.GetPosition() - pos);
			BinaryReader reader = writer.Flush();
			reader.ReadString(tempBuf);
			if (useVirtualName)
			{
				reader.ReadString(tempBuf.Clear());
			}
			else
			{
				#if UBA_DEBUG
				reader.ReadString(virtualName);
				#else
				reader.SkipString();
				#endif
			}
			mappedFileTableSize = reader.ReadU32();
			DEBUG_LOG_PIPE(TC("GetFileName"), TC("(%ls)"), tempBuf);
		}

		#if UBA_DEBUG
		if (useVirtualName)
		{ DEBUG_LOG_DETOURED(TC("Rpc_GetFullFileName"), TC("%s -> %s"), path, tempBuf.data); }
		else
		{ DEBUG_LOG_DETOURED(TC("Rpc_GetFullFileName"), TC("%s -> %s (%s)"), path, tempBuf.data, virtualName.data); }
		#endif

		g_mappedFileTable.Parse(mappedFileTableSize);
		path = tempBuf.data;
		pathLen = tempBuf.count;
	}

	void Rpc_GetFullFileName2(const tchar* path, StringBufferBase& outReal, StringBufferBase& outVirtual, const tchar* const* loaderPaths)
	{
		StringKey fileNameKey;
		StringBuffer<> temp2;
		if (IsAbsolutePath(path))
		{
			FixPath(temp2, path);
			path = temp2.data;
			fileNameKey = CaseInsensitiveFs ? ToStringKeyLower(temp2) : ToStringKey(temp2);
		}

		u32 mappedFileTableSize;

		{
			RPC_MESSAGE(GetFullFileName, getFullFileName)
			writer.WriteString(path);
			writer.WriteStringKey(fileNameKey);
			u16& bytes = *(u16*)writer.AllocWrite(2);
			auto pos = writer.GetPosition();
			if (loaderPaths)
				for (auto i=loaderPaths; *i; ++i)
					writer.WriteString(*i);
			bytes = u16(writer.GetPosition() - pos);
			BinaryReader reader = writer.Flush();
			reader.ReadString(outReal);
			reader.ReadString(outVirtual);
			mappedFileTableSize = reader.ReadU32();
			DEBUG_LOG_PIPE(TC("GetFileName"), TC("(%ls)"), tempBuf);
		}

		#if UBA_DEBUG
		DEBUG_LOG_DETOURED(TC("Rpc_GetFullFileName"), TC("%s -> %s (%s)"), path, outReal.data, outVirtual.data);
		#endif
		g_mappedFileTable.Parse(mappedFileTableSize);
	}

#endif // !UBA_STUB_BUILD
}
