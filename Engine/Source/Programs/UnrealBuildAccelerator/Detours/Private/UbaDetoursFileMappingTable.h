// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaBinaryReaderWriter.h"
#include "UbaDefinitions.h"
#include "UbaDetoursShared.h"
#include "UbaDirectoryTable.h"
#include "UbaSharedMemoryView.h"


namespace uba
{
	struct DetouredHandle;
	struct MemoryFile;

	struct FileInfo
	{
		// Size of file. Can be different than what Directory table says (For example for decompressed obj files)
		u64 size = InvalidValue;

		// Key of fixed up file name
		StringKey fileNameKey;

		// "Real" name of file. Can also be ids/handles to memory buffers etc
		const tchar* name = nullptr;

		// Original name of file as the detoured process sees it
		const tchar* originalName = nullptr;

		// Set if file is a memory file (writable, sharable across process hierarchies). isFileMap is false when set.
		MemoryFile* memoryFile = nullptr;

		// Holds previous file open access flags. Is used to figure out if file open is the first write
		u32 lastDesiredAccess = 0;

		// Last-seen offset in dirtable; if overlay has a newer one, Rpc_CheckRemapping is needed.
		DirTableOffset lastSeenTableOffset;

		// This is the Filemap handle and mapped view that represents the File HANDLE.
		// It can be used for other mappings too but doesnt' have to.
		bool isFileMap = false;

		// If this is true, fileMapMem will be deleted when
		bool freeFileMapOnClose = false;

		// Is set to true once file has been tracked to prevent tracking same file multiple times
		bool tracked = false;

		// If set to true, this means file was created during process group lifetime (which means it has its own file id etc)
		bool created = false;

		bool recursiveCall = false;

		// File mapping handle coming from session process (Set in NtCreateFile)
		SharedMemoryHandle trueFileMapHandle;

		// Offset into trueFileMapHandle
		u64 trueFileMapOffset = 0;

		// If file mapping is resolved, then this points to the memory. Is also used for compressed obj files
		u8* fileMapMem = nullptr;

		// Size of fileMapMem
		u64 fileMapMemSize = 0;

		#if PLATFORM_WINDOWS
		u32 refCount = 0;
		u32 fileMapDesiredAccess = PAGE_READONLY;
		u32 fileMapViewDesiredAccess = FILE_MAP_READ; // This is the parameter to MapViewOfFile
		#endif
	};

	class MappedFileTable
	{
	public:
		MappedFileTable(MemoryBlock& memoryBlock);

		void Init(const u8* mem, u32 tableCount, u32 tableSize);

		void ParseNoLock(u32 tableSize);
		void Parse(u32 tableSize);
		void SetDeleted(const StringKey& key, const tchar* name, bool deleted);

		MemoryBlock& m_memoryBlock;
		const u8* m_mem = 0;
		u32 m_memPosition = 0;
		ReaderWriterLock m_lookupLock;
		GrowingUnorderedMap<StringKey, FileInfo> m_lookup;
		Futex m_memLookupLock;
		struct Entry { int refCount = 0; DetouredHandle* handle = nullptr; };
		UnorderedMap<const void*, Entry> m_memLookup;
	};

	struct DirectoryTableEntryInformation : DirectoryTable::EntryInformation { DirTableOffset dirTableOffset; };

	enum : u8 { AccessFlag_Read = 1, AccessFlag_Write = 2 };

	void Rpc_CreateFileW(const StringView& fileName, const StringKey& fileNameKey, u8 accessFlags, tchar* outNewName, u64 newNameCapacity, u64& outSize, u32& outCloseId, bool lock);
	void Rpc_RegisterFileForWrite(StringView fileName, const StringKey& fileNameKey);
	void Rpc_CheckRemapping(const StringView& fileName, const StringKey& fileNameKey);
	void Rpc_CheckRemapping2(StringBufferBase& out, StringKey fileNameKey);
	bool Rpc_UpdateDirectory(const StringKey& dirKey, const tchar* dirName, u64 dirNameLen, bool lockDirTable = true);
	void Rpc_UpdateCloseHandle(const tchar* handleName, u32 closeId, bool deleteOnClose, const tchar* newName, SharedMemoryHandle memoryHandle, u64 fileSize, u64 lastWriteTime, u32 attributes, bool success);
	void Rpc_UpdateTables();
	bool Rpc_GetEntryOffset(DirTableOffset& out, const StringKey& entryNameKey, StringView entryName, bool checkIfDir = false);
	bool Rpc_GetEntryInformation(DirectoryTableEntryInformation& out, StringKey entryNameKey, StringView entryName, bool checkIfDir = false, bool followSymlink = true, StringBufferBase* outResolvedPath = nullptr, StringKey* outResolvedKey = nullptr);
	void Rpc_GetFullFileName(const tchar*& path, u64& pathLen, StringBufferBase& tempBuf, bool useVirtualName, const tchar* const* loaderPaths = nullptr);
	void Rpc_GetFullFileName2(const tchar* path, StringBufferBase& outReal, StringBufferBase& outVirtual, const tchar* const* loaderPaths = nullptr);
	void Rpc_GetWrittenFiles();

	SharedMemoryHandle Rpc_CreateSharedMemory(SharedMemoryView& out, SharedMemoryAllocatorHandle allocatorHandle, u64 capacity, u64 initialSize, const tchar* hint);
	void Rpc_CommitSharedMemory(SharedMemoryView& view, SharedMemoryAllocatorHandle allocatorHandle, SharedMemoryHandle handle, u64 size);
	void Rpc_GetSharedMemory(SharedMemoryView& view, SharedMemoryAllocatorHandle allocatorHandle, u8* allocatorMem, SharedMemoryHandle handle, u64 offset, u64 size, const tchar* hint, SharedMemoryMapType type = SharedMemoryMapType_ReadOnly);
	void Rpc_UpdateSharedMemory(SharedMemoryView& view, SharedMemoryAllocatorHandle allocatorHandle, SharedMemoryHandle handle, u64 capacity);

	struct DirHash
	{
		DirHash(const StringView& str);
		StringKey key;
		StringKeyHasher open;
	};
}