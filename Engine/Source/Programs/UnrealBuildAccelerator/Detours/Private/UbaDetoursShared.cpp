// Copyright Epic Games, Inc. All Rights Reserved.

#define UBA_IS_DETOURED_INCLUDE 1

#include "UbaDetoursShared.h"
#include "UbaDetoursFileMappingTable.h"
#include "UbaDirectoryTable.h"
#include "UbaTimer.h"

#include <atomic>

#if PLATFORM_MAC
#include <locale.h>
#include <pthread.h>
#include <mach/mach_init.h>
#endif

namespace uba
{
	constexpr u32 TrackInputsMemCapacity = 512 * 1024;

	VARIABLE_MEM(StringBuffer<512>, g_virtualApplication);
	VARIABLE_MEM(StringBuffer<512>, g_virtualApplicationDir);
	VARIABLE_MEM(ProcessStats, g_stats);
	VARIABLE_MEM(KernelStats, g_kernelStats);
	VARIABLE_MEM(Futex, g_communicationLock);
	VARIABLE_MEM(StringBuffer<512>, g_logName);
	VARIABLE_MEM(StringBuffer<512>, g_virtualWorkingDir);
	VARIABLE_MEM(StringBuffer<256>, g_exeDir);
	VARIABLE_MEM(StringBuffer<128>, g_systemRoot);
	VARIABLE_MEM(StringBuffer<256>, g_systemTemp);
	VARIABLE_MEM(MemoryBlock, g_memoryBlock);
	VARIABLE_MEM(MappedFileTable, g_mappedFileTable);
	VARIABLE_MEM(ReaderWriterLock, g_consoleStringCs);
	VARIABLE_MEM(ReaderWriterLock, g_vfsLock);
	VARIABLE_MEM(Futex, g_trackInputsLock);

	#if UBA_DEBUG_LOG_ENABLED
	VARIABLE_MEM(StringBuffer<MaxPath>, g_debugLogPath);
	VARIABLE_MEM(Futex, g_debugLogMutex);
	#endif

	DirectoryTable* g_directoryTable;
	bool g_conEnabled[2];
	u32 g_rulesIndex;
	ApplicationRules* g_rules;
	bool g_runningRemote;
	bool g_isChild;
	bool g_allowKeepFilesInMemory = IsWindows;
	bool g_allowOutputFiles = IsWindows;
	bool g_allowDiscardVirtualMemory;
	bool g_suppressLogging = false;
	u32 g_vfsEntryCount;
	u32 g_vfsMatchingLength;
	u8* g_trackInputsMem;
	u32 g_trackInputsBufPos;

#if UBA_STUB_BUILD
	// Storage accessors for the stub g_xxx macros — see UbaDetoursShared.h.
	void* g_virtualApplicationStorage()     { return g_virtualApplicationMem.data; }
	void* g_virtualApplicationDirStorage()  { return g_virtualApplicationDirMem.data; }
	void* g_exeDirStorage()                 { return g_exeDirMem.data; }
	void* g_logNameStorage()                { return g_logNameMem.data; }
	void* g_virtualWorkingDirStorage()      { return g_virtualWorkingDirMem.data; }
	void* g_systemTempStorage()             { return g_systemTempMem.data; }
	void* g_systemRootStorage()             { return g_systemRootMem.data; }
	void* g_statsStorage()                  { return g_statsMem.data; }
	void* g_kernelStatsStorage()            { return g_kernelStatsMem.data; }
	void* g_communicationLockStorage()      { return g_communicationLockMem.data; }
	void* g_memoryBlockStorage()            { return g_memoryBlockMem.data; }
	void* g_mappedFileTableStorage()        { return g_mappedFileTableMem.data; }
	void* g_trackInputsLockStorage()        { return g_trackInputsLockMem.data; }
	void* g_vfsLockStorage()                { return g_vfsLockMem.data; }
	#if UBA_DEBUG_LOG_ENABLED
	void* g_debugLogPathStorage()           { return g_debugLogPathMem.data; }
	void* g_debugLogMutexStorage()          { return g_debugLogMutexMem.data; }
	#endif
#endif

	void InitSharedVariables()
	{
		g_conEnabled[0] = true;
		g_conEnabled[1] = true;
		g_virtualApplicationMem.Create();
		g_virtualApplicationDirMem.Create();
		g_statsMem.Create();
		g_kernelStatsMem.Create();
		g_communicationLockMem.Create();
		g_logNameMem.Create();
		g_virtualWorkingDirMem.Create();
		g_exeDirMem.Create();
		g_systemRootMem.Create();
		g_systemTempMem.Create();

		#if UBA_DEBUG_LOG_ENABLED
		g_debugLogPathMem.Create();
		g_debugLogMutexMem.Create();
		#endif

		g_vfsEntryCount = 0;
		g_vfsMatchingLength = 0;

		u64 reserveSizeMb = IsWindows ? 256 : 1024; // The sync primitives on linux/macos is much bigger
		g_memoryBlockMem.Create(TC("GlobalDetour"), reserveSizeMb * 1024 * 1024);
		g_mappedFileTableMem.Create(g_memoryBlock);
		g_consoleStringCsMem.Create();
		g_vfsLockMem.Create();

		g_trackInputsLockMem.Create();
	}

#if UBA_DEBUG_LOG_ENABLED
	FileHandle g_debugFile = InvalidFileHandle;

	#if !UBA_STUB_BUILD
	thread_local u32 t_logScopeCount;
	constexpr const char g_emptyString[] = "                                                     ";
	constexpr const char* g_emptyStringEnd = ((const char*)g_emptyString) + sizeof_array(g_emptyString) - 1;
	#endif

	void WriteDebug(const void* data, u32 dataLen)
	{
		SuppressDetourScope sds;
		SCOPED_FUTEX(g_debugLogMutex, l);
		WriteDebugInner(data, dataLen);
	}

	void GetPrefixExtra(StringBufferBase& out)
	{
		(void)out;

		#if 0
		static u64 startTime = GetTime();
		u64 timeMs = TimeToMs(GetTime() - startTime);
		u64 ms = timeMs % 1000;
		u64 s = timeMs / 1000;
		out.Appendf(TC("[%5llu.%03llu]"), s, ms);
		#endif

		#if 0
		u64 timeMs = TimeToMs(GetTime()); // Absolute monotonic timestamp (no per-process baseline). All processes on the same machine see the same GetTime() values, so this aligns events across detour logs
		u64 ms = timeMs % 1000;
		u64 s = timeMs / 1000;
		out.Appendf(TC("[%6llu.%03llu]"), s, ms);
		#endif

		#if 0
		out.Appendf(TC("[%i]"), getpid());
		out.Appendf(TC("[%7u]"), GetCurrentThreadId());
		#endif
	}

	// Two-stage format (tchar→char with %S on Windows). Returns -1 on overflow rather than
	// triggering Windows' invalid-parameter abort, so caller can fall back gracefully.
	static int SafeTvsprintf(tchar* buf, u32 size, const tchar* format, va_list args)
	{
		#if PLATFORM_WINDOWS
		return _vsnwprintf_s(buf, size, _TRUNCATE, format, args);
		#else
		return vsnprintf(buf, size, format, args);
		#endif
	}

	// Lock-protected static large buffers for log lines. Static = signal-safe + stub-safe + no TLS;
	// stack allocation overflows Go's 8 KB goroutine stack → AL2023 sysmon stack-split panics.
	inline constexpr u32 LogBufSizeLarge = 64 * 1024;
	static tchar g_largeLogBufT[LogBufSizeLarge];
	static char  g_largeLogBufC[LogBufSizeLarge];

	void WriteDebugLogWithPrefix(const char* prefix, LogScope& scope, const tchar* command, const tchar* format, ...)
	{
		int errnoSaved = errno;

		SuppressDetourScope sds;

		#if PLATFORM_MAC
		static locale_t safeLocale = newlocale(LC_NUMERIC_MASK, "C", duplocale(LC_GLOBAL_LOCALE));
		locale_t oldLocale = uselocale(safeLocale);
		#endif

		StringBuffer<128> extra;
		GetPrefixExtra(extra);

		SCOPED_FUTEX(g_debugLogMutex, l);

		g_largeLogBufT[0] = 0;
		if (*format)
		{
			va_list arg;
			va_start(arg, format);
			SafeTvsprintf(g_largeLogBufT, LogBufSizeLarge, format, arg);
			va_end(arg);
		}

		#if !UBA_STUB_BUILD
		const char* emptyString = g_emptyStringEnd - Min(t_logScopeCount * 2, u32(sizeof_array(g_emptyString) - 1));
		#else
		const char* emptyString = TC("");
		#endif

		int large;
		#if PLATFORM_WINDOWS
		large = sprintf_s(g_largeLogBufC, LogBufSizeLarge, "%s %S   %s%S %S\n", prefix, extra.data, emptyString, command, g_largeLogBufT);
		#else
		large = snprintf(g_largeLogBufC, LogBufSizeLarge, "%s %s   %s%s %s\n", prefix, extra.data, emptyString, command, g_largeLogBufT);
		#endif
		if (large < 0)
			large = 0;
		else if (large >= int(LogBufSizeLarge))
			large = int(LogBufSizeLarge) - 1;

		WriteDebugInner(g_largeLogBufC, u32(large));

		#if PLATFORM_MAC
		uselocale(oldLocale);
		#endif

		errno = errnoSaved;
	}

	void WriteDebugLog(const tchar* format, ...)
	{
		int errnoSaved = errno;

		SuppressDetourScope sds;

		StringBuffer<128> extra;
		GetPrefixExtra(extra);

		SCOPED_FUTEX(g_debugLogMutex, l);
		g_largeLogBufT[0] = 0;
		if (*format)
		{
			va_list arg;
			va_start(arg, format);
			SafeTvsprintf(g_largeLogBufT, LogBufSizeLarge, format, arg);
			va_end(arg);
		}

		int large;
		#if PLATFORM_WINDOWS
		large = sprintf_s(g_largeLogBufC, LogBufSizeLarge, "%S %S\n", extra.data, g_largeLogBufT);
		#else
		large = snprintf(g_largeLogBufC, LogBufSizeLarge, "%s %s\n", extra.data, g_largeLogBufT);
		#endif
		if (large < 0)
			large = 0;
		else if (large >= int(LogBufSizeLarge))
			large = int(LogBufSizeLarge) - 1;

		WriteDebugInner(g_largeLogBufC, u32(large));

		errno = errnoSaved;
	}

	// LogScope is now empty stubs — each WriteDebugLog* call writes its line immediately.
	LogScope::LogScope()
	{
		#if !UBA_STUB_BUILD
		++t_logScopeCount;
		#endif
	}
	
	LogScope::~LogScope()
	{
		#if !UBA_STUB_BUILD
		--t_logScopeCount;
		#endif
	}
	void LogScope::Flush() {}
#endif

#if UBA_DEBUG_VALIDATE
	bool g_validateFileAccess = false;
#endif

	// TLS-free per-thread disallow-detour depth. Goroutine OS threads from raw clone() lack a glibc
	// TCB (FS base points at Go's M struct), so __tls_get_addr crashes on them. Use an open-addressed
	// array keyed by OS tid via raw SYS_gettid / GetCurrentThreadId / mach (no TLS access). Overflow
	// silently drops; sized well above any plausible concurrent-thread count.

	static constexpr u32 kDisallowSlots = 256;

	struct DisallowSlot
	{
		std::atomic<u64> tid;   // 0 = empty
		std::atomic<u32> depth;
	};
	static DisallowSlot g_disallowSlots[kDisallowSlots];

	static inline u64 GetOsThreadIdNoTls()
	{
		#if PLATFORM_WINDOWS
			return (u64)::GetCurrentThreadId();
		#elif PLATFORM_LINUX
			long tid;
			#if PLATFORM_CPU_ARM_FAMILY // SYS_gettid=178 on AArch64
				register long x8 asm("x8") = 178;
				register long x0 asm("x0");
				asm volatile("svc #0" : "=r"(x0) : "r"(x8) : "memory");
				tid = x0;
			#else                       // SYS_gettid=186 on x86_64
				register long rax asm("rax") = 186;
				asm volatile("syscall" : "+a"(rax) : : "rcx", "r11", "memory");
				tid = rax;
			#endif
			return (u64)tid;
		#elif PLATFORM_MAC
			// pthread_self reads a register; pthread_mach_thread_np does not touch TLS.
			return (u64)pthread_mach_thread_np(pthread_self());
		#else
			return 0;
		#endif
	}

	// Locate the slot for `tid`. allowAcquire=true grabs an empty slot via CAS if none matches.
	static DisallowSlot* FindOrAcquireSlot(u64 tid, bool allowAcquire)
	{
		// Splitmix-style mix to spread poorly-distributed tid values.
		u64 h = tid * 0x9E3779B97F4A7C15ULL;
		h ^= h >> 33;
		u32 start = (u32)(h & (kDisallowSlots - 1));
		for (u32 i = 0; i < kDisallowSlots; ++i)
		{
			u32 idx = (start + i) & (kDisallowSlots - 1);
			DisallowSlot& s = g_disallowSlots[idx];
			u64 cur = s.tid.load(std::memory_order_acquire);
			if (cur == tid)
				return &s;
			if (cur == 0 && allowAcquire)
			{
				u64 expected = 0;
				if (s.tid.compare_exchange_strong(expected, tid, std::memory_order_acq_rel, std::memory_order_acquire))
					return &s;
				if (expected == tid)
					return &s;
			}
		}
		return nullptr; // overflow — silently drop
	}

	u32 DisallowDetourDepth()
	{
		u64 tid = GetOsThreadIdNoTls();
		if (DisallowSlot* s = FindOrAcquireSlot(tid, /*allowAcquire*/ false))
			return s->depth.load(std::memory_order_acquire);
		return 0;
	}

	void DisallowDetourIncrement()
	{
		u64 tid = GetOsThreadIdNoTls();
		if (DisallowSlot* s = FindOrAcquireSlot(tid, /*allowAcquire*/ true))
			s->depth.fetch_add(1, std::memory_order_acq_rel);
	}

	void DisallowDetourDecrement()
	{
		u64 tid = GetOsThreadIdNoTls();
		if (DisallowSlot* s = FindOrAcquireSlot(tid, /*allowAcquire*/ false))
		{
			// Release slot at depth 0 so the array doesn't accumulate dead threads.
			u32 prev = s->depth.fetch_sub(1, std::memory_order_acq_rel);
			if (prev == 1)
				s->tid.store(0, std::memory_order_release);
		}
	}

	SuppressDetourScope::SuppressDetourScope()  { DisallowDetourIncrement(); }
	SuppressDetourScope::~SuppressDetourScope() { DisallowDetourDecrement(); }

	void InitTrackInput()
	{
		g_trackInputsMem = (u8*)g_memoryBlock.Allocate(TrackInputsMemCapacity, 1, TC("TrackInputs"));
	}

	void SendInput()
	{
		if (!g_trackInputsMem)
			return;

		u32 left = g_trackInputsBufPos;
		u32 reserveSize = left;
		u32 pos = 0;
		while (left)
		{
			RPC_MESSAGE(InputDependencies, sendInput)
			writer.Write7BitEncoded(reserveSize);
			reserveSize = 0;
			u32 toWrite = Min(left, u32(writer.GetCapacityLeft() - sizeof(u32)));
			writer.WriteU32(toWrite);
			writer.WriteBytes(g_trackInputsMem + pos, toWrite);
			writer.Flush();
			left -= toWrite;
			pos += toWrite;
		}
		g_trackInputsBufPos = 0;
	}


	void TrackInput(StringView file)
	{
		if (!g_trackInputsMem)
			return;

		// TODO: This should be caught earlier, temporary solution
		// Should add assert and investigate instead.
		#if PLATFORM_LINUX
		if (file.StartsWith(TCV("/dev/")) || file.StartsWith(TCV("/sys/")) || file.StartsWith(TCV("/proc/")) || file.StartsWith(TCV("/tmp/"))
			|| file.StartsWith(g_systemTemp))
			return;
		#endif


		SCOPED_FUTEX(g_trackInputsLock, l);

		if (g_trackInputsBufPos > TrackInputsMemCapacity - 2048)
			SendInput();
		UBA_ASSERTF(!file.StartsWith(TCV("\\")), TC("Error %s is not an absolute path. All tracked files must be absolute"), file.data);

		BinaryWriter w(g_trackInputsMem, g_trackInputsBufPos, TrackInputsMemCapacity);
		w.WriteString(file);
		g_trackInputsBufPos = u32(w.GetPosition());
	}

	void SkipTrackInput(StringView file)
	{
		// Just here to easily log out what we are ignoring in terms of input
	}

	bool IsTrackingInput()
	{
		return g_trackInputsMem != 0;
	}

	bool FixPath(StringBufferBase& out, const tchar* path, u32 pathLen)
	{
		return FixPath2(path, pathLen, g_virtualWorkingDir.data, g_virtualWorkingDir.count, out.data, out.capacity, &out.count);
	}

	bool FixPath(StringBufferBase& out, const StringView path)
	{
		return FixPath2(path.data, path.count, g_virtualWorkingDir.data, g_virtualWorkingDir.count, out.data, out.capacity, &out.count);
	}

	struct VfsEntry { StringView vfs; StringView local; bool fromHostInit; VfsEntry() : vfs(NoInit), local(NoInit), fromHostInit(false) {}; };
	static VfsEntry g_vfsEntries[256];
	static constexpr u32 kVfsBucketCount = 64; // power of 2; 6-bit hash
	static constexpr u32 kVfsBucketMask = kVfsBucketCount - 1;
	static u8  g_vfsBucketChain[256];               // entry indices, packed by bucket
	static u16 g_vfsBucketStart[kVfsBucketCount + 1]; // [b]..[b+1] = bucket b's range
	static u8  g_vfsRootEntries[8];                 // entries with vfs="/"
	static u8  g_vfsRootEntryCount;
	static u8  g_vfsBucketEntryCount;               // sum of all buckets
	static u32 g_vfsBucketDiscOffset;               // LCP of bucketed entries
	static const tchar* g_vfsBucketPrefix;          // first bucketed entry's vfs.data
	static bool g_vfsIndexValid;                    // false → fall back to linear scan

	static inline u32 VfsBucketHash(tchar c)
	{
		// Lowercase (StartsWith uses ignoreCase=true) → Knuth multiplicative → 6-bit bucket.
		u8 lc = (u8)((c >= 'A' && c <= 'Z') ? c - 'A' + 'a' : c);
		return ((u32)lc * 2654435769u) >> (32 - 6);
	}

	// Caller holds g_vfsLock for write.
	static void RecomputeVfsMatchingLength()
	{
		if (g_vfsEntryCount == 0) { g_vfsMatchingLength = 0; return; }
		u32 shortest = g_vfsEntries[0].vfs.count;
		for (u32 i = 1, e = g_vfsEntryCount; i != e; ++i)
		{
			VfsEntry& entry = g_vfsEntries[i];
			u32 sh = Min(shortest, entry.vfs.count);
			for (u32 k = 0; k != sh; ++k)
				if (g_vfsEntries[0].vfs.data[k] != entry.vfs.data[k])
					{ sh = k; break; }
			shortest = sh;
		}
		g_vfsMatchingLength = shortest;
	}

	// Caller holds g_vfsLock for write. Called only on mutation.
	static void RebuildVfsIndex()
	{
		g_vfsIndexValid = false;
		g_vfsRootEntryCount = 0;
		g_vfsBucketEntryCount = 0;
		g_vfsBucketDiscOffset = 0;
		g_vfsBucketPrefix = nullptr;
		for (u32 b = 0; b <= kVfsBucketCount; ++b)
			g_vfsBucketStart[b] = 0;
		if (g_vfsEntryCount == 0)
		{
			g_vfsIndexValid = true;
			return;
		}

		// Partition: root (vfs="/") vs bucketed.
		u8 bucketed[256];
		u8 bucketedCount = 0;
		for (u32 i = 0, e = g_vfsEntryCount; i != e; ++i)
		{
			const VfsEntry& en = g_vfsEntries[i];
			bool isRoot = (en.vfs.count == 1 && en.vfs.data[0] == '/');
			if (isRoot)
			{
				if (g_vfsRootEntryCount < sizeof_array(g_vfsRootEntries))
					g_vfsRootEntries[g_vfsRootEntryCount++] = (u8)i;
				else
					return; // root-list overflow: fall back to linear scan
			}
			else
			{
				bucketed[bucketedCount++] = (u8)i;
			}
		}
		g_vfsBucketEntryCount = bucketedCount;

		if (bucketedCount == 0)
		{
			g_vfsIndexValid = true;
			return;
		}

		// Compute LCP of bucketed vfs paths.
		const VfsEntry& first = g_vfsEntries[bucketed[0]];
		g_vfsBucketPrefix = first.vfs.data;
		u32 disc = first.vfs.count;
		for (u32 j = 1; j != bucketedCount; ++j)
		{
			const VfsEntry& en = g_vfsEntries[bucketed[j]];
			u32 sh = Min(disc, en.vfs.count);
			u32 k = 0;
			while (k != sh && first.vfs.data[k] == en.vfs.data[k])
				++k;
			disc = k;
		}
		// Need at least one byte past disc per entry; degenerate "two equal-prefix" entries → fallback.
		u32 minLen = first.vfs.count;
		for (u32 j = 1; j != bucketedCount; ++j)
			minLen = Min(minLen, g_vfsEntries[bucketed[j]].vfs.count);
		if (disc >= minLen)
			return; // linear-scan fallback
		g_vfsBucketDiscOffset = disc;

		// First pass: count entries per bucket.
		u16 counts[kVfsBucketCount] = {};
		for (u32 j = 0; j != bucketedCount; ++j)
		{
			const VfsEntry& en = g_vfsEntries[bucketed[j]];
			u32 b = VfsBucketHash(en.vfs.data[disc]) & kVfsBucketMask;
			++counts[b];
		}
		// Compute prefix sums into g_vfsBucketStart.
		u16 acc = 0;
		for (u32 b = 0; b != kVfsBucketCount; ++b)
		{
			g_vfsBucketStart[b] = acc;
			acc += counts[b];
		}
		g_vfsBucketStart[kVfsBucketCount] = acc;
		// Second pass: place entries; sort each bucket by vfs.count DESC.
		u16 cursors[kVfsBucketCount];
		for (u32 b = 0; b != kVfsBucketCount; ++b)
			cursors[b] = g_vfsBucketStart[b];
		for (u32 j = 0; j != bucketedCount; ++j)
		{
			u8 idx = bucketed[j];
			const VfsEntry& en = g_vfsEntries[idx];
			u32 b = VfsBucketHash(en.vfs.data[disc]) & kVfsBucketMask;
			g_vfsBucketChain[cursors[b]++] = idx;
		}
		// Insertion-sort each bucket by vfs.count DESC; bucket sizes are small (~8 in trusty).
		for (u32 b = 0; b != kVfsBucketCount; ++b)
		{
			u16 lo = g_vfsBucketStart[b];
			u16 hi = g_vfsBucketStart[b + 1];
			for (u16 i = lo + 1; i < hi; ++i)
			{
				u8 cur = g_vfsBucketChain[i];
				u32 curLen = g_vfsEntries[cur].vfs.count;
				u16 k = i;
				while (k > lo && g_vfsEntries[g_vfsBucketChain[k - 1]].vfs.count < curLen)
				{
					g_vfsBucketChain[k] = g_vfsBucketChain[k - 1];
					--k;
				}
				g_vfsBucketChain[k] = cur;
			}
		}

		g_vfsIndexValid = true;
	}

	void PopulateVfs(BinaryReader& vfsReader)
	{
		SCOPED_WRITE_LOCK(g_vfsLock, vfsLock);
		while (vfsReader.GetLeft())
		{
			vfsReader.ReadByte(); // Index, unused
			StringBuffer<> str;
			vfsReader.ReadString(str);
			if (!str.count)
			{
				vfsReader.SkipString();
				continue;
			}

			#if PLATFORM_WINDOWS
			str.Replace('/', '\\');
			#endif

			u32 index = g_vfsEntryCount++;
			UBA_ASSERT(index < sizeof_array(g_vfsEntries));
			VfsEntry& vfsEntry = g_vfsEntries[index];
			vfsEntry.vfs = g_memoryBlock.Strdup(str);
			vfsEntry.fromHostInit = true;

			if (index == 0)
				g_vfsMatchingLength = vfsEntry.vfs.count;
			else
			{
				u32 shortest = Min(g_vfsMatchingLength, vfsEntry.vfs.count);
				for (u32 i=0; i!=shortest; ++i)
				{
					if (g_vfsEntries[0].vfs.data[i] == vfsEntry.vfs.data[i])
						continue;
					shortest = i;
					break;
				}
				g_vfsMatchingLength = shortest;
			}
			vfsReader.ReadString(str.Clear());
			vfsEntry.local = g_memoryBlock.Strdup(str);
		}
		RebuildVfsIndex();
	}

	// Caller holds g_vfsLock(write) and is responsible for RebuildVfsIndex (deferred for bulk adopt).
	static bool AddVfsEntryLocked_NoIndex(StringView vfs, StringView local, bool dedup)
	{
		if (dedup)
		{
			for (u32 i = 0, e = g_vfsEntryCount; i != e; ++i)
			{
				if (g_vfsEntries[i].vfs.Equals(vfs))
				{
					g_vfsEntries[i].local = g_memoryBlock.Strdup(local);
					return true;
				}
			}
		}
		if (g_vfsEntryCount >= sizeof_array(g_vfsEntries))
			return false;
		u32 index = g_vfsEntryCount++;
		VfsEntry& entry = g_vfsEntries[index];
		entry.vfs = g_memoryBlock.Strdup(vfs);
		entry.local = g_memoryBlock.Strdup(local);
		if (index == 0)
			g_vfsMatchingLength = entry.vfs.count;
		else
		{
			u32 shortest = Min(g_vfsMatchingLength, entry.vfs.count);
			u32 m = 0;
			while (m != shortest && g_vfsEntries[0].vfs.data[m] == entry.vfs.data[m])
				++m;
			g_vfsMatchingLength = m;
		}
		return true;
	}

	bool AddVfsEntry(StringView vfs, StringView local)
	{
		SCOPED_WRITE_LOCK(g_vfsLock, vfsLock);
		bool ok = AddVfsEntryLocked_NoIndex(vfs, local, /*dedup=*/true);
		RebuildVfsIndex();
		return ok;
	}

	// Caller must invoke FinalizeVfsEntriesAfterBulk() once at end.
	bool AddVfsEntryBulkNoDedup(StringView vfs, StringView local)
	{
		SCOPED_WRITE_LOCK(g_vfsLock, vfsLock);
		return AddVfsEntryLocked_NoIndex(vfs, local, /*dedup=*/false);
	}

	void FinalizeVfsEntriesAfterBulk()
	{
		SCOPED_WRITE_LOCK(g_vfsLock, vfsLock);
		RebuildVfsIndex();
	}

	// Returns true if found. Swap-to-end (order-loose).
	bool RemoveVfsEntry(StringView vfs)
	{
		SCOPED_WRITE_LOCK(g_vfsLock, vfsLock);
		for (u32 i = 0, e = g_vfsEntryCount; i != e; ++i)
		{
			if (!g_vfsEntries[i].vfs.Equals(vfs))
				continue;
			if (i + 1 != e)
				g_vfsEntries[i] = g_vfsEntries[e - 1];
			g_vfsEntries[e - 1] = VfsEntry();
			--g_vfsEntryCount;
			RecomputeVfsMatchingLength();
			RebuildVfsIndex();
			return true;
		}
		return false;
	}

	bool IsVfsEnabled()
	{
		return g_vfsEntryCount > 0;
	}

	void Shared_BuildVfsEnv(StringBufferBase& out)
	{
		if (g_vfsEntryCount == 0)
			return;
		out.Append(TCV("UBA_VFS_OVERLAYS="));
		u32 prefixLen = out.count;
		SCOPED_READ_LOCK(g_vfsLock, lock);
		for (u32 i = 0, e = g_vfsEntryCount; i != e; ++i)
		{
			VfsEntry& entry = g_vfsEntries[i];
			if (entry.fromHostInit)
				continue;
			out.AppendValue((u64)entry.vfs.count).Append(':')
			   .Append(entry.vfs.data, entry.vfs.count).Append(':')
			   .AppendValue((u64)entry.local.count).Append(':')
			   .Append(entry.local.data, entry.local.count).Append(';');
		}
		if (out.count == prefixLen)
			out.Resize(0);
	}

	bool DevirtualizePath(StringBufferBase& path)
	{
		if (!g_vfsEntryCount)
			return false;

		SCOPED_READ_LOCK(g_vfsLock, vfsLock);
		if (!g_vfsEntryCount)
			return false;

		// Multi-pass for chained binds; per-entry applied[] caps each entry to one fire so the
		// vfs="/" pivot can't ping-pong with a bind that resolves under it.
		bool everSubstituted = false;
		bool applied[256] = {};
		for (u32 pass = 0, maxPasses = g_vfsEntryCount + 1; pass != maxPasses; ++pass)
		{
			if (!Equals(path.data, g_vfsEntries[0].vfs.data, Min(path.count, g_vfsMatchingLength), CaseInsensitiveFs))
				return everSubstituted;

			// TODO: This is not great, the dirs above the vfs root should be empty except the dir to the roots
			if (path.count < g_vfsMatchingLength)
			{
				path.Clear().Append(g_vfsEntries[0].local);
				return true;
			}

			// Longest-match: overlapping binds (root-of-sandbox + specific subpath) need specific
			// to win. vfs="/" only applies to NS-virtual paths — skipped when path starts with
			// any other entry's local (those locals ARE the real-host root prefixes).
			bool substituted = false;
			u32 bestIdx = ~u32(0);
			u32 bestLen = 0;

			if (g_vfsIndexValid)
			{
				if (g_vfsBucketEntryCount != 0
					&& path.count > g_vfsBucketDiscOffset
					&& Equals(path.data, g_vfsBucketPrefix, g_vfsBucketDiscOffset, CaseInsensitiveFs))
				{
					u32 b = VfsBucketHash(path.data[g_vfsBucketDiscOffset]) & kVfsBucketMask;
					u16 lo = g_vfsBucketStart[b];
					u16 hi = g_vfsBucketStart[b + 1];
					// Chain sorted by vfs.count DESC; first passing entry wins (longest match).
					for (u16 k = lo; k < hi; ++k)
					{
						u8 idx = g_vfsBucketChain[k];
						if (applied[idx])
							continue;
						VfsEntry& entry = g_vfsEntries[idx];
						if (!path.StartsWith(entry.vfs.data))
							continue;
						if (path.StartsWith(entry.local.data))
							continue;
						bestLen = entry.vfs.count;
						bestIdx = idx;
						break;
					}
				}
				// Root entries (vfs="/"): bucketed match always wins on length, so skip if found.
				if (bestIdx == ~u32(0))
				{
					for (u8 r = 0; r < g_vfsRootEntryCount; ++r)
					{
						u8 idx = g_vfsRootEntries[r];
						if (applied[idx])
							continue;
						VfsEntry& entry = g_vfsEntries[idx];
						if (!path.StartsWith(entry.vfs.data))
							continue;
						if (path.StartsWith(entry.local.data))
							continue;
						// Skip vfs="/" if path already starts with any other entry's local (host-real).
						bool isHostReal = false;
						for (u32 j = 0, e2 = g_vfsEntryCount; j != e2 && !isHostReal; ++j)
						{
							if (j == idx) continue;
							VfsEntry& other = g_vfsEntries[j];
							if (other.local.count > 1 && path.StartsWith(other.local.data))
								isHostReal = true;
						}
						if (isHostReal)
							continue;
						if (entry.vfs.count > bestLen)
						{
							bestLen = entry.vfs.count;
							bestIdx = idx;
						}
					}
				}
			}
			else
			{
				// Linear-scan fallback (index couldn't be built — degenerate duplicates / root overflow).
				for (u32 i=0, e=g_vfsEntryCount; i!=e; ++i)
				{
					if (applied[i])
						continue;
					VfsEntry& entry = g_vfsEntries[i];
					if (!path.StartsWith(entry.vfs.data))
						continue;
					if (path.StartsWith(entry.local.data))
						continue;
					if (entry.vfs.count == 1 && entry.vfs.data[0] == '/')
					{
						bool isHostReal = false;
						for (u32 j=0; j!=g_vfsEntryCount && !isHostReal; ++j)
						{
							if (j == i) continue;
							VfsEntry& other = g_vfsEntries[j];
							if (other.local.count > 1 && path.StartsWith(other.local.data))
								isHostReal = true;
						}
						if (isHostReal)
							continue;
					}
					if (entry.vfs.count > bestLen)
					{
						bestLen = entry.vfs.count;
						bestIdx = i;
					}
				}
			}
			if (bestIdx != ~u32(0))
			{
				VfsEntry& entry = g_vfsEntries[bestIdx];
				// vfs="/": strip 0 chars to keep the leading slash on the remainder.
				u32 stripCount = (entry.vfs.count == 1 && entry.vfs.data[0] == '/') ? 0 : entry.vfs.count;
				StringBuffer<MaxPath> temp2(path.data + stripCount);
				path.Clear().Append(entry.local).Append(temp2);
				applied[bestIdx] = true;
				substituted = true;
				everSubstituted = true;
			}
			if (!substituted)
				return everSubstituted;
		}
		return everSubstituted;
	}

	tchar DevirtualizeDrive(tchar drive)
	{
		if (!g_vfsEntryCount)
			return '\0';
		SCOPED_READ_LOCK(g_vfsLock, vfsLock);
		if (!g_vfsEntryCount)
			return '\0';
		return g_vfsEntries[0].local[0];
	}

	bool VirtualizePath(StringBufferBase& path)
	{
		if (!g_vfsEntryCount)
			return false;
		SCOPED_READ_LOCK(g_vfsLock, vfsLock);
		// Multi-pass: chained binds (specific bind + root pivot) need two layers; applied[] caps each entry to one fire.
		bool everSubstituted = false;
		bool applied[256] = {};
		for (u32 pass = 0, maxPasses = g_vfsEntryCount + 1; pass != maxPasses; ++pass)
		{
			bool substituted = false;
			for (u32 i = 0, e = g_vfsEntryCount; i != e; ++i)
			{
				if (applied[i]) continue;
				VfsEntry& entry = g_vfsEntries[i];
				if (path.count < entry.local.count || !path.StartsWith(entry.local.data))
					continue;
				StringBuffer<MaxPath> temp2(path.data + entry.local.count);
				// vfs="/": skip leading slash so remainder doesn't produce "//..." (symmetric with DevirtualizePath).
				bool vfsIsRoot = entry.vfs.count == 1 && entry.vfs.data[0] == '/';
				path.Clear();
				if (!vfsIsRoot)
					path.Append(entry.vfs);
				path.Append(temp2);
				applied[i] = true;
				substituted = true;
				everSubstituted = true;
				break;
			}
			if (!substituted)
				return everSubstituted;
		}
		return everSubstituted;
	}

	bool IsVfsParentDir(StringView path)
	{
		if (!g_vfsEntryCount || path.count == 0)
			return false;
		SCOPED_READ_LOCK(g_vfsLock, vfsLock);
		if (!g_vfsEntryCount)
			return false;

		// Bind-mount entries' vfs field is pre-pivot host form; callers see post-pivot. Prepend
		// the "/" overlay's local (= host NS root) to normalize before the boundary check.
		StringView nsRoot;
		bool haveNsRoot = false;
		for (u32 i = 0, e = g_vfsEntryCount; i != e; ++i)
		{
			if (g_vfsEntries[i].vfs.count == 1 && g_vfsEntries[i].vfs.data[0] == '/')
			{
				nsRoot = StringView(g_vfsEntries[i].local.data, g_vfsEntries[i].local.count);
				haveNsRoot = true;
				break;
			}
		}

		StringBuffer<MaxPath> prePivot;
		if (haveNsRoot && !path.StartsWith(nsRoot))
			prePivot.Append(nsRoot);
		prePivot.Append(path);
		while (prePivot.count > 1 && prePivot.data[prePivot.count - 1] == '/')
			prePivot.Resize(prePivot.count - 1);

		// Equal-length matches count: path IS the bind-mount target is also a valid namespace dir.
		// Boundary must be '/' or EOS. Skip vfs="/" — it matches everything.
		for (u32 i = 0, e = g_vfsEntryCount; i != e; ++i)
		{
			const VfsEntry& entry = g_vfsEntries[i];
			if (entry.vfs.count == 1 && entry.vfs.data[0] == '/')
				continue;
			if (entry.vfs.count >= prePivot.count
				&& (entry.vfs.count == prePivot.count || entry.vfs.data[prePivot.count] == '/')
				&& Equals(prePivot.data, entry.vfs.data, prePivot.count, CaseInsensitiveFs))
				return true;
		}
		return false;
	}

	// Pass-through subtrees: paths registered here + descendants bypass the detour (CanDetour2 false).
	// Used by Shared_mount fstype-tmpfs (nsjail's in-NS tmpfs at /tmp). Reuses g_vfsLock.
	struct PassThroughEntry { StringView root; };
	static PassThroughEntry g_passThroughRoots[16];
	static u8 g_passThroughRootCount;

	bool AddPassThroughRoot(StringView root)
	{
		if (!root.count) return false;
		SCOPED_WRITE_LOCK(g_vfsLock, vfsLock);
		for (u8 i = 0; i < g_passThroughRootCount; ++i) // dedup
			if (g_passThroughRoots[i].root.Equals(root))
				return true;
		if (g_passThroughRootCount >= sizeof_array(g_passThroughRoots))
			return false;
		g_passThroughRoots[g_passThroughRootCount++].root = g_memoryBlock.Strdup(root);
		return true;
	}

	bool IsUnderPassThroughRoot(StringView path)
	{
		if (!g_passThroughRootCount) // hot path: typically 0 outside nsjail edges
			return false;
		SCOPED_READ_LOCK(g_vfsLock, vfsLock);

		auto checkDirect = [&](StringView p) -> int {
			u32 longestPtRootLen = 0;
			for (u8 i = 0; i < g_passThroughRootCount; ++i)
			{
				StringView root = g_passThroughRoots[i].root;
				if (p.count < root.count) continue;
				if (!p.StartsWith(root.data)) continue;
				if (p.count != root.count
					&& p.data[root.count] != '/'
					&& !(root.count > 0 && root.data[root.count - 1] == '/'))
					continue;
				if (root.count > longestPtRootLen) longestPtRootLen = root.count;
			}
			return (int)longestPtRootLen;
		};
		auto vfsCoversMoreSpecifically = [&](StringView p, u32 longestPtRootLen) -> bool {
			for (u32 i = 0, e = g_vfsEntryCount; i != e; ++i)
			{
				const VfsEntry& entry = g_vfsEntries[i];
				if (entry.vfs.count == 1 && entry.vfs.data[0] == '/') continue;
				if (entry.vfs.count <= longestPtRootLen) continue;
				if (p.count < entry.vfs.count) continue;
				if (!p.StartsWith(entry.vfs.data)) continue;
				if (p.count != entry.vfs.count && p.data[entry.vfs.count] != '/') continue;
				return true;
			}
			return false;
		};

		// 1) Direct (path is host pre-pivot form).
		if (int n = checkDirect(path))
			return !vfsCoversMoreSpecifically(path, (u32)n);

		// 2) NS-virtual form: prepend nsRoot ("/" overlay's local) and re-check.
		StringView nsRoot;
		for (u32 i = 0, e = g_vfsEntryCount; i != e; ++i)
		{
			if (g_vfsEntries[i].vfs.count == 1 && g_vfsEntries[i].vfs.data[0] == '/')
			{
				nsRoot = StringView(g_vfsEntries[i].local.data, g_vfsEntries[i].local.count);
				break;
			}
		}
		if (!nsRoot.count || path.StartsWith(nsRoot))
			return false;
		// Reject prepend if path is already host-real (some vfs entry's LOCAL is a prefix).
		for (u32 i = 0, e = g_vfsEntryCount; i != e; ++i)
		{
			const VfsEntry& entry = g_vfsEntries[i];
			if (entry.vfs.count == 1 && entry.vfs.data[0] == '/') continue;
			if (path.count < entry.local.count) continue;
			if (entry.local.count == 0) continue;
			if (!path.StartsWith(entry.local.data)) continue;
			if (path.count != entry.local.count && path.data[entry.local.count] != '/') continue;
			return false;
		}

		StringBuffer<MaxPath> normalized;
		normalized.Append(nsRoot);
		normalized.Append(path);
		StringView queryPath(normalized.data, normalized.count);
		if (int n = checkDirect(queryPath))
			return !vfsCoversMoreSpecifically(queryPath, (u32)n);
		return false;
	}

	u32 BuildPassThroughRootsEnvValue(tchar* outBuf, u32 outCap)
	{
		if (!g_passThroughRootCount || !outBuf || outCap == 0)
			return 0;
		SCOPED_READ_LOCK(g_vfsLock, vfsLock);
		u32 written = 0;
		for (u8 i = 0; i < g_passThroughRootCount; ++i)
		{
			StringView root = g_passThroughRoots[i].root;
			// "<len>:<path>;" matches UBA_VFS_OVERLAYS framing; no partial entries.
			tchar tmp[32];
			int n = TSprintf_s(tmp, sizeof_array(tmp), TC("%u:"), root.count);
			if (n < 0) return 0;
			u32 need = (u32)n + root.count + 1; // +1 for ';'
			if (written + need >= outCap)
				return 0;
			for (int k = 0; k < n; ++k) outBuf[written++] = tmp[k];
			for (u32 k = 0; k < root.count; ++k) outBuf[written++] = root.data[k];
			outBuf[written++] = ';';
		}
		if (written < outCap) outBuf[written] = 0;
		return written;
	}

	// extern "C" forwarders for TestApp (LD_PRELOAD-only — stub linker script discards dynamics).
	// Need default visibility because UBT builds libUbaDetours.so with -fvisibility=hidden globally.
#if PLATFORM_LINUX && !UBA_STUB_BUILD
	#define UBA_TEST_EXPORT __attribute__((visibility("default")))
	extern "C" UBA_TEST_EXPORT int uba_AddPassThroughRoot(const char* path, unsigned int len)
	{
		return AddPassThroughRoot(StringView(path, len)) ? 1 : 0;
	}
	extern "C" UBA_TEST_EXPORT int uba_IsUnderPassThroughRoot(const char* path, unsigned int len)
	{
		return IsUnderPassThroughRoot(StringView(path, len)) ? 1 : 0;
	}
	// TestApp hook: register a vfs overlay in-process for mixed-config regression tests.
	extern "C" UBA_TEST_EXPORT int uba_AddVfsEntry(const char* vfs, unsigned int vlen, const char* local, unsigned int llen)
	{
		return AddVfsEntry(StringView(vfs, vlen), StringView(local, llen)) ? 1 : 0;
	}
#endif

	void Shared_BuildPassThroughEnv(StringBufferBase& out)
	{
		if (!g_passThroughRootCount) return;
		out.Append("UBA_PASSTHROUGH_ROOTS=");
		u32 startCount = out.count;
		u32 written = BuildPassThroughRootsEnvValue(out.data + out.count, out.capacity - out.count - 1);
		if (written == 0) { out.Resize(startCount - 22); return; } // roll back the prefix
		out.count += written;
		out.data[out.count] = 0;
	}

#if !UBA_STUB_BUILD
	void LogHeader(StringView cmdLine)
	{
		#if UBA_DEBUG_LOG_ENABLED
		if (g_debugFile == InvalidFileHandle)
			return;
		StringBuffer<256> computerName;
		GetComputerNameW(computerName);

		WriteDebugLog(TC("ProcessId: %u"), g_processId);
		WriteDebug("CmdLine: ", 9);

		#if PLATFORM_WINDOWS
		u32 left = cmdLine.count;
		const tchar* read = cmdLine.data;
		while (left)
		{
			char buf[1024];
			u32 written = 0;
			while (*read && written < sizeof(buf))
				buf[written++] = (char)*read++;
			WriteDebug(buf, written);
			left -= written;
		}
		#else
		WriteDebug(cmdLine.data, cmdLine.count*sizeof(tchar));
		#endif

		WriteDebug("\n", 1);
		WriteDebugLog(TC("WorkingDir: %s"), g_virtualWorkingDir.data);
		WriteDebugLog(TC("ExeDir: %s"), g_virtualApplicationDir.data);
		WriteDebugLog(TC("ExeDir (actual): %s"), g_exeDir.data);
		WriteDebugLog(TC("SystemTemp: %s"), g_systemTemp.data);
		WriteDebugLog(TC("Rules: %u (%u)"), g_rules->index, GetApplicationRules()[g_rules->index].hash);
		WriteDebugLog(TC("Host: %s"), computerName.data);
		WriteDebugLog(TC("HostType: %s"), g_runningRemote ? TC("Remote") : TC("Local"));

		static u32 reuseCounter;
		if (reuseCounter)
			WriteDebugLog(TC("ProcessReuseIndex: %u"), reuseCounter);
		++reuseCounter;
		WriteDebugLog(TC(""));
		#endif
	}

	void LogVfsInfo()
	{
		for (u32 i=0; i!=g_vfsEntryCount; ++i)
		{
			DEBUG_LOG(TC("Vfs: %s -> %s"), g_vfsEntries[i].vfs.data, g_vfsEntries[i].local.data);
		}
	}

	const tchar* GetApplicationShortName()
	{
		const tchar* lastBackslash = TStrrchr(g_virtualApplication.data, '\\');
		const tchar* lastSlash = TStrrchr(g_virtualApplication.data, '/');
		if (lastBackslash || lastSlash)
			return (lastBackslash > lastSlash ? lastBackslash : lastSlash) + 1;
		return g_virtualApplication.data;
	}

	ANALYSIS_NORETURN void FatalError(u32 code, const tchar* format, ...)
	{
		StringBuffer<2048> sb;
		sb.Append(GetApplicationShortName()).Append(TCV(" ERROR: "));
		va_list arg;
		va_start(arg, format);
		sb.Append(format, arg);
		va_end(arg);
		Rpc_WriteLog(sb.data, sb.count, true, true);

		#if PLATFORM_WINDOWS // Maybe all platforms should call exit()?
		ExitProcess(code);
		#else
		exit(code);
		#endif
	}

	bool TakeLockForRpc()
	{
		for (u32 i=0; i!=5; ++i)
		{
			if (g_communicationLock.TryEnter())
				return true;
			Sleep(100);
		}
		DEBUG_LOG(TC("TakeLockForRpc failed"));
		return false;
	}

	void Rpc_WriteLog(const tchar* text, u64 textCharLength, bool printInSession, bool isError)
	{
		DEBUG_LOG(TC("LOG  %.*s"), u32(textCharLength), text); // TODO: Investigate, deadlocks on non-windows
		// DEBUG_LOG(TC("LOG [%7u] %.*s"), GetCurrentThreadId(), u32(textCharLength), text);

		if (!TakeLockForRpc())
		{
			// Comm lock held by our own thread (assert mid-RPC, or signal handler).
			// Log RPC would hang — drop to stderr instead.
			#if !PLATFORM_WINDOWS
			const char prefix[] = "[UBA-LOG-NO-RPC] ";
			(void)!write(2, prefix, sizeof(prefix) - 1);
			(void)!write(2, text, (size_t)textCharLength);
			(void)!write(2, "\n", 1);
			#endif
			return;
		}
		RPC_MESSAGE_NO_LOCK(Log, log)
		writer.WriteBool(printInSession);
		writer.WriteBool(isError);
		writer.WriteString(text, textCharLength);
		writer.Flush();
		g_communicationLock.Leave();
	}

	void Rpc_WriteLogf(const tchar* format, ...)
	{
		va_list arg;
		va_start(arg, format);
		StringBuffer<2048> buffer;
		buffer.Append(format, arg);
		va_end(arg);
		Rpc_WriteLog(buffer.data, buffer.count, false, false);
	}

	UBA_NOINLINE void Rpc_ResolveCallstack(StringBufferBase& out, u32 skipCallstackCount, void* context)
	{
		if (!TakeLockForRpc())
		{
			out.Append(TCV("\n  Can't fetch callstack while in rpc"));
			
			// TODO: Add the offsets at least
			//void* callstack[100];
			//u32 callstackCount = GetCallstack(callstack, 100, skipCallstack, contextPtr);
			return;
		}

		RPC_MESSAGE_NO_LOCK(ResolveCallstack, callstack)
		auto written = (u32*)writer.AllocWrite(4);
		if (WriteCallstackInfo(writer, skipCallstackCount, context))
		{
			*written = u32(writer.GetPosition()) - 5;
			BinaryReader reader = writer.Flush();
			reader.ReadString(out);
		}
		else
		{
			out.Append(TCV("\n   Failed to resolve callstack\n"));
		}
		g_communicationLock.Leave();
	}

	//TODO: Implement SetConsoleTextAttribute.. clang is using it to color errors

	tchar g_consoleString[4096];
	u32 g_consoleStringIndex;

	template<typename CharType>
	void Shared_WriteConsoleT(const CharType* chars, u32 charCount, bool isError)
	{
		if (!g_conEnabled[isError?0:1] || g_suppressLogging)
			return;

		SCOPED_WRITE_LOCK(g_consoleStringCs, lock);
		const CharType* read = chars;
		tchar* write = g_consoleString + g_consoleStringIndex;
		int left = sizeof_array(g_consoleString) - g_consoleStringIndex - 1;
		int available = charCount;
		while (available)
		{
			if (*read == '\n' || !left)
			{
				*write = 0;
				u32 strLen = u32(write - g_consoleString);
				if (!g_rules->SuppressLogLine(g_consoleString, strLen))
					Rpc_WriteLog(g_consoleString, strLen, false, isError);
				write = g_consoleString;
				left = sizeof_array(g_consoleString) - 1;
			}
			else
			{
				*write = *read;
				++write;
			}
			++read;
			--left;
			--available;
		}
		g_consoleStringIndex = u32(write - g_consoleString);
	}

	void Shared_WriteConsole(const char* chars, u32 charCount, bool isError) { Shared_WriteConsoleT(chars, charCount, isError); }

	#if PLATFORM_WINDOWS
	void Shared_WriteConsole(const wchar_t* chars, u32 charCount, bool isError) { Shared_WriteConsoleT(chars, charCount, isError); }
	#endif

#endif // !UBA_STUB_BUILD

	const tchar* Shared_GetFileAttributes(FileAttributes& outAttr, StringView fileName, bool followSymlink, bool checkIfDir)
	{
		memset(&outAttr, 0, sizeof(FileAttributes));

		StringBuffer<MaxPath> fileNameForKey;
		fileNameForKey.Append(fileName);
		if (CaseInsensitiveFs)
			fileNameForKey.MakeLower();

		UBA_ASSERT(fileNameForKey.count);
		CHECK_PATH(fileNameForKey);
		StringKey fileNameKey = ToStringKey(fileNameForKey);

		memset(&outAttr.data, 0, sizeof(outAttr.data));

		u64 fileSize = InvalidValue;

		#if PLATFORM_WINDOWS
		if (fileName[1] == ':' && fileName[3] == 0 && (ToLower(fileName[0]) == ToLower(g_virtualWorkingDir[0]) || ToLower(fileName[0]) == g_systemRoot[0]))
		{
			// This is the root of the drive.. let's just return it as a directory
			outAttr.exists = true;
			outAttr.lastError = ErrorSuccess;
			outAttr.data.dwFileAttributes = FILE_ATTRIBUTE_DIRECTORY;
			return fileName.data;
		}
		#endif

		// This is an optimization where we populate directory table and use that to figure out if file exists or not..
		// .. in msvc's case it doesn't matter much because these tables are already up to date when msvc use CreateFile.
		// .. clang otoh is using CreateFile with tooons of different paths trying to open files.. in remote worker case this becomes super expensive
		DirectoryTableEntryInformation info;
		if (!Rpc_GetEntryInformation(info, fileNameKey, fileName, checkIfDir, followSymlink))
		{
			outAttr.exists = false;
			outAttr.lastError = ErrorFileNotFound;
			return fileName.data;
		}

		if (fileSize == InvalidValue)
			fileSize = info.size; // This can also be InvalidValue if file is open for write (Used in posix path)

		// Could be compressed and then directory table size is wrong
		if (CouldBeCompressedFile(fileName))
		{
			// If file is output file we accept wrong size because size is not supposed to be used anyway.
			// We don't want to trigger unnecessary download/decompress of file
			if (!g_rules->IsOutputFile(fileName, g_systemTemp))
			{
				StringBuffer<> temp;
				u32 closeId;
				Rpc_CreateFileW(fileName, fileNameKey, AccessFlag_Read, temp.data, temp.capacity, fileSize, closeId, true);
			}
		}

		outAttr.exists = true;
		outAttr.lastError = ErrorSuccess;

		UBA_ASSERTF(info.fileIndex, TC("No file index set for file %.*s"), fileName.count, fileName.data);
		outAttr.fileIndex = info.fileIndex;
		outAttr.volumeSerial = info.volumeSerial;

#if PLATFORM_WINDOWS
		LARGE_INTEGER li = ToLargeInteger(fileSize);
		outAttr.data.dwFileAttributes = info.attributes;
		outAttr.data.nFileSizeLow = li.LowPart;
		outAttr.data.nFileSizeHigh = li.HighPart;
		(u64&)outAttr.data.ftCreationTime = info.lastWrite;
		(u64&)outAttr.data.ftLastAccessTime = info.lastWrite;
		(u64&)outAttr.data.ftLastWriteTime = info.lastWrite;
#else
		outAttr.data.st_mtimespec = ToTimeSpec(info.lastWrite);
		outAttr.data.st_mode = (mode_t)info.attributes;
		outAttr.data.st_dev = info.volumeSerial;
		// st_ino from pathKey (XXH3-128), not info.fileIndex — info.fileIndex collides across unrelated paths
		// (kernel inode reuse + dirtable caching). mkfs.erofs hardlink-dedups on (st_dev, st_ino), so collisions
		// corrupt EROFS dirent topology. Same precedent: UbaDetoursFunctionsPosix.cpp:691 vfs-parent dirs.
		outAttr.data.st_ino = (decltype(outAttr.data.st_ino))fileNameKey.a;
		outAttr.data.st_size = fileSize;
#if !UBA_STUB_BUILD
		// getuid/getgid are libc; freestanding stub uses 0/0 (some tools like
		// git read these but the stub workload doesn't include them).
		static auto uid = getuid();
		static auto gid = getgid();
		outAttr.data.st_uid = uid;
		outAttr.data.st_gid = gid;
#endif
#endif

#if 0//UBA_DEBUG_VALIDATE
		if (g_validateFileAccess && !keepInMemory)
		{
			WIN32_FILE_ATTRIBUTE_DATA validate;
			memset(&validate, 0, sizeof(validate));
			SuppressDetourScope _;
			BOOL res = True_GetFileAttributesExW(fileName, GetFileExInfoStandard, &validate); (void)res;
			if (outAttr.exists)
			{
				UBA_ASSERTF(res != 0, L"File %ls exists even though uba claims it is not..", fileName.data);
				if (validate.dwFileAttributes & FILE_ATTRIBUTE_DIRECTORY)
					UBA_ASSERTF((outAttr.data.dwFileAttributes & FILE_ATTRIBUTE_DIRECTORY), L"File attributes are wrong for %ls", fileName.data);
				else
				{
					validate.ftCreationTime = outAttr.data.ftCreationTime; // Creation time is not really important
					validate.ftLastAccessTime = outAttr.data.ftLastAccessTime; // Access time is not really important
					validate.ftLastWriteTime = outAttr.data.ftLastWriteTime; // Write time is important, revisit this
					UBA_ASSERTF(memcmp(&validate, &outAttr.data, sizeof(WIN32_FILE_ATTRIBUTE_DATA)) == 0, L"File %ls is not up-to-date in cache", fileName.data);
				}
			}
			else
			{
				UBA_ASSERTF(res == 0, L"Can't find file %ls but validation checked that it is there", fileName.data); // This means most likely that Uba did not update attribute table for added files.
				DWORD lastError2 = GetLastError();
				if (lastError2 == ERROR_PATH_NOT_FOUND || lastError2 == ERROR_INVALID_NAME)
					lastError2 = ERROR_FILE_NOT_FOUND;
				UBA_ASSERT(outAttr.lastError == lastError2);
			}
		}
#endif
		return fileName.data;
	}
}
