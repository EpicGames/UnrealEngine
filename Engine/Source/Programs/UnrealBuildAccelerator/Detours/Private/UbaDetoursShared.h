// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaApplicationRules.h"
#include "UbaFileHandle.h"
#include "UbaStringBuffer.h"
#include "UbaPlatform.h"
#include "UbaProcessStats.h"
#include "UbaProtocol.h"
#if UBA_STUB_BUILD
// Stub build needs complete types for the shared-global macros below. FileMappingTable is excluded
// (it transitively includes us); TUs using g_mappedFileTable include UbaDetoursFileMappingTable.h directly.
#include "UbaMemory.h"
#include "UbaSynchronization.h"
#include "UbaDirectoryTable.h"
#endif

namespace uba
{
	class DirectoryTable;
	class MappedFileTable;
	struct FileMappingHandle;

	#if PLATFORM_WINDOWS
	DWORD Local_GetLongPathNameW(LPCWSTR lpszShortPath, LPWSTR lpszLongPath, DWORD cchBuffer);
	#endif

	void Rpc_WriteLog(const tchar* text, u64 textCharLength, bool printInSession, bool isError);
	void Rpc_WriteLogf(const tchar* format, ...);
	void Rpc_ResolveCallstack(StringBufferBase& out, u32 skipCallstackCount, void* context);

	const tchar* GetApplicationShortName();
	bool FixPath(StringBufferBase& out, const tchar* path, u32 pathLen = ~0u);
	bool FixPath(StringBufferBase& out, const StringView path);

	void PopulateVfs(BinaryReader& vfsReader);
	bool IsVfsEnabled();
	bool AddVfsEntry(StringView vfs, StringView local); // vfs/local must be absolute FixPath'd paths
	bool AddVfsEntryBulkNoDedup(StringView vfs, StringView local); // bulk fast path; caller MUST call FinalizeVfsEntriesAfterBulk()
	void FinalizeVfsEntriesAfterBulk();
	bool RemoveVfsEntry(StringView vfs);
	bool DevirtualizePath(StringBufferBase& path);
	tchar DevirtualizeDrive(tchar drive);
	bool VirtualizePath(StringBufferBase& path);
	bool IsVfsParentDir(StringView path); // True if path is an ancestor of any vfs entry; synthesizes "is a directory" for unbound bind-mount parents
	bool AddPassThroughRoot(StringView root); // Roots + descendants bypass detour; used by tmpfs mounts the host DIRTABLE can't model
	bool IsUnderPassThroughRoot(StringView path);
	u32 BuildPassThroughRootsEnvValue(tchar* outBuf, u32 outCap); // Encode as "<len>:<path>;..."; returns bytes written
	void Shared_BuildPassThroughEnv(StringBufferBase& out);
	void LogHeader(StringView cmdLine);
	void LogVfsInfo();

	#if UBA_DEBUG_LOG_ENABLED
		#define DEBUG_LOG_PREFIX(Prefix, Command, fmt, ...) \
			::uba::LogScope STRING_JOIN(ls, __LINE__); \
			do { UBA_FMT_CHECK(fmt, ##__VA_ARGS__); if (::uba::isLogging()) ::uba::WriteDebugLogWithPrefix(#Prefix, STRING_JOIN(ls, __LINE__), Command, fmt, ##__VA_ARGS__); } while(0)

		#define DEBUG_LOG_TRUE(Command, ...) DEBUG_LOG_PREFIX(T, Command, __VA_ARGS__)
		#define DEBUG_LOG_DETOURED(Command, ...) DEBUG_LOG_PREFIX(D, Command, __VA_ARGS__)
		#define DEBUG_LOG_PROXY(Command, ...) DEBUG_LOG_PREFIX(P, Command, __VA_ARGS__)
		#define DEBUG_LOG_PIPE(Command, ...) ts.Leave();
		#define DEBUG_LOG(fmt, ...) do { UBA_FMT_CHECK(fmt, ##__VA_ARGS__); if (::uba::isLogging()) ::uba::WriteDebugLog(fmt, ##__VA_ARGS__); } while(0)
	#else
		#define DEBUG_LOG(...) {}
		#define DEBUG_LOG_DETOURED(Command, ...) {}
		#define DEBUG_LOG_TRUE(Command, ...) {}
		#define DEBUG_LOG_PROXY(Command, ...) {}
		#define DEBUG_LOG_PIPE(...) ts.Leave();
	#endif

#if !UBA_STUB_BUILD
	// Normal build: each global is a reference backed by a pointer slot the dynamic linker fills in.
	extern StringBuffer<512>& g_virtualApplication;
	extern StringBuffer<512>& g_virtualApplicationDir;
	extern StringBuffer<256>& g_exeDir;
	extern StringBuffer<512>& g_logName;

	extern StringBuffer<512>& g_virtualWorkingDir;
	extern StringBuffer<256>& g_systemTemp;
	extern StringBuffer<128>& g_systemRoot;

	extern ProcessStats& g_stats;
	extern KernelStats& g_kernelStats;
	extern Futex& g_communicationLock;
	extern MemoryBlock& g_memoryBlock;
	extern MappedFileTable& g_mappedFileTable;
#endif
	// Stub-build declarations follow further down (after VariableMem is defined).
	extern bool g_conEnabled[2]; // 0=stderr, 1=stdout

	// TLS-free per-thread suppression state. Goroutine OS threads created via clone() lack a glibc
	// TCB, so thread_local access crashes them. Tracked via tiny open-addressed array keyed by OS tid.
	u32  DisallowDetourDepth();
	void DisallowDetourIncrement();
	void DisallowDetourDecrement();

	struct SuppressDetourScope
	{
		SuppressDetourScope();
		~SuppressDetourScope();
	};

	// Back-compat shim for existing `if (t_disallowDetour)` callsites.
	#define t_disallowDetour uba::DisallowDetourDepth()

	#if UBA_DEBUG_LOG_ENABLED
	extern FileHandle g_debugFile;
	extern Futex& g_debugLogMutex;
	extern StringBuffer<MaxPath>& g_debugLogPath;
	inline bool isLogging() { return g_debugFile != InvalidFileHandle; }
	struct LogScope { LogScope(); ~LogScope(); void Flush(); };
	void WriteDebugLogWithPrefix(const char* prefix, LogScope& scope, const tchar* command, const tchar* format, ...);
	void WriteDebugLog(const tchar* format, ...);
	void WriteDebugInner(const void* data, u32 dataLen); // Raw writer (platform-defined). Caller MUST hold g_debugLogMutex.
	void WriteDebug(const void* data, u32 dataLen);      // Locked public wrapper around WriteDebugInner.
	void FlushDebugLog();
	#endif
	#if UBA_DEBUG_VALIDATE
	extern bool g_validateFileAccess;
	#endif

	inline constexpr bool g_allowDirectoryCache = true;
	inline constexpr bool g_allowFileMappingDetour = true;
	inline constexpr bool g_allowFindFileDetour = true;
	inline constexpr bool g_allowListDirectoryHandle = true;

	extern DirectoryTable* g_directoryTable;
	extern u32 g_processId;
	extern u32 g_rulesIndex;
	extern ApplicationRules* g_rules;
	extern bool g_runningRemote;
	extern bool g_isChild;
	extern bool g_allowKeepFilesInMemory;
	extern bool g_allowOutputFiles;
	extern bool g_allowDiscardVirtualMemory;
	extern bool g_suppressLogging;

	#if PLATFORM_WINDOWS
	constexpr u32 ErrorSuccess = 0;
	constexpr u32 ErrorFileNotFound = ERROR_FILE_NOT_FOUND;
	using FileAttributesData = WIN32_FILE_ATTRIBUTE_DATA;
	#else
	using FileAttributesData = struct stat;
	constexpr u32 ErrorSuccess = 0;
	constexpr u32 ErrorFileNotFound = ENOENT;
	#endif


	struct FileAttributes
	{
		FileAttributesData data;
		u64 fileIndex;
		u32 volumeSerial;
		u8 exists;
		u32 lastError;
	};

	bool CouldBeCompressedFile(const StringView& fileName);

	void Shared_WriteConsole(const char* chars, u32 charCount, bool isError);
	void Shared_WriteConsole(const wchar_t* chars, u32 charCount, bool isError);

	const tchar* Shared_GetFileAttributes(FileAttributes& outAttr, StringView fileName, bool followSymlink = true, bool checkIfDir = false);

	void InitSharedVariables();
	void InitTrackInput();
	void TrackInput(StringView file);
	void SkipTrackInput(StringView file);
	bool IsTrackingInput();
	void SendInput();

	template<typename T> struct VariableMem { template<typename... Args> void Create(Args&&... args) { new (data) T(args...); }; u64 data[AlignUp(sizeof(T), sizeof(u64)) / sizeof(u64)]; };
#if UBA_STUB_BUILD
	// Stub build: emit storage only. The g_name macros below resolve through RIP-relative accessors —
	// no pointer slots, so no R_X86_64_RELATIVE relocations the objcopied blob couldn't apply.
	#define VARIABLE_MEM(type, name) VariableMem<type> name##Mem;

	// Untyped accessors so headers don't need the shared-globals' complete types in scope.
	void* g_virtualApplicationStorage();
	void* g_virtualApplicationDirStorage();
	void* g_exeDirStorage();
	void* g_logNameStorage();
	void* g_virtualWorkingDirStorage();
	void* g_systemTempStorage();
	void* g_systemRootStorage();
	void* g_statsStorage();
	void* g_kernelStatsStorage();
	void* g_communicationLockStorage();
	void* g_memoryBlockStorage();
	void* g_mappedFileTableStorage();
	void* g_trackInputsLockStorage();
	void* g_vfsLockStorage();
	#if UBA_DEBUG_LOG_ENABLED
	void* g_debugLogPathStorage();
	void* g_debugLogMutexStorage();
	#endif

	// Redirect bare `g_xxx` references through storage-accessor casts. Preprocessor expansion runs
	// before name lookup so callsites lower to RIP-relative lea + deref — no relocation.
	#define g_virtualApplication     (*static_cast<::uba::StringBuffer<512>*>(::uba::g_virtualApplicationStorage()))
	#define g_virtualApplicationDir  (*static_cast<::uba::StringBuffer<512>*>(::uba::g_virtualApplicationDirStorage()))
	#define g_exeDir                 (*static_cast<::uba::StringBuffer<256>*>(::uba::g_exeDirStorage()))
	#define g_logName                (*static_cast<::uba::StringBuffer<512>*>(::uba::g_logNameStorage()))
	#define g_virtualWorkingDir      (*static_cast<::uba::StringBuffer<512>*>(::uba::g_virtualWorkingDirStorage()))
	#define g_systemTemp             (*static_cast<::uba::StringBuffer<256>*>(::uba::g_systemTempStorage()))
	#define g_systemRoot             (*static_cast<::uba::StringBuffer<128>*>(::uba::g_systemRootStorage()))
	#define g_stats                  (*static_cast<::uba::ProcessStats*>(::uba::g_statsStorage()))
	#define g_kernelStats            (*static_cast<::uba::KernelStats*>(::uba::g_kernelStatsStorage()))
	#define g_communicationLock      (*static_cast<::uba::Futex*>(::uba::g_communicationLockStorage()))
	#define g_memoryBlock            (*static_cast<::uba::MemoryBlock*>(::uba::g_memoryBlockStorage()))
	#define g_mappedFileTable        (*static_cast<::uba::MappedFileTable*>(::uba::g_mappedFileTableStorage()))
	#define g_trackInputsLock        (*static_cast<::uba::Futex*>(::uba::g_trackInputsLockStorage()))
	#define g_vfsLock                (*static_cast<::uba::ReaderWriterLock*>(::uba::g_vfsLockStorage()))
	#if UBA_DEBUG_LOG_ENABLED
	#define g_debugLogPath           (*static_cast<::uba::StringBuffer<::uba::MaxPath>*>(::uba::g_debugLogPathStorage()))
	#define g_debugLogMutex          (*static_cast<::uba::Futex*>(::uba::g_debugLogMutexStorage()))
	#endif
#else
	#define VARIABLE_MEM(type, name) VariableMem<type> name##Mem; type& name = (type&)name##Mem.data;
#endif

	#define RPC_MESSAGE(messageName, timerName) \
		RPC_MESSAGE2(MessageType_##messageName, #messageName, g_stats.timerName)

	#define RPC_MESSAGE2(messageType, messageName, timer) \
		DEBUG_LOG(TC("RPC_MESSAGE %s"), TC(messageName)); \
		TimerScope ts(timer); \
		SCOPED_FUTEX(g_communicationLock, pcs); \
		BinaryWriter writer(ProcessCommunication_Write); \
		writer.WriteByte(messageType);

	#define RPC_MESSAGE_NO_LOCK(messageName, timerName) \
		DEBUG_LOG(TC("RPC_MESSAGE %s"), TC(#messageName)); \
		TimerScope ts(g_stats.timerName); \
		BinaryWriter writer(ProcessCommunication_Write); \
		writer.WriteByte(MessageType_##messageName);

	#define RPC_MESSAGE_END \
		ts.Leave(); \
		pcs.Leave();

	// Dummy code
	enum LogEntryType : u8 { LogEntryType_Error = 0, LogEntryType_Warning = 1, };
	class Logger { public: void Logf(...) {} void Info(...) {} bool Warning(...) { return false; } bool Error(...) { return false; } };
}
