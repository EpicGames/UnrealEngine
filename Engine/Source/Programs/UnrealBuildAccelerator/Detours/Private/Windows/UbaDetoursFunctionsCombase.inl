// Copyright Epic Games, Inc. All Rights Reserved.

HRESULT Detoured_CoCreateInstance(REFCLSID rclsid, LPUNKNOWN pUnkOuter, DWORD dwClsContext, REFIID riid, LPVOID *ppv)
{
	// Special handling for com object used to figure out visual studio installation. This is used as a fallback path for lld-link.exe among others
	// {177F0C4A-1CD3-4DE7-A32C-71DBBB9FA36D}
	static const CLSID CLSID_SetupConfiguration = { 0x177F0C4A, 0x1CD3, 0x4DE7, { 0xA3, 0x2C, 0x71, 0xDB, 0xBB, 0x9F, 0xA3, 0x6D } };

	if (IsEqualCLSID(rclsid, CLSID_SetupConfiguration))
	{
		DEBUG_LOG_DETOURED(L"CoCreateInstance", L"(VisualStudioSetup) %s", GuidToString((Guid&)rclsid).data);

		// TODO: Should probably not be hard coded and instead be transferred from host
		const wchar_t* dllPath = L"c:\\ProgramData\\Microsoft\\VisualStudio\\Setup\\x64\\Microsoft.VisualStudio.Setup.Configuration.Native.dll";
		HMODULE hMod = LoadLibraryW(dllPath);
		if (!hMod)
			return REGDB_E_CLASSNOTREG;

		typedef HRESULT (WINAPI* FnDllGetClassObject)(REFCLSID, REFIID, LPVOID*);
		auto DllGetClassObject = (FnDllGetClassObject)(void*)GetProcAddress(hMod, "DllGetClassObject");
		if (!DllGetClassObject)
			return REGDB_E_CLASSNOTREG;

		IClassFactory* pCF = nullptr;
		HRESULT hr = DllGetClassObject(rclsid, IID_IClassFactory, (void**)&pCF);
		if (FAILED(hr)) return hr;

		hr = pCF->CreateInstance(pUnkOuter, riid, ppv);
		pCF->Release();
		return hr;
	}

	DEBUG_LOG_DETOURED(L"CoCreateInstance", L"%s", GuidToString((Guid&)rclsid).data);
	return True_CoCreateInstance(rclsid, pUnkOuter, dwClsContext, riid, ppv);
}