// Copyright Epic Games, Inc. All Rights Reserved.

typedef struct _MODLOAD_DATA {} *PMODLOAD_DATA;
using SymLoadModuleExWFunc = BOOL(HANDLE hProcess, HANDLE hFile, PCWSTR ImageName, DWORD64 BaseOfDll, DWORD DllSize, PMODLOAD_DATA Data, DWORD Flags);
SymLoadModuleExWFunc* True_SymLoadModuleExW;

BOOL Detoured_SymLoadModuleExW(HANDLE hProcess, HANDLE hFile, PCWSTR ImageName, DWORD64 BaseOfDll, DWORD DllSize, PMODLOAD_DATA Data, DWORD Flags)
{
	UBA_ASSERT(g_isRunningWine);

	//u64 pathLen = wcslen(ImageName);
	//StringBuffer<512> tempBuf;
	//Rpc_GetFullFileName(ImageName, pathLen, tempBuf, false);
	DEBUG_LOG_TRUE(L"SymLoadModuleExW", L"(%ls)", ImageName);
	SuppressDetourScope _;
	return True_SymLoadModuleExW(hProcess, hFile, ImageName, BaseOfDll, DllSize, Data, Flags);
}