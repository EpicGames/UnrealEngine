// Copyright Epic Games, Inc. All Rights Reserved.

typedef PVOID DIGEST_HANDLE;
typedef BOOL(WINAPI* DIGEST_FUNCTION) (DIGEST_HANDLE refdata, PBYTE pData, DWORD dwLength);

using ImageGetDigestStreamFunc = BOOL(HANDLE FileHandle, DWORD DigestLevel, DIGEST_FUNCTION DigestFunction, DIGEST_HANDLE DigestHandle);
ImageGetDigestStreamFunc* True_ImageGetDigestStream;

BOOL Detoured_ImageGetDigestStream(HANDLE FileHandle, DWORD DigestLevel, DIGEST_FUNCTION DigestFunction, DIGEST_HANDLE DigestHandle)
{
	DEBUG_LOG_DETOURED(L"ImageGetDigestStream", L"%llu (%s)", u64(FileHandle), HandleToName(FileHandle));

	// This function is _very_ limited and does not cover all kinds of binaries. Only here to handle the dlls used by ShaderCompileWorker
	// Expects CERT_PE_IMAGE_DIGEST_ALL_IMPORT_INFO | CERT_PE_IMAGE_DIGEST_RESOURCES
	UBA_ASSERT(DigestLevel == (DWORD)(0x04 | 0x02));

	auto ErrorInvalidParameter = []() { SetLastError(ERROR_INVALID_PARAMETER); return false; };

	if (!FileHandle)
		return ErrorInvalidParameter();
	DWORD fileSize = GetFileSize(FileHandle, NULL);
	if (fileSize == INVALID_FILE_SIZE || fileSize < sizeof(IMAGE_DOS_HEADER))
		return ErrorInvalidParameter();

	auto mappingHandle = ::CreateFileMappingW(FileHandle, NULL, PAGE_READONLY, 0, 0, NULL);
	if (mappingHandle == INVALID_HANDLE_VALUE)
		return ErrorInvalidParameter();
	auto hg = MakeGuard([&]() { ::CloseHandle(mappingHandle); });

	auto mem = (BYTE*)::MapViewOfFile(mappingHandle, FILE_MAP_COPY, 0, 0, 0);
	if (!mem)
		return ErrorInvalidParameter();
	auto mg = MakeGuard([&]() { ::UnmapViewOfFile(mem); });

	auto& dosHeader = *(IMAGE_DOS_HEADER*)mem;
	if (dosHeader.e_magic != IMAGE_DOS_SIGNATURE)
		return ErrorInvalidParameter();
	DWORD offset = dosHeader.e_lfanew;
	if (!offset || offset > fileSize)
		return ErrorInvalidParameter();

	if (!DigestFunction(DigestHandle, mem, offset))
		return false;

	if (offset + sizeof(IMAGE_NT_HEADERS) > fileSize)
		return ErrorInvalidParameter();

	auto& ntHeader = *(IMAGE_NT_HEADERS*)(mem + offset);
	if (ntHeader.Signature != IMAGE_NT_SIGNATURE)
		return ErrorInvalidParameter();

	ntHeader.OptionalHeader.CheckSum = 0;
	if (ntHeader.OptionalHeader.Magic == 0x20b)
		ntHeader.OptionalHeader.DataDirectory[4] = { 0, 0 }; // Debug table

	DWORD headerSize = sizeof(ntHeader.Signature) + sizeof(ntHeader.FileHeader) + ntHeader.FileHeader.SizeOfOptionalHeader;
	if (!DigestFunction(DigestHandle, mem + offset, headerSize))
		return false;

	offset += headerSize;
	DWORD sectionCount = ntHeader.FileHeader.NumberOfSections;
	DWORD sectionsSize = sectionCount * sizeof(IMAGE_SECTION_HEADER);
	if (offset + sectionsSize > fileSize)
		return ErrorInvalidParameter();

	if (!DigestFunction(DigestHandle, mem + offset, sectionsSize))
		return false;

	auto sections = (IMAGE_SECTION_HEADER*)(mem + offset);

	for (IMAGE_SECTION_HEADER* i = sections, *e = sections + sectionCount; i < e; ++i)
		if (i->Characteristics & IMAGE_SCN_CNT_CODE)
			if (!DigestFunction(DigestHandle, mem + i->PointerToRawData, i->SizeOfRawData))
				return false;

	auto ReportSection = [&](const char* name)
		{
			for (IMAGE_SECTION_HEADER* i = sections, *e = sections + sectionCount; i < e; ++i)
				if (memcmp(i->Name, name, strlen(name)) == 0)
					if (DWORD offset = i->PointerToRawData)
						DigestFunction(DigestHandle, mem + offset, i->SizeOfRawData);
		};

	ReportSection(".rdata");
	ReportSection(".data");
	ReportSection(".pdata");
	ReportSection("_RDATA");
	ReportSection(".debug");
	ReportSection(".didat");
	ReportSection(".tls");
	ReportSection(".rsrc");
	ReportSection(".reloc");
	return true;
}
