// Copyright Epic Games, Inc. All Rights Reserved.

#define UBA_ASSERT_ON_DETOURED_STD_PIPE 0 // Errors on clang on llvm-symbolizer.exe.. needs investigation

static inline BOOL SetLastErrorFromNtStatus(NTSTATUS status)
{
	if (status)
		SetLastError(RtlNtStatusToDosError(status));
	else
		SetLastError(ERROR_SUCCESS);
	return status == 0;
}

DWORD Local_GetLongPathNameW(LPCWSTR lpszShortPath, LPWSTR lpszLongPath, DWORD cchBuffer)
{
	SCOPED_WRITE_LOCK(g_longPathNameCacheLock, lock);
	auto findIt = g_longPathNameCache.find(lpszShortPath);
	if (findIt != g_longPathNameCache.end())
	{
		const wchar_t* longPath = findIt->second;
		u32 len = u32(wcslen(longPath));
		if (len == 0)
		{
			DEBUG_LOG_DETOURED(L"GetLongPathNameW", L"(NOT_FOUND) (%s) -> 0", lpszShortPath);
			SetLastError(ERROR_FILE_NOT_FOUND);
			return 0;
		}
		SetLastError(ERROR_SUCCESS);
		if (cchBuffer <= len)
		{
			DEBUG_LOG_DETOURED(L"GetLongPathNameW", L"(BUFFER_SMALL) (%s) -> %u", lpszShortPath, len + 1);
			return len + 1;
		}
		memcpy(lpszLongPath, longPath, (len + 1) * 2);

		DEBUG_LOG_DETOURED(L"GetLongPathNameW", L"(%s) -> %u", lpszShortPath, len + 1);
		return len + 1;
	}

	const wchar_t* newLongPath = nullptr;
	DWORD res = 0;

	if (g_runningRemote)
	{
		u32 errorCode = 0;
		StringBuffer<> longName;
		{
			RPC_MESSAGE(GetLongPathName, longPathName)
			writer.WriteString(lpszShortPath);
			BinaryReader reader = writer.Flush();
			errorCode = reader.ReadU32();
			reader.ReadString(longName);
		}

		newLongPath = g_memoryBlock.Strdup(longName).data;

		if (longName.count == 0)
		{
			// Error
		}
		if (cchBuffer > longName.count)
		{
			memcpy(lpszLongPath, longName.data, (longName.count+1)*sizeof(wchar_t));
			res = longName.count;
		}
		else
		{
			res = longName.count + 1;
		}

		SetLastError(errorCode);

		DEBUG_LOG_DETOURED(L"GetLongPathNameW", L"%ls", lpszShortPath);
	}
	else
	{
		DEBUG_LOG_TRUE(L"GetLongPathNameW", L"(Detour disabled under this call to handle ~) (%ls)", lpszShortPath);

		SuppressDetourScope _;
		res = True_GetLongPathNameW(lpszShortPath, lpszLongPath, cchBuffer);
		if (res == 0)
			return res;
		newLongPath = g_memoryBlock.Strdup(lpszLongPath);
	}

	wchar_t* newShortPath = g_memoryBlock.Strdup(lpszShortPath);
	g_longPathNameCache.insert({ newShortPath, newLongPath });
	return res;
}

LPWSTR Detoured_GetCommandLineW()
{
	DETOURED_CALL(GetCommandLineW);
	LPWSTR str = True_GetCommandLineW();
	DEBUG_LOG_TRUE(L"GetCommandLineW", L"");// str);
	return str;
}

VOID Detoured_GetSystemInfo(LPSYSTEM_INFO lpSystemInfo)
{
	DETOURED_CALL(GetSystemInfo);
	DEBUG_LOG_TRUE(L"GetSystemInfo", L"");
	True_GetSystemInfo(lpSystemInfo);

	// This is a limitation in wine. There is a MAXIMUM_PROCESSORS preventing it from creating all registry keys under HARDWARE\DESCRIPTION\System\CentralProcessor
	if (g_isRunningWine && lpSystemInfo->dwNumberOfProcessors > 64)
		lpSystemInfo->dwNumberOfProcessors = 64;
}

DWORD Detoured_GetCurrentDirectoryW(DWORD nBufferLength, LPWSTR lpBuffer)
{
	DETOURED_CALL(GetCurrentDirectoryW);
	u64 length = g_virtualWorkingDir.count - 1; // Skip last slash
	SetLastError(ERROR_SUCCESS);
	if (lpBuffer == nullptr || nBufferLength < length + 1)
	{
		DEBUG_LOG_DETOURED(L"GetCurrentDirectoryW", L"(buffer too small: %lu) -> %llu", nBufferLength, length + 1);
		return DWORD(length + 1);
	}
	memcpy(lpBuffer, g_virtualWorkingDir.data, length * 2);
	lpBuffer[length] = 0; // Skip last slash
	DEBUG_LOG_DETOURED(L"GetCurrentDirectoryW", L"(%ls)", lpBuffer);
	return DWORD(length);

	//DEBUG_LOG_TRUE(L"GetCurrentDirectoryW", L"");
	//auto res = True_GetCurrentDirectoryW(nBufferLength, lpBuffer);
	//return res;
}

DWORD Detoured_GetCurrentDirectoryA(DWORD nBufferLength, LPSTR lpBuffer)
{
	DETOURED_CALL(GetCurrentDirectoryA);
	u64 length = g_virtualWorkingDir.count - 1; // Skip last slash
	SetLastError(ERROR_SUCCESS);
	if (lpBuffer == nullptr || nBufferLength < length + 1)
	{
		DEBUG_LOG_DETOURED(L"GetCurrentDirectoryA", L"(buffer too small: %lu)", nBufferLength);
		return DWORD(length + 1);
	}
	size_t res;
	errno_t err = wcstombs_s(&res, lpBuffer, nBufferLength, g_virtualWorkingDir.data, length);
	if (err)
		UBA_ASSERTF(false, L"wcstombs_s failed for string '%s' with error code: %u", g_virtualWorkingDir.data, err);
	DEBUG_LOG_DETOURED(L"GetCurrentDirectoryA", L"(%hs)", lpBuffer);
	return DWORD(length);
}

void Shared_SetCurrentDirectory(const wchar_t* workingDirBuffer)
{
	u32 charLen = 0;
	wchar_t temp[256];
	FixPath2(workingDirBuffer, ~0u, g_virtualWorkingDir.data, g_virtualWorkingDir.count, temp, sizeof_array(temp), &charLen);
	g_virtualWorkingDir.Clear().Append(temp).Append('\\');
}


BOOL Detoured_SetCurrentDirectoryW(LPCWSTR lpPathName)
{
	DETOURED_CALL(SetCurrentDirectoryW);

	Shared_SetCurrentDirectory(lpPathName);

	if (g_runningRemote)
	{
		DEBUG_LOG_DETOURED(L"SetCurrentDirectoryW", L"%ls -> true (%s)", lpPathName, g_virtualWorkingDir.data);
		return true;
	}

	DEBUG_LOG_TRUE(L"SetCurrentDirectoryW", L"%ls", lpPathName);
	return True_SetCurrentDirectoryW(lpPathName);
}

BOOL Detoured_DuplicateHandle(HANDLE hSourceProcessHandle, HANDLE hSourceHandle, HANDLE hTargetProcessHandle, LPHANDLE lpTargetHandle, DWORD dwDesiredAccess, BOOL bInheritHandle, DWORD dwOptions)
{
	if (isDetouredHandle(hTargetProcessHandle))
		hTargetProcessHandle = asDetouredHandle(hTargetProcessHandle).trueHandle;

	DETOURED_CALL(DuplicateHandle);
	if (hSourceHandle == PseudoHandle || !isDetouredHandle(hSourceHandle))
	{
		auto res = True_DuplicateHandle(hSourceProcessHandle, hSourceHandle, hTargetProcessHandle, lpTargetHandle, dwDesiredAccess, bInheritHandle, dwOptions);
		DEBUG_LOG_TRUE(L"DuplicateHandle",L"%llu (%llu) to %llu (%llu) -> %ls", uintptr_t(hSourceHandle), uintptr_t(hSourceProcessHandle), lpTargetHandle ? uintptr_t(*lpTargetHandle) : 0ull, uintptr_t(hTargetProcessHandle), ToString(res));
		return res;
	}
	if (hSourceProcessHandle != hTargetProcessHandle)
	{
		// This is not good. Means that the hosting process likely has hit the detour handle range. Large cooks can cause this
		//UBA_ASSERTF(false, TC("Duplicating detoured handles from one process to the other not supported..(SourceProcess: %llu SourceHandle: %llu)"), u64(hSourceProcessHandle), u64(hSourceHandle));
		auto res = True_DuplicateHandle(hSourceProcessHandle, hSourceHandle, hTargetProcessHandle, lpTargetHandle, dwDesiredAccess, bInheritHandle, dwOptions);
		DEBUG_LOG_TRUE(L"DuplicateHandle",L"%llu (%llu) to %llu (%llu) -> %ls", uintptr_t(hSourceHandle), uintptr_t(hSourceProcessHandle), lpTargetHandle ? uintptr_t(*lpTargetHandle) : 0ull, uintptr_t(hTargetProcessHandle), ToString(res));
		return res;
	}

	UBA_ASSERT(!(dwOptions & DUPLICATE_CLOSE_SOURCE));
	auto& dh = asDetouredHandle(hSourceHandle);

	HANDLE trueHandle = dh.trueHandle;
	HANDLE targetHandle = INVALID_HANDLE_VALUE;

	BOOL res = TRUE;
	if (trueHandle != INVALID_HANDLE_VALUE)
		res = True_DuplicateHandle(hSourceProcessHandle, trueHandle, hTargetProcessHandle, &targetHandle, dwDesiredAccess, bInheritHandle, dwOptions);
	else
		SetLastError(ERROR_SUCCESS);

	auto newDh = new DetouredHandle(dh.type);
	newDh->trueHandle = targetHandle;
	newDh->dirTableOffset = dh.dirTableOffset;
	newDh->fileObject = dh.fileObject;
	if (FileObject* fo = dh.fileObject)
	{
		//UBA_ASSERT(!fo->fileInfo->isFileMap);
		InterlockedIncrement(&fo->refCount);
	}
	*lpTargetHandle = makeDetouredHandle(newDh);
	DEBUG_LOG_DETOURED(L"DuplicateHandle", L"%llu (%llu) to %llu/%llu (%llu) -> %ls", u64(hSourceHandle), u64(hSourceProcessHandle), u64(*lpTargetHandle), u64(targetHandle), u64(hTargetProcessHandle), ToString(res));
	return res;
}

BOOL Detoured_CreateDirectoryW(LPCWSTR lpPathName, LPSECURITY_ATTRIBUTES lpSecurityAttributes)
{
	DETOURED_CALL(CreateDirectoryW);

	StringBuffer<> dirPath;
	if (!FixPath(dirPath, lpPathName))
	{
		SetLastError(ERROR_INVALID_NAME);
		DEBUG_LOG_DETOURED(L"CreateDirectoryW", L"%s -> %s (ERROR_INVALID_NAME)", lpPathName, ToString(false));
		return false;
	}

	if (IsTrackingInput() && dirPath.EndsWith(TCV("__pycache__"))) // If we are populating cache we need to make sure not to use pycache
	{
		SetLastError(ERROR_ACCESS_DENIED);
		DEBUG_LOG_DETOURED(L"CreateDirectoryW", L"(NOPYCACHE) %s -> %s (ERROR_ACCESS_DENIED)", dirPath.data, ToString(false));
		return false;
	}

	DevirtualizePath(dirPath);
	
	bool checkDir = false;
	if (dirPath.count == 3 && dirPath[2] == PathSeparator) // Root
	{
		dirPath.Resize(2);
		checkDir = true;
	}

	auto runError = [&](u32 error, const tchar* errorStr)
		{
			SetLastError(error);
			DEBUG_LOG_DETOURED(L"CreateDirectoryW", L"%s -> %s (%s)", dirPath.data, ToString(false), errorStr);
			return false;
		};

	StringKey dirPathKey = ToStringKeyLower(dirPath);

	DirectoryTableEntryInformation dirEntryInfo;
	if (Rpc_GetEntryInformation(dirEntryInfo, dirPathKey, dirPath, checkDir))
		return runError(ERROR_ALREADY_EXISTS, TC("ERROR_ALREADY_EXISTS"));

	const tchar* lastSeparator = dirPath.Last(PathSeparator);
	if (!lastSeparator)
		return runError(ERROR_PATH_NOT_FOUND, TC("ERROR_PATH_NOT_FOUND???"));

	StringView parentDirPath(dirPath.data, u32(lastSeparator - dirPath.data));
	DirectoryTableEntryInformation parentDirEntryInfo;
	if (!Rpc_GetEntryInformation(parentDirEntryInfo, ToStringKeyLower(parentDirPath), parentDirPath, false))
		return runError(ERROR_PATH_NOT_FOUND, TC("ERROR_PATH_NOT_FOUND"));

	// This should be a guaranteed create
	RPC_MESSAGE(CreateDirectory, createFile)
	writer.WriteStringKey(dirPathKey);
	writer.WriteString(dirPath);
	BinaryReader reader = writer.Flush();
	DirTableSize directoryTableSize = FromU64(reader.ReadU64());
	RPC_MESSAGE_END

	g_directoryTable->ParseDirectoryTable(directoryTableSize);

	SetLastError(ERROR_SUCCESS);
	DEBUG_LOG_DETOURED(L"CreateDirectoryW", L"%s -> %s", dirPath.data, ToString(true));
	return true;
}

#if DETOURED_INCLUDE_DEBUG
thread_local bool t_calledRemoveDirectoryW;
#endif

BOOL Detoured_RemoveDirectoryW(LPCWSTR lpPathName)
{
	DETOURED_CALL(RemoveDirectoryW);

	#if DETOURED_INCLUDE_DEBUG
	t_calledRemoveDirectoryW = true;
	#endif

	StringBuffer<> dirPath;
	FixPath(dirPath, lpPathName);
	DevirtualizePath(dirPath);

	StringKey dirPathKey = ToStringKeyLower(dirPath);

	DirectoryTableEntryInformation dirEntryInfo;
	if (!Rpc_GetEntryInformation(dirEntryInfo, dirPathKey, dirPath, true))
	{
		SetLastError(ERROR_FILE_NOT_FOUND);
		DEBUG_LOG_DETOURED(L"RemoveDirectoryW", L"%s -> %s (ERROR_FILE_NOT_FOUND)", dirPath.data, ToString(false));
		return false;
	}

	if (!IsDirectory(dirEntryInfo.attributes))
	{
		SetLastError(ERROR_DIRECTORY);
		DEBUG_LOG_DETOURED(L"RemoveDirectoryW", L"%s -> %s (ERROR_DIRECTORY)", dirPath.data, ToString(false));
		return false;
	}

	auto& dirTable = *g_directoryTable;

	SCOPED_READ_LOCK(dirTable.m_lookupLock, lookLock);
	auto findIt = dirTable.m_lookup.find(dirPathKey);
	bool foundDir = findIt != dirTable.m_lookup.end();
	lookLock.Leave();
	if (foundDir)
	{
		auto& dir = findIt->second;
		SCOPED_WRITE_LOCK(dir.lock, dirLock);
		if (!dirTable.IsDirectoryUpToDate(dir))
		{
			StringBuffer<> dirPathLower(dirPath);
			dirPathLower.MakeLower();
			DirHash hash(dirPathLower);
			dirTable.PopulateDirectoryNoLock(hash.open, dir);
		}
		for (auto& f : dir.files)
		{
			if (dirTable.GetAttributes(f.second) == 0)
				continue;

			SetLastError(ERROR_DIR_NOT_EMPTY);
			DEBUG_LOG_DETOURED(L"RemoveDirectoryW", L"%s -> %s (ERROR_DIR_NOT_EMPTY)", dirPath.data, ToString(false));
			return false;
		}
	}

	RPC_MESSAGE(RemoveDirectory, deleteFile)
	writer.WriteStringKey(dirPathKey);
	writer.WriteString(dirPath);
	BinaryReader reader = writer.Flush();
	DirTableSize directoryTableSize = FromU64(reader.ReadU64());
	RPC_MESSAGE_END;

	dirTable.ParseDirectoryTable(directoryTableSize);

	SetLastError(ERROR_SUCCESS);
	DEBUG_LOG_DETOURED(L"RemoveDirectoryW", L"%s -> %s", dirPath.data, ToString(true));
	return true;
}

BOOL Detoured_LockFile(HANDLE hFile, DWORD dwFileOffsetLow, DWORD dwFileOffsetHigh, DWORD nNumberOfBytesToLockLow, DWORD nNumberOfBytesToLockHigh)
{
	DETOURED_CALL(LockFile);
	HANDLE trueHandle = hFile;
	if (isDetouredHandle(hFile))
	{
		auto& dh = asDetouredHandle(hFile);
		trueHandle = dh.trueHandle;
		if (trueHandle == INVALID_HANDLE_VALUE) // Used by metallib compiler.. not sure why. But this file is only in memory so no point to lock it (I think?)
			return true;
	}
	DEBUG_LOG_TRUE(L"LockFile", L"%llu (%ls)", uintptr_t(hFile), HandleToName(hFile));
	return True_LockFile(trueHandle, dwFileOffsetLow, dwFileOffsetHigh, nNumberOfBytesToLockLow, nNumberOfBytesToLockHigh);
}

BOOL Detoured_LockFileEx(HANDLE hFile, DWORD dwFlags, DWORD dwReserved, DWORD nNumberOfBytesToLockLow, DWORD nNumberOfBytesToLockHigh, LPOVERLAPPED lpOverlapped)
{
	DETOURED_CALL(LockFileEx);
	HANDLE trueHandle = hFile;
	if (isDetouredHandle(hFile))
	{
		auto& dh = asDetouredHandle(hFile);
		trueHandle = dh.trueHandle;
		if (trueHandle == INVALID_HANDLE_VALUE)
			return true;
	}
	DEBUG_LOG_TRUE(L"LockFileEx", L"%llu %ls", uintptr_t(hFile), HandleToName(hFile));
	return True_LockFileEx(trueHandle, dwFlags, dwReserved, nNumberOfBytesToLockLow, nNumberOfBytesToLockHigh, lpOverlapped);
}

BOOL Detoured_UnlockFile(HANDLE hFile, DWORD dwFileOffsetLow, DWORD dwFileOffsetHigh, DWORD nNumberOfBytesToUnlockLow, DWORD nNumberOfBytesToUnlockHigh)
{
	DETOURED_CALL(UnlockFile);
	HANDLE trueHandle = hFile;
	if (isDetouredHandle(hFile))
	{
		auto& dh = asDetouredHandle(hFile);
		trueHandle = dh.trueHandle;
		UBA_ASSERT(trueHandle != INVALID_HANDLE_VALUE);
	}
	BOOL res = True_UnlockFile(trueHandle, dwFileOffsetLow, dwFileOffsetHigh, nNumberOfBytesToUnlockLow, nNumberOfBytesToUnlockHigh);
	DEBUG_LOG_TRUE(L"UnlockFile", L"%llu (%ls) -> %s", uintptr_t(hFile), HandleToName(hFile), ToString(res));
	return res;
}

BOOL Detoured_UnlockFileEx(HANDLE hFile, DWORD dwReserved, DWORD nNumberOfBytesToUnlockLow, DWORD nNumberOfBytesToUnlockHigh, LPOVERLAPPED lpOverlapped)
{
	DETOURED_CALL(UnlockFileEx);
	HANDLE trueHandle = hFile;
	if (isDetouredHandle(hFile))
	{
		auto& dh = asDetouredHandle(hFile);
		trueHandle = dh.trueHandle;
		UBA_ASSERT(trueHandle != INVALID_HANDLE_VALUE);
	}
	BOOL res = True_UnlockFileEx(trueHandle, dwReserved, nNumberOfBytesToUnlockLow, nNumberOfBytesToUnlockHigh, lpOverlapped);
	DEBUG_LOG_TRUE(L"UnlockFileEx", L"%llu (%ls) -> %s", uintptr_t(hFile), HandleToName(hFile), ToString(res));
	return res;
}

BOOL Detoured_WriteConsoleA(HANDLE hConsoleOutput, const VOID* lpBuffer, DWORD nNumberOfCharsToWrite, LPDWORD lpNumberOfCharsWritten, LPVOID lpReserved)
{
	DETOURED_CALL(WriteConsoleA);
	//DEBUG_LOG_DETOURED(L"WriteConsoleA", L"(%hs)", (char*)lpBuffer); // Too much spam
	Shared_WriteConsole((const char*)lpBuffer, nNumberOfCharsToWrite, false);
	if (lpNumberOfCharsWritten)
		*lpNumberOfCharsWritten = nNumberOfCharsToWrite;
	return TRUE;//True_WriteConsoleA(hConsoleOutput, lpBuffer, nNumberOfCharsToWrite, lpNumberOfCharsWritten, lpReserved);
}

BOOL Detoured_WriteConsoleW(HANDLE hConsoleOutput, const VOID* lpBuffer, DWORD nNumberOfCharsToWrite, LPDWORD lpNumberOfCharsWritten, LPVOID lpReserved)
{
	DETOURED_CALL(WriteConsoleW);
	//DEBUG_LOG_DETOURED(L"WriteConsoleW", L"(%s)", (const wchar_t*)lpBuffer); // Too much spam
	Shared_WriteConsole((const wchar_t*)lpBuffer, nNumberOfCharsToWrite, false);
	if (lpNumberOfCharsWritten)
		*lpNumberOfCharsWritten = nNumberOfCharsToWrite;
	return TRUE;//True_WriteConsoleW(hConsoleOutput, lpBuffer, nNumberOfCharsToWrite, lpNumberOfCharsWritten, lpReserved);
}

BOOL Detoured_ReadConsoleW(HANDLE hConsoleInput, LPVOID lpBuffer, DWORD nNumberOfCharsToRead, LPDWORD lpNumberOfCharsRead, PCONSOLE_READCONSOLE_CONTROL pInputControl)
{
	DETOURED_CALL(ReadConsoleW);
	//DEBUG_LOG_DETOURED(L"ReadConsoleW", L"(%llu)", u64(hConsoleInput)); // Too much spam
	HANDLE trueHandle = hConsoleInput;
	if (isDetouredHandle(hConsoleInput))
	{
		auto& dh = asDetouredHandle(hConsoleInput);
		trueHandle = dh.trueHandle;
		if (dh.type == HandleType_StdIn)
			trueHandle = True_GetStdHandle(STD_INPUT_HANDLE);
		UBA_ASSERTF(trueHandle != INVALID_HANDLE_VALUE, TC("ReadConsoleW got handle that can't be resolved %llu (%s)"), u64(hConsoleInput), HandleToName(hConsoleInput));
	}
	return True_ReadConsoleW(trueHandle, lpBuffer, nNumberOfCharsToRead, lpNumberOfCharsRead, pInputControl);
}

UINT Detoured_GetDriveTypeW(LPCWSTR lpRootPathName)
{
	DETOURED_CALL(GetDriveTypeW);
	if (g_runningRemote || IsVfsEnabled())
	{
		DEBUG_LOG_DETOURED(L"GetDriveType", L"%ls -> DRIVE_FIXED", lpRootPathName);
		return DRIVE_FIXED;
	}
	DEBUG_LOG_TRUE(L"GetDriveType", L"%ls", lpRootPathName);
	SuppressCreateFileDetourScope s; // Convenient since it will call NtQueryVolumeInformationFile
	return True_GetDriveTypeW(lpRootPathName);
}

BOOL Detoured_GetDiskFreeSpaceExW(LPCWSTR lpDirectoryName, PULARGE_INTEGER lpFreeBytesAvailableToCaller, PULARGE_INTEGER lpTotalNumberOfBytes, PULARGE_INTEGER lpTotalNumberOfFreeBytes)
{
	DETOURED_CALL(GetDiskFreeSpaceExW);
	StringBuffer<MaxPath> path;
	if (lpDirectoryName)
	{
		if (g_runningRemote)
		{
			UBA_ASSERT(lpDirectoryName[1] == ':');
			if (ToLower(lpDirectoryName[0]) == ToLower(g_virtualWorkingDir[0]))
			{
				if (lpDirectoryName[3] == 0)
					path.Append(g_exeDir.data, 3);
				else
					path.Append(g_exeDir);
				lpDirectoryName = path.data;
			}
		}
		else
		{
			FixPath(path, lpDirectoryName);
			DevirtualizePath(path);
			lpDirectoryName = path.data;
		}
	}

	DEBUG_LOG_TRUE(L"GetDiskFreeSpaceExW", L"%ls", lpDirectoryName);
	SuppressCreateFileDetourScope s; // Convenient since it will call NtQueryVolumeInformationFile
	return True_GetDiskFreeSpaceExW(lpDirectoryName, lpFreeBytesAvailableToCaller, lpTotalNumberOfBytes, lpTotalNumberOfFreeBytes);
}


BOOL Detoured_GetVolumeInformationByHandleW(HANDLE hFile, LPWSTR lpVolumeNameBuffer, DWORD nVolumeNameSize, LPDWORD lpVolumeSerialNumber, LPDWORD lpMaximumComponentLength, LPDWORD lpFileSystemFlags, LPWSTR lpFileSystemNameBuffer, DWORD nFileSystemNameSize)
{
	DETOURED_CALL(GetVolumeInformationByHandleW);
	HANDLE trueHandle = hFile;

	DirTableOffset entryOffset;

	if (isDetouredHandle(hFile))
	{
		DetouredHandle& dh = asDetouredHandle(hFile);
		trueHandle = dh.trueHandle;
		entryOffset = dh.dirTableOffset;
		UBA_ASSERT(IsValidEntry(entryOffset) || trueHandle != INVALID_HANDLE_VALUE);
	}
	else if (isListDirectoryHandle(hFile))
	{
		auto& listHandle = asListDirectoryHandle(hFile);
		entryOffset = listHandle.dirTableOffset;
		trueHandle = INVALID_HANDLE_VALUE;
	}

	if (IsValidEntry(entryOffset))
	{
		UBA_ASSERT(!lpVolumeNameBuffer);
		UBA_ASSERT(!lpMaximumComponentLength);
		if (lpFileSystemFlags) // esbuild.exe query this
		{
			// Standard nfts flags. It might be that we want to comment some of these outs because UBA (especially remotely) don't want to support these
			*lpFileSystemFlags = 0
			| FILE_CASE_SENSITIVE_SEARCH
			| FILE_CASE_PRESERVED_NAMES
			| FILE_UNICODE_ON_DISK
			| FILE_PERSISTENT_ACLS
			| FILE_FILE_COMPRESSION
			| FILE_VOLUME_QUOTAS
			| FILE_SUPPORTS_SPARSE_FILES
			| FILE_SUPPORTS_REPARSE_POINTS
			| FILE_SUPPORTS_OBJECT_IDS
			| FILE_SUPPORTS_ENCRYPTION
			| FILE_NAMED_STREAMS
			| FILE_SUPPORTS_HARD_LINKS
			| FILE_SUPPORTS_EXTENDED_ATTRIBUTES
			| FILE_SUPPORTS_OPEN_BY_FILE_ID
			| FILE_SUPPORTS_USN_JOURNAL
			;
		}
		DirectoryTable::EntryInformation entryInfo;
		g_directoryTable->GetEntryInformation(entryInfo, entryOffset);
		if (lpVolumeSerialNumber)
			*lpVolumeSerialNumber = entryInfo.volumeSerial;
		if (lpFileSystemNameBuffer)
		{
			UBA_ASSERT(nFileSystemNameSize > 5);
			wcscpy_s(lpFileSystemNameBuffer, nFileSystemNameSize, L"NTFS"); // TODO: Not everyone has NTFS?
		}
		SetLastError(ERROR_SUCCESS);
		DEBUG_LOG_DETOURED(L"GetVolumeInformationByHandleW", L"%llu (Serial: %u) (%ls) -> Success", uintptr_t(hFile), entryInfo.volumeSerial, HandleToName(hFile));
		return true;
	}

	DEBUG_LOG_TRUE(L"GetVolumeInformationByHandleW", L"%llu (%ls)", u64(hFile), HandleToName(hFile));
	return True_GetVolumeInformationByHandleW(trueHandle, lpVolumeNameBuffer, nVolumeNameSize, lpVolumeSerialNumber, lpMaximumComponentLength, lpFileSystemFlags, lpFileSystemNameBuffer, nFileSystemNameSize);
}

BOOL Detoured_GetVolumeInformationW(LPCWSTR lpRootPathName, LPWSTR lpVolumeNameBuffer, DWORD nVolumeNameSize, LPDWORD lpVolumeSerialNumber, LPDWORD lpMaximumComponentLength, LPDWORD lpFileSystemFlags, LPWSTR lpFileSystemNameBuffer, DWORD nFileSystemNameSize)
{
	DETOURED_CALL(GetVolumeInformationW);

	tchar buffer[128];
	if (IsVfsEnabled() && lpRootPathName)
	{
		UBA_ASSERT(lpRootPathName[1] == ':');
		TStrcpy_s(buffer, sizeof_array(buffer), lpRootPathName);
		buffer[0] = DevirtualizeDrive(buffer[0]);
		lpRootPathName = buffer;
	}

	if (g_runningRemote)
	{
		if (lpVolumeSerialNumber)
			*lpVolumeSerialNumber = lpRootPathName[0]; // Let's see if this works, LOL

		if (lpMaximumComponentLength)
			*lpMaximumComponentLength =  255; // TODO: Need to fix this

		//UBA_ASSERT(!lpVolumeNameBuffer);
		//UBA_ASSERT(!lpMaximumComponentLength);
		UBA_ASSERT(!lpFileSystemFlags);

		if (nFileSystemNameSize)
			wcscpy_s(lpFileSystemNameBuffer, nFileSystemNameSize, L"NTFS"); // TODO: Not everyone has NTFS?
		SetLastError(ERROR_SUCCESS);
		DEBUG_LOG_DETOURED(L"GetVolumeInformationW", L"%ls", lpRootPathName);
		return true;
	}

	DEBUG_LOG_TRUE(L"GetVolumeInformationW", L"%ls", lpRootPathName);
	SuppressCreateFileDetourScope s;
	auto res = True_GetVolumeInformationW(lpRootPathName, lpVolumeNameBuffer, nVolumeNameSize, lpVolumeSerialNumber, lpMaximumComponentLength, lpFileSystemFlags, lpFileSystemNameBuffer, nFileSystemNameSize);
	return res;
}

LPVOID Detoured_VirtualAlloc(LPVOID lpAddress, SIZE_T dwSize, DWORD flAllocationType, DWORD flProtect)
{
	DETOURED_CALL(VirtualAlloc);
	// Special cl.exe handling
	if (lpAddress != nullptr && g_clExeBaseReservedMemory != nullptr && lpAddress >= g_clExeBaseReservedMemory && uintptr_t(lpAddress) < uintptr_t(g_clExeBaseReservedMemory) + g_clExeBaseAddressSize)
	{
		DEBUG_LOG(L"VirtualAlloc releasing cl.exe reserved memory at 0x%llx", uintptr_t(lpAddress));
		VirtualFree(g_clExeBaseReservedMemory, 0, MEM_RELEASE);
		g_clExeBaseReservedMemory = nullptr;
	}


	u32 counter = 0;
	do
	{
		void* res = True_VirtualAlloc(lpAddress, dwSize, flAllocationType, flProtect);
		if (res)
			return res;
		if (!(flAllocationType & MEM_COMMIT))
			return res;
		DWORD error = GetLastError();
		if (error != ERROR_NOT_ENOUGH_MEMORY && error != ERROR_COMMITMENT_LIMIT)
			return res;
		StringBuffer<128> reason;
		reason.Append(TCV("VirtualAlloc ")).AppendValue(dwSize);
		Rpc_AllocFailed(reason.data, error);
		++counter;

	} while (counter <= 10);

	return nullptr;
}

PVOID Detoured_AddVectoredExceptionHandler(ULONG First, PVECTORED_EXCEPTION_HANDLER Handler)
{
	DEBUG_LOG_TRUE(L"AddVectoredExceptionHandler", L"");
	return True_AddVectoredExceptionHandler(0, Handler); // Must make sure it ends up after the lazy-cow exception handler
}

BOOL Detoured_PathSearchAndQualifyW(LPCWSTR pszPath, LPWSTR pszBuf, UINT cchBuf)
{
	FixPath2(pszPath, TStrlen(pszPath), g_virtualWorkingDir.data, g_virtualWorkingDir.count, pszBuf, cchBuf, nullptr);
	DEBUG_LOG_DETOURED(L"PathSearchAndQualifyW", L"(%ls) -> %s (%s)", pszPath, ToString(true), pszBuf);
	return true;
	//BOOL res = True_PathSearchAndQualifyW(pszPath, pszBuf, cchBuf);
	//DEBUG_LOG_TRUE(L"PathSearchAndQualifyW", L"(%ls) -> %s (%s)", pszPath, ToString(res), pszBuf);
	//return res;
}

BOOL Detoured_InitializeSynchronizationBarrier(LPSYNCHRONIZATION_BARRIER lpBarrier, LONG lTotalThreads, LONG lSpinCount)
{
	Rpc_WriteLogf(TC("You need a newer version of wine to be able to call InitializeSynchronizationBarrier (%s)"), g_virtualApplication.data);
	return false;
}

BOOL Detoured_DeleteSynchronizationBarrier(LPSYNCHRONIZATION_BARRIER lpBarrier)
{
	return false;
}

HANDLE Detoured_CreateToolhelp32Snapshot(DWORD dwFlags, DWORD th32ProcessID)
{
	DETOURED_CALL(CreateToolhelp32Snapshot);
	DEBUG_LOG_DETOURED(L"CreateToolhelp32Snapshot", L"(Disabled) %lu %lu -> INVALID_HANDLE_VALUE", dwFlags, th32ProcessID);
	SetLastError(ERROR_ACCESS_DENIED);
	return INVALID_HANDLE_VALUE;
	//return True_CreateToolhelp32Snapshot(dwFlags, th32ProcessID);
}

BOOL Detoured_FlushFileBuffers(HANDLE hFile)
{
	DETOURED_CALL(FlushFileBuffers);

	HANDLE trueHandle = hFile;
	if (isDetouredHandle(hFile))
	{
		DetouredHandle& dh = asDetouredHandle(hFile);
		FileInfo& fi = *dh.fileObject->fileInfo;
		if (fi.memoryFile)
		{
			DEBUG_LOG_DETOURED(L"FlushFileBuffers", L"%llu (%ls) -> Success", uintptr_t(hFile), HandleToName(hFile));
			SetLastError(ERROR_SUCCESS);
			return true;
		}
		trueHandle = dh.trueHandle;
		UBA_ASSERT(trueHandle != INVALID_HANDLE_VALUE);
	}

	BOOL res = True_FlushFileBuffers(trueHandle);
	DEBUG_LOG_TRUE(L"FlushFileBuffers", L"%llu (%ls) -> %ls", uintptr_t(hFile), HandleToName(hFile), ToString(res));
	return res;
}

BOOL Detoured_SetFileTime(HANDLE hFile, const FILETIME* lpCreationTime, const FILETIME* lpLastAccessTime, const FILETIME* lpLastWriteTime)
{
	DETOURED_CALL(SetFileTime);
	HANDLE trueHandle = hFile;
	if (isDetouredHandle(hFile))
	{
		if (!lpCreationTime && !lpLastWriteTime)
		{
			DEBUG_LOG_DETOURED(L"SetFileTime", L"%llu IGNORE (%ls)", uintptr_t(hFile), HandleToName(hFile));
			return TRUE;
		}
		DetouredHandle& dh = asDetouredHandle(hFile);
		trueHandle = dh.trueHandle;
		if (trueHandle == INVALID_HANDLE_VALUE)
		{
			// TODO: THIS NEED TO BE TRANSFERRED TO HOST
			Rpc_WriteLogf(L"SetFileTime called for %s. Must implement", dh.fileObject->fileInfo->originalName);
			DEBUG_LOG_DETOURED(L"SetFileTime", L"%llu MUSTIMPLEMENT (%ls)", uintptr_t(hFile), HandleToName(hFile));
			return true;
		}
	}
	DEBUG_LOG_TRUE(L"SetFileTime", L"%llu (%ls)", uintptr_t(hFile), HandleToName(hFile));
	return True_SetFileTime(trueHandle, lpCreationTime, lpLastAccessTime, lpLastWriteTime);
}

DWORD Detoured_GetFileType(HANDLE hFile)
{
	DETOURED_CALL(GetFileType);
	if (isDetouredHandle(hFile))
	{
		DetouredHandle& dh = asDetouredHandle(hFile);
		SetLastError(ERROR_SUCCESS);
		if (dh.type >= HandleType_StdErr)
		{
			DEBUG_LOG_DETOURED(L"GetFileType", L"%llu (%ls) -> FILE_TYPE_CHAR", uintptr_t(hFile), HandleToName(hFile));
			return FILE_TYPE_CHAR;
		}
		UBA_ASSERTF(dh.type == HandleType_File, L"HandleType: %u", dh.type);
		DEBUG_LOG_DETOURED(L"GetFileType", L"%llu (%ls) -> FILE_TYPE_DISK", uintptr_t(hFile), HandleToName(hFile));
		return FILE_TYPE_DISK;
	}
	if (isListDirectoryHandle(hFile))
	{
		DEBUG_LOG_DETOURED(L"GetFileType", L"%llu (%ls) -> FILE_TYPE_DISK", uintptr_t(hFile), HandleToName(hFile));
		SetLastError(ERROR_SUCCESS);
		return FILE_TYPE_DISK;
	}
	else if (hFile == PseudoHandle)
	{
		DEBUG_LOG_DETOURED(L"GetFileType", L"PseudoHandle -> FILE_TYPE_CHAR");
		SetLastError(ERROR_SUCCESS);
		return FILE_TYPE_CHAR;
	}

	DEBUG_LOG_TRUE(L"GetFileType", L"%llu (%ls)", uintptr_t(hFile), HandleToName(hFile));
	return True_GetFileType(hFile); // Calling NtQueryVolumeInformationFile FileFsDeviceInformation
}

BOOL Shared_GetFileAttributesExW(const tchar* caller, StringView fileName, GET_FILEEX_INFO_LEVELS fInfoLevelId, LPVOID lpFileInformation, LPCWSTR originalName)
{
	FileAttributes attr;
	Shared_GetFileAttributes(attr, fileName);
	SetLastError(attr.lastError);
	memcpy(lpFileInformation, &attr.data, sizeof(WIN32_FILE_ATTRIBUTE_DATA));
	DEBUG_LOG_DETOURED(caller, L"(%ls) -> %s", originalName, (attr.exists ? L"Exists" : L"NotFound"));
	return attr.exists;
}

BOOL Detoured_GetFileAttributesExW(LPCWSTR lpFileName, GET_FILEEX_INFO_LEVELS fInfoLevelId, LPVOID lpFileInformation)
{
	DETOURED_CALL(GetFileAttributesExW);
	StringView dosName(ToView(lpFileName));
	if (!CanDetour(dosName) || dosName.Contains(L"::")) // Some weird .net path used by dotnet.exe ... ignore for now!
	{
		//UBA_ASSERTF(!g_runningRemote, TC("GetFileAttributesExW can't handle this path when running remote: %s"), lpFileName);
		DEBUG_LOG_TRUE(L"GetFileAttributesExW", L"(%ls)", lpFileName);
		TimerScope ts(g_kernelStats.getFileInfo);
		return True_GetFileAttributesExW(lpFileName, fInfoLevelId, lpFileInformation);
	}

	StringBuffer<MaxPath> fixedName;
	FixPath(fixedName, dosName);
	DevirtualizePath(fixedName);

	if (!g_rules->CanExist(fixedName))
	{
		DEBUG_LOG_TRUE(L"GetFileAttributesExW", L"(CanNotExist) (%ls) -> NotFound", lpFileName);
		SetLastError(ERROR_FILE_NOT_FOUND);
		return FALSE;
	}

	return Shared_GetFileAttributesExW(TC("GetFileAttributesExW"), fixedName, fInfoLevelId, lpFileInformation, lpFileName);
}

DWORD Detoured_GetFileAttributesW(LPCWSTR lpFileName)
{
	DETOURED_CALL(GetFileAttributesW);
	StringView dosName(ToView(lpFileName));
	if (!CanDetour(dosName))
	{
		TimerScope ts(g_kernelStats.getFileInfo);
		DWORD res = True_GetFileAttributesW(lpFileName);
		DEBUG_LOG_TRUE(L"GetFileAttributesW", L"(NODETOUR) (%ls) -> %lu", lpFileName, res);
		return res;
	}

	StringBuffer<MaxPath> fixedPath;
	if (!FixPath(fixedPath, dosName))
	{
		DEBUG_LOG_DETOURED(L"GetFileAttributesW", L"(BAD_PATH) (%ls) -> INVALID_FILE_ATTRIBUTES", lpFileName);
		SetLastError(ERROR_INVALID_NAME);
		return INVALID_FILE_ATTRIBUTES;
	}
	DevirtualizePath(fixedPath);

	WIN32_FILE_ATTRIBUTE_DATA data;
	if (!Shared_GetFileAttributesExW(TC("GetFileAttributesW"), fixedPath, GetFileExInfoStandard, &data, lpFileName))
		return INVALID_FILE_ATTRIBUTES;

	return data.dwFileAttributes;
}

BOOL Detoured_SetFileAttributesW(LPCWSTR lpFileName, DWORD dwFileAttributes)
{
	DETOURED_CALL(SetFileAttributesW);

	StringBuffer<> fixedFile;
	FixPath(fixedFile, lpFileName);
	DevirtualizePath(fixedFile);
	StringKey fileKey = ToStringKeyLower(fixedFile);

	SCOPED_WRITE_LOCK(g_mappedFileTable.m_lookupLock, lock);
	auto it = g_mappedFileTable.m_lookup.find(fileKey);
	bool isMapped = it != g_mappedFileTable.m_lookup.end();
	lock.Leave();

	if (isMapped || g_rules->IsThrowAway(fixedFile, g_runningRemote))
	{
		DEBUG_LOG_DETOURED(L"SetFileAttributesW", L"(IGNORED) (%ls) %lu -> %s", lpFileName, dwFileAttributes, ToString(true));
		SetLastError(ERROR_SUCCESS);
		return true;
	}

	// TODO: This should be tracked in overlay instead and applied afterwards
	if (g_runningRemote && !fixedFile.StartsWith(g_systemTemp))
	{
		StringKey fileNameKey = ToStringKeyLower(fixedFile);
		DirectoryTableEntryInformation info;
		if (Rpc_GetEntryInformation(info, fileNameKey, fixedFile, true))
		{
			UBA_ASSERTF(info.attributes == dwFileAttributes, TC("Not implemented - Remote process wants to SetFileAttributes on file %s"), fixedFile.data);
			DEBUG_LOG_DETOURED(L"SetFileAttributesW", L"(IGNORED_REMOTE) (%ls) %lu", lpFileName, dwFileAttributes);
			return true;
		}
		DEBUG_LOG_DETOURED(L"SetFileAttributesW", L"(NotFound) (%ls) %lu -> %s", lpFileName, dwFileAttributes, ToString(false));
		SetLastError(ERROR_FILE_NOT_FOUND);
		return FALSE;
	}

	DEBUG_LOG_TRUE(L"SetFileAttributesW", L"(SUPPRESS)(%ls) %lu", lpFileName, dwFileAttributes);
	SuppressDetourScope _;
	TimerScope ts(g_kernelStats.setFileInfo);
	return True_SetFileAttributesW(lpFileName, dwFileAttributes);
}

DWORD Detoured_GetLongPathNameW(LPCWSTR lpszShortPath, LPWSTR lpszLongPath, DWORD cchBuffer)
{
	DETOURED_CALL(GetLongPathNameW);

	if (!lpszShortPath)
		return Local_GetLongPathNameW(lpszShortPath, lpszLongPath, cchBuffer);

	const wchar_t* path = lpszShortPath;
	if (wcsncmp(path, L"\\\\?\\", 4) == 0)
		path += 4;

	bool foundQuestionMark = false;
	for (const wchar_t* i = path, *e = i + 4; *i && i!=e; ++i)
		foundQuestionMark |= *i == '?';

	// TODO: Add support for ~ and "\\?\"
	if (!foundQuestionMark)
	{
		StringBuffer<> fixedName;
		FixPath(fixedName, path);
		bool success;

		{
			DEBUG_LOG_DETOURED(L"GetLongPathNameW", L"(%ls)", path);
			StringBuffer<> realName(fixedName);
			DevirtualizePath(realName);
			WIN32_FILE_ATTRIBUTE_DATA data;
			success = Shared_GetFileAttributesExW(TC("GetLongPathNameW"), realName, GetFileExInfoStandard, &data, path);
		}

		DWORD res = 0;
		if (success)
		{
			res = fixedName.count;
			memcpy(lpszLongPath, fixedName.data, res * 2 + 2);
		}

#if UBA_DEBUG_VALIDATE
		if (g_validateFileAccess)
		{
			if (!wcschr(path, '~') && !wcschr(path, '?'))
			{
				wchar_t temp[MaxPath];
				UBA_ASSERT(cchBuffer <= sizeof_array(temp));
				SuppressDetourScope _;
				DWORD res2 = True_GetLongPathNameW(path, temp, cchBuffer); (void)res2;
				UBA_ASSERT(res == res2);
			}
		}
#endif

		if (!success)
			SetLastError(ERROR_FILE_NOT_FOUND);
		return res;
	}

	return Local_GetLongPathNameW(lpszShortPath, lpszLongPath, cchBuffer);
}

DWORD Detoured_GetFullPathNameW(LPCWSTR lpFileName, DWORD nBufferLength, LPWSTR lpBuffer, LPWSTR* lpFilePart)
{
	DETOURED_CALL(GetFullPathNameW);
	if (!lpFileName)
	{
		SetLastError(ERROR_INVALID_NAME);
		return 0;
	}
	StringBuffer<> fullPath;
	FixPath(fullPath, lpFileName);
	u64 requiredSize = fullPath.count + 1;
	if (nBufferLength < requiredSize)
		return DWORD(requiredSize);
	memcpy(lpBuffer, fullPath.data, requiredSize * 2);
	if (lpFilePart)
		*lpFilePart = wcsrchr(lpBuffer, '\\') + 1;
	auto res = DWORD(fullPath.count);
	DEBUG_LOG_DETOURED(L"GetFullPathNameW", L"%ls TO %ls -> %lu", lpFileName, fullPath.data, res);
	SetLastError(ERROR_SUCCESS);
	return res;
}

DWORD Detoured_GetFullPathNameA(LPCSTR lpFileName, DWORD nBufferLength, LPSTR lpBuffer, LPSTR* lpFilePart)
{
	// Is verified that this does NOT always call GetFullPathNameW
	DETOURED_CALL(GetFullPathNameA);
	if (!lpFileName)
	{
		DEBUG_LOG(L"GetFullPathNameA (NULL_NAME)");
		SetLastError(ERROR_INVALID_NAME);
		return 0;
	}
	StringBuffer<> temp;
	temp.Append(lpFileName);
	StringBuffer<> fullPath;
	FixPath(fullPath, temp.data);
	u64 requiredSize = fullPath.count + 1;
	if (nBufferLength < requiredSize)
	{
		DEBUG_LOG(L"GetFullPathNameA (BUFFER_SMALL) %S", lpFileName);
		return DWORD(requiredSize);
	}
	if (!fullPath.Parse(lpBuffer, requiredSize))
	{
		DEBUG_LOG(L"GetFullPathNameA failed to parse %s to ansi", fullPath.data);
	}

	if (lpFilePart)
		*lpFilePart = strrchr(lpBuffer, '\\') + 1;
	auto res = DWORD(fullPath.count);
	DEBUG_LOG_DETOURED(L"GetFullPathNameA", L"%ls TO %ls -> %lu", temp.data, fullPath.data, res);
	SetLastError(ERROR_SUCCESS);
	return res;
}

BOOL Detoured_GetVolumePathNameW(LPCWSTR lpszFileName, LPWSTR lpszVolumePathName, DWORD cchBufferLength)
{
	DETOURED_CALL(GetVolumePathNameW);
	//if (lpszFileName[1] == ':')
	//{
	//	memcpy(lpszVolumePathName, lpszFileName, 6);
	//	lpszVolumePathName[3] = 0;
	//	return TRUE;
	//}

	if (g_runningRemote || IsVfsEnabled())
	{
		UBA_ASSERT(cchBufferLength > 3);
		memcpy(lpszVolumePathName, g_virtualWorkingDir.data, 6);
		lpszVolumePathName[3] = 0;
		DEBUG_LOG_DETOURED(L"GetVolumePathNameW", L"(%ls) -> %s", lpszFileName, lpszVolumePathName);
		SetLastError(ERROR_SUCCESS);
		return TRUE;
	}

	DEBUG_LOG_TRUE(L"GetVolumePathNameW", L"(%ls)", lpszFileName);
	SuppressCreateFileDetourScope cfs;


	// TODO: This is causing TONS of kernel calls

	auto res = True_GetVolumePathNameW(lpszFileName, lpszVolumePathName, cchBufferLength);
	return res;
}

DWORD Shared_GetModuleFileNameInner(const tchar* func, HMODULE hModule, const StringView& moduleName, LPWSTR lpFilename, DWORD nSize)
{
	if (nSize <= moduleName.count)
	{
		if (nSize)
		{
			memcpy(lpFilename, moduleName.data, nSize * 2);
			lpFilename[nSize - 1] = 0;
		}
		SetLastError(ERROR_INSUFFICIENT_BUFFER);
		DEBUG_LOG_DETOURED(func, L"%llu  %lu INSUFFICIENT BUFFER (%ls) -> %u", uintptr_t(hModule), nSize, moduleName.data, moduleName.count + 1);
		return nSize;
	}

	memcpy(lpFilename, moduleName.data, (moduleName.count+1) * 2);
	DEBUG_LOG_DETOURED(func, L"%llu  %lu (%ls) -> %u", uintptr_t(hModule), nSize, lpFilename, moduleName.count);
	SetLastError(ERROR_SUCCESS);
	return moduleName.count;
}

DWORD Shared_GetModuleFileNameW(const tchar* func, HMODULE hModule, LPWSTR lpFilename, DWORD nSize)
{
	// If null we use the virtual application name
	if (hModule == NULL)
	{
		if (g_virtualApplication.EndsWith(L".bat"))
		{
			DWORD res = True_GetModuleFileNameW(hModule, lpFilename, nSize);
			if (res == nSize && nSize && g_isRunningWine && GetLastError() == ERROR_INSUFFICIENT_BUFFER) // Wine bug, not terminating
				lpFilename[nSize-1] = 0;
			DEBUG_LOG_TRUE(L"GetModuleFileNameW", L"%llu  %lu (%ls) -> %lu", uintptr_t(hModule), nSize, lpFilename, res);
			return res;
		}
		return Shared_GetModuleFileNameInner(func, hModule, g_virtualApplication, lpFilename, nSize);
	}

	{
		// Check if there are any stored paths from dynamically loaded dlls
		SCOPED_READ_LOCK(g_loadedModulesLock, lock);
		auto findIt = g_loadedModules.find(hModule);
		if (findIt != g_loadedModules.end())
			return Shared_GetModuleFileNameInner(func, hModule, findIt->second, lpFilename, nSize);
	}

	if (!g_runningRemote && !IsVfsEnabled())
	{
		DWORD res = True_GetModuleFileNameW(hModule, lpFilename, nSize);
		if (res == nSize && nSize && g_isRunningWine && GetLastError() == ERROR_INSUFFICIENT_BUFFER) // Wine bug, not terminating
			lpFilename[nSize-1] = 0;
		DEBUG_LOG_TRUE(L"GetModuleFileNameW", L"%llu  %lu (%ls) -> %lu", uintptr_t(hModule), nSize, lpFilename, res);
		return res;
	}

	StringBuffer<512> moduleName;
	{
		DWORD res = True_GetModuleFileNameW(hModule, moduleName.data, moduleName.capacity);
		if (res == nSize && nSize && g_isRunningWine && GetLastError() == ERROR_INSUFFICIENT_BUFFER) // Wine bug, not terminating
			lpFilename[nSize-1] = 0;
		DEBUG_LOG_TRUE(L"GetModuleFileNameW", L"(INTERNAL) %llu  %u (%ls) -> %lu", uintptr_t(hModule), moduleName.capacity, moduleName.data, res);
		if (res == 0)
			return res;
		UBA_ASSERT(GetLastError() != ERROR_INSUFFICIENT_BUFFER);
		moduleName.count = res;
	}

	// This could be dlls that are loaded early one so might not exist in g_loadedModules
	// TODO: These could be wrong.. since the files could have been copied from different directories into the remote exedir
	if (!moduleName.StartsWith(g_exeDir.data))
	{
		VirtualizePath(moduleName);
		return Shared_GetModuleFileNameInner(func, hModule, moduleName, lpFilename, nSize);
	}

	StringBuffer<350> fileName;
	fileName.Append(g_virtualApplicationDir);
	fileName.Append(moduleName.data + g_exeDir.count);
	return Shared_GetModuleFileNameInner(func, hModule, fileName, lpFilename, nSize);
}

DWORD Detoured_GetModuleFileNameW(HMODULE hModule, LPWSTR lpFilename, DWORD nSize)
{
	DETOURED_CALL(GetModuleFileNameW);
	return Shared_GetModuleFileNameW(L"GetModuleFileNameW", hModule, lpFilename, nSize);
}

DWORD Detoured_GetModuleFileNameExW(HANDLE hProcess, HMODULE hModule, LPWSTR lpFilename, DWORD nSize)
{
	if (hProcess != (HANDLE)-1)
	{
		HANDLE trueHandle = hProcess;
		if (isDetouredHandle(hProcess))
			trueHandle = asDetouredHandle(hProcess).trueHandle;
		if (GetProcessId(trueHandle) != GetCurrentProcessId())
		{
			DWORD res = True_GetModuleFileNameExW(trueHandle, hModule, lpFilename, nSize);
			DEBUG_LOG_TRUE(L"GetModuleFileNameExW", L"%llu  %lu (%ls) -> %lu", uintptr_t(hModule), nSize, lpFilename, res);
			UBA_ASSERTF(!isDetouredHandle(hProcess), TC("GetModuleFileNameExW getting module filename from detoured process, path is potentially wrong. Need implementation"));
			UBA_ASSERTF(!g_runningRemote, TC("GetModuleFileNameExW on other processes not implemented (%s)"), lpFilename); // Not implemented
			return res;
		}
	}
	return Shared_GetModuleFileNameW(L"GetModuleFileNameExW", hModule, lpFilename, nSize);
}

DWORD Shared_GetModuleFileNameA(const tchar* func, HMODULE hModule, LPSTR lpFilename, DWORD nSize)
{
	// Verified called from used applications.. and does not automatically call W version
	StringBuffer<> temp;
	nSize = Min(nSize, (DWORD)temp.capacity);
	DWORD res = Shared_GetModuleFileNameW(func, hModule, temp.data, nSize);
	if (res == 0)
		return 0;
	temp.count = res;
	u32 res2 = temp.Parse(lpFilename, nSize);
	if (res2 == 0)
	{
		DEBUG_LOG(L"GetModuleFileNameA failed to parse %s to ansi", temp.data);
		return 0;
	}
	return res2 - 1;
}

DWORD Detoured_GetModuleFileNameA(HMODULE hModule, LPSTR lpFilename, DWORD nSize)
{
	// Verified called from used applications.. and does not automatically call W version
	DETOURED_CALL(GetModuleFileNameA);
	return Shared_GetModuleFileNameA(L"GetModuleFileNameA", hModule, lpFilename, nSize);
}

DWORD Detoured_GetModuleFileNameExA(HANDLE hProcess, HMODULE hModule, LPSTR lpFilename, DWORD nSize)
{
	// Verified called from used applications.. and does not automatically call W version
	DETOURED_CALL(GetModuleFileNameExA);
	if (hProcess != (HANDLE)-1)
	{
		HANDLE trueHandle = hProcess;
		if (isDetouredHandle(hProcess))
			trueHandle = asDetouredHandle(hProcess).trueHandle;
		if (GetProcessId(trueHandle) != GetCurrentProcessId())
		{
			DWORD res = True_GetModuleFileNameExA(trueHandle, hModule, lpFilename, nSize);
			DEBUG_LOG_TRUE(L"GetModuleFileNameExA", L"%llu  %lu (%hs) -> %lu", uintptr_t(hModule), nSize, lpFilename, res);
			UBA_ASSERTF(!isDetouredHandle(hProcess), TC("GetModuleFileNameExA getting module filename from detoured process, path is potentially wrong. Need implementation"));
			UBA_ASSERTF(!g_runningRemote, TC("GetModuleFileNameExA on other processes not implemented (%hs)"), lpFilename); // Not implemented
			return res;
		}
	}
	return Shared_GetModuleFileNameA(L"GetModuleFileNameExA", hModule, lpFilename, nSize);
}

BOOL Detoured_GetModuleHandleExW(DWORD dwFlags, LPCWSTR lpModuleName, HMODULE *phModule)
{
	bool isAddress = dwFlags & GET_MODULE_HANDLE_EX_FLAG_FROM_ADDRESS;

	StringBuffer<> path;
	if (!isAddress && lpModuleName && IsAbsolutePath(lpModuleName))
	{
		FixPath(path, lpModuleName);
		DevirtualizePath(path);
		lpModuleName = path.data;
	}

	// We don't want to trigger lots of downloading of dlls and searching through paths on remote machines... 
	// Will need to revisit this if it causes issues
	if (g_runningRemote)
		t_ntOpenFileDisallowed = true;

	BOOL res = True_GetModuleHandleExW(dwFlags, lpModuleName, phModule);

	DEBUG_LOG_TRUE(L"GetModuleHandleExW", L"%s -> %s (%llu)", isAddress ? TC("GET_MODULE_HANDLE_EX_FLAG_FROM_ADDRESS") : lpModuleName, ToString(res), u64(*phModule));

	if (g_runningRemote)
		t_ntOpenFileDisallowed = false;
	return res;
}

BOOL Shared_CopyFileExW(const tchar* func, bool isHardlink, LPCWSTR lpExistingFileName, LPCWSTR lpNewFileName, LPPROGRESS_ROUTINE lpProgressRoutine, LPVOID lpData, LPBOOL pbCancel, DWORD dwCopyFlags, LPSECURITY_ATTRIBUTES lpSecurityAttributes)
{
	StringBuffer<> fromPath;
	FixPath(fromPath, lpExistingFileName);
	DevirtualizePath(fromPath);
	StringKey fromKey = ToStringKeyLower(fromPath);

	DirectoryTableEntryInformation fromEntryInfo;
	if (!Rpc_GetEntryInformation(fromEntryInfo, fromKey, fromPath, false))
	{
		DEBUG_LOG_DETOURED(func, L"%s to %s flags: %lu -> Error (ERROR_FILE_NOT_FOUND)", lpExistingFileName, lpNewFileName, dwCopyFlags);
		SetLastError(ERROR_FILE_NOT_FOUND);
		return false;
	}
	UBA_ASSERT(!IsDirectory(fromEntryInfo.attributes));

	StringBuffer<> toPath;
	FixPath(toPath, lpNewFileName);
	DevirtualizePath(toPath);
	StringKey toKey = ToStringKeyLower(toPath);

	DirectoryTableEntryInformation toEntryInfo;
	Rpc_GetEntryInformation(toEntryInfo, toKey, toPath, false);
	if (toEntryInfo.attributes && (dwCopyFlags & COPY_FILE_FAIL_IF_EXISTS))
	{
		DEBUG_LOG_DETOURED(func, L"%s to %s flags: %lu -> Error (ERROR_ALREADY_EXISTS)", lpExistingFileName, lpNewFileName, dwCopyFlags);
		SetLastError(ERROR_ALREADY_EXISTS);
		return false;
	}
	UBA_ASSERT(!IsDirectory(toEntryInfo.attributes));

	// Once past this point the copy is expected to always succeed

	StringBuffer<> newFromPath;
	StringBuffer<> newToPath;

	RPC_MESSAGE(CopyFile, copyFile)
	writer.WriteStringKey(fromKey);
	writer.WriteString(fromPath);
	writer.WriteU64(fromEntryInfo.size);
	writer.WriteU64(fromEntryInfo.lastWrite);
	writer.WriteU32(fromEntryInfo.attributes);
	writer.WriteStringKey(toKey);
	writer.WriteString(toPath);
	BinaryReader reader = writer.Flush();
	reader.ReadString(newFromPath);
	reader.ReadString(newToPath);
	u32 closeId = reader.ReadU32();
	u32 lastError = reader.ReadU32();
	DirTableSize directoryTableSize = FromU64(reader.ReadU64());
	RPC_MESSAGE_END;

	g_directoryTable->ParseDirectoryTable(directoryTableSize);

	if (closeId == ~0u) // Copy was made server side
	{
		UBA_ASSERT(g_runningRemote);
		SetLastError(lastError);
		DEBUG_LOG_DETOURED(func, L"(File was copied server side)");
		return lastError == ERROR_SUCCESS;
	}

	UBA_ASSERT(newFromPath[0] != MemoryHandleChar);
	UBA_ASSERT(newFromPath[0] != FreeableMemoryHandleChar);
	UBA_ASSERT(newFromPath[0] != WrittenMemoryHandleChar);

	if (newFromPath[0] == '#')
		newFromPath.Clear().Append(fromPath);


	// TODO: This copy should probably be moved to session process instead.. to handle failing to copy better

	bool keepToFileInMemory = g_allowKeepFilesInMemory && g_rules->KeepInMemory(newToPath, g_systemTemp, g_runningRemote, true);

	bool res;
	if (keepToFileInMemory)
	{
		UBA_ASSERTF(false, TC("%s copying to memory file is not implemented (Used by link.exe -PGOOptimize). Either change application rules or implement code path (Destination file: %s)"), func, newToPath.data);
		SetLastError(ERROR_IO_DEVICE);
		return false;
	}
	else
	{
		SuppressCreateFileDetourScope cfs;
		if (isHardlink)
			res = True_CreateHardLinkW(newToPath.data, newFromPath.data, lpSecurityAttributes);
		else
			res = True_CopyFileExW(newFromPath.data, newToPath.data, lpProgressRoutine, lpData, pbCancel, dwCopyFlags);
	}
	DEBUG_LOG_TRUE(func, L"%ls to %ls flags: %lu (%ls to %ls) -> %s (%s)", lpExistingFileName, lpNewFileName, dwCopyFlags, newFromPath.data, newToPath.data, ToString(res), LastErrorToText().data);

	// We need to report the new file that has been added (and we must do it _after_ it has been copied
	if (!closeId)
		return res;

	bool deleteOnClose = res == false; // If failing to copy we set deleteOnClose
	Rpc_UpdateCloseHandle(newToPath.data, closeId, deleteOnClose, L"", {}, fromEntryInfo.size, fromEntryInfo.lastWrite, fromEntryInfo.attributes, true);

	return res;
}

BOOL Detoured_CopyFileExW(LPCWSTR lpExistingFileName, LPCWSTR lpNewFileName, LPPROGRESS_ROUTINE lpProgressRoutine, LPVOID lpData, LPBOOL pbCancel, DWORD dwCopyFlags)
{
	DETOURED_CALL(CopyFileExW);
	return Shared_CopyFileExW(TC("CopyFileExW"), false, lpExistingFileName, lpNewFileName, lpProgressRoutine, lpData, pbCancel, dwCopyFlags, NULL);
}

BOOL Detoured_CopyFileW(LPCWSTR lpExistingFileName, LPCWSTR lpNewFileName, BOOL bFailIfExists)
{
	DETOURED_CALL(CopyFileW);
	return Shared_CopyFileExW(TC("CopyFileW"), false, lpExistingFileName, lpNewFileName, (LPPROGRESS_ROUTINE)NULL, (LPVOID)NULL, (LPBOOL)NULL, bFailIfExists ? (DWORD)COPY_FILE_FAIL_IF_EXISTS : 0, NULL);
}


BOOL Detoured_CreateHardLinkW(LPCWSTR lpFileName, LPCWSTR lpExistingFileName, LPSECURITY_ATTRIBUTES lpSecurityAttributes)
{
	DETOURED_CALL(CreateHardLinkW);
	return Shared_CopyFileExW(TC("CreateHardLinkW"), true, lpExistingFileName, lpFileName, NULL, NULL, NULL, COPY_FILE_FAIL_IF_EXISTS, lpSecurityAttributes);
}

#if DETOURED_INCLUDE_DEBUG
thread_local bool t_calledDeleteFileW;
#endif

BOOL Detoured_DeleteFileW(LPCWSTR lpFileName)
{
	DETOURED_CALL(DeleteFileW);
	#if DETOURED_INCLUDE_DEBUG
	t_calledDeleteFileW = true;
	#endif

	LPCWSTR original = lpFileName;
	StringView dosName(ToView(lpFileName));

	if (!CanDetour(dosName))
	{
		DEBUG_LOG_TRUE(L"DeleteFileW", L"(%ls)", original);
		return True_DeleteFileW(original);
	}

	StringBuffer<> fixedName;
	FixPath(fixedName, dosName);
	DevirtualizePath(fixedName);

	StringKey fileNameKey = ToStringKeyLower(fixedName);

	DirectoryTableEntryInformation info;
	if (!Rpc_GetEntryInformation(info, fileNameKey, fixedName, false))
	{
		DEBUG_LOG_DETOURED(L"DeleteFileW", L"(%ls) -> Error (ERROR_FILE_NOT_FOUND)", lpFileName);
		SetLastError(ERROR_FILE_NOT_FOUND);
		return false;
	}
	UBA_ASSERT(!IsDirectory(info.attributes));

	DirTableSize directoryTableSize;
	bool result;
	u32 errorCode;
	{
		u32 closeId = 0;
		RPC_MESSAGE(DeleteFile, deleteFile)
		writer.WriteString(fixedName);
		writer.WriteStringKey(fileNameKey);
		writer.WriteU32(closeId);
		BinaryReader reader = writer.Flush();
		result = reader.ReadBool();
		errorCode = reader.ReadU32();
		directoryTableSize = FromU64(reader.ReadU64());
		pcs.Leave();
		DEBUG_LOG_PIPE(L"DeleteFile", L"%ls", lpFileName);
	}
	DEBUG_LOG_DETOURED(L"DeleteFileW", L"(%ls) -> %ls (%u)", lpFileName, ToString(result), errorCode);

	g_directoryTable->ParseDirectoryTable(directoryTableSize);
	g_mappedFileTable.SetDeleted(fileNameKey, lpFileName, true);
	SetLastError(errorCode);
	return result;
}

bool Shared_MoveFile(const tchar* func, LPCWSTR lpExistingFileName, LPCWSTR lpNewFileName, DWORD dwFlags)
{
	DETOURED_CALL(MoveFileExW);

	StringBuffer<> source;
	FixPath(source, lpExistingFileName);
	DevirtualizePath(source);
	StringKey sourceKey = ToStringKeyLower(source);

	DirectoryTableEntryInformation sourceEntryInfo;
	if (!Rpc_GetEntryInformation(sourceEntryInfo, sourceKey, source, false))
	{
		DEBUG_LOG_DETOURED(func, L"%s to %s -> Error (ERROR_FILE_NOT_FOUND)", lpExistingFileName, lpNewFileName);
		SetLastError(ERROR_FILE_NOT_FOUND);
		return false;
	}
	UBA_ASSERT(!IsDirectory(sourceEntryInfo.attributes));


	StringBuffer<> dest;
	FixPath(dest, lpNewFileName);
	DevirtualizePath(dest);
	StringKey destKey = ToStringKeyLower(dest);

	DirectoryTableEntryInformation destEntryInfo;
	if (Rpc_GetEntryInformation(destEntryInfo, destKey, dest, false))
	{
		if (!(dwFlags & MOVEFILE_REPLACE_EXISTING))
		{
			DEBUG_LOG_DETOURED(func, L"%s to %s -> Error (ERROR_ALREADY_EXISTS)", lpExistingFileName, lpNewFileName);
			SetLastError(ERROR_ALREADY_EXISTS);
			return false;
		}
	}
	UBA_ASSERTF(!IsDirectory(destEntryInfo.attributes), TC("Moving directories not implemented (moving %s to %s"), lpExistingFileName, lpNewFileName);

	#if 0 // This path needs to be revisited... [hka] I can't figure out how to hit this... but KeepInMemory should not be used outside of NtCreateFile anymore
	if (KeepInMemory(source, false))
	{
		UBA_ASSERT(false);

		SCOPED_WRITE_LOCK(g_mappedFileTable.m_lookupLock, lock);
		auto it = g_mappedFileTable.m_lookup.find(sourceKey);
		UBA_ASSERTF(it != g_mappedFileTable.m_lookup.end(), L"Can't find %ls", source.data);
		FileInfo& sourceInfo = it->second;
		lock.Leave();

		if (g_allowOutputFiles && g_rules->IsOutputFile(dest, g_systemTemp))
		{
			UBA_ASSERT(!sourceInfo.memoryFile->isLocalOnly);
			dest.MakeLower();
			StringKey destKey = ToStringKey(dest);
			SCOPED_WRITE_LOCK(g_mappedFileTable.m_lookupLock, lock2);
			auto insres = g_mappedFileTable.m_lookup.try_emplace(destKey);
			lock2.Leave();
			FileInfo& destInfo = insres.first->second;
			UBA_ASSERTF(!insres.second, TC("%s -> %s"), lpExistingFileName, lpNewFileName); // This is here just to get a chance to investigate this scenario.. might work

			HANDLE tempHandle = INVALID_HANDLE_VALUE;
			if (destInfo.isFileMap || (destInfo.memoryFile && destInfo.memoryFile->isLocalOnly)) // File has been read before, let's just ignore that and take the new memoryFile
			{
				destInfo.isFileMap = false;
				destInfo.trueFileMapHandle = 0;
				destInfo.trueFileMapOffset = 0;
				tempHandle = CreateFile(lpNewFileName, GENERIC_WRITE, 0, NULL, CREATE_ALWAYS, FILE_ATTRIBUTE_NORMAL, NULL);
				if (tempHandle == INVALID_HANDLE_VALUE)
					return false;
			}
			destInfo.memoryFile = sourceInfo.memoryFile;
			sourceInfo.memoryFile = nullptr;

			CloseHandle(tempHandle);
			DEBUG_LOG_DETOURED(func, L"(memfile->memfile) %ls to %ls -> Success", lpExistingFileName, lpNewFileName);
			SetLastError(ERROR_SUCCESS);
			return true;
		}

		DEBUG_LOG_DETOURED(func, L"(memfile->file) %ls to %ls", lpExistingFileName, lpNewFileName);

		HANDLE h = CreateFile(lpNewFileName, GENERIC_WRITE, 0, NULL, CREATE_ALWAYS, FILE_ATTRIBUTE_NORMAL, NULL);
		if (h == INVALID_HANDLE_VALUE)
			return false;
		auto cg = MakeGuard([&]() { CloseHandle(h); });
		UBA_ASSERT(sourceInfo.memoryFile->writtenSize < ~0u);
		u32 toWrite = u32(sourceInfo.memoryFile->writtenSize);
		u8* readPos = sourceInfo.memoryFile->baseAddress;
		while (toWrite)
		{
			DWORD written = 0;
			if (!WriteFile(h, readPos, toWrite, &written, NULL))
				return false;
			readPos += written;
			toWrite -= written;
		}
		SetLastError(ERROR_SUCCESS);
		return true;
	}
#endif

	RPC_MESSAGE(MoveFile, moveFile)
	writer.WriteStringKey(sourceKey);
	writer.WriteString(source);
	writer.WriteU64(sourceEntryInfo.size);
	writer.WriteU64(sourceEntryInfo.lastWrite);
	writer.WriteU32(sourceEntryInfo.attributes);
	writer.WriteStringKey(destKey);
	writer.WriteString(dest);
	writer.WriteU32(dwFlags);
	BinaryReader reader = writer.Flush();
	bool result = reader.ReadBool();
	u32 errorCode = reader.ReadU32();
	DirTableSize directoryTableSize = FromU64(reader.ReadU64());
	RPC_MESSAGE_END;
	DEBUG_LOG_PIPE(func, L"%ls to %ls", lpExistingFileName, lpNewFileName);

	DEBUG_LOG_DETOURED(func, L"(PIPE) (%ls to %ls) -> %ls (%u)", lpExistingFileName, lpNewFileName, ToString(result), errorCode);

	if (result)
	{
		g_directoryTable->ParseDirectoryTable(directoryTableSize);
		g_mappedFileTable.SetDeleted(sourceKey, source.data, true);
		g_mappedFileTable.SetDeleted(destKey, dest.data, false);
	}

	SetLastError(errorCode);

	return result;
}

BOOL Detoured_MoveFileExW(LPCWSTR lpExistingFileName, LPCWSTR lpNewFileName, DWORD dwFlags)
{
	return Shared_MoveFile(TC("MoveFileExW"), lpExistingFileName, lpNewFileName, dwFlags);
}

// MoveFileW ends up here
BOOL Detoured_MoveFileWithProgressW(LPCWSTR lpExistingFileName, LPCWSTR lpNewFileName, LPPROGRESS_ROUTINE lpProgressRoutine, LPVOID lpData, DWORD dwFlags)
{
	DETOURED_CALL(MoveFileWithProgressW);
	StringBuffer<> source;
	FixPath(source, lpExistingFileName);

	return Shared_MoveFile(TC("MoveFileWithProgressW"), lpExistingFileName, lpNewFileName, dwFlags);

	//UBA_ASSERT(!g_runningRemote);
	//DEBUG_LOG_TRUE(L"MoveFileWithProgressW", L"%ls to %ls", lpExistingFileName, lpNewFileName);
	//return True_MoveFileWithProgressW(lpExistingFileName, lpNewFileName, lpProgressRoutine, lpData, dwFlags);
}

bool Shared_GetNextFile(WIN32_FIND_DATA& outData, ListDirectoryHandle& listHandle)
{
	while (true)
	{
		if (listHandle.it == listHandle.fileTableOffsets.size())
			return false;

		constexpr u32 maxLen = sizeof_array(outData.cFileName);

		if (listHandle.it < 0)
		{
			if (listHandle.it == -2)
				wcscpy_s(outData.cFileName, maxLen, L".");
			else
				wcscpy_s(outData.cFileName, maxLen, L"..");
			outData.nFileSizeHigh = 0;
			outData.nFileSizeLow = 0;
			outData.dwFileAttributes = FILE_ATTRIBUTE_DIRECTORY;
			outData.cAlternateFileName[0] = 0;
			(u64&)outData.ftLastWriteTime = 0;
			(u64&)outData.ftCreationTime = 0;
			(u64&)outData.ftLastAccessTime = 0;

			++listHandle.it;
			return true;
		}

		DirectoryTable::EntryInformation info;
		DirTableOffset fileTableOffset = listHandle.fileTableOffsets[listHandle.it++];
		g_directoryTable->GetEntryInformation(info, fileTableOffset, outData.cFileName, maxLen);
		if (info.attributes == 0) // File was deleted
			continue;

		LARGE_INTEGER li = ToLargeInteger(info.size);
		outData.nFileSizeHigh = li.HighPart;
		outData.nFileSizeLow = li.LowPart;
		outData.dwFileAttributes = info.attributes;
		outData.cAlternateFileName[0] = 0;
		(u64&)outData.ftLastWriteTime = info.lastWrite;

		// TODO: These are wrong.. 
		(u64&)outData.ftCreationTime = info.lastWrite;
		(u64&)outData.ftLastAccessTime = info.lastWrite;
		return true;
	}
}

HANDLE Local_FindFirstFileExW(LPCWSTR lpFileName, FINDEX_INFO_LEVELS fInfoLevelId, LPVOID lpFindFileData, FINDEX_SEARCH_OPS fSearchOp, LPVOID lpSearchFilter, DWORD dwAdditionalFlags, const wchar_t* funcName)
{
	DEBUG_LOG_TRUE(funcName, L"(NODETOUR) (%ls)", lpFileName);
	SuppressCreateFileDetourScope s; // Needed for cmd.exe copy right now.. NtCreate's flags are set the same as directory search but the first file is not a directory.
	auto res = True_FindFirstFileExW(lpFileName, fInfoLevelId, lpFindFileData, fSearchOp, lpSearchFilter, dwAdditionalFlags);
	UBA_ASSERT(!isDetouredHandle(res) && !isListDirectoryHandle(res));
	return res;
}


HANDLE Shared_FindFirstFileExW(LPCWSTR lpFileName, FINDEX_INFO_LEVELS fInfoLevelId, LPVOID lpFindFileData, FINDEX_SEARCH_OPS fSearchOp, LPVOID lpSearchFilter, DWORD dwAdditionalFlags, const wchar_t* funcName)
{
	if (t_disallowDetour != 0 || Equals(lpFileName, L"nul") || !g_allowFindFileDetour)
	{
		return Local_FindFirstFileExW(lpFileName, fInfoLevelId, lpFindFileData, fSearchOp, lpSearchFilter, dwAdditionalFlags, funcName);
	}

	StringBuffer<> lowerName;
	if (!FixPath(lowerName, lpFileName))
	{
		DEBUG_LOG_DETOURED(funcName, L"(%ls) -> InvalidName", lpFileName);
		SetLastError(ERROR_INVALID_NAME);
		return INVALID_HANDLE_VALUE;
	}
	DevirtualizePath(lowerName);

	//if (lowerName.StartsWith(g_systemTemp.data) || lowerName.StartsWith(g_systemRoot.data))
	if (g_systemRoot.count && lowerName.StartsWith(g_systemRoot.data))
		return Local_FindFirstFileExW(lpFileName, fInfoLevelId, lpFindFileData, fSearchOp, lpSearchFilter, dwAdditionalFlags, funcName);

	lowerName.MakeLower();
	wchar_t* buf = lowerName.data;

	wchar_t* fileName = lowerName.data;
	wchar_t* lastBackslash = wcsrchr(fileName, '\\');
	if (lastBackslash)
		fileName = lastBackslash + 1;

	UBA_ASSERTF(lastBackslash, TC("No expected backslash in %s (%s)"), fileName, lpFileName);
	u32 bufChars = u32(lastBackslash - buf + 1);

	if (wcscmp(fileName, L"*") == 0 || wcscmp(fileName, L"*.*") == 0)
	{
		*fileName = 0;
	}

	// We must remove a slash at the end so it matches our cache entries
	if (bufChars > 2)
	{
		wchar_t* temp = (wchar_t*)buf;
		if (temp[bufChars - 1] == '\\')
			temp[--bufChars] = 0;
	}

	DirHash hash(StringView(buf, bufChars));

	auto& dirTable = *g_directoryTable;

	SCOPED_WRITE_LOCK(dirTable.m_lookupLock, lookLock);
	auto insres = dirTable.m_lookup.try_emplace(hash.key, g_memoryBlock);
	DirectoryTable::Directory& dir = insres.first->second;
	if (insres.second)
	{
		CHECK_PATH(StringView(buf, bufChars));
		if (dirTable.EntryExistsNoLock(hash.key, StringView(buf, bufChars)) != DirectoryTable::Exists_No)
			Rpc_UpdateDirectory(hash.key, buf, bufChars, false);
	}
	bool exists = false;
	if (dir.latestOffset != InvalidTableOffset || dir.latestOverlayOffset != InvalidTableOffset)
	{
		DirTableOffset entryOffset;
		dirTable.GetLatestOffset(entryOffset, dir);
		DirectoryTable::EntryInformation entryInfo;
		dirTable.GetEntryInformation(entryInfo, entryOffset);
		exists = entryInfo.attributes != 0;
	}

#if UBA_DEBUG_VALIDATE
	HANDLE validateHandle = INVALID_HANDLE_VALUE;
	/*
	if (g_validateFileAccess)
	{
		NTSTATUS res = exists ? 0 : -1;
		IO_STATUS_BLOCK IoStatusBlock2;
		NTSTATUS res2 = True_NtCreateFile(&validateHandle, DesiredAccess, ObjectAttributes, &IoStatusBlock2, AllocationSize, FileAttributes, ShareAccess, CreateDisposition, CreateOptions, EaBuffer, EaLength);
		UBA_ASSERT(res < 0 && res2 < 0 || res >= 0 && res2 >= 0);
	}
	*/
#endif

	if (!exists)
	{
		//if (g_systemTemp.StartsWith(lowerName.data)) // TODO: This is a big hack. We should make sure the uba system temp folder is virtualized and is always some root path that never can collide with the host file system
		//	return Local_FindFirstFileExW(lpFileName, fInfoLevelId, lpFindFileData, fSearchOp, lpSearchFilter, dwAdditionalFlags, funcName);

		DEBUG_LOG_DETOURED(funcName, L"(%ls) -> NotFound", lpFileName);
		SetLastError(ERROR_FILE_NOT_FOUND);
		return INVALID_HANDLE_VALUE;
	}

	// TODO: Add support for more modes
	UBA_ASSERT(fInfoLevelId == FindExInfoBasic || fInfoLevelId == FindExInfoStandard);
	UBA_ASSERT(fSearchOp == FindExSearchNameMatch);
	UBA_ASSERT(lpSearchFilter == nullptr);
	//UBA_ASSERT(dwAdditionalFlags == 0);

	dirTable.PopulateDirectory(hash.open, dir);

	auto listHandle = new ListDirectoryHandle{ hash.key, dir };

	if (!*fileName)
		listHandle->it = -2;
	else
		listHandle->it = 0;

	SCOPED_READ_LOCK(dir.lock, lock);
	listHandle->fileTableOffsets.resize(dir.files.size());
	u32 it = 0;
	for (auto& pair : dir.files)
		listHandle->fileTableOffsets[it++] = pair.second;
	lock.Leave();

	listHandle->wildcard = fileName;
#if UBA_DEBUG_VALIDATE
	if (g_validateFileAccess)
		listHandle->validateHandle = validateHandle;
#endif

	auto& data = *(WIN32_FIND_DATA*)lpFindFileData;
	while (true)
	{
		if (!Shared_GetNextFile(data, *listHandle))
		{
			delete listHandle;
			//if (g_systemTemp.StartsWith(lowerName.data)) // TODO: This is a big hack. We should make sure the uba system temp folder is virtualized and is always some root path that never can collide with the host file system
			//	return Local_FindFirstFileExW(lpFileName, fInfoLevelId, lpFindFileData, fSearchOp, lpSearchFilter, dwAdditionalFlags, funcName);
			DEBUG_LOG_DETOURED(funcName, L"(%ls) -> NotFound(2)", lpFileName);
			return INVALID_HANDLE_VALUE;
		}
		if (listHandle->wildcard.empty() || PathMatchSpecW(data.cFileName, listHandle->wildcard.c_str()))
			break;
	}

	HANDLE res = makeListDirectoryHandle(listHandle);
	DEBUG_LOG_DETOURED(funcName, L"(%ls) \"%ls\" -> %llu", lpFileName, data.cFileName, uintptr_t(res));
	return res;
}

HANDLE Detoured_FindFirstFileExW(LPCWSTR lpFileName, FINDEX_INFO_LEVELS fInfoLevelId, LPVOID lpFindFileData, FINDEX_SEARCH_OPS fSearchOp, LPVOID lpSearchFilter, DWORD dwAdditionalFlags)
{
	DETOURED_CALL(FindFirstFileExW);
	return Shared_FindFirstFileExW(lpFileName, fInfoLevelId, lpFindFileData, fSearchOp, lpSearchFilter, dwAdditionalFlags, L"FindFirstFileExW");
}

HANDLE Detoured_FindFirstFileW(LPCWSTR lpFileName, LPWIN32_FIND_DATAW lpFindFileData)
{
	DETOURED_CALL(FindFirstFileW);
	return Shared_FindFirstFileExW(lpFileName, FindExInfoStandard, lpFindFileData, FindExSearchNameMatch, NULL, 0, L"FindFirstFileW");
}

BOOL Detoured_FindNextFileW(HANDLE hFindFile, LPWIN32_FIND_DATAW lpFindFileData)
{
	DETOURED_CALL(FindNextFileW);
	if (isListDirectoryHandle(hFindFile))
	{
		auto& listHandle = asListDirectoryHandle(hFindFile);
		auto& data = *(WIN32_FIND_DATA*)lpFindFileData;
		while (true)
		{
			if (!Shared_GetNextFile(data, listHandle))
			{
				DEBUG_LOG_DETOURED(L"FindNextFileW", L"%llu (NOMORE) -> False", u64(hFindFile));
				SetLastError(ERROR_NO_MORE_FILES);
				return false;
			}

			if (IsTrackingInput() && Equals(data.cFileName, L"__pycache__")) // If we are populating cache we need to make sure not to use pycache
				continue;

			if (listHandle.wildcard.empty() || PathMatchSpecW(data.cFileName, listHandle.wildcard.c_str()))
			{
				DEBUG_LOG_DETOURED(L"FindNextFileW", L"%llu (%ls) -> True", u64(hFindFile), data.cFileName);
				SetLastError(ERROR_SUCCESS);
				return true;
			}
		}
	}

	UBA_ASSERT(!isDetouredHandle(hFindFile));
	DEBUG_LOG_TRUE(L"FindNextFileW", L"%llu", uintptr_t(hFindFile));
	return True_FindNextFileW(hFindFile, lpFindFileData);
}

HANDLE Detoured_FindFirstFileA(LPCSTR lpFileName, LPWIN32_FIND_DATAA lpFindFileData)
{
	DETOURED_CALL(FindFirstFileW);

	StringBuffer<> fileName;
	fileName.Append(lpFileName);

	WIN32_FIND_DATAW findFileData;
	HANDLE res = Shared_FindFirstFileExW(fileName.data, FindExInfoStandard, &findFileData, FindExSearchNameMatch, NULL, 0, L"FindFirstFileA");
	if (res == INVALID_HANDLE_VALUE)
		return res;

	memcpy(lpFindFileData, &findFileData, 48); // 48 is not exact but it is at least down to where the name starts (the types are identical down to the name)
	size_t destLen;
	errno_t err = wcstombs_s(&destLen, lpFindFileData->cFileName, MAX_PATH, findFileData.cFileName, MAX_PATH-1);
	UBA_ASSERT(err == 0);(void)err;

	return res;
}

BOOL Detoured_FindNextFileA(HANDLE hFindFile, LPWIN32_FIND_DATAA lpFindFileData)
{
	WIN32_FIND_DATAW findFileData;
	if (!Detoured_FindNextFileW(hFindFile, &findFileData))
		return false;

	memcpy(lpFindFileData, &findFileData, 48); // 48 is not exact but it is at least down to where the name starts (the types are identical down to the name)
	size_t destLen;
	errno_t err = wcstombs_s(&destLen, lpFindFileData->cFileName, MAX_PATH, findFileData.cFileName, MAX_PATH-1);
	UBA_ASSERT(err == 0);(void)err;

	return true;
}

BOOL Detoured_FindClose(HANDLE handle)
{
	DETOURED_CALL(FindClose);
	if (isListDirectoryHandle(handle))
	{
		DEBUG_LOG_DETOURED(L"FindClose", L"%llu -> Success", uintptr_t(handle));
		delete& asListDirectoryHandle(handle);
		SetLastError(ERROR_SUCCESS);
		return true;
	}
	UBA_ASSERT(!isDetouredHandle(handle));
	BOOL res = True_FindClose(handle);
	DEBUG_LOG_TRUE(L"FindClose", L"%llu -> %ls", uintptr_t(handle), ToString(res));
	return res;
}

BOOL Detoured_GetFileInformationByHandleEx(HANDLE hFile, FILE_INFO_BY_HANDLE_CLASS fileInformationClass, LPVOID lpFileInformation, DWORD dwBufferSize)
{
	DETOURED_CALL(GetFileInformationByHandleEx);

	HANDLE trueHandle = hFile;

	DirTableOffset entryOffset;

	#if UBA_DEBUG_VALIDATE
	const wchar_t* originalName = nullptr;
	#endif

	auto& dirTable = *g_directoryTable;

	u64 fileSize = InvalidValue;

	if (isDetouredHandle(hFile))
	{
		DetouredHandle& dh = asDetouredHandle(hFile);
		trueHandle = dh.trueHandle;
		entryOffset = dh.dirTableOffset;

		FileObject& fo = *dh.fileObject;
		FileInfo& fi = *fo.fileInfo;
		fileSize = fi.size;

		if (!IsValidEntry(entryOffset) && trueHandle == INVALID_HANDLE_VALUE)
		{
			MemoryFile* mf = fi.memoryFile;
			UBA_ASSERTF(mf || fi.name[0] == '$', L"GetFileInformationByHandleEx called on file %s which has no entry offset or real handle", HandleToName(hFile));

			DEBUG_LOG_DETOURED(L"GetFileInformationByHandleEx", L"(MEMORY) (%u) %llu (%ls) -> %s", fileInformationClass, uintptr_t(hFile), HandleToName(hFile), ToString(TRUE));

			if (fileInformationClass == FileIdInfo)
			{
				auto& data = *(FILE_ID_INFO*)lpFileInformation;
				data.VolumeSerialNumber = mf->volumeSerial;
				u64* id = (u64*)&data.FileId;
				id[0] = 0;
				id[1] = mf->fileIndex;
				return TRUE;
			}
			else if (fileInformationClass == FileStandardInfo)
			{
				auto& data = *(FILE_STANDARD_INFO*)lpFileInformation;
				data.EndOfFile = ToLargeInteger(mf->writtenSize);
				data.AllocationSize = ToLargeInteger(mf->committedSize);
				data.DeletePending = fo.deleteOnClose;
				data.NumberOfLinks = 1;
				data.Directory = false;
				return TRUE;
			}
			else if (fileInformationClass == FileRemoteProtocolInfo)
			{
				UBA_ASSERT(fi.name[0] == '$');
				auto& data = *(FILE_REMOTE_PROTOCOL_INFO*)lpFileInformation;
				memset(&data, 0, sizeof(FILE_REMOTE_PROTOCOL_INFO));
				data.StructureVersion = 1;
				data.StructureSize = sizeof(FILE_REMOTE_PROTOCOL_INFO);
				data.Protocol = 0;  // means "not remote"
				data.ProtocolMajorVersion = 0;
				data.ProtocolMinorVersion = 0;
				data.ProtocolRevision = 0;
				data.Flags = 0;
				return TRUE;
			}
			else
			{
				UBA_ASSERTF(!mf, L"GetFileInformationByHandleEx called for memory file using class %u which is not implemented (%s)", fileInformationClass, HandleToName(hFile));
			}
		}
		else if (fileInformationClass == FileIdBothDirectoryRestartInfo || fileInformationClass == FileIdBothDirectoryInfo)
		{
			DEBUG_LOG_DETOURED(L"GetFileInformationByHandleEx", L"(NOT_SUPPORTED) (%u) %llu (%ls) -> FALSE (ERROR_ACCESS_DENIED)", fileInformationClass, uintptr_t(hFile), HandleToName(hFile));
			SetLastError(ERROR_ACCESS_DENIED);
			return FALSE;
		}

#if UBA_DEBUG_VALIDATE
		originalName = dh.fileObject->fileInfo->originalName;
#endif
	}
	else if (isListDirectoryHandle(hFile))
	{
		if (fileInformationClass == FileFullDirectoryRestartInfo  || fileInformationClass == FileFullDirectoryInfo)
		{
			DEBUG_LOG_DETOURED(L"GetFileInformationByHandleEx", L"(FileFullDirectoryInfo) %llu (%ls)", uintptr_t(hFile), HandleToName(hFile));
			IO_STATUS_BLOCK io;
			NTSTATUS status = Detoured_NtQueryDirectoryFile(hFile, NULL, NULL, NULL, &io, lpFileInformation, dwBufferSize, FileDirectoryInformation, TRUE, NULL, (fileInformationClass == FileFullDirectoryRestartInfo));
			return SetLastErrorFromNtStatus(status);
		}
		if (fileInformationClass == FileIdBothDirectoryRestartInfo || fileInformationClass == FileIdBothDirectoryInfo)
		{
			DEBUG_LOG_DETOURED(L"GetFileInformationByHandleEx", L"(FileIdBothDirectoryInfo) %llu (%ls)", uintptr_t(hFile), HandleToName(hFile));
			IO_STATUS_BLOCK io;
			NTSTATUS status = Detoured_NtQueryDirectoryFile(hFile, NULL, NULL, NULL, &io, lpFileInformation, dwBufferSize, FileIdBothDirectoryInformation, TRUE, NULL, (fileInformationClass == FileIdBothDirectoryRestartInfo));
			return SetLastErrorFromNtStatus(status);
		}

		auto& listHandle = asListDirectoryHandle(hFile);
		if (!dirTable.GetLatestOffset(entryOffset, listHandle.dir))
			UBA_ASSERT(false);
		trueHandle = INVALID_HANDLE_VALUE;
	}

	if (IsValidEntry(entryOffset))
	{
		DirectoryTable::EntryInformation entryInfo;
		dirTable.GetEntryInformation(entryInfo, entryOffset);

		UBA_ASSERT(entryInfo.attributes);

		if (fileInformationClass == FileBasicInfo)
		{
			auto& data = *(FILE_BASIC_INFO*)lpFileInformation;
			data.CreationTime = ToLargeInteger(entryInfo.lastWrite);
			data.LastAccessTime = ToLargeInteger(entryInfo.lastWrite);
			data.LastWriteTime = ToLargeInteger(entryInfo.lastWrite);
			data.ChangeTime = ToLargeInteger(entryInfo.lastWrite);
			data.FileAttributes = entryInfo.attributes;
			DEBUG_LOG_DETOURED(L"GetFileInformationByHandleEx", L"(DIRTABLE) (FileBasicInfo) %llu (%ls) -> %s", uintptr_t(hFile), HandleToName(hFile), ToString(TRUE));
			return TRUE;
		}
		else if (fileInformationClass == FileIdInfo)
		{
			auto& data = *(FILE_ID_INFO*)lpFileInformation;
			data.VolumeSerialNumber = entryInfo.volumeSerial;
			u64* id = (u64*)&data.FileId;
			id[0] = 0;
			id[1] = entryInfo.fileIndex;
			DEBUG_LOG_DETOURED(L"GetFileInformationByHandleEx", L"(DIRTABLE) (FileIdInfo) %llu (VolumeSerial: %u FileIndex: %llu), (%ls)", uintptr_t(hFile), entryInfo.volumeSerial, entryInfo.fileIndex, HandleToName(hFile));
			return TRUE;
		}
		else if (fileInformationClass == FileStandardInfo)
		{
			if (fileSize == InvalidValue)  // Always use FileInfo size if available since file could be decompressed and then directory info is wrong
				fileSize = entryInfo.size;
			auto& data = *(FILE_STANDARD_INFO*)lpFileInformation;
			data.EndOfFile = ToLargeInteger(fileSize);
			data.AllocationSize = ToLargeInteger(entryInfo.size);
			data.DeletePending = false;
			data.NumberOfLinks = 1;
			data.Directory = (entryInfo.attributes & FILE_ATTRIBUTE_DIRECTORY) != 0;


#if UBA_DEBUG_VALIDATE
			if (g_validateFileAccess && originalName)
			{
				SuppressDetourScope _;
				WIN32_FILE_ATTRIBUTE_DATA validData;
				if (True_GetFileAttributesExW(originalName, GetFileExInfoStandard, &validData))
				{
					u64 size = ToLargeInteger(validData.nFileSizeHigh, validData.nFileSizeLow).QuadPart; (void)size;
					UBA_ASSERTF(u64(data.EndOfFile.QuadPart) == size, L"File size used: %llu Actual file size: %llu (%s)", data.EndOfFile.QuadPart, size, originalName);
				}
				else
				{
					Rpc_WriteLogf(L"FAILED TO GET FILE ATTRIBUTES %s", originalName);
				}
			}
#endif

			DEBUG_LOG_DETOURED(L"GetFileInformationByHandleEx", L"(DIRTABLE) (FileStandardInfo) %llu (%ls)", uintptr_t(hFile), HandleToName(hFile));
			return TRUE;
		}
		else if (fileInformationClass == FileRemoteProtocolInfo)
		{
			SetLastError(ERROR_INVALID_PARAMETER);
			DEBUG_LOG_DETOURED(L"GetFileInformationByHandleEx", L"(DIRTABLE) (FileRemoteProtocolInfo) %llu (%s) -> %s", uintptr_t(hFile), HandleToName(hFile), ToString(FALSE));
			return FALSE;
		}
		else if (fileInformationClass == FileAttributeTagInfo)
		{
			auto& data = *(FILE_ATTRIBUTE_TAG_INFO*)lpFileInformation;
			data.FileAttributes = entryInfo.attributes;
			data.ReparseTag = 0;
			DEBUG_LOG_DETOURED(L"GetFileInformationByHandleEx", L"(DIRTABLE) (FileAttributeTagInfo) %llu (%ls)", uintptr_t(hFile), HandleToName(hFile));
			return TRUE;
		}
		else
		{
			UBA_ASSERTF(trueHandle != INVALID_HANDLE_VALUE, L"GetFileInformationByHandleEx with class %u not Implemented (%ls)", fileInformationClass, HandleToName(hFile));
		}
	}
	DEBUG_LOG_TRUE(L"GetFileInformationByHandleEx", L"(%u) %llu (%ls)", u32(fileInformationClass), uintptr_t(hFile), HandleToName(hFile));
	TimerScope ts(g_kernelStats.getFileInfo);
	return True_GetFileInformationByHandleEx(trueHandle, fileInformationClass, lpFileInformation, dwBufferSize); /// calls GetFileInformationByHandleEx
}

BOOL Detoured_GetFileInformationByHandle(HANDLE hFile, LPBY_HANDLE_FILE_INFORMATION lpFileInformation)
{
	DETOURED_CALL(GetFileInformationByHandle);

	HANDLE trueHandle = hFile;
	if (isDetouredHandle(hFile))
	{
		auto& dh = asDetouredHandle(hFile);
		DirTableOffset dirTableOffset = dh.dirTableOffset;

		UBA_ASSERT(dh.fileObject->fileInfo);
		FileInfo& fi = *dh.fileObject->fileInfo;

		if (IsValidEntry(dirTableOffset))
		{
			DirectoryTable::EntryInformation entryInfo;
			g_directoryTable->GetEntryInformation(entryInfo, dirTableOffset);
			lpFileInformation->dwFileAttributes = entryInfo.attributes;
			(u64&)lpFileInformation->ftCreationTime = entryInfo.lastWrite;
			(u64&)lpFileInformation->ftLastAccessTime = entryInfo.lastWrite;
			(u64&)lpFileInformation->ftLastWriteTime = entryInfo.lastWrite;
			lpFileInformation->dwVolumeSerialNumber = entryInfo.volumeSerial;
			LARGE_INTEGER li = ToLargeInteger(entryInfo.fileIndex);
			lpFileInformation->nFileIndexHigh = li.HighPart;
			lpFileInformation->nFileIndexLow = li.LowPart;
			lpFileInformation->nNumberOfLinks = 1;//~u32(0); // TODO
			u64 fileSize = fi.size;
			if (fileSize == InvalidValue)
			{
				fileSize = entryInfo.size;
				if (fileSize == InvalidValue) // File is still open?
					fileSize = 0;
			}
#if UBA_DEBUG_VALIDATE
			if (g_validateFileAccess && !(entryInfo.attributes & FILE_ATTRIBUTE_DIRECTORY))
			{
				SuppressDetourScope _;
				WIN32_FILE_ATTRIBUTE_DATA data;
				if (dh.trueHandle != INVALID_HANDLE_VALUE)
				{
					BY_HANDLE_FILE_INFORMATION bhfi;
					auto res2 = True_GetFileInformationByHandle(dh.trueHandle, &bhfi);
					UBA_ASSERT(res2 == TRUE);
					u64 size = ToLargeInteger(bhfi.nFileSizeHigh, bhfi.nFileSizeLow).QuadPart; (void)size;
					u64 fileIndex = ToLargeInteger(bhfi.nFileIndexHigh, bhfi.nFileIndexLow).QuadPart; (void)fileIndex;
					UBA_ASSERTF(fileSize == size, L"File size used: %llu Actual file size: %llu (%s)", fileSize, size, fi.originalName);
					//UBA_ASSERTF(entryInfo.attributes == bhfi.dwFileAttributes, L"Attributes used: 0x%x Actual: 0x%x (%s)", entryInfo.attributes, bhfi.dwFileAttributes, fi.originalName);
					UBA_ASSERTF(entryInfo.volumeSerial == bhfi.dwVolumeSerialNumber, L"VolumeSerial used: %u Actual: %u (%s)", entryInfo.volumeSerial, bhfi.dwVolumeSerialNumber, fi.originalName);
					UBA_ASSERTF(entryInfo.fileIndex == fileIndex, L"FileIndex used: %llu Actual: %llu (%s)", entryInfo.fileIndex, fileIndex, fi.originalName);
					UBA_ASSERTF(bhfi.nNumberOfLinks == 1, L"Links used: %i Actual: %u (%s)", 1, bhfi.nNumberOfLinks, fi.originalName);
				}
				else if (True_GetFileAttributesExW(fi.originalName, GetFileExInfoStandard, &data))
				{
					u64 size = ToLargeInteger(data.nFileSizeHigh, data.nFileSizeLow).QuadPart; (void)size;
					UBA_ASSERTF(fileSize == size, L"File size used: %llu Actual file size: %llu (%s)", fileSize, size, fi.originalName);
				}
				else
				{
					Rpc_WriteLogf(L"FAILED TO GET FILE ATTRIBUTES %s", fi.originalName);
				}
			}
#endif


			li = ToLargeInteger(fileSize);
			lpFileInformation->nFileSizeHigh = li.HighPart;
			lpFileInformation->nFileSizeLow = li.LowPart;
			DEBUG_LOG_DETOURED(L"GetFileInformationByHandle", L"(file) %llu (%ls) -> Success (size: %llu)", uintptr_t(hFile), HandleToName(hFile), fileSize);
			SetLastError(ERROR_SUCCESS);
			return TRUE;
		}

		if (MemoryFile* mf = fi.memoryFile)
		{
			DEBUG_LOG_DETOURED(L"GetFileInformationByHandle", L"(memoryfile) %llu (%ls) -> Success (Size: %llu)", uintptr_t(hFile), HandleToName(hFile), mf->writtenSize);
			lpFileInformation->dwFileAttributes = FILE_ATTRIBUTE_NORMAL;
			(u64&)lpFileInformation->ftCreationTime = mf->fileTime;
			(u64&)lpFileInformation->ftLastAccessTime = mf->fileTime;
			(u64&)lpFileInformation->ftLastWriteTime = mf->fileTime;
			lpFileInformation->dwVolumeSerialNumber = mf->volumeSerial;
			LARGE_INTEGER li = ToLargeInteger(mf->fileIndex);
			lpFileInformation->nFileIndexHigh = li.HighPart;
			lpFileInformation->nFileIndexLow = li.LowPart;
			lpFileInformation->nNumberOfLinks = 1;//~u32(0); // TODO
			li = ToLargeInteger(mf->writtenSize);
			lpFileInformation->nFileSizeHigh = li.HighPart;
			lpFileInformation->nFileSizeLow = li.LowPart;
			SetLastError(ERROR_SUCCESS);
			return TRUE;
		}

		if (g_runningRemote || fi.isFileMap)
		{
			StringBuffer<> fixedName;
			FixPath(fixedName, fi.originalName);

			FileAttributes attr;
			Shared_GetFileAttributes(attr, fixedName);

			if (!attr.exists)
			{
				// this could be a file that was created locally and is not propagated to directory table


				SetLastError(ERROR_FILE_NOT_FOUND);
				DEBUG_LOG_DETOURED(L"GetFileInformationByHandle", L"(remote) %llu (%ls) -> NotFound", uintptr_t(hFile), HandleToName(hFile));
				return false;
			}

			if (attr.fileIndex || dh.trueHandle == INVALID_HANDLE_VALUE)
			{
				UBA_ASSERTF(attr.fileIndex, TC("No file index set for %s"), HandleToName(hFile));

				// TODO: This scenario can happen in very complicated python scripts.. might need more investigation
				LARGE_INTEGER test;
				test.LowPart = attr.data.nFileSizeLow;
				test.HighPart = attr.data.nFileSizeHigh;
				if (test.QuadPart == InvalidValue)
				{
					//auto findIt = g_mappedFileTable.m_lookup.find(fi.fileNameKey);
					//UBA_ASSERT(findIt != g_mappedFileTable.m_lookup.end());

					UBA_ASSERT(!fi.isFileMap && dh.trueHandle != INVALID_HANDLE_VALUE);
					TimerScope ts(g_kernelStats.getFileInfo);
					auto res = True_GetFileInformationByHandle(dh.trueHandle, lpFileInformation); // Calls NtQueryInformationFile
					DEBUG_LOG_TRUE(L"GetFileInformationByHandle", L"(SPECIAL) %llu (%ls) -> %u (Size: %llu)", uintptr_t(hFile), HandleToName(hFile), res, ToLargeInteger(lpFileInformation->nFileSizeHigh, lpFileInformation->nFileSizeLow).QuadPart);
					return res;
				}

				LARGE_INTEGER li = ToLargeInteger(attr.fileIndex);
				/*
				#if UBA_DEBUG_VALIDATE
				if (g_validateFileAccess)
				{
					HANDLE h = True_CreateFileW(fileName, 0, FILE_SHARE_READ, NULL, OPEN_ALWAYS, FILE_FLAG_BACKUP_SEMANTICS, NULL);
					auto res = True_GetFileInformationByHandle(h, lpFileInformation);
					True_CloseHandle(h);

					UBA_ASSERT(attr.data.dwFileAttributes == lpFileInformation->dwFileAttributes);
					UBA_ASSERT(attr.volumeSerial == lpFileInformation->dwVolumeSerialNumber);
					UBA_ASSERT(li.HighPart == lpFileInformation->nFileIndexHigh);
					UBA_ASSERT(li.LowPart == lpFileInformation->nFileIndexLow);
					//return res;
				}
				#endif
				*/
				SetLastError(ERROR_SUCCESS);

				UBA_ASSERTF(attr.volumeSerial, TC("No volume serial set for %s"), HandleToName(hFile));

				lpFileInformation->dwFileAttributes = attr.data.dwFileAttributes;
				lpFileInformation->ftCreationTime = attr.data.ftCreationTime;
				lpFileInformation->ftCreationTime = attr.data.ftCreationTime;
				lpFileInformation->ftLastAccessTime = attr.data.ftLastAccessTime;
				lpFileInformation->ftLastWriteTime = attr.data.ftLastWriteTime;
				lpFileInformation->dwVolumeSerialNumber = attr.volumeSerial;
				lpFileInformation->nFileIndexHigh = li.HighPart;
				lpFileInformation->nFileIndexLow = li.LowPart;
				lpFileInformation->nNumberOfLinks = 1;//~u32(0); // TODO
				lpFileInformation->nFileSizeHigh = attr.data.nFileSizeHigh;
				lpFileInformation->nFileSizeLow = attr.data.nFileSizeLow;
				DEBUG_LOG_DETOURED(L"GetFileInformationByHandle", L"(remote) %llu (%ls) -> Success", uintptr_t(hFile), HandleToName(hFile));
				return TRUE;
			}
		}
		UBA_ASSERTF(dh.trueHandle != INVALID_HANDLE_VALUE, TC("GetFileInformationByHandle needs true handle for %ls"), HandleToName(hFile));
		trueHandle = dh.trueHandle;
	}

	TimerScope ts(g_kernelStats.getFileInfo);
	auto res = True_GetFileInformationByHandle(trueHandle, lpFileInformation); // Calls NtQueryInformationFile
	DEBUG_LOG_TRUE(L"GetFileInformationByHandle", L"%llu (%ls) -> %u", uintptr_t(hFile), HandleToName(hFile), res);
	return res;
}

BOOL Detoured_GetFileInformationByName(PCWSTR FileName, FILE_INFO_BY_NAME_CLASS FileInformationClass, PVOID FileInfoBuffer, ULONG FileInfoBufferSize)
{
	DETOURED_CALL(GetFileInformationByName);

#if 0
	enum FILE_INFO_BY_NAME_CLASS
	{
		FileStatByNameInfo,
		FileStatLxByNameInfo,
		FileCaseSensitiveByNameInfo,
		FileStatBasicByNameInfo,
		MaximumFileInfoByNameClass
	};
	
	if (FileInformationClass == FileStatBasicByNameInfo)
	{
		struct FILE_STAT_BASIC_INFORMATION
		{
			LARGE_INTEGER FileId;
			LARGE_INTEGER CreationTime;
			LARGE_INTEGER LastAccessTime;
			LARGE_INTEGER LastWriteTime;
			LARGE_INTEGER ChangeTime;
			LARGE_INTEGER AllocationSize;
			LARGE_INTEGER EndOfFile;
			ULONG FileAttributes;
			ULONG ReparseTag;
			ULONG NumberOfLinks;
			ACCESS_MASK EffectiveAccess;
		};
		FILE_STAT_BASIC_INFORMATION
	}
#endif
	if (g_runningRemote)
	{
		SetLastError(ERROR_CALL_NOT_IMPLEMENTED);
		DEBUG_LOG_DETOURED(L"GetFileInformationByName", L"(%ls) (class %u) -> %s (ERROR_CALL_NOT_IMPLEMENTED)", FileName, FileInformationClass, ToString(false));
		return false;
	}

	StringBuffer<> pathName;
	FixPath(pathName, FileName);
	DevirtualizePath(pathName);
	StringKey sourceKey = ToStringKeyLower(pathName);

#if UBA_DEBUG
	SCOPED_WRITE_LOCK(g_mappedFileTable.m_lookupLock, lock);
	auto it = g_mappedFileTable.m_lookup.find(sourceKey);
	if (it != g_mappedFileTable.m_lookup.end())
	{
		FileInfo& fi = it->second;
		UBA_ASSERT(!fi.memoryFile);
	}
	lock.Leave();
#endif

	BOOL res = True_GetFileInformationByName(pathName.data, FileInformationClass, FileInfoBuffer, FileInfoBufferSize);
	DEBUG_LOG_TRUE(L"GetFileInformationByName", L"(%ls) (class %u) -> %s", pathName.data, FileInformationClass, ToString(res));
	return res;
}


BOOL Detoured_SetFileInformationByHandle(HANDLE hFile, FILE_INFO_BY_HANDLE_CLASS FileInformationClass, LPVOID lpFileInformation, DWORD dwBufferSize)
{
	DETOURED_CALL(SetFileInformationByHandle);

	if (g_isRunningWine)
	{
		if (FileInformationClass == FileRenameInfoEx) // DOES NOT EXIST IN WINE 10.7
		{
			auto& info = *(FILE_RENAME_INFO*)lpFileInformation;
			DWORD flags = info.Flags;
			UBA_ASSERT(flags == 0x00000001); // FILE_RENAME_REPLACE_IF_EXISTS
			info.ReplaceIfExists = true;

			DEBUG_LOG_TRUE(L"SetFileInformationByHandle", L"%llu (FileRenameInfoEx)", uintptr_t(hFile));
			bool res = True_SetFileInformationByHandle(hFile, FileRenameInfo, lpFileInformation, dwBufferSize);
			info.Flags = flags;
			return res;
		}
		if (FileInformationClass == FileAllocationInfo) // DOES NOT CALL NtSetFileInformation in wine
		{
			IO_STATUS_BLOCK io;
			return SetLastErrorFromNtStatus(Detoured_NtSetInformationFile(hFile, &io, lpFileInformation, dwBufferSize, FileAllocationInformation));
		}
	}

	DEBUG_LOG_TRUE(L"SetFileInformationByHandle", L"%llu (%u)", uintptr_t(hFile), FileInformationClass);
	return True_SetFileInformationByHandle(hFile, FileInformationClass, lpFileInformation, dwBufferSize); // In here to be tabbed in log
}

HANDLE Detoured_CreateFileMappingW(HANDLE hFile, LPSECURITY_ATTRIBUTES lpFileMappingAttributes, DWORD flProtect, DWORD dwMaximumSizeHigh, DWORD dwMaximumSizeLow, LPCWSTR lpName)
{
	DETOURED_CALL(CreateFileMappingW);
	HANDLE trueHandle = hFile;
	FileObject* fo = nullptr;
	if (isDetouredHandle(hFile))
	{
		auto& dh = asDetouredHandle(hFile);

		fo = dh.fileObject;
		fo->wasUsed = true;

		FileInfo& fi = *fo->fileInfo;
		if (fi.memoryFile || fi.isFileMap)
		{
			auto mdh = new DetouredHandle(HandleType_FileMapping);
			if (fi.isFileMap)
			{
				// If protection levels are the same we can reuse the "built-in" file mapping
				UBA_ASSERTF((flProtect == PAGE_WRITECOPY ? PAGE_READONLY : flProtect) == fi.fileMapDesiredAccess, L"Code path not implemented (%ls)", HandleToName(hFile));
			}
			mdh->fileObject = dh.fileObject;

			u64 maxSize = ToLargeInteger(dwMaximumSizeHigh, dwMaximumSizeLow).QuadPart;

			if (MemoryFile* mf = fi.memoryFile)
			{
				if (!(flProtect & MEM_RESERVE) && maxSize)
				{
					mf->EnsureCommitted(*mdh, maxSize);
					if (!mf->writtenSize && (flProtect & PAGE_READWRITE)) // TODO: Maybe we should always set writtenSize?
						mf->writtenSize = maxSize;
				}
			}
			InterlockedIncrement(&dh.fileObject->refCount);
			HANDLE res = makeDetouredHandle(mdh);
			DEBUG_LOG_DETOURED(L"CreateFileMappingW", L"(%ls) File %llu Protect %lu Size %llu (%ls) -> %llu", (fi.memoryFile ? L"MEMORYFILE" : L"FILEMAP"), uintptr_t(hFile), flProtect, maxSize, HandleToName(hFile), uintptr_t(res));
			SetLastError(ERROR_SUCCESS);
			return res;
		}
		UBA_ASSERT(dh.trueHandle != INVALID_HANDLE_VALUE);
		trueHandle = dh.trueHandle;
	}

	TimerScope ts(g_kernelStats.createFileMapping);
	HANDLE mappingHandle = True_CreateFileMappingW(trueHandle, lpFileMappingAttributes, flProtect, dwMaximumSizeHigh, dwMaximumSizeLow, lpName);
	if (!mappingHandle)
	{
		DEBUG_LOG_TRUE(L"CreateFileMappingW", L"File %llu (%s) -> Error", uintptr_t(hFile), HandleToName(hFile));
		return NULL;
	}

	if (g_allowFileMappingDetour)
	{
		if (GetLastError() == ERROR_ALREADY_EXISTS)
			ToInvestigate(L"Mapping already exists");
		auto detouredHandle = new DetouredHandle(HandleType_FileMapping);
		detouredHandle->trueHandle = mappingHandle;
		//detouredHandle->fileObject = fo;
		//if (fo)
		//	InterlockedIncrement(&fo->refCount);
		mappingHandle = makeDetouredHandle(detouredHandle);
	}
	DEBUG_LOG_TRUE(L"CreateFileMappingW", L"File %llu, Size: %llu (%s) -> %llu", uintptr_t(hFile), ToLargeInteger(dwMaximumSizeHigh, dwMaximumSizeLow).QuadPart, HandleToName(hFile), u64(mappingHandle));
	return mappingHandle;
}

HANDLE Detoured_CreateFileMappingA(HANDLE hFile, LPSECURITY_ATTRIBUTES lpFileMappingAttributes, DWORD flProtect, DWORD dwMaximumSizeHigh, DWORD dwMaximumSizeLow, LPCSTR lpName)
{
	const wchar_t* name = nullptr;
	wchar_t temp[512];
	if (lpName)
	{
		swprintf_s(temp, sizeof_array(temp), L"%hs", lpName);
		name = temp;
	}
	return Detoured_CreateFileMappingW(hFile, lpFileMappingAttributes, flProtect, dwMaximumSizeHigh, dwMaximumSizeLow, name);
}

HANDLE Detoured_OpenFileMappingW(DWORD dwDesiredAccess, BOOL bInheritHandle, LPCWSTR lpName)
{
	DETOURED_CALL(OpenFileMappingW);
	HANDLE mappingHandle = True_OpenFileMappingW(dwDesiredAccess, bInheritHandle, lpName);
	if (!mappingHandle)
	{
		DEBUG_LOG_TRUE(L"OpenFileMappingW", L"%ls -> Error", lpName);
		return NULL;
	}
	if (g_allowFileMappingDetour)
	{
		auto detouredHandle = new DetouredHandle(HandleType_FileMapping);
		detouredHandle->trueHandle = mappingHandle;
		mappingHandle = makeDetouredHandle(detouredHandle);
	}
	DEBUG_LOG_TRUE(L"OpenFileMappingW", L"%ls -> %llu", lpName, u64(mappingHandle));
	return mappingHandle;
}

auto& g_specialViewsLock = *new ReaderWriterLock();
auto& g_specialViews = *new List<SharedMemoryView>();


LPVOID Detoured_MapViewOfFileEx(HANDLE hFileMappingObject, DWORD dwDesiredAccess, DWORD dwFileOffsetHigh, DWORD dwFileOffsetLow, SIZE_T dwNumberOfBytesToMap, LPVOID lpBaseAddress)
{
	DETOURED_CALL(MapViewOfFileEx);
	HANDLE trueMappingObject = hFileMappingObject;
	if (isDetouredHandle(hFileMappingObject))
	{
		auto& dh = asDetouredHandle(hFileMappingObject);
		if (dh.fileObject)
		{
			u64 offset = ToLargeInteger(dwFileOffsetHigh, dwFileOffsetLow).QuadPart;

			FileInfo& fi = *dh.fileObject->fileInfo;
			if (fi.fileMapMem && lpBaseAddress && lpBaseAddress != fi.fileMapMem) // This scenario happens with pch files in msvc cl.exe
			{
				// This is really hacky.. cl.exe keeps the old mapping and creates a new one with an offset which is set copy-on-write..
				// This is where cl.exe "continues" its compile from the pch
				UBA_ASSERT(dwDesiredAccess == FILE_MAP_COPY);

				SCOPED_WRITE_LOCK(g_specialViewsLock, lock);
				SharedMemoryView& view = g_specialViews.emplace_back(g_sharedMemoryFileMappingBackend);
				lock.Leave();

				SharedMemoryMapType mapType = g_useLazyCow ? SharedMemoryMapType_LazyCopyOnWrite : SharedMemoryMapType_CopyOnWrite;

				RPC_MESSAGE(GetSharedMemory, getSharedMemory)
				writer.WriteU64(fi.trueFileMapHandle.ToU64());
				writer.WriteU64(offset + fi.trueFileMapOffset);
				writer.WriteU64(dwNumberOfBytesToMap);
				BinaryReader reader = writer.Flush();
				view.Init(g_sharedMemoryAllocator, dwNumberOfBytesToMap, lpBaseAddress);
				UBA_FOR_ASSERT(bool success =) view.AddRequestedMemory(g_sharedMemoryAllocator, reader, mapType, 0);
				RPC_MESSAGE_END
				UBA_ASSERT(success);

				SetLastError(ERROR_SUCCESS);
				//UBA_ASSERT(!fi.trueFileMapOffset);
				void* res = view.GetMemory();
				if (g_useLazyCow)
				{
					UBA_ASSERT(dwNumberOfBytesToMap);
					RegisterLazyCowRange((u8*)res, dwNumberOfBytesToMap);
				}
				DEBUG_LOG_TRUE(L"MapViewOfFileEx", L"(INTERNAL) New FileObject for different base address %llu Size %llu CoW (%s) -> 0x%llx", uintptr_t(hFileMappingObject), dwNumberOfBytesToMap, HandleToName(hFileMappingObject), uintptr_t(res));
				return res;
			}
			else if (!fi.fileMapMem)
				fi.fileMapViewDesiredAccess = dwDesiredAccess;

			SharedMemoryMapType type = SharedMemoryMapType_ReadOnly;
			if (dwDesiredAccess & FILE_MAP_COPY)
				type = g_useLazyCow ? SharedMemoryMapType_LazyCopyOnWrite : SharedMemoryMapType_CopyOnWrite;
				//type = SharedMemoryMapType_CopyOnWrite;


			if (!EnsureMapped(dh, dwFileOffsetHigh, dwFileOffsetLow, dwNumberOfBytesToMap, lpBaseAddress, type))
				return nullptr;

			SetLastError(ERROR_SUCCESS);

			u8* mem = fi.fileMapMem ? fi.fileMapMem : fi.memoryFile->baseAddress;

			mem += offset;

			if (fi.memoryFile && (dwDesiredAccess & FILE_MAP_WRITE)) // We assume changes will happen
				fi.memoryFile->isReported = false;

			DEBUG_LOG_DETOURED(L"MapViewOfFileEx", L"%llu Size %llu %s (%s) -> 0x%llx", uintptr_t(hFileMappingObject), dwNumberOfBytesToMap, (type == SharedMemoryMapType_ReadOnly ? L"ReadOnly" : L"CoW"), HandleToName(hFileMappingObject), u64(mem));

			if (type == SharedMemoryMapType_LazyCopyOnWrite)
			{
				// dwNumberOfBytesToMap==0 legally means "map to end of file"; derive the real span from FileInfo.
				u64 regSize = dwNumberOfBytesToMap ? dwNumberOfBytesToMap : (fi.fileMapMemSize > offset ? fi.fileMapMemSize - offset : 0);
				if (regSize)
					RegisterLazyCowRange((u8*)mem, regSize);
			}

			SCOPED_FUTEX(g_mappedFileTable.m_memLookupLock, lock);
			auto& entry = g_mappedFileTable.m_memLookup[mem];
			if (!entry.handle)
			{
				InterlockedIncrement(&dh.fileObject->refCount);
				auto newDh = new DetouredHandle(dh.type);
				newDh->fileObject = dh.fileObject;
				entry.handle = newDh;
			}
			++entry.refCount;
			return mem;
		}
		UBA_ASSERT(dh.trueHandle != INVALID_HANDLE_VALUE);
		trueMappingObject = dh.trueHandle;
	}

	bool useLazyCow = g_useLazyCow && dwNumberOfBytesToMap != 0 && dwFileOffsetHigh == 0 && dwFileOffsetLow == 0 && (dwDesiredAccess & FILE_MAP_COPY) != 0;
	if (useLazyCow)
		dwDesiredAccess = FILE_MAP_READ;

	TimerScope ts(g_kernelStats.mapViewOfFile);
	void* res = True_MapViewOfFileEx(trueMappingObject, dwDesiredAccess, dwFileOffsetHigh, dwFileOffsetLow, dwNumberOfBytesToMap, lpBaseAddress);
	DEBUG_LOG_TRUE(L"MapViewOfFileEx", L"%llu (size %llu)%s (%s) -> 0x%llx", uintptr_t(hFileMappingObject), dwNumberOfBytesToMap, ((dwDesiredAccess & FILE_MAP_COPY) ? L" CoW" : L""), HandleToName(hFileMappingObject), uintptr_t(res));

	if (useLazyCow)
		RegisterLazyCowRange((u8*)res, dwNumberOfBytesToMap);

	return res;
}

LPVOID Detoured_MapViewOfFile(HANDLE hFileMappingObject, DWORD dwDesiredAccess, DWORD dwFileOffsetHigh, DWORD dwFileOffsetLow, SIZE_T dwNumberOfBytesToMap)
{
	DETOURED_CALL(MapViewOfFile);
	return Detoured_MapViewOfFileEx(hFileMappingObject, dwDesiredAccess, dwFileOffsetHigh, dwFileOffsetLow, dwNumberOfBytesToMap, nullptr);
}

PVOID Detoured_MapViewOfFile3(HANDLE FileMapping, HANDLE Process, PVOID BaseAddress, ULONG64 Offset, SIZE_T ViewSize, ULONG AllocationType, ULONG PageProtection, MEM_EXTENDED_PARAMETER *ExtendedParameters, ULONG ParameterCount)
{
	UBA_ASSERT(!isDetouredHandle(Process));

	HANDLE trueHandle = FileMapping;
	if (isDetouredHandle(FileMapping))
	{
		auto& dh = asDetouredHandle(FileMapping);
		trueHandle = dh.trueHandle;
		if (dh.fileObject)
		{
			FileInfo& fi = *dh.fileObject->fileInfo;

			UBA_ASSERT(AllocationType == MEM_REPLACE_PLACEHOLDER);
			UBA_ASSERT(PageProtection == PAGE_READONLY || PageProtection == PAGE_WRITECOPY);
			UBA_ASSERT(!ExtendedParameters);
			UBA_ASSERT(!ParameterCount);

			SharedMemoryMapType type = SharedMemoryMapType_ReadOnly;
			if (PageProtection & PAGE_WRITECOPY)
				type = SharedMemoryMapType_CopyOnWrite;

			SharedMemoryView view(g_sharedMemoryFileMappingBackend);
			view.SetMemory((u8*)BaseAddress, ViewSize);

			RPC_MESSAGE(GetSharedMemory, getSharedMemory)
			writer.WriteU64(fi.trueFileMapHandle.ToU64());
			writer.WriteU64(Offset + fi.trueFileMapOffset);
			writer.WriteU64(ViewSize);
			BinaryReader reader = writer.Flush();
			bool success = view.Init(g_sharedMemoryAllocator, nullptr, reader, ViewSize, L"", type);
			RPC_MESSAGE_END

			view.DetachMemory();

			void* res = success ? BaseAddress : nullptr;
			DEBUG_LOG_TRUE(L"MapViewOfFile3", L"%llu (offs %llu size %llu)%s %s (%s) -> 0x%llx", uintptr_t(FileMapping), Offset, ViewSize, ((PageProtection & PAGE_WRITECOPY) ? L" CoW" : L"ReadOnly"), fi.originalName, HandleToName(FileMapping), uintptr_t(res));
			return res;
		}
		UBA_ASSERT(trueHandle != INVALID_HANDLE_VALUE);
	}

	//DEBUG_LOG_TRUE(L"MapViewOfFile3", L""); // Spammy since SharedMemoryAllocator use it
	TimerScope ts(g_kernelStats.mapViewOfFile3);
	return True_MapViewOfFile3(trueHandle, Process, BaseAddress, Offset, ViewSize, AllocationType, PageProtection, ExtendedParameters, ParameterCount);
}

BOOL Detoured_FlushViewOfFile(LPCVOID lpBaseAddress, SIZE_T dwNumberOfBytesToFlush)
{
	DETOURED_CALL(FlushViewOfFile);
	DEBUG_LOG_DETOURED(L"FlushViewOfFile", L"(NO-OP!!!!) %p (size %llu) -> true", lpBaseAddress, dwNumberOfBytesToFlush);
	return true;
}

BOOL Detoured_UnmapViewOfFileEx(PVOID lpBaseAddress, ULONG UnmapFlags)
{
	DETOURED_CALL(UnmapViewOfFileEx);

	if (g_useLazyCow)
		UnregisterLazyCowRange((u8*)lpBaseAddress);

	{
		SCOPED_FUTEX(g_mappedFileTable.m_memLookupLock, lock);
		auto it = g_mappedFileTable.m_memLookup.find(lpBaseAddress);
		if (it != g_mappedFileTable.m_memLookup.end())
		{
			auto& entry = it->second;
			if (!--entry.refCount)
			{
				if (entry.handle)
					Detoured_NtClose(makeDetouredHandle(entry.handle));
				g_mappedFileTable.m_memLookup.erase(it);
			}
			SetLastError(ERROR_SUCCESS);
			return true;
		}
	}

	SCOPED_WRITE_LOCK(g_specialViewsLock, lock);
	for (auto i=g_specialViews.begin(), e=g_specialViews.end(); i!=e; ++i)
	{
		if (i->GetMemory() != lpBaseAddress)
			continue;
		g_specialViews.erase(i);
		SetLastError(ERROR_SUCCESS);
		return true;
	}
	lock.Leave();

	auto res = True_UnmapViewOfFileEx(lpBaseAddress, UnmapFlags); (void)res;
	DEBUG_LOG_TRUE(L"UnmapViewOfFileEx", L"0x%llx -> %ls", uintptr_t(lpBaseAddress), ToString(res));

	// TerminateProcess unmaps same memory address twice.. causing this log entry. Ignore for now
	//if (res == 0)
	//	ToInvestigate(L"Failed to Unmap 0x%llx -> %u", uintptr_t(lpBaseAddress), GetLastError());
	return TRUE;
}

BOOL Detoured_UnmapViewOfFile(LPCVOID lpBaseAddress)
{
	DETOURED_CALL(UnmapViewOfFile);
	return Detoured_UnmapViewOfFileEx((PVOID)lpBaseAddress, 0);
}

DWORD Detoured_GetFinalPathNameByHandleW(HANDLE hFile, LPTSTR lpszFilePath, DWORD cchFilePath, DWORD dwFlags)
{
	DETOURED_CALL(GetFinalPathNameByHandleW);

	HANDLE trueHandle = hFile;
	if (isDetouredHandle(hFile))
	{
		auto& dh = asDetouredHandle(hFile);
		auto fo = dh.fileObject;
		UBA_ASSERT(fo);
		auto& fi = *fo->fileInfo;
		UBA_ASSERT(fi.originalName);
		const wchar_t* fileName = fi.originalName;

		if (dwFlags == 0 || dwFlags == 2)
		{
			if (!fo->newName.empty())
				fileName = fo->newName.c_str();

			StringBuffer<> buffer;
			FixPath(fileName, g_virtualWorkingDir.data, g_virtualWorkingDir.count, buffer);
			VirtualizePath(buffer);
			u32 requiredBufferSize = buffer.count;
			if (dwFlags == 2)
				requiredBufferSize += 4;

			if (cchFilePath <= requiredBufferSize)
			{
				SetLastError(ERROR_NOT_ENOUGH_MEMORY);
				DEBUG_LOG_DETOURED(L"GetFinalPathNameByHandleW", L"%llu (%lu) (%ls) -> Error (not enough mem)", uintptr_t(hFile), dwFlags, lpszFilePath);
				return requiredBufferSize + 1;
			}

			// Unfortunately casing can be wrong here.. and we need to fix that. Let's use the directory table for that
			// Note, this really only matters when building linux target from windows.. then there is path validation that errors if this is not properly fixed
			g_directoryTable->GetFinalPath(buffer.Clear(), fileName);
			VirtualizePath(buffer);
			buffer.data[0] = ToUpper(buffer.data[0]);
			if (dwFlags == 2)
				buffer.Prepend(AsView(L"\\??\\"));
			UBA_ASSERTF(requiredBufferSize == buffer.count, TC("Length mismatch %u vs %u (%s)"), requiredBufferSize, buffer.count, buffer.data);

			memcpy(lpszFilePath, buffer.data, (buffer.count + 1) * sizeof(tchar));
			DEBUG_LOG_DETOURED(L"GetFinalPathNameByHandleW", L"%llu (%lu) (%ls) -> Success", uintptr_t(hFile), dwFlags, lpszFilePath);

			SetLastError(ERROR_SUCCESS);
			return buffer.count;
		}
		trueHandle = dh.trueHandle;
		UBA_ASSERTF(trueHandle != INVALID_HANDLE_VALUE, L"GetFinalPathNameByHandleW using flags (%lu) on detoured file not handled (%s)", dwFlags, fileName);
	}

	auto res = True_GetFinalPathNameByHandleW(trueHandle, lpszFilePath, cchFilePath, dwFlags); // Calls NtQueryInformationFile and NtQueryObject
	DEBUG_LOG_TRUE(L"GetFinalPathNameByHandleW", L"%llu (%lu) (%ls) -> %lu", uintptr_t(hFile), dwFlags, (res != 0 ? lpszFilePath : L"UNKNOWN"), res);
	return res;
}

DWORD Detoured_SearchPathW(LPCWSTR lpPath, LPCWSTR lpFileName, LPCWSTR lpExtension, DWORD nBufferLength, LPWSTR lpBuffer, LPWSTR* lpFilePart)
{
	DETOURED_CALL(SearchPathW);
	if (g_runningRemote && !t_disallowDetour)
	{
		g_rules->RepairMalformedLibPath(lpFileName);

		const wchar_t* original = lpFileName; (void)original;
		u64 pathLen = wcslen(lpFileName);
		StringBuffer<512> tempBuf;
		Rpc_GetFullFileName(lpFileName, pathLen, tempBuf, true);
		UBA_ASSERT(nBufferLength > pathLen);
		memcpy(lpBuffer, lpFileName, pathLen * sizeof(tchar) + 2);
		DEBUG_LOG_DETOURED(L"SearchPathW", L"%ls %ls -> %ls", lpPath, original, lpFileName);
		SetLastError(ERROR_SUCCESS);
		return DWORD(pathLen);
	}

	DWORD res = True_SearchPathW(lpPath, lpFileName, lpExtension, nBufferLength, lpBuffer, lpFilePart);
	if (res && IsVfsEnabled() && (!g_systemRoot.count || !StartsWith(lpBuffer, g_systemRoot.data)))
	{
		UBA_ASSERT(res < nBufferLength);
		StringBuffer<> temp(lpBuffer);
		if (VirtualizePath(temp))
		{
			memcpy(lpBuffer, temp.data, sizeof(tchar)*temp.count+1);
			res = temp.count;
			if (lpFilePart)
				*lpFilePart = TStrrchr(lpBuffer, '\\') + 1;
		}

	}
	DEBUG_LOG_TRUE(L"SearchPathW", L"%ls %ls -> %s", lpPath, lpFileName, lpBuffer);
	return res;
}

using AdditionalLoads = Vector<HMODULE, GrowingAllocator>;
using VisitedModules = UnorderedSet<StringKey, std::hash<StringKey>, std::equal_to<StringKey>, GrowingAllocator>;

HMODULE Recursive_LoadLibraryExW(const StringView& filePath, LPCWSTR originalName, DWORD dwFlags, AdditionalLoads& additionalLoads, VisitedModules& visitedModules)
{
	if (!visitedModules.insert(ToStringKeyNoCheck(filePath.data, filePath.count)).second)
		return 0;

	// Important that this code is not doing allocations.. it could cause a recursive stack overflow
	struct Import { wchar_t name[128]; bool isKnown;  Import(const wchar_t* s, bool ik) : isKnown(ik) { wcscpy_s(name, sizeof_array(name), s); } };
	Vector<Import, GrowingAllocator> importedModules(g_memoryBlock);
	{
		SuppressCreateFileDetourScope cfs;
		StringBuffer<256> error;
		BinaryInfo info;
		if (!ParseBinary(filePath, {}, info, [&](const wchar_t* import, bool isKnown, const tchar* const* importLoaderPaths)
			{
				if (!GetModuleHandleW(import))
					importedModules.emplace_back(import, isKnown);
			}, error))
		{
			FatalError(9887, L"Failed to find imports for binary %s (%s)", filePath.data, originalName);
		}
	}
	for (auto& importedModule : importedModules)
	{
		if (importedModule.isKnown && !g_isRunningWine)
			continue;

		{
			SuppressCreateFileDetourScope cfs;
			HMODULE checkModule = GetModuleHandleW(importedModule.name); // This function ends up in NtCreateFile when running in wine
			if (checkModule)
				continue;
		}

		if (importedModule.isKnown) // We need to catch dbghelp.dll and imagehlp.dll
		{
			if (HMODULE h = True_LoadLibraryExW(importedModule.name, 0, 0))
			{
				OnModuleLoaded(h, ToView(importedModule.name));
				additionalLoads.push_back(h);
			}
			continue;
		}

		const wchar_t* path = importedModule.name;
		if (path[1] == ':')
			if (const wchar_t* lastSlash = wcsrchr(path, '\\'))
				path = lastSlash + 1;
		u64 pathLen = wcslen(path);
		StringBuffer<512> tempBuf;
		if (g_runningRemote)
			Rpc_GetFullFileName(path, pathLen, tempBuf, false);

		if (HMODULE r = Recursive_LoadLibraryExW(ToView(path), importedModule.name, dwFlags, additionalLoads, visitedModules))
			additionalLoads.push_back(r);
	}

	StringBuffer<512> newName;
	if (originalName[1] != ':' && !filePath.Equals(originalName))
	{
		newName.Append(g_virtualApplicationDir).Append(originalName);
		originalName = newName.data;
	}

	TrackInput(ToView(originalName));

	DEBUG_LOG_TRUE(L"INTERNAL LoadLibraryExW", L"%ls", originalName);

	SuppressCreateFileDetourScope cfs;
	auto res = True_LoadLibraryExW(filePath.data, 0, 0);
	if (res)
	{
		if (originalName[1] == ':')
		{
			// TODO: Virtualize!
			SCOPED_WRITE_LOCK(g_loadedModulesLock, lock);
			g_loadedModules[res] = originalName;
		}
		OnModuleLoaded(res, filePath);
	}
	return res;
}

#if defined(_M_ARM64) 
	#define ReadTebOffset(offset) (((BYTE*)NtCurrentTeb()) + offset)
#else
	#define ReadTebOffset(offset) __readgsqword(offset)
#endif

HMODULE LoadModuleNoSyscall(StringView moduleName)
{
	ULONG disp;
	PVOID cookie;
	LdrLockLoaderLock(0, &disp, &cookie);
	auto lockGuard = MakeGuard([&](){ LdrUnlockLoaderLock(0, cookie); });

	StringView moduleFileName = moduleName.GetFileName();
	PPEB peb = (PPEB)ReadTebOffset(0x60);
	PPEB_LDR_DATA ldr = peb->Ldr;
	PLIST_ENTRY head = &ldr->InMemoryOrderModuleList;
	for (PLIST_ENTRY entry = head->Flink; entry != head; entry = entry->Flink)
	{
		PLDR_DATA_TABLE_ENTRY mod = CONTAINING_RECORD(entry, LDR_DATA_TABLE_ENTRY, InMemoryOrderLinks);
		StringView fullDllName(mod->FullDllName.Buffer, mod->FullDllName.Length/2);
		StringView dllName = fullDllName.GetFileName();
		if (!dllName.Equals(moduleFileName))
			continue;

		// Check if memory is still committed.. if not it might have been unloaded
		MEMORY_BASIC_INFORMATION mbi;
		bool res = VirtualQuery(mod->DllBase, &mbi, sizeof(mbi)) != 0;
		UBA_ASSERT(res);
		if (!res)
			return NULL;
		if (mbi.State != MEM_COMMIT)
			return nullptr;
		NTSTATUS ntRes = LdrAddRefDll(0, mod->DllBase);
		UBA_ASSERT(ntRes == STATUS_SUCCESS);
		if (ntRes != STATUS_SUCCESS)
			return NULL;
		return (HMODULE)mod->DllBase;
	}
	return NULL;  // Module not found
}

HMODULE Shared_LoadLibrary(LPCWSTR lpLibFileName, HANDLE hFile, DWORD dwFlags, bool& mightExist)
{
	if (!g_runningRemote && !IsTrackingInput() || t_disallowDetour)
		return NULL;

	StringView pathView(ToView(lpLibFileName));
	if (pathView.StartsWith(TCV("\\\\?\\")))
		pathView = pathView.Skip(4);

	StringBuffer<> path;
	path.Append(pathView).FixPathSeparators();

	bool detourDll = path.EndsWith(L".exe") || path.EndsWith(L".dll") || path.EndsWith(L".node");
	if (detourDll && g_systemRoot.count && path.StartsWith(g_systemRoot.data))
		detourDll = g_runningRemote && GetFileAttributes(path.data) == INVALID_FILE_ATTRIBUTES; // It might be that remote machine actually doesn't have the file in system32. then we need to detour

	if (!detourDll)
		return NULL;
		
	// When running remote the path will not be correct (since the dll is loaded from a temporary local folder)
	// This code assumes that we never load the same dll (same exact name) from different paths and will break if we do.
	if (HMODULE res = LoadModuleNoSyscall(path))
		return res;

	u64 pathLen = path.count;
	u64 toSkip = 0;
	if (path.StartsWith(g_exeDir.data))
		toSkip = g_exeDir.count;
	else if (path.StartsWith(g_virtualApplicationDir.data))
		toSkip = g_virtualApplicationDir.count;
	const wchar_t* fileName = path.data + toSkip;
	pathLen -= toSkip;

	StringBuffer<512> tempBuf;
	const wchar_t* newPath = fileName;
	u64 newPathLen = pathLen;
	if (g_runningRemote)
	{
		Rpc_GetFullFileName(newPath, newPathLen, tempBuf, false);
		if (newPath[1] != ':')
		{
			mightExist = false;
			return 0;
		}
	}
	AdditionalLoads additionalLoads(g_memoryBlock); // Don't do allocations
	VisitedModules visitedModules(g_memoryBlock);
	HMODULE res = Recursive_LoadLibraryExW(ToView(newPath), fileName, dwFlags, additionalLoads, visitedModules);
	for (HMODULE h : additionalLoads)
		FreeLibrary(h);
	return res;
}

HMODULE Detoured_LoadLibraryExW(LPCWSTR lpLibFileName, HANDLE hFile, DWORD dwFlags)
{
	DETOURED_CALL(LoadLibraryExW);

	if (!g_rules->AllowLoadLibrary(lpLibFileName))
	{
		DEBUG_LOG_DETOURED(L"LoadLibraryExW", L"SKIPPED (%ls)", lpLibFileName);
		return 0;
	}

	if (IsKnownSystemFile(lpLibFileName))
	{
		SuppressCreateFileDetourScope cfs;
		const tchar* fileName = lpLibFileName;
		if (auto lastSeparator = TStrrchr(fileName, PathSeparator))
			fileName = lastSeparator + 1;
		DEBUG_LOG_TRUE(L"LoadLibraryExW", L"(KNOWN) %ls (%s)", lpLibFileName, fileName);
		return True_LoadLibraryExW(fileName, hFile, LOAD_LIBRARY_SEARCH_SYSTEM32);
	}

	DEBUG_LOG_DETOURED(L"LoadLibraryExW", L"(%ls)", lpLibFileName);

	bool mightExist = true;
	if (HMODULE res = Shared_LoadLibrary(lpLibFileName, hFile, dwFlags, mightExist))
		return res;

	if (!mightExist)
	{
		DEBUG_LOG_TRUE(L"LoadLibraryExW", L"(NOTFOUND) %s", lpLibFileName);
		return 0;
	}
	
	StringBuffer<> fileName;
	if (lpLibFileName[1] == ':')
	{
		FixPath(fileName, lpLibFileName);
		DevirtualizePath(fileName);
	}
	else
		fileName.Append(lpLibFileName);

	DEBUG_LOG_TRUE(L"LoadLibraryExW", L"%ls (%s)", lpLibFileName, fileName.data);
	return True_LoadLibraryExW(fileName.data, hFile, dwFlags);
}

HANDLE Detoured_GetStdHandle(DWORD nStdHandle)
{
	DETOURED_CALL(GetStdHandle);
	if (g_isDetachedProcess)
	{
		HANDLE res = g_stdHandle[nStdHandle + 12]; // STD_INPUT_HANDLE -10, STD_OUTPUT_HANDLE -11, STD_ERROR_HANDLE -12
		DEBUG_LOG_DETOURED(L"GetStdHandle", L"%lu -> %llu", nStdHandle, u64(res));
		SetLastError(ERROR_SUCCESS);
		return res;
	}

	auto res = True_GetStdHandle(nStdHandle);
	DEBUG_LOG_TRUE(L"GetStdHandle", L"%lu -> %llu", nStdHandle, u64(res));
	return res;
}

bool g_setStdHandleCalled;

BOOL Detoured_SetStdHandle(DWORD nStdHandle, HANDLE hHandle)
{
	DETOURED_CALL(SetStdHandle);

	if (g_isDetachedProcess)
	{
		DEBUG_LOG_DETOURED(L"SetStdHandle", L"(SKIPPED!) %s %llu -> Success", StdToString(nStdHandle), u64(hHandle));
		return true;
	}

	if (nStdHandle == STD_OUTPUT_HANDLE)
		g_stdHandle[1] = (hHandle != g_nullFile && GetFileType(hHandle) == FILE_TYPE_CHAR) ? hHandle : 0;
	else if (nStdHandle == STD_ERROR_HANDLE)
		g_stdHandle[0] = (hHandle != g_nullFile && GetFileType(hHandle) == FILE_TYPE_CHAR) ? hHandle : 0;

	//UBA_ASSERTF(!isDetouredHandle(hHandle), L"Trying to use handle %ls for std stream", HandleToName(hHandle));
	HANDLE trueHandle = hHandle;

	// TODO: Reason we have change to true handle is because this is transferred to child processes which can't use this process detoured handles
	// ... need to fix this.
	if (isDetouredHandle(hHandle))
	{
		auto& dh = asDetouredHandle(hHandle);
		trueHandle = dh.trueHandle;
		UBA_ASSERT(trueHandle != INVALID_HANDLE_VALUE);
	}
	g_setStdHandleCalled = true;

	auto res = True_SetStdHandle(nStdHandle, trueHandle);
	DEBUG_LOG_TRUE(L"SetStdHandle", L"%s %llu -> %s", StdToString(nStdHandle), u64(trueHandle), ToString(res));
	return res;
}


BOOL Detoured_GetConsoleMode(HANDLE hConsoleHandle, LPDWORD lpMode)
{
	DETOURED_CALL(GetConsoleMode);
	if (hConsoleHandle == g_stdHandle[0] || hConsoleHandle == g_stdHandle[1])
	{
		*lpMode = 0xffff;
		return true;
	}
	else if (hConsoleHandle == g_stdHandle[2])
	{
		*lpMode = 0xffff;
		return true;
	}

	if (g_isDetachedProcess)
	{
		SetLastError(ERROR_INVALID_HANDLE);
		DEBUG_LOG_DETOURED(L"GetConsoleMode", L"%llu -> Error", uintptr_t(hConsoleHandle));
		return false;
	}

	auto res = True_GetConsoleMode(hConsoleHandle, lpMode);
	DEBUG_LOG_TRUE(L"GetConsoleMode", L"%llu %lu -> %ls", uintptr_t(hConsoleHandle), *lpMode, ToString(res));
	return res;
}

BOOL Detoured_SetConsoleMode(HANDLE hConsoleHandle, DWORD mode)
{
	DETOURED_CALL(SetConsoleMode);

	HANDLE trueHandle = hConsoleHandle;

	const tchar* dbgstr = TC("unknown");

	if (g_isDetachedProcess)
	{
		if (hConsoleHandle == g_stdHandle[0])
		{
			trueHandle = True_GetStdHandle(STD_ERROR_HANDLE);
			dbgstr = TC("error");
		}
		else if (hConsoleHandle == g_stdHandle[1])
		{
			trueHandle = True_GetStdHandle(STD_OUTPUT_HANDLE);
			dbgstr = TC("output");
		}
		else if (hConsoleHandle == g_stdHandle[2])
		{
			trueHandle = True_GetStdHandle(STD_INPUT_HANDLE);
			dbgstr = TC("input");
		}
	}

	if (hConsoleHandle == trueHandle && isDetouredHandle(hConsoleHandle))
	{
		DetouredHandle& dh = asDetouredHandle(hConsoleHandle);
		trueHandle = dh.trueHandle;
		UBA_ASSERTF(trueHandle != INVALID_HANDLE_VALUE, TC("SetConsoleMode is using %s as handle to set mode 0x%lx"), HandleToName(hConsoleHandle), mode);
		dbgstr = TC("detoured");
	}

	BOOL res = True_SetConsoleMode(trueHandle, mode);
	DEBUG_LOG_DETOURED(L"SetConsoleMode", L"(%s) %llu (0x%lx) -> %s", dbgstr, u64(hConsoleHandle), mode, ToString(res));
	return res;
}

BOOL Detoured_GetConsoleTitleW(LPTSTR lpConsoleTitle, DWORD nSize)
{
	DETOURED_CALL(GetConsoleTitleW);
	DEBUG_LOG_DETOURED(L"GetConsoleTitleW", L"");
	SetLastError(ERROR_SUCCESS);
	*lpConsoleTitle = 0;
	return true;
}

BOOL Detoured_CreateProcessW(LPCWSTR lpApplicationName, LPWSTR lpCommandLine, LPSECURITY_ATTRIBUTES lpProcessAttributes, LPSECURITY_ATTRIBUTES lpThreadAttributes, BOOL bInheritHandles,
	DWORD dwCreationFlags, LPVOID lpEnvironment, LPCWSTR lpCurrentDirectory, LPSTARTUPINFOW lpStartupInfo, LPPROCESS_INFORMATION lpProcessInformation)
{
	StringView originalCmd = ToView(lpCommandLine ? lpCommandLine : TC(""));

	DETOURED_CALL(CreateProcessW);
	DEBUG_LOG_DETOURED(L"CreateProcessW", L"%ls %ls CreationFlags: 0x%lx StartupFlags: 0x%lu Stdin: %llu Stdout: %llu Stderr: %llu WorkDir: %s Inherit: %s Env: %llu", lpApplicationName, originalCmd.data, dwCreationFlags, lpStartupInfo->dwFlags, u64(lpStartupInfo->hStdInput), u64(lpStartupInfo->hStdOutput), u64(lpStartupInfo->hStdError), (lpCurrentDirectory ? lpCurrentDirectory : L""), bInheritHandles?L"true":L"false", u64(lpEnvironment));

	StringView testName = ToView(lpApplicationName ? lpApplicationName : TC(""));

	if (!testName.count)
	{
		if (!originalCmd.count)
		{
			SetLastError(ERROR_FILE_NOT_FOUND);
			return FALSE;
		}

		testName = originalCmd;
	}

	// Don't know if this is sufficient. But we can't have child processes inherit detoured handles and maybe this detouring has to happen further down the stack
	// TODO: Maybe not modify lpStartupInfo directly
	if (lpStartupInfo->cbReserved2)
	{
		BYTE* p = lpStartupInfo->lpReserved2;
		DWORD count = *(DWORD*)p;
		BYTE*   flags   = p + sizeof(DWORD);
		HANDLE* handles = (HANDLE*)(flags + count);
		for (DWORD i = 0; i < count; i++)
		{
			HANDLE h = handles[i];
			if (h != INVALID_HANDLE_VALUE && isDetouredHandle(h))
				handles[i] = asDetouredHandle(h).trueHandle;
		}
	}

	// Debug binaries started when process crash... we don't want to detour these.
	if (testName.Contains(L"winedbg") || testName.Contains(L"werfault.exe") || testName.Contains(L"vsjitdebugger.exe") || testName.Contains(L"crashpad_handler.exe"))
	{
		if (g_runningRemote)
		{
			UbaAssert(L"Unhandled exception/crash. Suppress debugger startup and try to report issue instead. This message is here to hopefully see callstack", "", 0, "", false, 0, nullptr, 1);
			return false;
		}
		else
		{
			SuppressDetourScope _;
			BOOL res = True_CreateProcessW(lpApplicationName, lpCommandLine, lpProcessAttributes, lpThreadAttributes, bInheritHandles, dwCreationFlags, lpEnvironment, lpCurrentDirectory, lpStartupInfo, lpProcessInformation);
			True_WaitForSingleObject(lpProcessInformation->hProcess, 10000);
			return res;
		}
	}

	bool isChild = true;
	// We don't care about tracking mspdbsrv or vctip.. they are services just spawned by this process
	if (testName.Contains(L"mspdbsrv.exe") || testName.Contains(L"vctip.exe"))
	{
		if (!g_runningRemote)
		{
			SuppressDetourScope _;
			return True_CreateProcessW(lpApplicationName, lpCommandLine, lpProcessAttributes, lpThreadAttributes, bInheritHandles, dwCreationFlags, lpEnvironment, lpCurrentDirectory, lpStartupInfo, lpProcessInformation);
		}
		isChild = false;
	}

	tchar* tempAllocated = nullptr;
	tchar* finalCommandLine = lpCommandLine;
	HANDLE nulStdIn = INVALID_HANDLE_VALUE;
	auto g = MakeGuard([&]() { if (tempAllocated) delete[] tempAllocated; if (nulStdIn != INVALID_HANDLE_VALUE) CloseHandle(nulStdIn); });

	StringView commandLineWithoutApplication;
	{
		const tchar* endOfApplication;
		if (originalCmd.data[0] == '"')
		{
			const tchar* quoteEnd = TStrchr(originalCmd.data+1, '"');
			UBA_ASSERT(quoteEnd);
			endOfApplication = quoteEnd + 1;

			// This is an annoying special case that it seems like DetourCreateProcessWithDllsW does not like.. it does not separate the arguments properly
			// so if there is "link.exe"/LIB as a command it will not produce proper wmain(argv)
			if (*endOfApplication != ' ')
			{
				finalCommandLine = tempAllocated = new tchar[originalCmd.count + 1];
				u64 firstPart = endOfApplication - originalCmd.data;
				memcpy(finalCommandLine, originalCmd.data, firstPart*sizeof(tchar));
				finalCommandLine[firstPart] = ' ';
				memcpy(finalCommandLine + firstPart + 1, endOfApplication, (originalCmd.count - firstPart + 1)*sizeof(tchar));
			}
		}
		else if (const tchar* firstSpace = TStrchr(originalCmd.data, ' '))
			endOfApplication = firstSpace + 1;
		else
			endOfApplication = originalCmd.data + originalCmd.count;

		commandLineWithoutApplication = StringView(endOfApplication, originalCmd.count - u32(endOfApplication - originalCmd.data));
	}

	StringBuffer<512> application;
	if (lpApplicationName && *lpApplicationName)
	{
		if (lpApplicationName[1] == ':') // Only fixup absolute paths (since we want to devirtualize them
			FixPath(application, lpApplicationName);
		else
			application.Append(lpApplicationName);
		UBA_ASSERTF(application.count, TC("Invalid application name from application field: %s"), lpApplicationName);
	}
	else
	{
		StringBuffer<512> temp;
		if (originalCmd.data[0] == '"')
			temp.Append(originalCmd.data+1, commandLineWithoutApplication.data - originalCmd.data - 2);
		else if (commandLineWithoutApplication.count == 0)
			temp.Append(originalCmd);
		else
			temp.Append(originalCmd.data, commandLineWithoutApplication.data - originalCmd.data - 1);
		if (temp[1] == ':')
			FixPath(application, temp.data);
		else
			application.Append(temp.data);
		UBA_ASSERTF(application.count, TC("Invalid application from command line (name: %s)"), originalCmd.data);
		if (application.count < 4 || application[application.count - 4] != '.')
			application.Append(TCV(".exe")); // TODO: Maybe also handle other extensions?
	}
	DevirtualizePath(application);

	bool startSuspended = (dwCreationFlags & CREATE_SUSPENDED) != 0;

	StringBuffer<> workingDir(lpCurrentDirectory ? lpCurrentDirectory : g_virtualWorkingDir.data);
	DevirtualizePath(workingDir);

	if (testName.Contains(TC("UbaCli.exe")))
	{
		{
			RPC_MESSAGE(RunSpecialProgram, createProcess)
			writer.WriteString(application);
			writer.WriteLongString(commandLineWithoutApplication);
			writer.WriteString(workingDir);
			writer.Flush();
		}
		Rpc_UpdateTables();

		UBA_ASSERT(g_systemRoot.count);
		StringBuffer<256> cmdExe;
		cmdExe.Append(g_systemRoot).EnsureEndsWithSlash().Append(TCV("system32\\cmd.exe"));
		SuppressDetourScope _;
		return True_CreateProcessW(cmdExe.data, (LPWSTR)TC("cmd.exe /c exit 0"), lpProcessAttributes, lpThreadAttributes, bInheritHandles, dwCreationFlags, lpEnvironment, workingDir.data, lpStartupInfo, lpProcessInformation);
	}

	// Need to turn into cmd.exe manually since we don't it to tamper with the path (remote execution will end up using the true path instead of virtualized)
	if (application.EndsWith(TCV(".bat")))
	{
		//UBA_ASSERT(false);
		StringView extra(TCV("cmd.exe /C "));
		UBA_ASSERT(!application.Contains(' '));
		u64 len = finalCommandLine ? TStrlen(finalCommandLine) : 0;
		tchar* newCommandLine = new tchar[application.count + extra.count + len + 1];
		memcpy(newCommandLine, extra.data, application.count*sizeof(tchar));
		memcpy(newCommandLine + extra.count, application.data, application.count*sizeof(tchar));
		memcpy(newCommandLine + extra.count + application.count, finalCommandLine ? finalCommandLine : TC(""), (len + 1)*sizeof(tchar));
		delete[] tempAllocated;
		finalCommandLine = tempAllocated = newCommandLine;
		application.Clear().Append(TCV("C:\\Windows\\system32\\cmd.exe"));
	}

	bool is32Bit = application.EndsWith(TC("gperf.exe")); // Hard coded for 32-bit application gperf.exe... this could be extended but will only work for apps that does not produce output files since we don't track non-detoured applications
	if (is32Bit)
	{
		const wchar_t* realApplication = application.data;
		u64 pathLen = application.count;
		StringBuffer<512> tempBuf;
		if (g_runningRemote)
			Rpc_GetFullFileName(realApplication, pathLen, tempBuf, false);
		SuppressDetourScope _;
		return True_CreateProcessW(realApplication, finalCommandLine, lpProcessAttributes, lpThreadAttributes, bInheritHandles, dwCreationFlags, lpEnvironment, lpCurrentDirectory, lpStartupInfo, lpProcessInformation);
	}

	TString currentDir;
	u32 processId = 0;
	u32 childRulesIndex = 0;
	char dllBuffer[1024];
	char* dllIt = dllBuffer;
	u32 dllCount = 0;
	LPCSTR dlls[2];
	{
		RPC_MESSAGE(CreateProcess, createProcess)
		writer.WriteString(application);
		writer.WriteLongString(commandLineWithoutApplication, CommunicationMemSize - 4096); // don't compress if there is room in the writer memory
		writer.WriteString(workingDir);
		writer.WriteBool(startSuspended);
		writer.WriteBool(isChild);
		BinaryReader reader = writer.Flush();
		processId = reader.ReadU32();
		if (!processId)
		{
			pcs.Leave();
			UBA_ASSERTF(processId > 0, L"Failed to create process %s", originalCmd.data);
			SetLastError(ERROR_ACCESS_DENIED); // This is wrong but there should be info in the log
			return FALSE;
		}

		childRulesIndex = reader.ReadU32();

		while (true)
		{
			u32 dllNameSize = reader.ReadU32();
			if (!dllNameSize)
				break;
			dlls[dllCount++] = dllIt;
			reader.ReadBytes(dllIt, dllNameSize);
			dllIt += dllNameSize;
			*dllIt++ = 0;
		}

		currentDir = reader.ReadString();
		reader.ReadString(application.Clear());
		DEBUG_LOG_PIPE(L"CreateProcess", L"%ls %ls", application.data, originalCmd.data);
	}

	auto handleFileDetour = [&](HANDLE& handle, HandleType type, const tchar* name, DWORD nativeHandleId)
		{
			if (!isDetouredHandle(handle))
				return;
			DetouredHandle& dh = asDetouredHandle(handle);
			if (dh.type == type)
			{
				handle = g_isDetachedProcess ? 0 : True_GetStdHandle(nativeHandleId);
				return;
			}

			handle = dh.trueHandle;
			if (dh.trueHandle != INVALID_HANDLE_VALUE)
				return;

			#if UBA_ASSERT_ON_DETOURED_STD_PIPE
			auto assertGuard = MakeGuard([&]() { UBA_ASSERTF(false, L"%s is detoured and there is no proper handle (%s)", name, lpApplicationName); });
			#endif

			if (dh.type != HandleType_File)
				return;
			if (!dh.fileObject)
				return;
			auto& fo = *dh.fileObject;
			if (!fo.fileInfo)
				return;
			FileInfo& fi = *fo.fileInfo;
			auto mf = fi.memoryFile;
			if (!mf)
				return;
			if (mf->isLocalOnly)
				return;

			// TODO: This is a memory file
			#if UBA_ASSERT_ON_DETOURED_STD_PIPE
			assertGuard.Cancel();
			#endif
		};


	handleFileDetour(lpStartupInfo->hStdError, HandleType_StdErr, TC("hStdError"), STD_ERROR_HANDLE);
	handleFileDetour(lpStartupInfo->hStdOutput, HandleType_StdOut, TC("hStdOutput"), STD_OUTPUT_HANDLE);


	if (isDetouredHandle(lpStartupInfo->hStdInput))
	{
		DetouredHandle& dh = asDetouredHandle(lpStartupInfo->hStdInput);
		if (dh.type == HandleType_StdIn)
			lpStartupInfo->hStdInput = g_isDetachedProcess ? 0 : True_GetStdHandle(STD_INPUT_HANDLE);
		else if (dh.type == HandleType_StdErr)
			lpStartupInfo->hStdInput = g_isDetachedProcess ? 0 : True_GetStdHandle(STD_ERROR_HANDLE);
		else
		{
			#if UBA_ASSERT_ON_DETOURED_STD_PIPE
			UBA_ASSERTF(false, L"hStdInput is detoured (type %i, true: %llu) (%s)", int(dh.type), dh.trueHandle, lpApplicationName);
			#endif
			
			lpStartupInfo->hStdInput = dh.trueHandle;
		}
	}

	if (GetApplicationRules()[childRulesIndex].rules->AllowDetach())
	{
		if (lpStartupInfo->dwFlags & STARTF_USESTDHANDLES)
		{
			HANDLE in = lpStartupInfo->hStdInput;
			if (in && in != INVALID_HANDLE_VALUE && !isDetouredHandle(in) && True_GetFileType(in) == FILE_TYPE_CHAR)
			{
				SECURITY_ATTRIBUTES nulSa = { sizeof(nulSa), NULL, TRUE };
				nulStdIn = CreateFileW(L"NUL", GENERIC_READ, FILE_SHARE_READ | FILE_SHARE_WRITE, &nulSa, OPEN_EXISTING, 0, NULL);
				lpStartupInfo->hStdInput = nulStdIn;
				DEBUG_LOG_DETOURED(L"CreateProcessW", L"Redirected console stdin to NUL (%ls)", testName.data);
			}
		}
		else
		{
			// TODO: GenerateTLH triggers this assert

			// Without STARTF_USESTDHANDLES the detached child inherits our std handles through the PEB, so we can't
			// redirect its console stdin here. Only dangerous when we hold a real console (non-detached parent). Not
			// observed yet (CRT _wspawn always sets STARTF_USESTDHANDLES) -> assert so we catch it.
			//UBA_FOR_ASSERT(HANDLE trueIn = True_GetStdHandle(STD_INPUT_HANDLE));
			//UBA_ASSERTF(g_isDetachedProcess || !trueIn || True_GetFileType(trueIn) != FILE_TYPE_CHAR, L"Spawning detached child without STARTF_USESTDHANDLES while holding a console stdin; child may block on stdin (%ls)", testName.data);
		}
	}

	bool allowCreationFlagChange = true;
	if (dwCreationFlags & EXTENDED_STARTUPINFO_PRESENT)
	{
		struct PROC_THREAD_ATTRIBUTE { DWORD_PTR Attribute; SIZE_T Size; ULONG_PTR Value; };
		struct PROC_THREAD_ATTRIBUTE_LIST_INTERNAL { DWORD Unknown1; DWORD AttributeCount; DWORD Unknown2; DWORD Unknown3; DWORD Unknown4; DWORD Unknown5; PROC_THREAD_ATTRIBUTE Attributes[1]; };
		auto& siex = *(STARTUPINFOEXW*)lpStartupInfo;
		if (auto attrList = (PROC_THREAD_ATTRIBUTE_LIST_INTERNAL*)siex.lpAttributeList)
		{
			for (DWORD i = 0; i < attrList->AttributeCount; i++)
			{
				DWORD_PTR attr = attrList->Attributes[i].Attribute;
				SIZE_T size = attrList->Attributes[i].Size;
				ULONG_PTR value = attrList->Attributes[i].Value;
				DWORD attrType = attr & 0x0000FFFF;
				if (attrType == ProcThreadAttributeHandleList)
				{
					HANDLE* handles = (HANDLE*)value;
					DWORD handleCount = (DWORD)(size / sizeof(HANDLE));
					for (DWORD h = 0; h < handleCount; h++)
					{
						if (isDetouredHandle(handles[h]))
						{
							DetouredHandle& dh = asDetouredHandle(handles[h]);
							UBA_ASSERT(dh.trueHandle != INVALID_HANDLE_VALUE);
							handles[h] = dh.trueHandle;
						}
						
					}
					allowCreationFlagChange = false;
				}
				else if (attrType == ProcThreadAttributeParentProcess)
				{
					UBA_ASSERT(!isDetouredHandle(*(HANDLE*)value));
				}
				else if (attrType == 1) // ProcThreadAttributeExtendedFlags
				{
					//struct PS_ATTRIBUTE { ULONG  Attribute; SIZE_T Size; union { ULONG_PTR Value; PVOID ValuePtr; }; PSIZE_T ReturnLength; };
				}
				else
				{
					// Don't know what 1 is, Ignore
					UBA_ASSERTF(false, TC("Check if thread attribute %lu needs to be handled"), attrType);
				}
			}
		}
	}

	#if UBA_DEBUG
	bool debugWineCalls = false;//application.EndsWith(L"node.exe");
	if (debugWineCalls && !lpEnvironment)
		lpEnvironment = True_GetEnvironmentStringsW();
	#endif

	// Sometimes programs decides to change environment variables such as TEMP/TMP/SystemRoot etc which will break when running remotely
	Vector<tchar> tempEnvironment;
	if (lpEnvironment)
	{
		u32 reserveSize = g_systemTemp.count*2 + 4 + 5 + 3; // Two paths plus TEMP=/TMP= and 3 terminators
		for(StringView env = ToView((tchar*)lpEnvironment); env.count; env = ToView(env.data + env.count + 1))
			reserveSize += env.count+1;
		tempEnvironment.reserve(reserveSize);

		auto addChars = [&](StringView text)
			{
				u64 destPos = tempEnvironment.size();
				tempEnvironment.resize(tempEnvironment.size() + text.count);
				tchar* dest = tempEnvironment.data() + destPos;
				memcpy(dest, text.data, text.count*sizeof(tchar));
			};
		UBA_ASSERT(dwCreationFlags & CREATE_UNICODE_ENVIRONMENT);

		int addTempIndex = 0;
		for(StringView env = ToView((tchar*)lpEnvironment); env.count;)
		{
			u32 equalPos;
			env.Contains('=', &equalPos);
			StringView nameAndEquals = env.SubStr(0, equalPos+1);
			if (nameAndEquals.Equals(TCV("TEMP=")) || nameAndEquals.Equals(TCV("TMP="))) // We skip the already existing ones
			{
				env = ToView(env.data + env.count + 1);
				continue;
			}
			if (addTempIndex == 2) // Both TEMP and TMP are added, just add the rest
			{
				addChars({env.data, env.count+1});
				env = ToView(env.data + env.count + 1);
				continue;
			}
			StringView toAdd = addTempIndex == 0 ? TCV("TEMP=") : TCV("TMP=");
			int diff = _wcsnicmp(env.data, toAdd.data, toAdd.count); // Add them sorted, TEMP first and TMP second
			if (diff < 0)
			{
				addChars({env.data, env.count+1});
				env = ToView(env.data + env.count + 1);
				continue;
			}
			UBA_ASSERT(diff != 0);
			addChars({toAdd.data, toAdd.count});
			addChars({g_systemTemp.data, g_systemTemp.count+1});
			++addTempIndex;
		}
		while (addTempIndex < 2)
		{
			StringView toAdd = addTempIndex == 0 ? TCV("TEMP=") : TCV("TMP=");
			addChars({toAdd.data, toAdd.count});
			addChars({g_systemTemp.data, g_systemTemp.count+1});
			++addTempIndex;
		}

		#if UBA_DEBUG
		if (debugWineCalls)
			addChars(TCV("WINEDEBUG=,+file\0")); //"WINEDEBUG=+relay,+process,+file\0"
		#endif

		tempEnvironment.push_back(0);
		lpEnvironment = tempEnvironment.data();
	}

	// CREATE_NO_WINDOW is an optimization that prevents additional overhead per spawned child process.
	// .. disabled for now because there are hierarchies of processes that does not like this. Haven't figured out exactly why
	// .. can be reproed by running some of the very complex cmake/python scripts when compiling clang (lto.exports etc)
	bool createNoWindow = false; //!g_setStdHandleCalled; // If console mode has been called there are probably some sort of stdin/out rerouting stuff.. so let's not set CREATE_NO_WINDOW

	if (allowCreationFlagChange && (dwCreationFlags & (DETACHED_PROCESS | CREATE_NO_WINDOW)) == 0)
	{
		if (g_rules->AllowDetach())
			dwCreationFlags |= DETACHED_PROCESS;
		else if (createNoWindow)
			dwCreationFlags |= CREATE_NO_WINDOW;
		else
		{
			lpStartupInfo->dwFlags |= STARTF_USESHOWWINDOW;
			lpStartupInfo->wShowWindow = SW_HIDE;
		}
	}

	dwCreationFlags |= CREATE_SUSPENDED;
	BOOL res = true;
	u32 lastError = ERROR_SUCCESS;
	u32 retryCount = 0;

	//u64 createProcessStart = GetTime();

	DisallowDetourIncrement();
	while (true)
	{
		res = true;
		if (DetourCreateProcessWithDllsW(application.data, finalCommandLine, NULL, NULL, bInheritHandles, dwCreationFlags, lpEnvironment, currentDir.c_str(), lpStartupInfo, lpProcessInformation, dllCount, dlls, True_CreateProcessW))
			break;
		res = false;
		lastError = GetLastError();
		if (lastError != ERROR_ACCESS_DENIED && lastError != ERROR_INTERNAL_ERROR)
			break;
		// We have no idea why this is happening.. but it seems to recover when retrying.
		// Could it be related to two process spawning at the exact same time or something?
		// It happens extremely rarely and can happen on both host and remotes
		if (retryCount++ >= 5)
			break;
		const wchar_t* errorText = lastError == ERROR_ACCESS_DENIED ? L"access denied" : L"internal error";
		Rpc_WriteLogf(L"DetourCreateProcessWithDllEx failed with %ls (code: %u), retrying %ls (Working dir: %ls)", errorText, lastError, originalCmd.data, currentDir.c_str());
		Sleep(100 + (rand() % 200)); // We have no idea
		continue;
	}
	DisallowDetourDecrement();

	//DEBUG_LOG(L"CREATE PROCESS TOOK: %llums", TimeToMs(GetTime() - createProcessStart));

	#if UBA_DEBUG
	USHORT processMachine = 0;
	USHORT nativeMachine = 0;
	if (IsWow64Process2(lpProcessInformation->hProcess, &processMachine, &nativeMachine))
	{
		bool is64Bit = processMachine == IMAGE_FILE_MACHINE_UNKNOWN;
		UBA_ASSERTF(is64Bit, TC("Can't detour 32-bit application %s"), application.data);
	}
	#endif

	u32 trueProcessId = lpProcessInformation->dwProcessId;
	if (isChild)
	{
		RPC_MESSAGE(StartProcess, startProcess)
		--g_stats.createProcess.count; // Don't want double count for one process
		writer.WriteU32(processId);
		writer.WriteBool(res);
		writer.WriteU32(lastError);
		writer.WriteU64(u64(lpProcessInformation->hProcess));
		writer.WriteU32(trueProcessId);
		writer.WriteU64(u64(lpProcessInformation->hThread));
		writer.Flush();
		DEBUG_LOG_PIPE(L"StartProcess", L"%ls %ls", application.data, originalCmd.data);
	}

	UBA_ASSERTF(res, L"Failed to spawn process %ls (Error code: %u)", finalCommandLine, lastError);

	HANDLE trueHandle = lpProcessInformation->hProcess;
	if (!res || trueHandle == NULL)
	{
		DEBUG_LOG_DETOURED(L"CreateProcessW", L"FAILED");
		return FALSE;
	}
	auto detouredHandle = new DetouredHandle(HandleType_Process);
	detouredHandle->trueHandle = trueHandle;
	lpProcessInformation->hProcess = makeDetouredHandle(detouredHandle);

	DEBUG_LOG_DETOURED(L"CreateProcessW", L"(RESULT) %s %llu (true: %llu pid: %u)", application.data, u64(lpProcessInformation->hProcess), u64(trueHandle), trueProcessId);
	return TRUE;
}

BOOL Detoured_CreateProcessA(LPCSTR lpApplicationName, LPSTR lpCommandLine, LPSECURITY_ATTRIBUTES lpProcessAttributes, LPSECURITY_ATTRIBUTES lpThreadAttributes, BOOL bInheritHandles,
	DWORD dwCreationFlags, LPVOID lpEnvironment, LPCSTR lpCurrentDirectory, LPSTARTUPINFOA lpStartupInfo, LPPROCESS_INFORMATION lpProcessInformation)
{
	wchar_t* lpApplicationNameW = nullptr;
	TString lpApplicationNameTemp;
	if (lpApplicationName)
	{
		lpApplicationNameTemp = TString(lpApplicationName, lpApplicationName + strlen(lpApplicationName));
		lpApplicationNameW = lpApplicationNameTemp.data();
	}
	wchar_t* lpCommandLineW = nullptr;
	TString lpCommandLineTemp;
	if (lpCommandLine)
	{
		lpCommandLineTemp = TString(lpCommandLine, lpCommandLine + strlen(lpCommandLine));
		lpCommandLineW = lpCommandLineTemp.data();
	}
	wchar_t* lpCurrentDirectoryW = nullptr;
	TString lpCurrentDirectoryTemp;
	if (lpCurrentDirectory)
	{
		lpCurrentDirectoryTemp = TString(lpCurrentDirectory, lpCurrentDirectory + strlen(lpCurrentDirectory));
		lpCurrentDirectoryW = lpCurrentDirectoryTemp.data();
	}

	UBA_ASSERT(!lpStartupInfo->lpReserved);
	UBA_ASSERT(!lpStartupInfo->lpDesktop);
	UBA_ASSERT(!lpStartupInfo->lpTitle);

	STARTUPINFOW lpStartupInfoW = *(LPSTARTUPINFOW)lpStartupInfo;
	return Detoured_CreateProcessW(lpApplicationNameW, lpCommandLineW, lpProcessAttributes, lpThreadAttributes, bInheritHandles, dwCreationFlags, lpEnvironment, lpCurrentDirectoryW, &lpStartupInfoW, lpProcessInformation);
}

BOOL Detoured_RegisterWaitForSingleObject(PHANDLE phNewWaitObject, HANDLE hObject, WAITORTIMERCALLBACK Callback, PVOID Context, ULONG dwMilliseconds, ULONG dwFlags)
{
	HANDLE trueHandle = hObject;
	if (isDetouredHandle(hObject))
		trueHandle = asDetouredHandle(hObject).trueHandle;

	return True_RegisterWaitForSingleObject(phNewWaitObject, trueHandle, Callback, Context, dwMilliseconds, dwFlags);
}

BOOL Detoured_GetExitCodeProcess(HANDLE hProcess, LPDWORD lpExitCode)
{
	DETOURED_CALL(GetExitCodeProcess);
	HANDLE trueHandle = hProcess;
	if (isDetouredHandle(hProcess))
		trueHandle = asDetouredHandle(hProcess).trueHandle;
	BOOL res = True_GetExitCodeProcess(trueHandle, lpExitCode);

	DEBUG_LOG_DETOURED(L"GetExitCodeProcess", L"%llu Exit code: %lu -> %ls", uintptr_t(trueHandle), *lpExitCode, ToString(res));

	if (res != STILL_ACTIVE)
		Rpc_UpdateTables();

	return res;
}

BOOL Detoured_CreateTimerQueueTimer(PHANDLE phNewTimer, HANDLE TimerQueue, WAITORTIMERCALLBACK Callback, PVOID Parameter, DWORD DueTime, DWORD Period, ULONG Flags)
{
	DETOURED_CALL(CreateTimerQueueTimer);
	BOOL res = True_CreateTimerQueueTimer(phNewTimer, TimerQueue, Callback, Parameter, DueTime, Period, Flags);
	DEBUG_LOG_TRUE(L"CreateTimerQueueTimer", L"%p -> %ls", *phNewTimer, ToString(res));
	return res;
}

BOOL Detoured_DeleteTimerQueueTimer(HANDLE TimerQueue, HANDLE Timer, HANDLE CompletionEvent)
{
	DETOURED_CALL(DeleteTimerQueueTimer);
	BOOL res = True_DeleteTimerQueueTimer(TimerQueue, Timer, CompletionEvent);
	if (!res && IsRunningWine())
	{
		DEBUG_LOG_DETOURED(L"DeleteTimerQueueTimer", L"%p %p %p -> %ls (WINE ignored)", TimerQueue, Timer, CompletionEvent, ToString(res));
		return true;
	}
	DEBUG_LOG_TRUE(L"DeleteTimerQueueTimer", L"%p %p %p -> %ls", TimerQueue, Timer, CompletionEvent, ToString(res));
	return res;
}

DWORD Detoured_WaitForSingleObject(HANDLE hHandle, DWORD dwMilliseconds)
{
	return WaitForSingleObjectEx(hHandle, dwMilliseconds, false);
}

// Both WaitForSingleObject and WaitForSingleObjectEx is needed to support Wine
DWORD Detoured_WaitForSingleObjectEx(HANDLE hHandle, DWORD dwMilliseconds, BOOL bAlertable)
{
	DETOURED_CALL(WaitForSingleObjectEx);
	bool isProcess = false;
	HANDLE trueHandle = hHandle;
	if (isDetouredHandle(hHandle))
	{
		DetouredHandle& dh = asDetouredHandle(hHandle);
		trueHandle = asDetouredHandle(hHandle).trueHandle;
		isProcess = dh.type == HandleType_Process;
	}

	auto res = True_WaitForSingleObjectEx(trueHandle, dwMilliseconds, bAlertable);

	if (res != WAIT_OBJECT_0 || !isProcess)
		return res;

#if UBA_DEBUG_LOG_ENABLED
	if (isLogging())
	{
		auto lastError = GetLastError();
		DWORD exitCode;
		True_GetExitCodeProcess(trueHandle, &exitCode);
		DEBUG_LOG_DETOURED(L"WaitForSingleObjectEx", L"for process %llu (0x%llx). Exit code: %lu", u64(hHandle), u64(trueHandle), exitCode);
		SetLastError(lastError);
	}
#endif

	Rpc_UpdateTables();

	return res;
}
DWORD Detoured_WaitForMultipleObjects(DWORD nCount, const HANDLE* lpHandles, BOOL bWaitAll, DWORD dwMilliseconds)
{
	DETOURED_CALL(WaitForMultipleObjects);
	
	UBA_ASSERT(nCount != 0);

	bool hasProcess = false;
	auto tempHandles = (HANDLE*)_malloca(nCount * sizeof(HANDLE));
	auto isProcess = (bool*)_malloca(nCount * sizeof(bool));
	if (!tempHandles || !isProcess)
		FatalError(1355, L"Here to please static analyzer");

#ifndef __clang_analyzer__
	auto g = MakeGuard([&]() { _freea(tempHandles); _freea(isProcess); });
#endif

	for (u32 i = 0; i != nCount; ++i)
	{
		HANDLE hHandle = lpHandles[i];
		isProcess[i] = false;
		if (isDetouredHandle(hHandle))
		{
			DetouredHandle& dh = asDetouredHandle(hHandle);
			hHandle = asDetouredHandle(hHandle).trueHandle;
			if (dh.type == HandleType_Process)
			{
				hasProcess = true;
				isProcess[i] = true;
			}
		}
		tempHandles[i] = hHandle;
	}

	auto res = True_WaitForMultipleObjectsEx(nCount, tempHandles, bWaitAll, dwMilliseconds, false);


	if (!hasProcess)
		return res;

	if (res < WAIT_OBJECT_0 || res > WAIT_OBJECT_0 + nCount)
		return res;
	
	if (!isProcess[res - WAIT_OBJECT_0])
		return res;

	DEBUG_LOG_DETOURED(L"WaitForMultipleObjects", L"");

	Rpc_UpdateTables();

	return res;
}

DWORD Detoured_WaitForMultipleObjectsEx(DWORD nCount, const HANDLE* lpHandles, BOOL bWaitAll, DWORD dwMilliseconds, BOOL bAlertable)
{
	DETOURED_CALL(WaitForMultipleObjectsEx);

	bool isProcess = false;
	HANDLE* tempHandles = (HANDLE*)_malloca(nCount * sizeof(HANDLE));
	if (!tempHandles)
		FatalError(1355, L"Here to please static analyzer");

	for (u32 i = 0; i != nCount; ++i)
	{
		HANDLE hHandle = lpHandles[i];
		if (isDetouredHandle(hHandle))
		{
			DetouredHandle& dh = asDetouredHandle(hHandle);
			hHandle = asDetouredHandle(hHandle).trueHandle;
			isProcess |= dh.type == HandleType_Process;
		}
		tempHandles[i] = hHandle;
	}

	auto res = True_WaitForMultipleObjectsEx(nCount, tempHandles, bWaitAll, dwMilliseconds, bAlertable);
#ifndef __clang_analyzer__
	_freea(tempHandles);
#endif

	if (!isProcess || res != WAIT_OBJECT_0)
		return res;

	DEBUG_LOG_DETOURED(L"WaitForMultipleObjectsEx", L"");

	Rpc_UpdateTables();

	return res;
}

LANGID Detoured_GetUserDefaultUILanguage()
{
	DETOURED_CALL(GetUserDefaultUILanguage);
	DEBUG_LOG_DETOURED(L"GetUserDefaultUILanguage", L"");
	//UBA_ASSERTF(g_runningRemote || True_GetUserDefaultUILanguage() == g_uiLanguage, L"Session process has language id %u while this process is set to use %u", g_uiLanguage, True_GetUserDefaultUILanguage());
	return LANGID(g_uiLanguage);
}

BOOL Detoured_GetThreadPreferredUILanguages(DWORD dwFlags, PULONG pulNumLanguages, PZZWSTR pwszLanguagesBuffer, PULONG pcchLanguagesBuffer)
{
	DETOURED_CALL(GetThreadPreferredUILanguages);

	if (dwFlags & MUI_LANGUAGE_ID)
	{
		UBA_ASSERT(pulNumLanguages);
		UBA_ASSERT(pcchLanguagesBuffer);
		//UBA_ASSERT(!(dwFlags & ~MUI_LANGUAGE_ID);
		*pulNumLanguages = 1;
		*pcchLanguagesBuffer = 6;

		if (!pwszLanguagesBuffer)
		{
			DEBUG_LOG_DETOURED(L"GetThreadPreferredUILanguages", L"(nobuf) -> TRUE");
			return TRUE;
		}
		swprintf_s(pwszLanguagesBuffer, 6, L"%04x", g_uiLanguage);
		pwszLanguagesBuffer[5] = 0;
		DEBUG_LOG_DETOURED(L"GetThreadPreferredUILanguages", L"(%s) -> TRUE", pwszLanguagesBuffer);
		return TRUE;
	}
	else // MUI_LANGUAGE_NAME
	{
		// TODO: We need to get the string of g_uiLanguage
		//UBA_ASSERTF(!g_runningRemote, L"GetThreadPreferredUILanguages uses unsupported flag on remote execution: %u", dwFlags);
		auto res = True_GetThreadPreferredUILanguages(dwFlags, pulNumLanguages, pwszLanguagesBuffer, pcchLanguagesBuffer);
		DEBUG_LOG_TRUE(L"GetThreadPreferredUILanguages", L"-> %ls", ToString(res));
		return res;
	}
}

#if DETOURED_INCLUDE_DEBUG

BOOL Detoured_DeviceIoControl(HANDLE hDevice, DWORD dwIoControlCode, LPVOID lpInBuffer, DWORD nInBufferSize, LPVOID lpOutBuffer, DWORD  nOutBufferSize, LPDWORD lpBytesReturned, LPOVERLAPPED lpOverlapped)
{
	DEBUG_LOG_PROXY(L"DeviceIoControl", L"");
	return True_DeviceIoControl(hDevice, dwIoControlCode, lpInBuffer, nInBufferSize, lpOutBuffer, nOutBufferSize, lpBytesReturned, lpOverlapped);
}

BOOL Detoured_GetDiskFreeSpaceExA(LPCSTR lpDirectoryName, PULARGE_INTEGER lpFreeBytesAvailableToCaller, PULARGE_INTEGER lpTotalNumberOfBytes, PULARGE_INTEGER lpTotalNumberOfFreeBytes)
{
	DETOURED_CALL(GetDiskFreeSpaceExA);
	DEBUG_LOG_TRUE(L"GetDiskFreeSpaceExA", L"%hs", lpDirectoryName);
	return True_GetDiskFreeSpaceExA(lpDirectoryName, lpFreeBytesAvailableToCaller, lpTotalNumberOfBytes, lpTotalNumberOfFreeBytes);
}

DWORD Detoured_GetLongPathNameA(LPCSTR lpszShortPath, LPSTR lpszLongPath, DWORD cchBuffer)
{
	DETOURED_CALL(GetLongPathNameA);
	DEBUG_LOG_TRUE(L"GetLongPathNameA", L"");
	UBA_ASSERT(!g_runningRemote);
	return True_GetLongPathNameA(lpszShortPath, lpszLongPath, cchBuffer);
}

BOOL Detoured_GetVolumePathNameA(LPCSTR lpszFileName, LPSTR lpszVolumePathName, DWORD cchBufferLength)
{
	DETOURED_CALL(GetVolumePathNameA);
	DEBUG_LOG_TRUE(L"GetVolumePathNameA", L"%S", lpszFileName);
	return True_GetVolumePathNameA(lpszFileName, lpszVolumePathName, cchBufferLength);
}

HANDLE Detoured_CreateFileW(LPCWSTR lpFileName, DWORD dwDesiredAccess, DWORD dwShareMode, LPSECURITY_ATTRIBUTES lpSecurityAttributes, DWORD dwCreationDisposition, DWORD dwFlagsAndAttributes, HANDLE hTemplateFile)
{
	DETOURED_CALL(CreateFileW);
	DEBUG_LOG_PROXY(L"CreateFileW", L"%ls", lpFileName);
	return True_CreateFileW(lpFileName, dwDesiredAccess, dwShareMode, lpSecurityAttributes, dwCreationDisposition, dwFlagsAndAttributes, hTemplateFile);
}

HANDLE Detoured_CreateFileA(LPCSTR lpFileName, DWORD dwDesiredAccess, DWORD dwShareMode, LPSECURITY_ATTRIBUTES lpSecurityAttributes, DWORD dwCreationDisposition, DWORD dwFlagsAndAttributes, HANDLE hTemplateFile)
{
	DETOURED_CALL(CreateFileA);
	DEBUG_LOG_TRUE(L"CreateFileA", L"%hs", lpFileName);
	return True_CreateFileA(lpFileName, dwDesiredAccess, dwShareMode, lpSecurityAttributes, dwCreationDisposition, dwFlagsAndAttributes, hTemplateFile);
}

DWORD Detoured_GetFileSize(HANDLE hFile, LPDWORD lpFileSizeHigh)
{
	DETOURED_CALL(GetFileSize);
	DEBUG_LOG_PROXY(L"GetFileSize", L"%llu (%ls)", uintptr_t(hFile), HandleToName(hFile));
	return True_GetFileSize(hFile, lpFileSizeHigh); // Calls NtQueryInformationFile
}

DWORD Detoured_GetFileSizeEx(HANDLE hFile, PLARGE_INTEGER lpFileSize)
{
	DETOURED_CALL(GetFileSizeEx);
	DEBUG_LOG_PROXY(L"GetFileSizeEx", L"%llu (%ls)", uintptr_t(hFile), HandleToName(hFile));
	return True_GetFileSizeEx(hFile, lpFileSize); // Calls NtQueryInformationFile
}

BOOL Detoured_GetFileTime(HANDLE hFile, LPFILETIME lpCreationTime, LPFILETIME lpLastAccessTime, LPFILETIME lpLastWriteTime)
{
	DETOURED_CALL(GetFileTime);
	DEBUG_LOG_TRUE(L"GetFileTime", L"%llu (%ls)", uintptr_t(hFile), HandleToName(hFile));
	return True_GetFileTime(hFile, lpCreationTime, lpLastAccessTime, lpLastWriteTime);
}

DWORD Detoured_SetFilePointer(HANDLE hFile, LONG lDistanceToMove, PLONG lpDistanceToMoveHigh, DWORD dwMoveMethod)
{
	DETOURED_CALL(SetFilePointer);
	DEBUG_LOG_PROXY(L"SetFilePointer", L"%llu %lli %lu (%ls)", uintptr_t(hFile), ToLargeInteger(lpDistanceToMoveHigh ? *lpDistanceToMoveHigh : 0ll, lDistanceToMove).QuadPart, dwMoveMethod, HandleToName(hFile));
	return True_SetFilePointer(hFile, lDistanceToMove, lpDistanceToMoveHigh, dwMoveMethod);
}

DWORD Detoured_SetFilePointerEx(HANDLE hFile, LARGE_INTEGER liDistanceToMove, PLARGE_INTEGER lpNewFilePointer, DWORD dwMoveMethod)
{
	DETOURED_CALL(SetFilePointerEx);
	DEBUG_LOG_PROXY(L"SetFilePointerEx", L"%llu %lli %lu (%ls)", uintptr_t(hFile), liDistanceToMove.QuadPart, dwMoveMethod, HandleToName(hFile));
	return True_SetFilePointerEx(hFile, liDistanceToMove, lpNewFilePointer, dwMoveMethod); // This ends up in NtQueryInformationFile and NtSetInformationFile
}

BOOL Detoured_SetEndOfFile(HANDLE hFile)
{
	DETOURED_CALL(SetEndOfFile);
	DEBUG_LOG_PROXY(L"SetEndOfFile", L"%llu (%ls)", uintptr_t(hFile), HandleToName(hFile));
	return True_SetEndOfFile(hFile);
}

DWORD Detoured_GetFileAttributesA(LPCSTR lpFileName)
{
	// Is verified that both windows and wine are calling GetFileAttributesW
	DEBUG_LOG_TRUE(L"GetFileAttributesA", L"%S", lpFileName);
	return True_GetFileAttributesA(lpFileName);
}

BOOL Detoured_GetFileAttributesExA(LPCSTR lpFileName, GET_FILEEX_INFO_LEVELS fInfoLevelId, LPVOID lpFileInformation)
{
	DETOURED_CALL(GetFileAttributesExA);
	DEBUG_LOG_TRUE(L"GetFileAttributesExA", L"%S", lpFileName); // Calls ExW on both windows and wine
	return True_GetFileAttributesExA(lpFileName, fInfoLevelId, lpFileInformation);
}

#if !defined(_M_ARM64)
HMODULE Detoured_LoadLibraryW(LPCWSTR lpLibFileName)
{
	DETOURED_CALL(LoadLibraryW);
	DEBUG_LOG_TRUE(L"LoadLibraryW", L"(%ls)", lpLibFileName);
	return True_LoadLibraryW(lpLibFileName);
}
#endif

BOOL Detoured_SetDllDirectoryW(LPCWSTR lpPathName)
{
	DETOURED_CALL(SetDllDirectoryW);
	BOOL res = True_SetDllDirectoryW(lpPathName);
	DEBUG_LOG_TRUE(L"SetDllDirectoryW", L"(%ls) -> %s", lpPathName, ToString(res));
	return res;
}

BOOL Detoured_GetDllDirectoryW(DWORD nBufferLength, LPWSTR lpBuffer)
{
	DETOURED_CALL(GetDllDirectoryW);
	BOOL res = True_GetDllDirectoryW(nBufferLength, lpBuffer);
	DEBUG_LOG_TRUE(L"GetDllDirectoryW", L"%lu -> %s (%ls)", nBufferLength, ToString(res), lpBuffer);
	return res;
}

DWORD Detoured_GetModuleBaseNameA(HANDLE hProcess, HMODULE hModule, LPSTR lpBaseName, DWORD nSize)
{
	DETOURED_CALL(GetModuleBaseNameA);
	DEBUG_LOG_TRUE(L"GetModuleBaseNameA", L"");

	char temp[1024];
	DWORD res = GetModuleFileNameExA(hProcess, hModule, temp, sizeof_array(temp)); (void)res;
	UBA_ASSERT(res != 0 && res < sizeof_array(temp));
	char* moduleName = temp;
	if (char* lastSlash = strrchr(temp, '\\'))
		moduleName = lastSlash + 1;
	DWORD len = (DWORD)strlen(moduleName);
	UBA_ASSERTF(len < nSize, L"Module name %hs does not fit in buffer size (is %lu, needs %lu)", moduleName, nSize, len);
	strcpy_s(lpBaseName, nSize, moduleName);
	memset(lpBaseName + len, 0, nSize - len);
	return len;
}

DWORD Detoured_GetModuleBaseNameW(HANDLE hProcess, HMODULE hModule, LPWSTR lpBaseName, DWORD nSize)
{
	DETOURED_CALL(GetModuleBaseNameW);
	DEBUG_LOG_TRUE(L"GetModuleBaseNameW", L"");

	wchar_t temp[1024];
	DWORD res = GetModuleFileNameExW(hProcess, hModule, temp, sizeof_array(temp)); (void)res;
	UBA_ASSERT(res != 0 && res < sizeof_array(temp));
	wchar_t* moduleName = temp;
	if (wchar_t* lastSlash = wcsrchr(temp, '\\'))
		moduleName = lastSlash + 1;
	DWORD len = (DWORD)wcslen(moduleName);
	UBA_ASSERTF(len < nSize, L"Module name %s does not fit in buffer size (is %lu, needs %lu)", moduleName, nSize, len);
	wcscpy_s(lpBaseName, nSize, moduleName);
	return len;
}

LPTOP_LEVEL_EXCEPTION_FILTER Detoured_SetUnhandledExceptionFilter(LPTOP_LEVEL_EXCEPTION_FILTER lpTopLevelExceptionFilter)
{
	DETOURED_CALL(SetUnhandledExceptionFilter);
	DEBUG_LOG_TRUE(L"SetUnhandledExceptionFilter", L"");
	return True_SetUnhandledExceptionFilter(lpTopLevelExceptionFilter);
}

BOOL Detoured_FlushInstructionCache(HANDLE hProcess, LPCVOID lpBaseAddress, SIZE_T dwSize)
{
	DETOURED_CALL(FlushInstructionCache);
	UBA_ASSERT(!isDetouredHandle(hProcess));
	BOOL res = True_FlushInstructionCache(hProcess, lpBaseAddress, dwSize);
	//DEBUG_LOG_DETOURED(L"FlushInstructionCache", L"%llu -> %ls", uintptr_t(hProcess), ToString(res));
	return res;
}

HANDLE Detoured_CreateFile2(LPCWSTR lpFileName, DWORD dwDesiredAccess, DWORD dwShareMode, DWORD dwCreationDisposition, LPCREATEFILE2_EXTENDED_PARAMETERS pCreateExParams)
{
	DETOURED_CALL(CreateFile2);
	DEBUG_LOG_TRUE(L"CreateFile2", L"(%ls)", lpFileName);
	return True_CreateFile2(lpFileName, dwDesiredAccess, dwShareMode, dwCreationDisposition, pCreateExParams);
}

HANDLE Detoured_CreateFileTransactedW(LPCWSTR lpFileName, DWORD dwDesiredAccess, DWORD dwShareMode, LPSECURITY_ATTRIBUTES lpSecurityAttributes, DWORD dwCreationDisposition,
	DWORD dwFlagsAndAttributes, HANDLE hTemplateFile, HANDLE hTransaction, PUSHORT pusMiniVersion, PVOID lpExtendedParameter)
{
	DETOURED_CALL(CreateFileTransactedW);
	DEBUG_LOG_TRUE(L"CreateFileTransacted", L"(%ls)", lpFileName);
	return True_CreateFileTransactedW(lpFileName, dwDesiredAccess, dwShareMode, lpSecurityAttributes, dwCreationDisposition, dwFlagsAndAttributes, hTemplateFile, hTransaction, pusMiniVersion, lpExtendedParameter);
}

HFILE Detoured_OpenFile(LPCSTR lpFileName, LPOFSTRUCT lpReOpenBuff, UINT uStyle)
{
	DETOURED_CALL(OpenFile);
	DEBUG_LOG_TRUE(L"OpenFile", L"(%hs)", lpFileName);
	return True_OpenFile(lpFileName, lpReOpenBuff, uStyle);
}

HANDLE Detoured_ReOpenFile(HANDLE hOriginalFile, DWORD dwDesiredAccess, DWORD dwShareMode, DWORD dwFlagsAndAttributes)
{
	DETOURED_CALL(ReOpenFile);
	if (isDetouredHandle(hOriginalFile))
	{
		DEBUG_LOG_DETOURED(L"TODO ReOpenFile", L"(%ls)", HandleToName(hOriginalFile));
		return INVALID_HANDLE_VALUE;
	}
	DEBUG_LOG_TRUE(L"ReOpenFile", L"(%ls)", HandleToName(hOriginalFile));
	return True_ReOpenFile(hOriginalFile, dwDesiredAccess, dwShareMode, dwFlagsAndAttributes);
}

BOOL Detoured_ReadFile(HANDLE hFile, LPVOID lpBuffer, DWORD nNumberOfBytesToRead, LPDWORD lpNumberOfBytesRead, LPOVERLAPPED lpOverlapped)
{
	DETOURED_CALL(ReadFile);
	t_ntReadFileCalled = false;
	BOOL res = True_ReadFile(hFile, lpBuffer, nNumberOfBytesToRead, lpNumberOfBytesRead, lpOverlapped);
	UBA_ASSERT(t_ntReadFileCalled);
	return res;
}

BOOL Detoured_ReadFileEx(HANDLE hFile, LPVOID lpBuffer, DWORD nNumberOfBytesToRead, LPOVERLAPPED lpOverlapped, LPOVERLAPPED_COMPLETION_ROUTINE lpCompletionRoutine)
{
	DETOURED_CALL(ReadFileEx);
	DEBUG_LOG_TRUE(L"ReadFileEx", L"%llu (%ls)", uintptr_t(hFile), HandleToName(hFile));
	t_ntReadFileCalled = false;
	BOOL res = True_ReadFileEx(hFile, lpBuffer, nNumberOfBytesToRead, lpOverlapped, lpCompletionRoutine);
	UBA_ASSERT(t_ntReadFileCalled);
	return res;
}

BOOL Detoured_ReadFileScatter(HANDLE hFile, FILE_SEGMENT_ELEMENT* aSegmentArray, DWORD nNumberOfBytesToRead, LPDWORD lpReserved, LPOVERLAPPED lpOverlapped)
{
	DETOURED_CALL(ReadFileScatter);
	DEBUG_LOG_TRUE(L"ReadFileScatter", L"%llu (%ls)", uintptr_t(hFile), HandleToName(hFile));
	UBA_ASSERT(!isDetouredHandle(hFile));
	UBA_ASSERT(!isListDirectoryHandle(hFile));
	return True_ReadFileScatter(hFile, aSegmentArray, nNumberOfBytesToRead, lpReserved, lpOverlapped);
}

BOOL Detoured_WriteFile(HANDLE hFile, LPCVOID lpBuffer, DWORD nNumberOfBytesToWrite, LPDWORD lpNumberOfBytesWritten, LPOVERLAPPED lpOverlapped)
{
	t_ntWriteFileCalled = false;
	BOOL res = True_WriteFile(hFile, lpBuffer, nNumberOfBytesToWrite, lpNumberOfBytesWritten, lpOverlapped);
	UBA_ASSERT(t_ntWriteFileCalled);
	return res;
}

BOOL Detoured_WriteFileEx(HANDLE hFile, LPCVOID lpBuffer, DWORD nNumberOfBytesToWrite, LPOVERLAPPED lpOverlapped, LPOVERLAPPED_COMPLETION_ROUTINE lpCompletionRoutine)
{
	t_ntWriteFileCalled = false;
	BOOL res = True_WriteFileEx(hFile, lpBuffer, nNumberOfBytesToWrite, lpOverlapped, lpCompletionRoutine);
	UBA_ASSERT(t_ntWriteFileCalled);
	return res;
}

#if !defined(_M_ARM64)
void Detoured_SetLastError(DWORD dwErrCode)
{
	DETOURED_CALL(SetLastError);
	if (dwErrCode != ERROR_SUCCESS)
		while (false) {}
	True_SetLastError(dwErrCode);
}
#endif

#if !defined(_M_ARM64)
DWORD Detoured_GetLastError()
{
	DETOURED_CALL(GetLastError);
	auto res = True_GetLastError();
	if (res != ERROR_SUCCESS)
		while (false) {}
	return res;
}
#endif

BOOL Detoured_SetFileValidData(HANDLE hFile, LONGLONG ValidDataLength)
{
	DETOURED_CALL(SetFileValidData);
	DEBUG_LOG_TRUE(L"SetFileValidData", L"(%ls)", HandleToName(hFile));
	UBA_ASSERT(!isDetouredHandle(hFile));
	return True_SetFileValidData(hFile, ValidDataLength);
}

BOOL Detoured_ReplaceFileW(LPCWSTR lpReplacedFileName, LPCWSTR lpReplacementFileName, LPCWSTR lpBackupFileName, DWORD dwReplaceFlags, LPVOID lpExclude, LPVOID lpReserved)
{
	UBA_ASSERT(!g_runningRemote);
	DETOURED_CALL(ReplaceFileW);
	DEBUG_LOG_TRUE(L"ReplaceFileW", L"");
	return True_ReplaceFileW(lpReplacedFileName, lpReplacementFileName, lpBackupFileName, dwReplaceFlags, lpExclude, lpReserved);
}

BOOL Detoured_CreateHardLinkA(LPCSTR lpFileName, LPCSTR lpExistingFileName, LPSECURITY_ATTRIBUTES lpSecurityAttributes)
{
	UBA_ASSERT(!g_runningRemote);
	DETOURED_CALL(CreateHardLinkA);
	DEBUG_LOG_TRUE(L"CreateHardLinkA", L"");
	return True_CreateHardLinkA(lpFileName, lpExistingFileName, lpSecurityAttributes);
}

BOOL Detoured_DeleteFileA(LPCSTR lpFileName)
{
	DETOURED_CALL(DeleteFileA);
	DEBUG_LOG_PROXY(L"DeleteFileA", L"");
	t_calledDeleteFileW = false;
	BOOL res = True_DeleteFileA(lpFileName);
	UBA_ASSERT(t_calledDeleteFileW);
	return res;
}

DWORD Detoured_GetShortPathNameW(LPCWSTR lpszLongPath, LPWSTR lpszShortPath, DWORD cchBuffer)
{
	// This is using FindFirstFileExW/FindClose so it is fine to let it through
	DETOURED_CALL(GetShortPathNameW);
	DEBUG_LOG_TRUE(L"GetShortPathNameW", L"");
	return True_GetShortPathNameW(lpszLongPath, lpszShortPath, cchBuffer);
}

BOOL Detoured_NeedCurrentDirectoryForExePathW(LPCWSTR ExeName)
{
	//UBA_ASSERT(!g_runningRemote); // This is probably fine
	DEBUG_LOG_TRUE(L"NeedCurrentDirectoryForExePathW", L"");
	return True_NeedCurrentDirectoryForExePathW(ExeName);
}

BOOL Detoured_ReadDirectoryChangesW(HANDLE hDirectory, LPVOID lpBuffer, DWORD nBufferLength, BOOL bWatchSubtree, DWORD dwNotifyFilter, LPDWORD lpBytesReturned, LPOVERLAPPED lpOverlapped, LPOVERLAPPED_COMPLETION_ROUTINE lpCompletionRoutine)
{
	UBA_ASSERT(!g_runningRemote && !isDetouredHandle(hDirectory));
	DEBUG_LOG_TRUE(L"ReadDirectoryChangesW", L"");
	return True_ReadDirectoryChangesW(hDirectory, lpBuffer, nBufferLength, bWatchSubtree, dwNotifyFilter, lpBytesReturned, lpOverlapped, lpCompletionRoutine);
}

BOOL Detoured_SetCurrentDirectoryA(LPCSTR lpPathName)
{
	DETOURED_CALL(SetCurrentDirectoryA);
	DEBUG_LOG_TRUE(L"SetCurrentDirectoryA", L"%hs", lpPathName);
	return True_SetCurrentDirectoryA(lpPathName);
}
BOOLEAN Detoured_CreateSymbolicLinkW(LPCWSTR lpSymlinkFileName, LPCWSTR lpTargetFileName, DWORD dwFlags)
{
	UBA_ASSERT(!g_runningRemote);
	DETOURED_CALL(CreateSymbolicLinkW);
	DEBUG_LOG_TRUE(L"CreateSymbolicLinkW", L"");
	return True_CreateSymbolicLinkW(lpSymlinkFileName, lpTargetFileName, dwFlags);
}

BOOLEAN Detoured_CreateSymbolicLinkA(LPCSTR lpSymlinkFileName, LPCSTR lpTargetFileName, DWORD  dwFlags)
{
	UBA_ASSERT(!g_runningRemote);
	DETOURED_CALL(CreateSymbolicLinkA);
	DEBUG_LOG_TRUE(L"CreateSymbolicLinkA", L"");
	return True_CreateSymbolicLinkA(lpSymlinkFileName, lpTargetFileName, dwFlags);
}

DWORD Detoured_SetEnvironmentVariableW(LPCWSTR lpName, LPWSTR lpValue)
{
	DETOURED_CALL(SetEnvironmentVariableW);
	DWORD res = True_SetEnvironmentVariableW(lpName, lpValue);
	DEBUG_LOG_TRUE(L"SetEnvironmentVariableW", L"%ls -> %ls", lpName, lpValue);
	return res;
}

DWORD Detoured_GetEnvironmentVariableW(LPCWSTR lpName, LPWSTR lpBuffer, DWORD nSize)
{
	DETOURED_CALL(GetEnvironmentVariableW);
	DWORD res = True_GetEnvironmentVariableW(lpName, lpBuffer, nSize);
	DEBUG_LOG_TRUE(L"GetEnvironmentVariableW", L"%ls -> %ls", lpName, res ? lpBuffer : L"NOTFOUND");
	return res;
}

DWORD Detoured_GetEnvironmentVariableA(LPCSTR lpName, LPSTR lpBuffer, DWORD nSize)
{
	DETOURED_CALL(GetEnvironmentVariableA);
	DWORD res = True_GetEnvironmentVariableA(lpName, lpBuffer, nSize);
	DEBUG_LOG_TRUE(L"GetEnvironmentVariableA", L"%hs -> %hs", lpName, res ? lpBuffer : "NOTFOUND");
	return res;
}

LPWCH Detoured_GetEnvironmentStringsW()
{
	DETOURED_CALL(GetEnvironmentStringsW);
	auto res = True_GetEnvironmentStringsW();
	DEBUG_LOG_TRUE(L"GetEnvironmentStringsW", L"-> 0x%llu", u64(res));

	#if 0 // Enable to print out environment variables in the log
	auto it = res;
	while (*it)
	{
		DEBUG_LOG(L"		VAR: %ls", it);
		it += wcslen(it) + 1;
	}
	#endif

	return res;
}

DWORD Detoured_GetTempPathW(DWORD nBufferLength, LPWSTR lpBuffer)
{
	DETOURED_CALL(GetTempPathW);
	DWORD res = True_GetTempPathW(nBufferLength, lpBuffer);
	DEBUG_LOG_TRUE(L"GetTempPathW", L"%s -> %lu", lpBuffer, res);
	return res;
}

DWORD Detoured_GetTempPath2W(DWORD nBufferLength, LPWSTR lpBuffer)
{
	DETOURED_CALL(GetTempPathW);
	DWORD res = True_GetTempPath2W(nBufferLength, lpBuffer);
	DEBUG_LOG_TRUE(L"GetTempPath2W", L"%s -> %lu", lpBuffer, res);
	return res;
}

DWORD Detoured_ExpandEnvironmentStringsW(LPCWSTR lpSrc, LPWSTR lpDst, DWORD nSize)
{
	DETOURED_CALL(ExpandEnvironmentStringsW);
	DEBUG_LOG_TRUE(L"ExpandEnvironmentStringsW", L"%ls", lpSrc);
	return True_ExpandEnvironmentStringsW(lpSrc, lpDst, nSize);
}

HANDLE Detoured_FindFirstFileNameW(LPCWSTR lpFileName, DWORD dwFlags, LPDWORD StringLength, PWSTR LinkName)
{
	DETOURED_CALL(FindFirstFileNameW);
	DEBUG_LOG_TRUE(L"FindFirstFileNameW", L"%ls", lpFileName);
	return True_FindFirstFileNameW(lpFileName, dwFlags, StringLength, LinkName);
}

UINT Detoured_GetTempFileNameW(LPCWSTR lpPathName, LPCWSTR lpPrefixString, UINT uUnique, LPTSTR lpTempFileName)
{
	DETOURED_CALL(GetTempFileNameW);
	DEBUG_LOG_TRUE(L"GetTempFileNameW", L"%s %s", lpPathName, lpPrefixString);
	return True_GetTempFileNameW(lpPathName, lpPrefixString, uUnique, lpTempFileName);
}

DWORD Detoured_GetShortPathNameA(LPCSTR lpszLongPath, LPSTR lpszShortPath, DWORD cchBuffer)
{
	// This is using FindFirstFileExW/FindClose so it is fine to let it through
	DETOURED_CALL(GetShortPathNameA);
	DWORD res = True_GetShortPathNameA(lpszLongPath, lpszShortPath, cchBuffer);
	DEBUG_LOG_TRUE(L"GetShortPathNameA", L"%hs (%lu) -> %lu (%hs)", lpszLongPath, cchBuffer, res, res ? lpszShortPath : "error");
	return res;
}

DWORD Detoured_GetTempPathA(DWORD nBufferLength, LPSTR lpBuffer)
{
	DETOURED_CALL(GetTempPathA);
	DEBUG_LOG_TRUE(L"GetTempPathA", L"");
	return True_GetTempPathA(nBufferLength, lpBuffer);
}

UINT Detoured_GetTempFileNameA(LPCSTR lpPathName, LPCSTR lpPrefixString, UINT uUnique, LPSTR lpTempFileName)
{
	DETOURED_CALL(GetTempFileNameA);
	DEBUG_LOG_TRUE(L"GetTempFileNameA", L"");
	return True_GetTempFileNameA(lpPathName, lpPrefixString, uUnique, lpTempFileName);
}

BOOL Detoured_CreateDirectoryExW(LPCWSTR lpTemplateDirectory, LPCWSTR lpNewDirectory, LPSECURITY_ATTRIBUTES lpSecurityAttributes)
{
	DETOURED_CALL(CreateDirectoryExW);
	DEBUG_LOG_TRUE(L"CreateDirectoryExW", L"");
	return True_CreateDirectoryExW(lpTemplateDirectory, lpNewDirectory, lpSecurityAttributes);
}

BOOL Detoured_RemoveDirectoryA(LPCSTR lpPathName)
{
	t_calledRemoveDirectoryW = false;
	BOOL res = True_RemoveDirectoryA(lpPathName);
	UBA_ASSERT(t_calledRemoveDirectoryW);
	return res;
}
/*
DWORD Detoured_GetSecurityInfo(HANDLE handle, SE_OBJECT_TYPE ObjectType, SECURITY_INFORMATION SecurityInfo, PSID* ppsidOwner, PSID* ppsidGroup, PACL* ppDacl, PACL* ppSacl, PSECURITY_DESCRIPTOR* ppSecurityDescriptor)
{
	DETOURED_CALL(GetSecurityInfo);
	if (isDetouredHandle(handle))
	{
		handle = asDetouredHandle(handle).trueHandle;
		UBA_ASSERTF(handle != INVALID_HANDLE_VALUE, L"GetSecurityInfo");
	}

	DEBUG_LOG_TRUE(L"GetSecurityInfo", L"");
	return True_GetSecurityInfo(handle, ObjectType, SecurityInfo, ppsidOwner, ppsidGroup, ppDacl, ppSacl, ppSecurityDescriptor);
}

BOOL Detoured_DecryptFileW(LPCWSTR lpFileName, DWORD dwReserved)
{
	DETOURED_CALL(DecryptFileW);
	DEBUG_LOG_TRUE(L"DecryptFileW", L"");
	return True_DecryptFileW(lpFileName, dwReserved);
}

BOOL Detoured_DecryptFileA(LPCSTR lpFileName, DWORD dwReserved)
{
	DETOURED_CALL(DecryptFileA);
	DEBUG_LOG_TRUE(L"DecryptFileA", L"");
	return True_DecryptFileA(lpFileName, dwReserved);
}

BOOL Detoured_EncryptFileW(LPCWSTR lpFileName)
{
	DETOURED_CALL(EncryptFileW);
	DEBUG_LOG_TRUE(L"EncryptFileW", L"");
	return True_EncryptFileW(lpFileName);
}

BOOL Detoured_EncryptFileA(LPCSTR lpFileName)
{
	DETOURED_CALL(EncryptFileA);
	DEBUG_LOG_TRUE(L"EncryptFileA", L"");
	return True_EncryptFileA(lpFileName);
}

DWORD Detoured_OpenEncryptedFileRawW(LPCWSTR lpFileName, ULONG ulFlags, PVOID* pvContext)
{
	DETOURED_CALL(OpenEncryptedFileRawW);
	DEBUG_LOG_TRUE(L"OpenEncryptedFileRawW", L"");
	return True_OpenEncryptedFileRawW(lpFileName, ulFlags, pvContext);
}

DWORD Detoured_OpenEncryptedFileRawA(LPCSTR lpFileName, ULONG ulFlags, PVOID* pvContext)
{
	DETOURED_CALL(OpenEncryptedFileRawA);
	DEBUG_LOG_TRUE(L"OpenEncryptedFileRawA", L"");
	return True_OpenEncryptedFileRawA(lpFileName, ulFlags, pvContext);
}
*/
HANDLE Detoured_OpenFileById(HANDLE hFile, LPFILE_ID_DESCRIPTOR lpFileID, DWORD dwDesiredAccess, DWORD dwShareMode, LPSECURITY_ATTRIBUTES lpSecurityAttributes, DWORD dwFlags)
{
	DETOURED_CALL(OpenFileById);
	DEBUG_LOG_TRUE(L"OpenFileById", L"");
	UBA_ASSERT(!isDetouredHandle(hFile));
	return True_OpenFileById(hFile, lpFileID, dwDesiredAccess, dwShareMode, lpSecurityAttributes, dwFlags);
}

HANDLE Detoured_CreateEventW(LPSECURITY_ATTRIBUTES lpEventAttributes, BOOL bManualReset, BOOL bInitialState, LPCWSTR lpName)
{
	DETOURED_CALL(CreateEvent);
	if (lpName)
	{
		DEBUG_LOG_TRUE(L"CreateEvent", L"%ls", lpName);
	}
	return True_CreateEventW(lpEventAttributes, bManualReset, bInitialState, lpName);
}

HANDLE Detoured_CreateEventExW(LPSECURITY_ATTRIBUTES lpEventAttributes, LPCWSTR lpName, DWORD dwFlags, DWORD dwDesiredAccess)
{
	DETOURED_CALL(CreateEventEx);
	if (lpName)
	{
		DEBUG_LOG_TRUE(L"CreateEventEx", L"%ls", lpName);
	}
	return True_CreateEventExW(lpEventAttributes, lpName, dwFlags, dwDesiredAccess);
}

HANDLE Detoured_CreateMutexExW(LPSECURITY_ATTRIBUTES lpMutexAttributes, LPCWSTR lpName, DWORD dwFlags, DWORD dwDesiredAccess)
{
	DETOURED_CALL(CreateMutexEx);
	if (lpName)
	{
		DEBUG_LOG_TRUE(L"CreateMutexEx", L"%ls", lpName);
	}
	return True_CreateMutexExW(lpMutexAttributes, lpName, dwFlags, dwDesiredAccess);
}

HANDLE Detoured_CreateWaitableTimerExW(LPSECURITY_ATTRIBUTES lpTimerAttributes, LPCWSTR lpTimerName, DWORD dwFlags, DWORD dwDesiredAccess)
{
	DETOURED_CALL(CreateWaitableTimerExW);
	if (lpTimerName)
	{
		DEBUG_LOG_TRUE(L"CreateWaitableTimerExW", L"%ls", lpTimerName);
	}
	return True_CreateWaitableTimerExW(lpTimerAttributes, lpTimerName, dwFlags, dwDesiredAccess);
}

HANDLE Detoured_CreateIoCompletionPort(HANDLE FileHandle, HANDLE ExistingCompletionPort, ULONG_PTR CompletionKey, DWORD NumberOfConcurrentThreads)
{
	DETOURED_CALL(CreateIoCompletionPort);
	UBA_ASSERT(!isDetouredHandle(FileHandle));
	HANDLE trueHandle = FileHandle;
	if (isDetouredHandle(FileHandle))
		trueHandle = asDetouredHandle(FileHandle).trueHandle;
	HANDLE res = True_CreateIoCompletionPort(trueHandle, ExistingCompletionPort, CompletionKey, NumberOfConcurrentThreads);
	DEBUG_LOG_TRUE(L"CreateIoCompletionPort", L"%llu %llu (%llu) -> %llu", u64(FileHandle), u64(ExistingCompletionPort), u64(CompletionKey), u64(res));
	return res;
}

HANDLE Detoured_CreateRemoteThread(HANDLE hProcess, LPSECURITY_ATTRIBUTES lpThreadAttributes, SIZE_T dwStackSize, LPTHREAD_START_ROUTINE lpStartAddress, LPVOID lpParameter, DWORD dwCreationFlags, LPDWORD lpThreadId)
{
	DEBUG_LOG_TRUE(L"CreateRemoteThread", L"");
	return True_CreateRemoteThread(hProcess, lpThreadAttributes, dwStackSize, lpStartAddress, lpParameter, dwCreationFlags, lpThreadId);
}

BOOL Detoured_PostQueuedCompletionStatus(HANDLE CompletionPort, DWORD dwNumberOfBytesTransferred, ULONG_PTR dwCompletionKey, LPOVERLAPPED lpOverlapped)
{
	UBA_ASSERT(!isDetouredHandle(CompletionPort));
	DEBUG_LOG_TRUE(L"PostQueuedCompletionStatus", L"%llu", u64(dwCompletionKey));
	return True_PostQueuedCompletionStatus(CompletionPort, dwNumberOfBytesTransferred, dwCompletionKey, lpOverlapped);
}

BOOL Detoured_GetQueuedCompletionStatusEx(HANDLE CompletionPort, LPOVERLAPPED_ENTRY lpCompletionPortEntries, ULONG ulCount, PULONG ulNumEntriesRemoved, DWORD dwMilliseconds, BOOL fAlertable)
{
	DETOURED_CALL(GetQueuedCompletionStatusEx);
	//DEBUG_LOG_PROXY(L"GetQueuedCompletionStatusEx", L""); // Ends up in NtRemoveIoCompletionEx
	return True_GetQueuedCompletionStatusEx(CompletionPort, lpCompletionPortEntries, ulCount, ulNumEntriesRemoved, dwMilliseconds, fAlertable);
}

BOOL Detoured_CancelIo(HANDLE hFile)
{
	UBA_ASSERT(!isDetouredHandle(hFile));
	DEBUG_LOG_TRUE(L"CancelIo", L"");
	return True_CancelIo(hFile);
}

BOOL Detoured_CancelIoEx(HANDLE hFile, LPOVERLAPPED lpOverlapped)
{
	UBA_ASSERT(!isDetouredHandle(hFile));
	DEBUG_LOG_TRUE(L"CancelIoEx", L"");
	return True_CancelIoEx(hFile, lpOverlapped);
}

BOOL Detoured_SetFileCompletionNotificationModes(HANDLE FileHandle, UCHAR Flags)
{
	UBA_ASSERT(!isDetouredHandle(FileHandle));
	DEBUG_LOG_TRUE(L"SetFileCompletionNotificationModes", L"");
	return True_SetFileCompletionNotificationModes(FileHandle, Flags);
}

BOOL Detoured_CreatePipe(PHANDLE hReadPipe, PHANDLE hWritePipe, LPSECURITY_ATTRIBUTES lpPipeAttributes, DWORD nSize)
{
	DETOURED_CALL(CreatePipe);
	DEBUG_LOG_TRUE(L"CreatePipe", L"");
	bool res = True_CreatePipe(hReadPipe, hWritePipe, lpPipeAttributes, nSize);
	DEBUG_LOG_TRUE(L"CreatePipe", L"(POST) %llu %llu -> %s", u64(*hReadPipe), u64(*hWritePipe), ToString(res));
	return res;
}


BOOL Detoured_SetHandleInformation(HANDLE hObject, DWORD dwMask, DWORD dwFlags)
{
	DETOURED_CALL(SetHandleInformation);
	DEBUG_LOG_TRUE(L"SetHandleInformation", L"%llu", uintptr_t(hObject));
	return True_SetHandleInformation(hObject, dwMask, dwFlags); // Calls NtQueryObject and NtSetInformationObject internally
}

HANDLE Detoured_CreateNamedPipeW(LPCWSTR lpName, DWORD dwOpenMode, DWORD dwPipeMode, DWORD nMaxInstances, DWORD nOutBufferSize, DWORD nInBufferSize, DWORD nDefaultTimeOut, LPSECURITY_ATTRIBUTES lpSecurityAttributes)
{
	DETOURED_CALL(CreateNamedPipeW);
	HANDLE h = True_CreateNamedPipeW(lpName, dwOpenMode, dwPipeMode, nMaxInstances, nOutBufferSize, nInBufferSize, nDefaultTimeOut, lpSecurityAttributes);
	DEBUG_LOG_TRUE(L"CreateNamedPipeW", L"%ls -> %llu", lpName, u64(h));
	return h;
}

BOOL Detoured_CallNamedPipeW(LPCWSTR lpNamedPipeName, LPVOID lpInBuffer, DWORD nInBufferSize, LPVOID lpOutBuffer, DWORD nOutBufferSize, LPDWORD lpBytesRead, DWORD nTimeOut)
{
	DETOURED_CALL(CreateNamedPipeW);
	DEBUG_LOG_TRUE(L"CallNamedPipeW", L"%ls %lu %lu", lpNamedPipeName, nInBufferSize, nOutBufferSize);
	return True_CallNamedPipeW(lpNamedPipeName, lpInBuffer, nInBufferSize, lpOutBuffer, nOutBufferSize, lpBytesRead, nTimeOut);
}

BOOL Detoured_PeekNamedPipe(HANDLE hNamedPipe, LPVOID lpBuffer, DWORD nBufferSize, LPDWORD lpBytesRead, LPDWORD lpTotalBytesAvail, LPDWORD lpBytesLeftThisMessage)
{
	UBA_ASSERT(!isDetouredHandle(hNamedPipe));
	DEBUG_LOG_TRUE(L"PeekNamedPipe", L"");
	return True_PeekNamedPipe(hNamedPipe, lpBuffer, nBufferSize, lpBytesRead, lpTotalBytesAvail, lpBytesLeftThisMessage);
}

BOOL Detoured_GetKernelObjectSecurity(HANDLE Handle, SECURITY_INFORMATION RequestedInformation, PSECURITY_DESCRIPTOR pSecurityDescriptor, DWORD nLength, LPDWORD lpnLengthNeeded)
{
	DEBUG_LOG_TRUE(L"GetKernelObjectSecurity", L"%llu", uintptr_t(Handle));
	t_ntQuerySecurityObjectCalled = false;
	BOOL res = True_GetKernelObjectSecurity(Handle, RequestedInformation, pSecurityDescriptor, nLength, lpnLengthNeeded);
	UBA_ASSERT(t_ntQuerySecurityObjectCalled);
	return res;
}

BOOL Detoured_ImpersonateNamedPipeClient(HANDLE hNamedPipe)
{
	UBA_ASSERT(!isDetouredHandle(hNamedPipe));
	return True_ImpersonateNamedPipeClient(hNamedPipe);
}

BOOL Detoured_TransactNamedPipe(HANDLE hNamedPipe, LPVOID lpInBuffer, DWORD nInBufferSize, LPVOID lpOutBuffer, DWORD nOutBufferSize, LPDWORD lpBytesRead, LPOVERLAPPED lpOverlapped)
{
	UBA_ASSERT(!isDetouredHandle(hNamedPipe));
	return True_TransactNamedPipe(hNamedPipe, lpInBuffer, nInBufferSize, lpOutBuffer, nOutBufferSize, lpBytesRead, lpOverlapped);
}

BOOL Detoured_SetNamedPipeHandleState(HANDLE hNamedPipe, LPDWORD lpMode, LPDWORD lpMaxCollectionCount, LPDWORD lpCollectDataTimeout)
{
	UBA_ASSERT(!isDetouredHandle(hNamedPipe));
	return True_SetNamedPipeHandleState(hNamedPipe, lpMode, lpMaxCollectionCount, lpCollectDataTimeout);
}

BOOL Detoured_GetNamedPipeInfo(HANDLE hNamedPipe, LPDWORD lpFlags, LPDWORD lpOutBufferSize, LPDWORD lpInBufferSize, LPDWORD lpMaxInstances)
{
	UBA_ASSERT(!isDetouredHandle(hNamedPipe));
	return True_GetNamedPipeInfo(hNamedPipe, lpFlags, lpOutBufferSize, lpInBufferSize, lpMaxInstances);
}

BOOL Detoured_GetNamedPipeHandleStateW(HANDLE hNamedPipe, LPDWORD lpState, LPDWORD lpCurInstances, LPDWORD lpMaxCollectionCount, LPDWORD lpCollectDataTimeout, LPWSTR lpUserName, DWORD nMaxUserNameSize)
{
	UBA_ASSERT(!isDetouredHandle(hNamedPipe));
	return True_GetNamedPipeHandleStateW(hNamedPipe, lpState, lpCurInstances, lpMaxCollectionCount, lpCollectDataTimeout, lpUserName, nMaxUserNameSize);
}

BOOL Detoured_GetNamedPipeServerProcessId(HANDLE Pipe, PULONG ServerProcessId)
{
	UBA_ASSERT(!isDetouredHandle(Pipe));
	return True_GetNamedPipeServerProcessId(Pipe, ServerProcessId);
}

BOOL Detoured_GetNamedPipeServerSessionId(HANDLE Pipe, PULONG ServerSessionId)
{
	UBA_ASSERT(!isDetouredHandle(Pipe));
	return True_GetNamedPipeServerSessionId(Pipe, ServerSessionId);
}


HANDLE Detoured_OpenFileMappingA(DWORD dwDesiredAccess, BOOL bInheritHandle, LPCSTR lpName)
{
	DETOURED_CALL(OpenFileMappingA);
	DEBUG_LOG_TRUE(L"OpenFileMappingA", L"");
	return True_OpenFileMappingA(dwDesiredAccess, bInheritHandle, lpName);
}

DWORD Detoured_GetMappedFileNameW(HANDLE hProcess, LPVOID lpv, LPWSTR lpFilename, DWORD nSize)
{
	DETOURED_CALL(GetMappedFileNameW);
	DEBUG_LOG_TRUE(L"GetMappedFileNameW", L"");
	return True_GetMappedFileNameW(hProcess, lpv, lpFilename, nSize);
}

BOOL Detoured_IsProcessorFeaturePresent(DWORD ProcessorFeature)
{
	DETOURED_CALL(IsProcessorFeaturePresent);
	auto res = True_IsProcessorFeaturePresent(ProcessorFeature);
	DEBUG_LOG_TRUE(L"IsProcessorFeaturePresent", L"%lu -> %ls", ProcessorFeature, ToString(res));
	return res;
}

PVOID Detoured_VirtualAlloc2(HANDLE Process, PVOID BaseAddress, SIZE_T Size, ULONG AllocationType, ULONG PageProtection, MEM_EXTENDED_PARAMETER *ExtendedParameters, ULONG ParameterCount)
{
	UBA_ASSERT(!isDetouredHandle(Process));
	auto res = True_VirtualAlloc2(Process, BaseAddress, Size, AllocationType, PageProtection, ExtendedParameters, ParameterCount);
	//DEBUG_LOG_TRUE(L"VirtualAlloc2", L"-> 0x%llx", uintptr_t(res)); // Calling NtAllocateVirtualMemoryEx
	return res;
}

PVOID Detoured_VirtualAlloc2FromApp(HANDLE Process, PVOID BaseAddress, SIZE_T Size, ULONG AllocationType, ULONG PageProtection, MEM_EXTENDED_PARAMETER *ExtendedParameters, ULONG ParameterCount)
{
	UBA_ASSERT(!isDetouredHandle(Process));
	auto res = True_VirtualAlloc2FromApp(Process, BaseAddress, Size, AllocationType, PageProtection, ExtendedParameters, ParameterCount);
	DEBUG_LOG_TRUE(L"VirtualAlloc2FromApp", L"-> 0x%llx", uintptr_t(res));
	return res;
}

BOOL Detoured_UnmapViewOfFile2(HANDLE Process, PVOID BaseAddress, ULONG UnmapFlags)
{
	DETOURED_CALL(UnmapViewOfFile2);
	auto res = True_UnmapViewOfFile2(Process, BaseAddress, UnmapFlags);
	//DEBUG_LOG_TRUE(L"UnmapViewOfFile2", L"0x%llx -> %ls", uintptr_t(BaseAddress), ToString(res)); // Spammy since SharedMemoryAllocator use it
	return res;
}

BOOL Detoured_GetComputerNameA(LPSTR lpBuffer, LPDWORD nSize)
{
	DETOURED_CALL(GetComputerNameA);
	auto res = True_GetComputerNameA(lpBuffer, nSize);
	DEBUG_LOG_TRUE(L"GetComputerNameA", L"-> %s", ToString(res));
	return res;
}

HMODULE Detoured_GetModuleHandleW(LPCWSTR lpModuleName)
{
	DETOURED_CALL(GetModuleHandleW);
	auto res = True_GetModuleHandleW(lpModuleName);
	DEBUG_LOG_TRUE(L"GetModuleHandleW", L"%s -> %llx", lpModuleName, u64(res));
	return res;
}

VOID Detoured_GetStartupInfoW(LPSTARTUPINFOW lpStartupInfo)
{
	DETOURED_CALL(GetStartupInfoW);
	True_GetStartupInfoW(lpStartupInfo);
	DEBUG_LOG_TRUE(L"GetStartupInfoW", L"stdout: %llu", u64(lpStartupInfo->hStdOutput));
	
	if (lpStartupInfo->cbReserved2 && lpStartupInfo->lpReserved2)
	{
		BYTE* p = lpStartupInfo->lpReserved2;
		DWORD count = *(DWORD*)p;
		BYTE*   flags   = p + sizeof(DWORD);
		HANDLE* handles = (HANDLE*)(flags + count);  // may be unaligned, but x64 handles it

		for (DWORD i = 0; i < count; i++)
		{
			HANDLE h = handles[i];
			if (h == INVALID_HANDLE_VALUE)
				continue;
			UBA_ASSERT(!isDetouredHandle(h)); // Detoured handles can't be inherited. This is not good
			// flags[i] bits: FOPEN=0x01, FTEXT=0x80, FPIPE=0x08, FDEV=0x40, etc.
			// do something with h and flags[i]
		}
	}
}

VOID Detoured_GetSystemTimeAsFileTime(LPFILETIME lpSystemTimeAsFileTime)
{
	DETOURED_CALL(GetSystemTimeAsFileTime);
	DEBUG_LOG_TRUE(L"GetSystemTimeAsFileTime", L"");
	True_GetSystemTimeAsFileTime(lpSystemTimeAsFileTime);
}

SIZE_T Detoured_GetLargePageMinimum()
{
	DETOURED_CALL(GetLargePageMinimum);
	auto res = True_GetLargePageMinimum();
	DEBUG_LOG_TRUE(L"GetLargePageMinimum", L"-> %llu", u64(res));
	return res;
}

// These functions require a very new version of win api (1-1-7).. so skip detouring these for now
//HANDLE Detoured_CreateFileMapping2(HANDLE File, SECURITY_ATTRIBUTES* SecurityAttributes, ULONG DesiredAccess, ULONG PageProtection, ULONG AllocationAttributes, ULONG64 MaximumSize, PCWSTR Name, MEM_EXTENDED_PARAMETER* ExtendedParameters, ULONG ParameterCount)
//{
//	DEBUG_LOG_TRUE(L"CreateFileMapping2", L"(%ls)", HandleToName(File));
//	UBA_ASSERT(!isDetouredHandle(File));
//	return True_CreateFileMapping2(File, SecurityAttributes, DesiredAccess, PageProtection, AllocationAttributes, MaximumSize, Name, ExtendedParameters, ParameterCount);
//}

//HANDLE Detoured_CreateFileMappingNumaW(HANDLE hFile, LPSECURITY_ATTRIBUTES lpFileMappingAttributes, DWORD flProtect, DWORD dwMaximumSizeHigh, DWORD dwMaximumSizeLow, LPCWSTR lpName, DWORD nndPreferred)
//{
//	DEBUG_LOG_TRUE(L"CreateFileMappingNumaW", L"(%ls)", HandleToName(hFile));
//	UBA_ASSERT(!isDetouredHandle(hFile));
//	return True_CreateFileMappingNumaW(hFile, lpFileMappingAttributes, flProtect, dwMaximumSizeHigh, dwMaximumSizeLow, lpName, nndPreferred);
//}

void Detoured_ExitProcess(UINT uExitCode)
{
	return True_ExitProcess(uExitCode);
}

LPSTR Detoured_GetCommandLineA()
{
	DETOURED_CALL(GetCommandLineA);
	auto str = True_GetCommandLineA();
	DEBUG_LOG_TRUE(L"GetCommandLineA", L"");// str);
	return str;
}

LPWSTR* Detoured_CommandLineToArgvW(LPCWSTR lpCmdLine, int *pNumArgs)
{
	DETOURED_CALL(CommandLineToArgvW);
	LPWSTR* res = True_CommandLineToArgvW(lpCmdLine, pNumArgs);
	DEBUG_LOG_TRUE(L"CommandLineToArgvW", L" -> %ls", res[0]);
	return res;
}

void* Detoured_GetProcAddress(HMODULE hModule, LPCSTR lpProcName)
{
	void* res = True_GetProcAddress(hModule, lpProcName);

	StringBuffer<> func;
	if ((ULONG_PTR)lpProcName >> 16)
		func.Append(lpProcName);
	else
		func.Append(TCV("Ordinal:")).AppendValue(LOWORD(lpProcName));
	DEBUG_LOG_TRUE(L"GetProcAddress", L"%s -> 0x%p", func.data, res);
	return res;
}

BOOL Detoured_FreeLibrary(HMODULE hModule)
{
	DETOURED_CALL(FreeLibrary);
	BOOL res = True_FreeLibrary(hModule);
	DEBUG_LOG_TRUE(L"FreeLibrary", L"%llu -> %ls", uintptr_t(hModule), ToString(res));
	return res;
}

LSTATUS Detoured_RegOpenKeyW(HKEY hKey, LPCWSTR lpSubKey, PHKEY phkResult)
{
	DETOURED_CALL(RegOpenKeyW);
	//UBA_NOT_IMPLEMENTED();
	SuppressCreateFileDetourScope cfs;
	auto res = True_RegOpenKeyW(hKey, lpSubKey, phkResult);
	DEBUG_LOG_TRUE(L"RegOpenKeyW", L"(%ls) -> %ls", lpSubKey, ToString(res == ERROR_SUCCESS));
	return res;
}

LSTATUS Detoured_RegOpenKeyExW(HKEY hKey, LPCWSTR lpSubKey, DWORD ulOptions, REGSAM samDesired, PHKEY phkResult)
{
	DETOURED_CALL(RegOpenKeyExW);
	//UBA_NOT_IMPLEMENTED();
	SuppressCreateFileDetourScope cfs;
	auto res = True_RegOpenKeyExW(hKey, lpSubKey, ulOptions, samDesired, phkResult);
	DEBUG_LOG_TRUE(L"RegOpenKeyExW", L"(%ls) -> %ls (%llu)", lpSubKey, ToString(res == ERROR_SUCCESS), phkResult ? u64(*phkResult) : 0ull);
	return res;
}

LSTATUS Detoured_RegCreateKeyExW(HKEY hKey, LPCWSTR lpSubKey, DWORD Reserved, LPWSTR lpClass, DWORD dwOptions, REGSAM samDesired, const LPSECURITY_ATTRIBUTES lpSecurityAttributes, PHKEY phkResult, LPDWORD lpdwDisposition)
{
	DETOURED_CALL(RegCreateKeyExW);
	//UBA_NOT_IMPLEMENTED();
	SuppressCreateFileDetourScope cfs;
	auto res = True_RegCreateKeyExW(hKey, lpSubKey, Reserved, lpClass, dwOptions, samDesired, lpSecurityAttributes, phkResult, lpdwDisposition);
	DEBUG_LOG_TRUE(L"RegCreateKeyExW", L"(%ls) -> %ls", lpSubKey, ToString(res == ERROR_SUCCESS));
	return res;
}

LSTATUS Detoured_RegOpenKeyExA(HKEY hKey, LPCSTR lpSubKey, DWORD ulOptions, REGSAM samDesired, PHKEY phkResult)
{
	DETOURED_CALL(RegOpenKeyExA);
	auto res = True_RegOpenKeyExA(hKey, lpSubKey, ulOptions, samDesired, phkResult);
	//UBA_NOT_IMPLEMENTED();
	DEBUG_LOG_TRUE(L"RegOpenKeyExA", L"%llu (%hs) -> %ls", uintptr_t(*phkResult), lpSubKey, ToString(res == ERROR_SUCCESS));
	return res;
}

LSTATUS Detoured_RegCloseKey(HKEY hKey)
{
	DETOURED_CALL(RegCloseKey);
	return True_RegCloseKey(hKey);
}

HANDLE Detoured_CreateConsoleScreenBuffer(DWORD dwDesiredAccess, DWORD dwShareMode, const SECURITY_ATTRIBUTES* lpSecurityAttributes, DWORD dwFlags, LPVOID lpScreenBufferData)
{
	DETOURED_CALL(CreateConsoleScreenBuffer);
	DEBUG_LOG_TRUE(L"CreateConsoleScreenBuffer", L"");
	return True_CreateConsoleScreenBuffer(dwDesiredAccess, dwShareMode, lpSecurityAttributes, dwFlags, lpScreenBufferData);
}

HANDLE Detoured_OpenProcess(DWORD dwDesiredAccess, BOOL bInheritHandle, DWORD dwProcessId)
{
	DETOURED_CALL(OpenProcess);
	HANDLE h = True_OpenProcess(dwDesiredAccess, bInheritHandle, dwProcessId);
	DEBUG_LOG_TRUE(L"OpenProcess", L"%lu -> %llu", dwProcessId, u64(h));
	return h;
}

ULONG Detoured_RemoveVectoredExceptionHandler(PVOID Handle)
{
	DEBUG_LOG_TRUE(L"RemoveVectoredExceptionHandler", L"");
	return True_RemoveVectoredExceptionHandler(Handle);
}

VOID Detoured_RaiseFailFastException(PEXCEPTION_RECORD pExceptionRecord, PCONTEXT pContextRecord, DWORD dwFlags)
{
	DEBUG_LOG_TRUE(L"RaiseFailFastException", L"");
	return True_RaiseFailFastException(pExceptionRecord, pContextRecord, dwFlags);
}

BOOL Detoured_TerminateJobObject(HANDLE hJob, UINT uExitCode)
{
	DEBUG_LOG_TRUE(L"TerminateJobObject", L"");
	return True_TerminateJobObject(hJob, uExitCode);
}

UINT Detoured_SetErrorMode(UINT uMode)
{
	DEBUG_LOG_TRUE(L"SetErrorMode", L"%u", uMode);
	return True_SetErrorMode(uMode);
}
BOOL Detoured_CreateProcessAsUserW(HANDLE hToken, LPCWSTR lpApplicationName, LPWSTR lpCommandLine, LPSECURITY_ATTRIBUTES lpProcessAttributes, LPSECURITY_ATTRIBUTES lpThreadAttributes, BOOL bInheritHandles, DWORD dwCreationFlags, LPVOID lpEnvironment, LPCWSTR lpCurrentDirectory, LPSTARTUPINFOW lpStartupInfo, LPPROCESS_INFORMATION lpProcessInformation)
{
	DETOURED_CALL(CreateProcessAsUserW);
	DEBUG_LOG_DETOURED(L"CreateProcessAsUserW", L"%ls %ls %lu", lpApplicationName, lpCommandLine ? lpCommandLine : L"", dwCreationFlags);
	return True_CreateProcessAsUserW(hToken, lpApplicationName, lpCommandLine, lpProcessAttributes, lpThreadAttributes, bInheritHandles, dwCreationFlags, lpEnvironment, lpCurrentDirectory, lpStartupInfo, lpProcessInformation);
}

BOOL WINAPI Detoured_SetConsoleCtrlHandler(PHANDLER_ROUTINE HandlerRoutine, BOOL Add)
{
	DETOURED_CALL(SetConsoleCtrlHandler);
	DEBUG_LOG_DETOURED(L"SetConsoleCtrlHandler", L"");
	return TRUE;
}

UINT Detoured_GetConsoleOutputCP()
{
	DETOURED_CALL(GetConsoleOutputCP);
	DEBUG_LOG_DETOURED(L"GetConsoleOutputCP", L"-> 437");
	return 437;
	//auto res = True_GetConsoleOutputCP();
	//return res;
}

BOOL Detoured_ReadConsoleInputA(HANDLE hConsoleInput, PINPUT_RECORD lpBuffer, DWORD nLength, LPDWORD lpNumberOfEventsRead)
{
	DETOURED_CALL(ReadConsoleInput);
	DEBUG_LOG_DETOURED(L"ReadConsoleInput", L"");
	return FALSE;// True_ReadConsoleInput(hConsoleInput, lpBuffer, nLength, lpNumberOfEventsRead);
}

HWND Detoured_GetConsoleWindow()
{
	DETOURED_CALL(GetConsoleWindow);
	HWND res = True_GetConsoleWindow();
	DEBUG_LOG_TRUE(L"GetConsoleWindow", L"-> %llu", uintptr_t(res));
	return res;
}

BOOL Detoured_SetConsoleCursorPosition(HANDLE hConsoleOutput, COORD dwCursorPosition)
{
	DETOURED_CALL(SetConsoleCursorPosition);
	DEBUG_LOG_DETOURED(L"SetConsoleCursorPosition", L"");
	UBA_ASSERT(!isDetouredHandle(hConsoleOutput));
	return True_SetConsoleCursorPosition(hConsoleOutput, dwCursorPosition);
}


BOOL Detoured_GetConsoleScreenBufferInfo(HANDLE hConsoleOutput, PCONSOLE_SCREEN_BUFFER_INFO lpConsoleScreenBufferInfo)
{
	DETOURED_CALL(GetConsoleScreenBufferInfo);
	bool res = True_GetConsoleScreenBufferInfo(hConsoleOutput, lpConsoleScreenBufferInfo);
	DEBUG_LOG_DETOURED(L"GetConsoleScreenBufferInfo", L"(%llu) -> %s", u64(hConsoleOutput), ToString(res));
	return res;
	/*
	// We make these up just to make clang output errors with proper line breaks..
	lpConsoleScreenBufferInfo->dwSize = COORD{ 120,200 };
	lpConsoleScreenBufferInfo->dwCursorPosition = COORD{ 0,0 };
	lpConsoleScreenBufferInfo->wAttributes = 0;
	lpConsoleScreenBufferInfo->srWindow = SMALL_RECT{ 0,0,120,200 };
	lpConsoleScreenBufferInfo->dwMaximumWindowSize = COORD{120,8000};
	return true;
	*/
}

BOOL Detoured_ScrollConsoleScreenBufferW(HANDLE hConsoleOutput, const SMALL_RECT* lpScrollRectangle, const SMALL_RECT* lpClipRectangle, COORD dwDestinationOrigin, const CHAR_INFO* lpFill)
{
	DETOURED_CALL(ScrollConsoleScreenBufferW);
	DEBUG_LOG_DETOURED(L"ScrollConsoleScreenBufferW", L"");
	return True_ScrollConsoleScreenBufferW(hConsoleOutput, lpScrollRectangle, lpClipRectangle, dwDestinationOrigin, lpFill);
}

BOOL Detoured_FillConsoleOutputAttribute(HANDLE hConsoleOutput, WORD wAttribute, DWORD nLength, COORD dwWriteCoord, LPDWORD lpNumberOfAttrsWritten)
{
	DETOURED_CALL(FillConsoleOutputAttribute);
	DEBUG_LOG_DETOURED(L"FillConsoleOutputAttribute", L"");
	return True_FillConsoleOutputAttribute(hConsoleOutput, wAttribute, nLength, dwWriteCoord, lpNumberOfAttrsWritten);
}

BOOL Detoured_FillConsoleOutputCharacterW(HANDLE hConsoleOutput, TCHAR cCharacter, DWORD nLength, COORD dwWriteCoord, LPDWORD lpNumberOfCharsWritten)
{
	DETOURED_CALL(FillConsoleOutputCharacterW);
	DEBUG_LOG_DETOURED(L"FillConsoleOutputCharacterW", L"");
	return True_FillConsoleOutputCharacterW(hConsoleOutput, cCharacter, nLength, dwWriteCoord, lpNumberOfCharsWritten);
}

BOOL Detoured_FlushConsoleInputBuffer(HANDLE hConsoleInput)
{
	DETOURED_CALL(FlushConsoleInputBuffer);
	DEBUG_LOG_DETOURED(L"FlushConsoleInputBuffer", L"");
	HANDLE trueHandle = hConsoleInput;
	if (isDetouredHandle(hConsoleInput))
	{
		trueHandle = asDetouredHandle(hConsoleInput).trueHandle;
		UBA_ASSERT(trueHandle != INVALID_HANDLE_VALUE);
	}
	return True_FlushConsoleInputBuffer(trueHandle);
}

BOOL Detoured_SetConsoleTextAttribute(HANDLE hConsoleOutput, WORD wAttributes)
{
	DETOURED_CALL(SetConsoleTextAttribute);
	DEBUG_LOG_DETOURED(L"SetConsoleTextAttribute", L"%llu %i", u64(hConsoleOutput), wAttributes);
	return True_SetConsoleTextAttribute(hConsoleOutput, wAttributes);
}

BOOL Detoured_SetConsoleTitleW(LPCTSTR lpConsoleTitle)
{
	DETOURED_CALL(SetConsoleTitleW);
	DEBUG_LOG_DETOURED(L"SetConsoleTitleW", L"");
	return true;
}

int Detoured_GetLocaleInfoEx(LPCWSTR lpLocaleName, LCTYPE LCType, LPWSTR lpLCData, int cchData)
{
	DETOURED_CALL(GetLocaleInfoEx);
	//DEBUG_LOG_TRUE(L"GetLocaleInfoEx", L"(%ls)", lpLocaleName);
	auto res = True_GetLocaleInfoEx(lpLocaleName, LCType, lpLCData, cchData);
	//UBA_ASSERTF(false, L"%i %ls %u %ls %i", res, lpLocaleName, LCType, lpLCData, cchData);
	return res;
}

int Detoured_GetUserDefaultLocaleName(LPWSTR lpLocaleName, int cchLocaleName)
{
	DETOURED_CALL(GetUserDefaultLocaleName);
	//wcscpy_s(lpLocaleName, cchLocaleName, L"en-US");
	//int res = 6;
	int res = True_GetUserDefaultLocaleName(lpLocaleName, cchLocaleName);
	DEBUG_LOG_TRUE(L"GetUserDefaultLocaleName", L"(%ls) -> %u", lpLocaleName, res);
	return res;
}

BOOL Detoured_IsValidCodePage(UINT CodePage)
{
	DETOURED_CALL(IsValidCodePage);
	BOOL res = True_IsValidCodePage(CodePage);
	DEBUG_LOG_TRUE(L"IsValidCodePage", L"-> %u", res);
	return res;
}

UINT Detoured_GetACP()
{
	DETOURED_CALL(GetACP);
	auto res = True_GetACP();
	DEBUG_LOG_TRUE(L"GetACP", L"-> %u", res);
	return res;
}

LPCWSTR Detoured_PathFindFileNameW(LPCWSTR pszPath) // This is called by Ps4SymbolTool.exe and vctip.exe
{
	auto res = True_PathFindFileNameW(pszPath);
	DEBUG_LOG_TRUE(L"PathFindFileNameW", L"(%ls) -> %ls", pszPath, res);
	return res;
}

BOOL Detoured_PathIsRelativeW(LPCWSTR pszPath)
{
	//UBA_ASSERTF(!g_runningRemote, L"%ls", pszPath); // intel compiler uses PathIsRelativeW.. don't know if this function touches file system but will comment out this assert for now
	auto res = True_PathIsRelativeW(pszPath);
	DEBUG_LOG_TRUE(L"PathIsRelativeW", L"(%ls) -> %u", pszPath, res);
	return res;
}
BOOL Detoured_PathIsDirectoryEmptyW(LPCWSTR pszPath)
{
	UBA_ASSERTF(!g_runningRemote, L"%ls", pszPath);
	auto res = True_PathIsDirectoryEmptyW(pszPath);
	DEBUG_LOG_TRUE(L"PathIsDirectoryEmptyW", L"(%ls) -> %u", pszPath, res);
	return res;
}

void Detoured_UrlCreateFromPathW(PCWSTR pszPath, PWSTR pszUrl, DWORD* pcchUrl, DWORD dwFlags)
{
	DEBUG_LOG_TRUE(L"UrlCreateFromPathW", L"(%ls)", pszPath);
	True_UrlCreateFromPathW(pszPath, pszUrl, pcchUrl, dwFlags);
}

void Detoured_PathCreateFromUrlW(PCWSTR pszUrl, PWSTR pszPath, DWORD* pcchPath, DWORD dwFlags)
{
	DEBUG_LOG_TRUE(L"PathCreateFromUrlW", L"(%ls)", pszUrl);
	True_PathCreateFromUrlW(pszUrl, pszPath, pcchPath, dwFlags);
}

HRESULT Detoured_SHCreateStreamOnFileW(LPCWSTR pszFile, DWORD grfMode, IStream** ppstm)
{
	//UBA_ASSERTF(!g_runningRemote, L"%ls", pszFile);
	return True_SHCreateStreamOnFileW(pszFile, grfMode, ppstm);
}

BOOL Detoured_PathFileExistsW(LPCWSTR pszPath)
{
	DEBUG_LOG_DETOURED(L"PathFileExistsW", L"CALLING GetFileAttributesW (%s)", pszPath);
	DWORD attributes = Detoured_GetFileAttributesW(pszPath);
	return attributes != INVALID_FILE_ATTRIBUTES;

	//UBA_ASSERTF(!g_runningRemote, L"%ls", pszPath);
	//auto res = True_PathFileExistsW(pszPath);
	//DEBUG_LOG_TRUE(L"PathFileExistsW", L"(%ls) -> %ls", pszPath, res);
	//return res;
}

#endif // DETOURED_INCLUDE_DEBUG
