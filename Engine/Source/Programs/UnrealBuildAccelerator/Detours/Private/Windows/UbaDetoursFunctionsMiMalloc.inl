// Copyright Epic Games, Inc. All Rights Reserved.

#if UBA_USE_MIMALLOC

#define DETOURED_CALL_MEM(x) DETOURED_CALL(x)

#define TRACK_UCRT_ALLOC_ENABLED 0

#if TRACK_UCRT_ALLOC_ENABLED
Atomic<u64> g_reallocCount;
Atomic<u64> g_allocCount;
Atomic<u64> g_freeCount;
#define TRACK_UCRT_ALLOC(x) ++g_##x##Count;
#else
#define TRACK_UCRT_ALLOC(x)
#endif

inline bool IsInMiMalloc(void* ptr)
{
	return (((uintptr_t)ptr - 1) & ~MI_SEGMENT_MASK) && mi_is_in_heap_region(ptr);
}

#if UBA_DEBUG
#define VALIDATE_IN_MIMALLOC(ptr) UBA_ASSERT(!ptr || IsInMiMalloc(ptr));
#else
#define VALIDATE_IN_MIMALLOC(ptr)
#endif

void* Detoured_malloc(size_t size)
{
	DETOURED_CALL_MEM(malloc);
	TRACK_UCRT_ALLOC(alloc)
	return mi_malloc(size);
}

void* Detoured_calloc(size_t number, size_t size)
{
	DETOURED_CALL_MEM(calloc);
	TRACK_UCRT_ALLOC(alloc)
	return mi_calloc(number, size);
}

void* Detoured__recalloc(void* memblock, size_t num, size_t size)
{
	DETOURED_CALL_MEM(_recalloc);
	VALIDATE_IN_MIMALLOC(memblock);
	TRACK_UCRT_ALLOC(realloc)
	return mi_recalloc(memblock, num, size);
}

void* Detoured_realloc(void* memblock, size_t size)
{
	DETOURED_CALL_MEM(realloc);
	VALIDATE_IN_MIMALLOC(memblock);
	TRACK_UCRT_ALLOC(realloc)
	return mi_realloc(memblock, size);
}

void* Detoured__expand(void* memblock, size_t size)
{
	DETOURED_CALL_MEM(_expand);
	VALIDATE_IN_MIMALLOC(memblock);
	TRACK_UCRT_ALLOC(realloc)
	return mi_expand(memblock, size);
}

size_t Detoured__msize(void* memblock)
{
	DETOURED_CALL_MEM(free);
	VALIDATE_IN_MIMALLOC(memblock);
	return mi_usable_size(memblock);
}

void Detoured_free(void* memblock)
{
	DETOURED_CALL_MEM(free);
	TRACK_UCRT_ALLOC(free)
	if (IsInMiMalloc(memblock))
		return mi_free(memblock);
	else
		True_free(memblock);
}

char* Detoured__strdup(const char* s)
{
	DETOURED_CALL_MEM(_strdup);
	TRACK_UCRT_ALLOC(alloc)
	return mi_strdup(s);
}

wchar_t* Detoured__wcsdup(const wchar_t* s)
{
	DETOURED_CALL_MEM(_wcsdup);
	TRACK_UCRT_ALLOC(alloc)
#if MI_MALLOC_VERSION < 30000
	return (wchar_t*)mi_wcsdup((const unsigned short*)s);
#else
	return mi_wcsdup(s);
#endif
}

wchar_t* Detoured__mbsdup(const wchar_t* s)
{
	DETOURED_CALL_MEM(_mbsdup);
	TRACK_UCRT_ALLOC(alloc)
	return (wchar_t*)mi_mbsdup((const unsigned char*)(s));
}

void* Detoured__aligned_malloc(size_t size, size_t alignment)
{
	DETOURED_CALL_MEM(_aligned_malloc);
	TRACK_UCRT_ALLOC(alloc)
	return mi_malloc_aligned(size, alignment);
}

void* Detoured__aligned_recalloc(void* memblock, size_t num, size_t size, size_t alignment)
{
	DETOURED_CALL_MEM(_aligned_recalloc);
	VALIDATE_IN_MIMALLOC(memblock);
	TRACK_UCRT_ALLOC(realloc)
	return mi_aligned_recalloc(memblock, num, size, alignment);
}

void* Detoured__aligned_realloc(void* memblock, size_t size, size_t alignment)
{
	DETOURED_CALL_MEM(_aligned_realloc);
	VALIDATE_IN_MIMALLOC(memblock);
	TRACK_UCRT_ALLOC(realloc)
	return mi_realloc_aligned(memblock, size, alignment);
}

void Detoured__aligned_free(void* memblock)
{
	DETOURED_CALL_MEM(_aligned_free);
	VALIDATE_IN_MIMALLOC(memblock);
	TRACK_UCRT_ALLOC(free)
	return mi_free(memblock);
}

void* Detoured__aligned_offset_malloc(size_t size, size_t alignment, size_t offset)
{
	DETOURED_CALL_MEM(_aligned_offset_malloc);
	TRACK_UCRT_ALLOC(alloc)
	return mi_malloc_aligned_at(size, alignment, offset);
}

void* Detoured__aligned_offset_recalloc(void* memblock, size_t num, size_t size, size_t alignment, size_t offset)
{
	DETOURED_CALL_MEM(_aligned_offset_recalloc);
	VALIDATE_IN_MIMALLOC(memblock);
	TRACK_UCRT_ALLOC(realloc)
	return mi_recalloc_aligned_at(memblock, num, size, alignment, offset);
}

void* Detoured__aligned_offset_realloc(void* memblock, size_t size, size_t alignment, size_t offset)
{
	DETOURED_CALL_MEM(_aligned_offset_realloc);
	VALIDATE_IN_MIMALLOC(memblock);
	TRACK_UCRT_ALLOC(realloc)
	return mi_realloc_aligned_at(memblock, size, alignment, offset);
}

errno_t Detoured__wdupenv_s(wchar_t** buffer, size_t* numberOfElements, const wchar_t* varname)
{
	DETOURED_CALL_MEM(_wdupenv_s);
#if MI_MALLOC_VERSION < 30000
	auto res = mi_wdupenv_s((unsigned short**)buffer, numberOfElements, (const unsigned short*)varname);
#else
	auto res = mi_wdupenv_s(buffer, numberOfElements, varname);
#endif
	TRACK_UCRT_ALLOC(alloc)
	DEBUG_LOG_DETOURED(L"_wdupenv_s", L"(%ls) -> %u", varname, res);
	return res;
}

errno_t Detoured__dupenv_s(char** buffer, size_t* numberOfElements, const char* varname)
{
	DETOURED_CALL_MEM(_dupenv_s);
	auto res = mi_dupenv_s(buffer, numberOfElements, varname);
	TRACK_UCRT_ALLOC(alloc)
	DEBUG_LOG_DETOURED(L"_dupenv_s", L"(%hs) -> %u", varname, res);
	return res;
}

void* Detoured__malloc_base(size_t _Size)
{
	TRACK_UCRT_ALLOC(alloc)
	return mi_malloc(_Size);
}

void* Detoured__calloc_base(size_t _Count, size_t _Size)
{
	TRACK_UCRT_ALLOC(alloc)
	return mi_calloc(_Count, _Size);
}

void* Detoured__realloc_base(void* memblock, size_t size)
{
	TRACK_UCRT_ALLOC(realloc)
	if (memblock && !IsInMiMalloc(memblock))
		return True__realloc_base(memblock, size);
	return mi_realloc(memblock, size);
}

void Detoured__free_base(void* memblock)
{
	if (!memblock)
		return;
	TRACK_UCRT_ALLOC(free)
	if (IsInMiMalloc(memblock))
		mi_free(memblock);
	else
		True__free_base(memblock);
}

void* Detoured__expand_base(void* memblock, size_t size)
{
	TRACK_UCRT_ALLOC(realloc)
	if (memblock && !IsInMiMalloc(memblock))
		return True__expand_base(memblock, size);
	return mi_expand(memblock, size);
}

size_t Detoured__msize_base(void* memblock)
{
	DETOURED_CALL_MEM(_msize_base);
	if (memblock && !IsInMiMalloc(memblock))
		return True__msize_base(memblock);
	return mi_usable_size(memblock);
}

void* Detoured__recalloc_base(void* memblock, size_t num, size_t size)
{
	TRACK_UCRT_ALLOC(realloc)
	if (memblock && !IsInMiMalloc(memblock))
		return True__recalloc_base(memblock, num, size);
	return mi_recalloc(memblock, num, size);
}

#if DETOURED_INCLUDE_DEBUG

size_t Detoured__aligned_msize(void* p, size_t alignment, size_t offset)
{
	UBA_ASSERTF(false, L"_aligned_msize called but is only detoured in debug");
	DETOURED_CALL(_aligned_msize);
	VALIDATE_IN_MIMALLOC(p);
	return mi_usable_size(p);
}

//void Detoured__free_dbg(void* userData, int blockType)
//{
//	DETOURED_CALL(_free_dbg);
//	//DEBUG_LOG_TRUE(L"_free_dbg", L"");
//	return True__free_dbg(userData, blockType);
//}

#endif
#define TRACK_UCRT_ALLOC_ENABLED 0
#endif
