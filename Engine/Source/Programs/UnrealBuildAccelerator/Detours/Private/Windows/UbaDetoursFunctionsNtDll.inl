// Copyright Epic Games, Inc. All Rights Reserved.

StringBuffer<64> ToString(FILE_INFORMATION_CLASS c)
{
	switch (u32(c))
	{
		#define FILE_INFO_CLASS(name, value) case value: return StringBuffer<64>(TCV(#name));
		FILE_INFO_CLASSES
		#undef FILE_INFO_CLASS
	default:
		return StringBuffer<64>(TCV("class ")).AppendValue(u32(c));
	}
}

#if UBA_DEBUG_LOG_ENABLED
StringBuffer<64> ToString(FS_INFORMATION_CLASS c)
{
	switch (u32(c))
	{
	case FileFsVolumeInformation: return StringBuffer<64>(TCV("FileFsVolumeInformation"));
	case FileFsDeviceInformation: return StringBuffer<64>(TCV("FileFsDeviceInformation"));
	case FileFsAttributeInformation: return StringBuffer<64>(TCV("FileFsAttributeInformation"));
	default:
		return StringBuffer<64>(TCV("class ")).AppendValue(u32(c));
	}
}
#endif

bool IsContentWrite(u32 desiredAccess, u32 createDisposition)
{
	if (desiredAccess & (FILE_WRITE_DATA | FILE_APPEND_DATA | GENERIC_WRITE))
		return true;
	if (createDisposition == FILE_CREATE || createDisposition == FILE_OVERWRITE || createDisposition == FILE_OVERWRITE_IF)
		return true;
	return false;
}

bool IsContentRead(u32 desiredAccess, u32 createDisposition)
{
	return (desiredAccess & (GENERIC_READ | FILE_READ_DATA)) != 0;
}

bool IsContentUse(u32 desiredAccess, u32 createDisposition)
{
	return IsContentRead(desiredAccess, createDisposition) || IsContentWrite(desiredAccess, createDisposition);
}

bool IsWrite(u32 desiredAccess, u32 createDisposition)
{
	return IsContentWrite(desiredAccess, createDisposition);
}

u8 GetFileAccessFlags(DWORD desiredAccess, u32 createDisposition)
{
	u8 access = 0;
	if (IsContentRead(desiredAccess, createDisposition))
		access |= AccessFlag_Read;
	if (IsWrite(desiredAccess, createDisposition))
		access |= AccessFlag_Write;
	return access;
}

const tchar* StdToString(DWORD stdHandle)
{
	if (stdHandle == STD_OUTPUT_HANDLE)
		return L"StdOut";
	if (stdHandle == STD_INPUT_HANDLE)
		return L"StdIn";
	if (stdHandle == STD_ERROR_HANDLE)
		return L"StdErr";
	return L"StdINVALID";
}

#if UBA_DEBUG_LOG_ENABLED
StringBuffer<32> ToString(NTSTATUS s)
{
	StringBuffer<32> res;
	if (NT_SUCCESS(s))
		return res.Append(L"Success");
	if (s == STATUS_OBJECT_NAME_NOT_FOUND)
		return res.Append(L"STATUS_OBJECT_NAME_NOT_FOUND");
	if (s == STATUS_OBJECT_PATH_NOT_FOUND)
		return res.Append(L"STATUS_OBJECT_PATH_NOT_FOUND");
	if (s == STATUS_INVALID_HANDLE)
		return res.Append(L"STATUS_INVALID_HANDLE");
	if (s == STATUS_SHARING_VIOLATION)
		return res.Append(L"STATUS_SHARING_VIOLATION");
	if (s == STATUS_ACCESS_DENIED)
		return res.Append(L"STATUS_ACCESS_DENIED");
	if (s == STATUS_DIRECTORY_NOT_EMPTY)
		return res.Append(L"STATUS_DIRECTORY_NOT_EMPTY");
	return res.Appendf(L"Error (0x%x)", u32(s));
}
#endif

extern "C" RTL_PATH_TYPE NTAPI RtlDetermineDosPathNameType_U(PCWSTR DosFileName);
extern "C" ULONG NTAPI RtlIsDosDeviceName_U(PCWSTR DosFileName);

static void FixupDosPath(PCWSTR& dosPath, StringBufferBase& temp)
{
	if (!dosPath)
		return;

	if (RtlIsDosDeviceName_U(dosPath) != 0)
		return;

	switch (RtlDetermineDosPathNameType_U(dosPath))
	{
	case RtlPathTypeRelative:
		if (TStrchr(dosPath, ':')) // There was a colon somewhere later in the path
			break;
		temp.Append(g_virtualWorkingDir).EnsureEndsWithSlash().Append(dosPath);
		dosPath = temp.data;
		break;
	case RtlPathTypeRooted:
		temp.Append(g_virtualWorkingDir[0]).Append(':').Append(dosPath);
		dosPath = temp.data;
		break;
	case RtlPathTypeDriveRelative:
		wchar_t envVar[4] = { L'=', dosPath[0], L':', L'\0' };
		temp.count = GetEnvironmentVariableW(envVar, temp.data, temp.capacity);
		if (temp.count == 0)
			temp.Append(dosPath[0]).Append(TCV(":\\"));
		temp.EnsureEndsWithSlash().Append(dosPath + 2);
		dosPath = temp.data;
		break;
	}
}

NTSTATUS Detoured_RtlDosPathNameToNtPathName_U_WithStatus(PCWSTR dosPath, PUNICODE_STRING ntPath, PWSTR* filePart, VOID* reserved)
{
	const tchar* original = dosPath;(void)original;
	StringBuffer<> temp;
	FixupDosPath(dosPath, temp);
	auto res = True_RtlDosPathNameToNtPathName_U_WithStatus(dosPath, ntPath, filePart, reserved);
	//DEBUG_LOG(L"RtlDosPathNameToNtPathName_U_WithStatus:           %s -> %.*s", original, ntPath->Length/2, ntPath->Buffer);
	return res;
}

NTSTATUS Detoured_RtlDosPathNameToRelativeNtPathName_U_WithStatus(PCWSTR dosPath, PUNICODE_STRING ntPath, PWSTR* filePart, RTL_RELATIVE_NAME_U* relativeName)
{
	const tchar* original = dosPath;(void)original;
	StringBuffer<> temp;
	FixupDosPath(dosPath, temp);
	auto res = True_RtlDosPathNameToRelativeNtPathName_U_WithStatus(dosPath, ntPath, filePart, relativeName);
	//DEBUG_LOG(L"RtlDosPathNameToRelativeNtPathName_U_WithStatus:   %s -> %.*s", original, ntPath->Length/2, ntPath->Buffer);
	return res;
}

DWORD WINAPI Detoured_RtlGetFullPathName_U(const WCHAR* name, ULONG size, WCHAR* buffer, WCHAR** file_part)
{
	const tchar* original = name;(void)original;
	StringBuffer<> temp;
	FixupDosPath(name, temp);
	auto res = True_RtlGetFullPathName_U(name, size, buffer, file_part);
	//DEBUG_LOG(L"RtlGetFullPathName_U:                              %s -> %s", original, buffer);
	return res;
}

ULONG WINAPI Detoured_RtlGetFullPathName_UEx(const WCHAR* name, ULONG size, WCHAR* buffer, WCHAR** file_part, RTL_PATH_TYPE* type)
{
	const tchar* original = name;(void)original;
	StringBuffer<> temp;
	FixupDosPath(name, temp);
	auto res = True_RtlGetFullPathName_UEx(name, size, buffer, file_part, type);
	//DEBUG_LOG(L"RtlGetFullPathName_UEx:                            %s -> %s", original, buffer);
	return res;
}

BOOLEAN NTAPI Detoured_RtlDosPathNameToRelativeNtPathName_U(PCWSTR dosPath, PUNICODE_STRING ntPath, PWSTR* filePart, RTL_RELATIVE_NAME_U* relativeName)
{
	const tchar* original = dosPath;(void)original;
	StringBuffer<> temp;
	FixupDosPath(dosPath, temp);
	auto res = True_RtlDosPathNameToRelativeNtPathName_U(dosPath, ntPath, filePart, relativeName);
	//DEBUG_LOG(L"RtlDosPathNameToRelativeNtPathName_U:              %s -> %.*s", original, ntPath->Length/2, ntPath->Buffer);
	return res;
}

NTSTATUS Detoured_NtQueryVolumeInformationFile(HANDLE FileHandle, PIO_STATUS_BLOCK IoStatusBlock, PVOID FsInformation, ULONG Length, FS_INFORMATION_CLASS FsInformationClass)
{
	IoStatusBlock->Status = STATUS_SUCCESS;
	IoStatusBlock->Information = 0;

	auto runDefaultFsDeviceInfo = [&]()
		{
			auto& info = *(FILE_FS_DEVICE_INFORMATION*)FsInformation;
			info.DeviceType = FILE_DEVICE_DISK;
			info.Characteristics = FILE_DEVICE_IS_MOUNTED; // | FILE_REMOTE_DEVICE; // Use remote device to turn off some optimizations
			IoStatusBlock->Information = sizeof(FILE_FS_DEVICE_INFORMATION);
			DEBUG_LOG_DETOURED(L"NtQueryVolumeInformationFile", L"%llu (FileFsDeviceInformation) (%ls) -> %ls", uintptr_t(FileHandle), HandleToName(FileHandle), ToString(STATUS_SUCCESS).data);
			return STATUS_SUCCESS;
		};

	auto runDefaultFsVolumeInfo = [&](DirTableOffset entryOffset)
		{
			ULONG required = offsetof(FILE_FS_VOLUME_INFORMATION, VolumeLabel);
			if (Length < required)
			{
				IoStatusBlock->Status = STATUS_BUFFER_TOO_SMALL;
				IoStatusBlock->Information = 0;
				DEBUG_LOG_DETOURED(L"NtQueryVolumeInformationFile", L"%llu (FileFsVolumeInformation) (%ls) -> %ls", uintptr_t(FileHandle), HandleToName(FileHandle), ToString(STATUS_BUFFER_TOO_SMALL).data);
				return STATUS_BUFFER_TOO_SMALL;
			}

			auto& info = *(FILE_FS_VOLUME_INFORMATION*)FsInformation;
			UBA_ASSERT(IsValidEntry(entryOffset));
			DirectoryTable::EntryInformation entryInfo;
			g_directoryTable->GetEntryInformation(entryInfo, entryOffset);
			UBA_ASSERT(entryInfo.attributes != 0);
			info.VolumeCreationTime.QuadPart = 123;
			info.VolumeSerialNumber = entryInfo.volumeSerial;
			info.VolumeLabelLength = 0;
			info.SupportsObjects = true;
			info.VolumeLabel[0] = 0;
			IoStatusBlock->Information = offsetof(FILE_FS_VOLUME_INFORMATION, VolumeLabel) + info.VolumeLabelLength;
			DEBUG_LOG_DETOURED(L"NtQueryVolumeInformationFile", L"%llu (FileFsVolumeInformation) (%ls) -> %ls", uintptr_t(FileHandle), HandleToName(FileHandle), ToString(STATUS_SUCCESS).data);
			return STATUS_SUCCESS;
		};

	DETOURED_CALL(NtQueryVolumeInformationFile);
	HANDLE TrueHandle = FileHandle;
	if (isDetouredHandle(FileHandle))
	{
		auto& dh = asDetouredHandle(FileHandle);
		if (dh.fileObject->fileInfo->memoryFile)
		{
			if (FsInformationClass == FileFsDeviceInformation)
				return runDefaultFsDeviceInfo();
		}
		TrueHandle = dh.trueHandle;
		if (TrueHandle == INVALID_HANDLE_VALUE)
		{
			if (FsInformationClass == FileFsVolumeInformation)
				return runDefaultFsVolumeInfo(dh.dirTableOffset);
			if (FsInformationClass == FileFsDeviceInformation)
				return runDefaultFsDeviceInfo();
			UBA_ASSERTF(false, L"NtQueryVolumeInformationFile using class %u not handled %ls (%ls)", FsInformationClass, dh.fileObject->fileInfo->name, dh.fileObject->fileInfo->originalName);
		}
	}
	else if (isListDirectoryHandle(FileHandle))
	{
		auto& listHandle = asListDirectoryHandle(FileHandle);

		if (FsInformationClass == FileFsDeviceInformation)
			return runDefaultFsDeviceInfo();
		if (FsInformationClass == FileFsVolumeInformation)
		{
			DirTableOffset tableOffset;
			g_directoryTable->GetLatestOffset(tableOffset, listHandle.dir);
			return runDefaultFsVolumeInfo(tableOffset);
		}
		UBA_ASSERTF(false, L"NtQueryVolumeInformationFile called in ListDirectoryHandle using class %u which is not implemented (%ls)", FsInformationClass, HandleToName(FileHandle));
	}
	auto res = True_NtQueryVolumeInformationFile(TrueHandle, IoStatusBlock, FsInformation, Length, FsInformationClass);
	DEBUG_LOG_TRUE(L"NtQueryVolumeInformationFile", L"%llu (%s) (%ls) -> %ls", uintptr_t(FileHandle), ToString(FsInformationClass).data, HandleToName(FileHandle), ToString(res).data);
	return res;
}

NTSTATUS Detoured_NtQueryInformationFile(HANDLE FileHandle, PIO_STATUS_BLOCK IoStatusBlock, PVOID FileInformation, ULONG Length, FILE_INFORMATION_CLASS FileInformationClass)
{
	IoStatusBlock->Information = 0;
	IoStatusBlock->Status = STATUS_SUCCESS;

	DETOURED_CALL(NtQueryInformationFile);
	if (isListDirectoryHandle(FileHandle))
	{
		auto& listHandle = asListDirectoryHandle(FileHandle);
		if (FileInformationClass == FileIsRemoteDeviceInformation)
		{
			auto& info = *(FILE_IS_REMOTE_DEVICE_INFORMATION*)FileInformation;
			info.IsRemote = FALSE;
			DEBUG_LOG_DETOURED(L"NtQueryInformationFile", L"(FileIsRemoteDeviceInformation) %llu (%ls) -> Success", uintptr_t(FileHandle), HandleToName(FileHandle));
			return STATUS_SUCCESS;
		}
		else if (FileInformationClass == FileIdInformation)
		{
			auto& info = *(FILE_ID_INFORMATION*)FileInformation;
			if (listHandle.dir.latestOffset != InvalidTableOffset || listHandle.dir.latestOverlayOffset != InvalidTableOffset)
			{
				DirTableOffset entryOffset;
				g_directoryTable->GetLatestOffset(entryOffset, listHandle.dir);
				DirectoryTable::EntryInformation entryInfo;
				g_directoryTable->GetEntryInformation(entryInfo, entryOffset);
				info.VolumeSerialNumber = entryInfo.volumeSerial;
				u64* id = (u64*)&info.FileId;
				id[0] = 0;
				id[1] = entryInfo.fileIndex;
			}
			else
			{
				UBA_ASSERT(false);
				info.VolumeSerialNumber = 0;//attr.volumeSerial;
				memcpy(info.FileId.Identifier, &listHandle.dirNameKey, 16);
			}
			DEBUG_LOG_DETOURED(L"NtQueryInformationFile", L"(FileIdInformation) %llu (%ls) -> Success", uintptr_t(FileHandle), HandleToName(FileHandle));
			return STATUS_SUCCESS;
		}
		/*
		else if (FileInformationClass == FileNameInformation)
		{
			auto& info = *(FILE_NAME_INFORMATION*)FileInformation;
			u32 nameLen = u32(wcslen(listHandle.name));
			//UBA_ASSERT(info.FileNameLength/2 > nameLen);
			memcpy(info.FileName, listHandle.name, nameLen*2+2);
			info.FileNameLength = nameLen*2;
			UBA_ASSERT(false);
			return STATUS_SUCCESS;
		}
		else if (FileInformationClass == 55) // Undefined, some old compilers using this it seems
		{
			DEBUG_LOG_DETOURED(L"NtQueryInformationFile", L"TODO_THIS (55) %llu (%ls) -> Error", uintptr_t(FileHandle), HandleToName(FileHandle));
			return STATUS_NOT_SUPPORTED;
		}
		*/
		else
		{
			FatalError(1348, L"NtQueryInformationFile (%s) not implemented", ToString(FileInformationClass).data);
		}
	}

	HANDLE trueHandle = FileHandle;
	auto runTrue = [&]()
		{
			UBA_ASSERT(trueHandle != INVALID_HANDLE_VALUE);
			TimerScope ts(g_kernelStats.getFileInfo);
			auto res = True_NtQueryInformationFile(trueHandle, IoStatusBlock, FileInformation, Length, FileInformationClass);
			DEBUG_LOG_TRUE(L"NtQueryInformationFile", L"(%s) %llu (%ls) -> %ls", ToString(FileInformationClass).data, uintptr_t(FileHandle), HandleToName(FileHandle), ToString(res).data);
			return res;
		};

	if (!isDetouredHandle(FileHandle))
		return runTrue();

	auto& dh = asDetouredHandle(FileHandle);
	trueHandle = dh.trueHandle;

	auto setBasicInformation = [](FILE_BASIC_INFORMATION& out, DirectoryTable::EntryInformation& entryInfo)
		{
			out.CreationTime.QuadPart = entryInfo.lastWrite;
			out.LastAccessTime.QuadPart = entryInfo.lastWrite;
			out.LastWriteTime.QuadPart = entryInfo.lastWrite;
			out.ChangeTime.QuadPart = entryInfo.lastWrite;
			out.FileAttributes = entryInfo.attributes;
			return STATUS_SUCCESS;
		};

	auto getFileSizeAndAllocation = [&](u64& outSize, u64& outAllocationSize, FileInfo& fi, DirectoryTable::EntryInformation* entryInfo)
		{
			if (fi.memoryFile)
			{
				outSize = fi.memoryFile->writtenSize;
				outAllocationSize = fi.memoryFile->committedSize;
			}
			else if (fi.fileMapMem)
			{
				outSize = fi.fileMapMemSize;
				outAllocationSize = outSize;
			}
			else if (fi.size != InvalidValue)
			{
				outSize = fi.size;
				outAllocationSize = outSize;
			}
			else if (entryInfo)
			{
				outSize = entryInfo->size;
				outAllocationSize = outSize;
			}
			else
				return false;
			return true;
		};

	auto setStandardInformation = [&](FILE_STANDARD_INFORMATION& out, FileInfo& fi, DirectoryTable::EntryInformation* entryInfo)
		{
			u64 size = 0;
			u64 allocationSize = 0;
			if (!getFileSizeAndAllocation(size, allocationSize, fi, entryInfo))
				return STATUS_INVALID_HANDLE;
			out.EndOfFile.QuadPart = size;
			out.AllocationSize.QuadPart = allocationSize;
			out.DeletePending = FALSE;
			out.NumberOfLinks = 0;
			out.Directory = FALSE;
			return STATUS_SUCCESS;
		};

	auto setNameInformation = [&](FILE_NAME_INFORMATION& out, FileInfo& fi, u32 offset)
		{
			UBA_ASSERT(fi.originalName[1] == ':');
			const wchar_t* fileName = fi.originalName + 2; // skip drive letter and colon
			ULONG nameBytes = (ULONG)(wcslen(fileName) * sizeof(WCHAR));
			ULONG required = offset + nameBytes;
			if (Length < offset)
			{
				IoStatusBlock->Status = STATUS_INFO_LENGTH_MISMATCH;
				return STATUS_INFO_LENGTH_MISMATCH;
			}
			ULONG copy = Min(nameBytes, Length - offset);
			out.FileNameLength = nameBytes;
			if (copy)
				memcpy(out.FileName, fileName, copy);
			IoStatusBlock->Information = Min(Length, required);
			return STATUS_SUCCESS;
		};

	if (FileInformationClass == FileStandardInformation)
	{
		if (dh.type != HandleType_File)
		{
			if (dh.type == HandleType_StdOut || dh.type == HandleType_StdErr)
			{
				if (trueHandle == INVALID_HANDLE_VALUE)
					trueHandle = True_GetStdHandle(dh.type == HandleType_StdOut ? STD_OUTPUT_HANDLE : STD_ERROR_HANDLE);
				return runTrue();
			}
			UBA_ASSERTF(false, TC("NtQueryInformationFile is called on unsupported detoured handle: %u"), u32(dh.type));
		}
		FileInfo& fi = *dh.fileObject->fileInfo;
		if (!fi.memoryFile && !fi.isFileMap && fi.created) // fi.created means that it was created in process hierarchy and other fields might be outdated when it comes to size
			return runTrue();
		if (setStandardInformation(*(FILE_STANDARD_INFORMATION*)FileInformation, fi, nullptr) != STATUS_SUCCESS)
			return runTrue();
		DEBUG_LOG_DETOURED(L"NtQueryInformationFile", L"(FileStandardInformation) %llu (%s) -> %ls", uintptr_t(FileHandle), HandleToName(FileHandle), ToString(STATUS_SUCCESS).data);
		return STATUS_SUCCESS;
	}
	if (FileInformationClass == FilePositionInformation)
	{
		if (dh.type >= HandleType_StdErr)
		{
			if (dh.trueHandle != INVALID_HANDLE_VALUE)
				return runTrue();
			return STATUS_SUCCESS;
		}
		UBA_ASSERT(dh.type == HandleType_File);
		FileInfo& fi = *dh.fileObject->fileInfo;
		if (!fi.memoryFile && !fi.isFileMap)
			return runTrue();
		auto& info =  *(FILE_POSITION_INFORMATION*)FileInformation;
		info.CurrentByteOffset.QuadPart = dh.pos;
		DEBUG_LOG_DETOURED(L"NtQueryInformationFile", L"(FilePositionInformation) %llu (%s) -> %ls", uintptr_t(FileHandle), HandleToName(FileHandle), ToString(STATUS_SUCCESS).data);
		return STATUS_SUCCESS;
	}
	if (FileInformationClass == FileBasicInformation)
	{
		if (!IsValidEntry(dh.dirTableOffset))
			return runTrue();
		DirectoryTable::EntryInformation entryInfo;
		g_directoryTable->GetEntryInformation(entryInfo, dh.dirTableOffset);
		UBA_ASSERT(entryInfo.attributes != 0);
		setBasicInformation(*(FILE_BASIC_INFORMATION*)FileInformation, entryInfo);
		DEBUG_LOG_DETOURED(L"NtQueryInformationFile", L"(FileBasicInformation) %llu (%s) -> %ls", uintptr_t(FileHandle), HandleToName(FileHandle), ToString(STATUS_SUCCESS).data);
		return STATUS_SUCCESS;
	}
	if (FileInformationClass == FileAllInformation)
	{
		if (!IsValidEntry(dh.dirTableOffset))
			return runTrue();
		if (dh.trueHandle != INVALID_HANDLE_VALUE)
			return runTrue();
		UBA_ASSERT(dh.type == HandleType_File);

		DirectoryTable::EntryInformation entryInfo;
		g_directoryTable->GetEntryInformation(entryInfo, dh.dirTableOffset);
		UBA_ASSERT(entryInfo.attributes != 0);
		FileInfo& fi = *dh.fileObject->fileInfo;

		// TODO This code path is here to handle nodejs queries.. Is not properly implemented and miss things
		auto& info = *(FILE_ALL_INFORMATION*)FileInformation;
		NTSTATUS res = STATUS_SUCCESS;
		
		res = res == STATUS_SUCCESS ? setBasicInformation(info.BasicInformation, entryInfo) : res;
		res = res == STATUS_SUCCESS ? setStandardInformation(info.StandardInformation, fi, &entryInfo) : res;
		res = res == STATUS_SUCCESS ? setNameInformation(info.NameInformation, fi, FIELD_OFFSET(FILE_ALL_INFORMATION, NameInformation.FileName)) : res;
		info.InternalInformation.IndexNumber.QuadPart = entryInfo.fileIndex;
		info.EaInformation.EaSize = 0;
		info.AccessInformation.AccessFlags = dh.fileObject->desiredAccess;
		info.PositionInformation.CurrentByteOffset.QuadPart = 0;
		info.ModeInformation.Mode = 0;
		info.AlignmentInformation.AlignmentRequirement = 0;
		DEBUG_LOG_DETOURED(L"NtQueryInformationFile", L"(FileAllInformation) %llu (%s) -> %ls", uintptr_t(FileHandle), HandleToName(FileHandle), ToString(res).data);
		return res;
	}
	if (FileInformationClass == FileNetworkOpenInformation)
	{
		if (!IsValidEntry(dh.dirTableOffset))
			return runTrue();
		DirectoryTable::EntryInformation entryInfo;
		g_directoryTable->GetEntryInformation(entryInfo, dh.dirTableOffset);
		UBA_ASSERT(entryInfo.attributes != 0);
		FileInfo& fi = *dh.fileObject->fileInfo;

		u64 size = 0;
		u64 allocationSize = 0;
		getFileSizeAndAllocation(size, allocationSize, fi, &entryInfo);

		auto& info = *(FILE_NETWORK_OPEN_INFORMATION*)FileInformation;
		info.CreationTime.QuadPart = entryInfo.lastWrite;
		info.LastAccessTime.QuadPart = entryInfo.lastWrite;
		info.LastWriteTime.QuadPart = entryInfo.lastWrite;
		info.ChangeTime.QuadPart = entryInfo.lastWrite;
		info.EndOfFile.QuadPart = size;
		info.AllocationSize.QuadPart = allocationSize;
		info.FileAttributes = entryInfo.attributes;
		DEBUG_LOG_DETOURED(L"NtQueryInformationFile", L"(FileNetworkOpenInformation) %llu (%s) -> %ls", uintptr_t(FileHandle), HandleToName(FileHandle), ToString(STATUS_SUCCESS).data);
		return STATUS_SUCCESS;
	}
	if (FileInformationClass == FileNormalizedNameInformation)
	{
		if (dh.trueHandle != INVALID_HANDLE_VALUE)
			return runTrue();
		const tchar* src = TC("");//dh.fileObject->fileInfo->originalName;
		if (Length < sizeof(ULONG))
			return STATUS_INFO_LENGTH_MISMATCH;
		auto& info = *(FILE_NAME_INFORMATION*)FileInformation;
		u64 destLen = Length - FIELD_OFFSET(FILE_NAME_INFORMATION, FileName);
		u64 srcLen = TStrlen(src)*sizeof(tchar);
		u64 toWrite = Min(srcLen, destLen);
		info.FileNameLength = (ULONG)srcLen;
		memcpy(info.FileName, src, toWrite);
		NTSTATUS res = destLen >= srcLen ? STATUS_SUCCESS : STATUS_BUFFER_OVERFLOW;
		IoStatusBlock->Information = FIELD_OFFSET(FILE_NAME_INFORMATION, FileName) + toWrite;
		IoStatusBlock->Status = res;
		DEBUG_LOG_DETOURED(L"NtQueryInformationFile", L"(FileNormalizedNameInformation) %llu (%s) -> %ls", uintptr_t(FileHandle), HandleToName(FileHandle), ToString(res).data);
		return res;
	}
	if (FileInformationClass == FileNameInformation)
	{
		if (dh.trueHandle != INVALID_HANDLE_VALUE)
			return runTrue();
		NTSTATUS res = setNameInformation(*(FILE_NAME_INFORMATION*)FileInformation, *dh.fileObject->fileInfo, FIELD_OFFSET(FILE_NAME_INFORMATION, FileName));
		DEBUG_LOG_DETOURED(L"NtQueryInformationFile", L"(FileNameInformation) %llu (%s) -> %ls", uintptr_t(FileHandle), HandleToName(FileHandle), ToString(res).data);
		return res;
	}

	UBA_ASSERTF(false, L"NtQueryInformationFile (%u) failed using detoured handle %ls (%ls)", FileInformationClass, dh.fileObject->fileInfo->name, dh.fileObject->fileInfo->originalName);
	return runTrue();
}

NTSTATUS NTAPI Detoured_NtQueryDirectoryFile(HANDLE FileHandle, HANDLE Event, PIO_APC_ROUTINE ApcRoutine, PVOID ApcContext, PIO_STATUS_BLOCK IoStatusBlock, PVOID FileInformation, ULONG Length,
	FILE_INFORMATION_CLASS FileInformationClass, BOOLEAN ReturnSingleEntry, PUNICODE_STRING FileName, BOOLEAN RestartScan)
{
	DETOURED_CALL(NtQueryDirectoryFile);

	if (isListDirectoryHandle(FileHandle))
	{
		auto returnFunc = [&](NTSTATUS res)
			{
				DEBUG_LOG_DETOURED(L"NtQueryDirectoryFile", L"%llu %s -> %s", u64(FileHandle), HandleToName(FileHandle), ToString(res).data);
				return res;
			};

		IoStatusBlock->Information = 0;

		auto& listHandle = asListDirectoryHandle(FileHandle);
		NTSTATUS res = STATUS_NO_MORE_FILES;

		UBA_ASSERT(Event == 0 && ApcRoutine == nullptr && ApcContext == nullptr);

		if (RestartScan)
		{
			// Check so directory has not changed since this handle was opened.. RestartScan means that it needs to be up-to-date
			auto& dir = listHandle.dir;
			if (!g_directoryTable->IsDirectoryUpToDate(dir))
			{
				StringBuffer<> fileNameLower(listHandle.originalName);
				fileNameLower.MakeLower();
				DirHash hash(fileNameLower);
				SCOPED_READ_LOCK(dir.lock, lock);
				g_directoryTable->PopulateDirectoryNoLock(hash.open, dir);
				listHandle.fileTableOffsets.resize(dir.files.size());
				u32 it = 0;
				for (auto& pair : dir.files)
					listHandle.fileTableOffsets[it++] = pair.second;
				lock.Leave();
			}
			listHandle.it = 0;
		}
		u8* prevInformation = nullptr;
		u8* it = (u8*)FileInformation;
		u8* bufferEnd = it + Length;

		u32 structSize = 0;
		u32 fileNameOffset = 0;
		if (FileInformationClass == FileDirectoryInformation)
		{
			structSize = sizeof(FILE_DIRECTORY_INFORMATION);
			fileNameOffset = FIELD_OFFSET(FILE_DIRECTORY_INFORMATION, FileName);
		}
		else if (FileInformationClass == FileIdBothDirectoryInformation)
		{
			structSize = sizeof(FILE_ID_BOTH_DIR_INFORMATION);
			fileNameOffset = FIELD_OFFSET(FILE_ID_BOTH_DIR_INFORMATION, FileName);
		}
		else
		{
			UBA_ASSERT(false);
			IoStatusBlock->Status = STATUS_OBJECT_NAME_NOT_FOUND;
			return returnFunc(STATUS_OBJECT_NAME_NOT_FOUND);
		}

		if (Length < structSize)
			return returnFunc(STATUS_BUFFER_OVERFLOW);

		// Fix for buggy program (rustc.exe). even if there are zero entries they still seem to check this variable
		memset(FileInformation, 0, structSize);

		while (true)
		{
			if (listHandle.it == listHandle.fileTableOffsets.size())
				break;

			DirTableOffset fileOffset = listHandle.fileTableOffsets[listHandle.it++];

			DirectoryTable::EntryInformation entryInfo;
			wchar_t fileName[512];
			g_directoryTable->GetEntryInformation(entryInfo, fileOffset, fileName, sizeof_array(fileName));
			if (entryInfo.attributes == 0) // File was deleted
				continue;

			if (FileName && !StringView(FileName->Buffer, FileName->Length / 2).Equals(fileName))
				continue;

			u32 fileNameBytes = u32(wcslen(fileName) * 2);

			wchar_t* fileNamePos = (wchar_t*)((u8*)it + fileNameOffset);
			u8* writeEnd = (u8*)fileNamePos + fileNameBytes;
			if (writeEnd > bufferEnd)
			{
				--listHandle.it;
				if (!prevInformation)
					res = STATUS_BUFFER_OVERFLOW;
				break;
			}

			memset(it, 0, structSize);
			auto& info = *(FILE_DIRECTORY_INFORMATION*)it;

			memcpy(fileNamePos, fileName, fileNameBytes);

			info.FileNameLength = fileNameBytes;
			info.FileAttributes = entryInfo.attributes;
			info.LastWriteTime.QuadPart = entryInfo.lastWrite;
			info.EndOfFile.QuadPart = entryInfo.size;
			//info.FileIndex = entryInfo.fileIndex; // This needs serialno too?
			info.AllocationSize.QuadPart = entryInfo.size;
			info.CreationTime.QuadPart = entryInfo.lastWrite;

			if (prevInformation)
			{
				((FILE_DIRECTORY_INFORMATION*)prevInformation)->NextEntryOffset = u32(it - prevInformation);
			}

			prevInformation = it;
			it = (u8*)fileNamePos + info.FileNameLength + 2;

			// Very spammy
			//DEBUG_LOG_DETOURED(L"NtQueryDirectoryFile", L"%llu %.*s", u64(FileHandle), int(fileNameBytes/sizeof(tchar)), fileNamePos);

			res = STATUS_SUCCESS;

			if (ReturnSingleEntry)
				break;
		}

		IoStatusBlock->Status = res;
		IoStatusBlock->Information = it - (u8*)FileInformation;

#if 0//UBA_DEBUG_VALIDATE
		if (false) // Sorting can mismatch
		{
			u8 info2Mem[1024];
			UBA_ASSERT(Length <= sizeof(info2Mem));
			auto& info2 = *(FILE_DIRECTORY_INFORMATION*)info2Mem;
			NTSTATUS res2;
			do
			{
				res2 = True_NtQueryDirectoryFile(listHandle.validateHandle, Event, ApcRoutine, ApcContext, IoStatusBlock, &info2, Length, FileInformationClass, ReturnSingleEntry, FileName, RestartScan);
				if (res2 >= 0)
				{
					info2.FileName[info2.FileNameLength / 2] = 0;
					ToLower(info2.FileName);
				}
			} while (wcscmp(info2.FileName, L".") == 0 || wcscmp(info2.FileName, L"..") == 0);
			UBA_ASSERT(res < 0 && res2 < 0 || res >= 0 && res2 >= 0);
			UBA_ASSERT(res < 0 || wcscmp(info.FileName, info2.FileName) == 0);
		}
#endif
		return returnFunc(res);
	}

	HANDLE trueHandle = FileHandle;
	if (isDetouredHandle(FileHandle))
	{
		DetouredHandle& h = asDetouredHandle(FileHandle);
		trueHandle = h.trueHandle;
		UBA_ASSERTF(trueHandle != INVALID_HANDLE_VALUE, L"NtQueryDirectoryFile (%s) not implemented for detoured handles (%ls)", ToString(FileInformationClass).data, HandleToName(FileHandle));
	}

	NTSTATUS res = True_NtQueryDirectoryFile(trueHandle, Event, ApcRoutine, ApcContext, IoStatusBlock, FileInformation, Length, FileInformationClass, ReturnSingleEntry, FileName, RestartScan);

#if UBA_DEBUG_LOG_ENABLED
	if (res == STATUS_SUCCESS)
	{
		u8* it = (u8*)FileInformation;
		while (true)
		{
			const wchar_t* fileNamePos;
			if (FileInformationClass == FileDirectoryInformation)
				fileNamePos = ((FILE_DIRECTORY_INFORMATION*)it)->FileName;
			else if (FileInformationClass == 2)//FileFullDirectoryInformation)
				fileNamePos = ((FILE_FULL_DIR_INFORMATION*)it)->FileName;
			else
				break;
			StringBuffer<> b;
			b.Append(fileNamePos, ((FILE_DIRECTORY_INFORMATION*)it)->FileNameLength / 2);
			DEBUG_LOG_TRUE(L"NtQueryDirectoryFile", L"%llu %ls", u64(FileHandle), b.data);

			u32 nextOffset = ((FILE_DIRECTORY_INFORMATION*)it)->NextEntryOffset;
			if (!nextOffset)
				break;
			it += nextOffset;
			//DEBUG_LOG_TRUE(L"NtQueryDirectoryFile", L"(%u) %llu (%ls) -> %ls", FileInformationClass, uintptr_t(FileHandle), HandleToName(FileHandle), ToString(res));
		}
		//DEBUG_LOG_TRUE(L"NtQueryDirectoryFile", L"(%u) %llu (%ls) -> %ls", FileInformationClass, uintptr_t(FileHandle), HandleToName(FileHandle), ToString(res));
	}
#endif

	DEBUG_LOG_TRUE(L"NtQueryDirectoryFile", L"%llu -> %s", u64(FileHandle), ToString(res).data);
	return res;
}

NTSTATUS Detoured_NtQueryFullAttributesFile(POBJECT_ATTRIBUTES ObjectAttributes, PVOID Attributes)
{
	StringView fileName(ObjectAttributes->ObjectName->Buffer, ObjectAttributes->ObjectName->Length/sizeof(tchar));

	if (!g_rules->CanDetour(fileName, g_runningRemote) || fileName.Contains(L"::")) // Some weird .net path used by dotnet.exe ... ignore for now!
	{
		TimerScope ts(g_kernelStats.getFileInfo);
		auto res = True_NtQueryFullAttributesFile(ObjectAttributes, Attributes);
		DEBUG_LOG_TRUE(L"NtQueryFullAttributesFile", L"(%.*s) -> %s", fileName.count, fileName.data, ToString(res).data);
		return res;
	}


	StringBuffer<MaxPath> fixedName;
	FixPath(fixedName, fileName);

	DevirtualizePath(fixedName);

	FileAttributes attr;
	Shared_GetFileAttributes(attr, fixedName);

	UBA_ASSERT(!ObjectAttributes->RootDirectory);

	NTSTATUS res = STATUS_OBJECT_NAME_NOT_FOUND;
	if (attr.exists && attr.lastError == ErrorSuccess)
	{
		WIN32_FILE_ATTRIBUTE_DATA& data = attr.data;
		res = STATUS_SUCCESS;
		auto& info = *(FILE_NETWORK_OPEN_INFORMATION*)Attributes;;
		info.CreationTime = ToLargeInteger(data.ftCreationTime.dwHighDateTime, data.ftCreationTime.dwLowDateTime);
		info.LastAccessTime = ToLargeInteger(data.ftLastAccessTime.dwHighDateTime, data.ftLastAccessTime.dwLowDateTime);
		info.LastWriteTime = ToLargeInteger(data.ftLastWriteTime.dwHighDateTime, data.ftLastWriteTime.dwLowDateTime);
		info.ChangeTime = ToLargeInteger(data.ftLastWriteTime.dwHighDateTime, data.ftLastWriteTime.dwLowDateTime);
		info.AllocationSize = ToLargeInteger(data.nFileSizeHigh, data.nFileSizeLow);
		info.EndOfFile = info.AllocationSize;
		info.FileAttributes = data.dwFileAttributes;
	}

	DEBUG_LOG_DETOURED(L"NtQueryFullAttributesFile", L"(%.*s) -> %s (Size: %llu)", fileName.count, fileName.data, ToString(res).data, ToLargeInteger(attr.data.nFileSizeHigh, attr.data.nFileSizeLow).QuadPart);
	return res;
}

NTSTATUS Detoured_NtSetInformationFile(HANDLE FileHandle, PIO_STATUS_BLOCK IoStatusBlock, PVOID FileInformation, ULONG Length, FILE_INFORMATION_CLASS FileInformationClass)
{
	DETOURED_CALL(NtSetInformationFile);

	IoStatusBlock->Information = 0;
	IoStatusBlock->Status = STATUS_SUCCESS;

	HANDLE trueHandle = FileHandle;

	auto runTrue = [&]()
		{
			UBA_ASSERT(trueHandle != INVALID_HANDLE_VALUE);
			TimerScope ts(g_kernelStats.setFileInfo);
			auto res = True_NtSetInformationFile(trueHandle, IoStatusBlock, FileInformation, Length, FileInformationClass);
			DEBUG_LOG_TRUE(L"NtSetInformationFile", L"(%s) %llu (%s) -> %s", ToString(FileInformationClass).data, uintptr_t(FileHandle), HandleToName(FileHandle), ToString(res).data);
			return res;
		};

	if (!isDetouredHandle(FileHandle))
	{
		UBA_ASSERT(!isListDirectoryHandle(FileHandle));
		return runTrue();
	}

	DetouredHandle& h = asDetouredHandle(FileHandle);
	trueHandle = h.trueHandle;

	if (FileInformationClass == FileBasicInformation)
	{
		auto& info = *(FILE_BASIC_INFORMATION*)FileInformation;
		u32 attr = info.FileAttributes;
		bool attrUnchanged = attr == (g_isRunningWine ? 0 : 0xFFFFFFFF);
		if (attrUnchanged && info.CreationTime.QuadPart == -1 && info.LastWriteTime.QuadPart == -1)
		{
			DEBUG_LOG_DETOURED(L"NtSetInformationFile", L"(FileBasic) %llu IGNORE (%ls)", uintptr_t(FileHandle), HandleToName(FileHandle));
			return STATUS_SUCCESS;
		}
		return runTrue();
	}
	if (FileInformationClass == FilePositionInformation)
	{
		auto& info = *(FILE_POSITION_INFORMATION*)FileInformation;
		if (h.type >= HandleType_StdErr)
		{
			info.CurrentByteOffset.QuadPart = 0;
			return STATUS_SUCCESS;
		}
		UBA_ASSERT(h.type == HandleType_File);
		if (h.type >= HandleType_StdErr)
		{
			UBA_ASSERT(false); // TODO: What should we do with this?
			return STATUS_SUCCESS;
		}

		FileInfo& fi = *h.fileObject->fileInfo;
		if (fi.memoryFile || fi.isFileMap)
		{
			h.pos = info.CurrentByteOffset.QuadPart;
			DEBUG_LOG_DETOURED(L"NtSetInformationFile", L"(FilePosition MEMORY) %llu %s -> Success", uintptr_t(FileHandle), HandleToName(FileHandle));
			return STATUS_SUCCESS;
		}
		return runTrue();
	}
	if (FileInformationClass == FileEndOfFileInformation)
	{
		UBA_ASSERT(h.type == HandleType_File);
		FileInfo& fi = *h.fileObject->fileInfo;
		if (MemoryFile* mf = fi.memoryFile)
		{
			DEBUG_LOG_DETOURED(L"NtSetInformationFile", L"(FileEndOfFile MEMORY) %llu NewSize: %llu (%ls) -> Success", uintptr_t(FileHandle), mf->writtenSize, HandleToName(FileHandle));
			mf->writtenSize = h.pos;
			mf->isReported = false;
			mf->EnsureCommitted(h, mf->writtenSize);

			auto& info = *(FILE_END_OF_FILE_INFORMATION*)FileInformation;
			info.EndOfFile.QuadPart = h.pos;
			return STATUS_SUCCESS;
		}
		return runTrue();
	}
	if (FileInformationClass == FileRenameInformation || FileInformationClass == FileRenameInformationEx)
	{
		// We can end up in here through MoveFileEx and SetFileInformationByHandle
		auto& info = *(FILE_RENAME_INFORMATION*)FileInformation;
		StringView nameView(info.FileName, info.FileNameLength / 2);
		if (nameView.StartsWith(L"\\??\\"))
			nameView = nameView.Skip(4);
		StringBuffer<> newName;
		FixPath(newName, nameView.data, nameView.count);
		DevirtualizePath(newName);
		FileObject& fo = *h.fileObject;
		fo.newName = newName.ToString();

		StringKey newFileNameKey = ToStringKeyLower(newName);

		DEBUG_LOG_DETOURED(L"NtSetInformationFile", L"File is set to be renamed on close (from %ls to %ls)", HandleToName(FileHandle), fo.newName.c_str());

		if (auto memoryFile = fo.fileInfo->memoryFile)
		{
			memoryFile->isReported = false;
			return STATUS_SUCCESS;
		}

		UBA_ASSERT(!fo.fileInfo->isFileMap);

		if (g_runningRemote) // This needs a proper solution as the comments above.
			return STATUS_SUCCESS;

		// In case we are using vfs we need to replace the information before calling the true NtSetInformationFile
		u8 tempBuffer[sizeof(FILE_RENAME_INFORMATION) + 512];
		auto& info2 = *(FILE_RENAME_INFORMATION*)tempBuffer;
		memcpy(&info2, &info, sizeof(FILE_RENAME_INFORMATION));
		memcpy(info2.FileName, TC("\\??\\"), 8);
		memcpy(info2.FileName + 4, newName.data, newName.count * 2);
		info2.FileNameLength = (newName.count+4) * 2;
			
		FileInformation = &info2;
		Length = sizeof(FILE_RENAME_INFORMATION) + info2.FileNameLength + 2;
		return runTrue();
	}
	if (FileInformationClass == FileDispositionInformation)
	{
		auto& info = *(FILE_DISPOSITION_INFORMATION*)FileInformation;
		FileObject& fo = *h.fileObject;
		if (info.DeleteFile)
		{
			DEBUG_LOG_DETOURED(L"NtSetInformationFile", L"File is set to be deleted on close (%ls)", HandleToName(FileHandle));
			fo.deleteOnClose = true;
		}
		else if (fo.deleteOnClose)
		{
			DEBUG_LOG_DETOURED(L"NtSetInformationFile", L"File is set to NOT be deleted on close (%ls)", HandleToName(FileHandle));
			fo.deleteOnClose = false;
		}
		else
		{
			DEBUG_LOG_DETOURED(L"NtSetInformationFile", L"%llu (FileDispositionInfo %u)", uintptr_t(FileHandle), info.DeleteFileW);
		}
		if (fo.fileInfo->memoryFile)
			return STATUS_SUCCESS;
	}
	if (FileInformationClass == FileAllocationInformation)
	{
		FileObject& fo = *h.fileObject;
		FileInfo& fi = *fo.fileInfo;
		if (MemoryFile* mf = fi.memoryFile)
		{
			auto& info = *(FILE_ALLOCATION_INFORMATION*)FileInformation;
			mf->EnsureCommitted(h, info.AllocationSize.QuadPart);
			DEBUG_LOG_TRUE(L"NtSetInformationFile", L"%llu (FileAllocationInfo) Size: %llu", uintptr_t(FileHandle), info.AllocationSize.QuadPart);
			return STATUS_SUCCESS;
		}
	}

	if (trueHandle == INVALID_HANDLE_VALUE)
	{
		//UBA_ASSERT(!g_runningRemote);
		// TODO: This needs to be sent back to Session.. so session can set whatever needs to be set.
		DEBUG_LOG_DETOURED(L"NtSetInformationFile", L"(%s) SKIPPED!!!!!!!!! %llu (%ls) -> Skipped", ToString(FileInformationClass).data, uintptr_t(FileHandle), HandleToName(FileHandle));
		return STATUS_SUCCESS;
	}
	return runTrue();
}

NTSTATUS Detoured_NtSetInformationObject(HANDLE ObjectHandle, OBJECT_INFORMATION_CLASS ObjectInformationClass, PVOID ObjectInformation, ULONG Length)
{
	if (isDetouredHandle(ObjectHandle))
	{
		DetouredHandle& h = asDetouredHandle(ObjectHandle);
		ObjectHandle = h.trueHandle;
		UBA_ASSERT(ObjectHandle != INVALID_HANDLE_VALUE);
	}
	auto res = True_NtSetInformationObject(ObjectHandle, ObjectInformationClass, ObjectInformation, Length);
	DEBUG_LOG_TRUE(L"NtSetInformationObject", L"(%u) %llu (%ls) -> %ls", ObjectInformationClass, uintptr_t(ObjectHandle), HandleToName(ObjectHandle), ToString(res).data);
	return res;
}

NTSTATUS Detoured_NtRemoveIoCompletionEx(HANDLE IoCompletionHandle, FILE_IO_COMPLETION_INFORMATION *info, ULONG count, ULONG *written, LARGE_INTEGER* timeout, BOOLEAN alertable)
{
	auto res = True_NtRemoveIoCompletionEx(IoCompletionHandle, info, count, written, timeout, alertable);
	DEBUG_LOG_TRUE(L"NtRemoveIoCompletionEx", L"%llu (Count: %lu Key: %llu Timeout: %lluns) -> %s (%lu)", u64(IoCompletionHandle), count, (written && *written) ? u64((ULONG_PTR)info[0].KeyContext) : 0ull, timeout ? u64(timeout->QuadPart) : 0ull, ToString(res).data, *written);
	if (res == STATUS_SUCCESS)
		Rpc_UpdateTables(); // This is a bit ugly but we know this is how msbuild worker nodes sync with each other..
	return res;
}

struct JobInfo { Vector<HANDLE> processes; bool shouldKill = false; };
auto& g_jobsLock = *new ReaderWriterLock();
auto& g_jobs = *new UnorderedMap<FileObject*, JobInfo>();

void TerminateProcessesInJob(JobInfo&& info, u32 exitCode)
{
	for (auto& process : info.processes)
	{
		DWORD pid = GetProcessId(process);
		DEBUG_LOG(L"PROCESS %lu WILL BE TERMINATED by job", pid);
		RPC_MESSAGE(ExitChildProcess, detach) // Let it be part of detach timer for now
			writer.WriteU32(pid);
		writer.WriteU32(exitCode);
		writer.Flush();
	}
}

void CloseProcess(u32 exitStatus)
{
	{
		SCOPED_WRITE_LOCK(g_jobsLock, lock);
		for (auto& kv : g_jobs)
			if (kv.second.shouldKill)
				TerminateProcessesInJob(std::move(kv.second), exitStatus);
	}
	CloseCaches();
	SendExitMessage(exitStatus, GetTime());
	PostDeinit();
	#if UBA_DEBUG_LOG_ENABLED
	FlushDebugLog();
	#endif
}

NTSTATUS NTAPI Detoured_NtTerminateProcess(HANDLE ProcessHandle, NTSTATUS ExitStatus)
{
	// NtTerminateProcess can be reached in three ways.. one is through ExitProcess, TerminateProcess and NtTerminateProcess
	// If called through ExitMessage we don't want to do anymore work here.
	DETOURED_CALL(TerminateProcess);
	DEBUG_LOG_DETOURED(L"NtTerminateProcess", L"%llu (%ls) ExitCode: %lu", u64(ProcessHandle), HandleToName(ProcessHandle), ExitStatus);

	HANDLE trueProcess = ProcessHandle;
	bool isSelf = trueProcess == NtCurrentProcess() || trueProcess == NULL;
	if (!isSelf)
		if (isDetouredHandle(trueProcess))
			trueProcess = asDetouredHandle(trueProcess).trueHandle;

	DWORD pid = GetProcessId(trueProcess);
	if (!isSelf)
		isSelf = pid == GetCurrentProcessId();

	if (isSelf)
	{
		// Not called through ExitProcess
		if (!g_exitMessageSent && !DisallowDetourDepth())
			CloseProcess((DWORD)ExitStatus);
	}
	else
	{
		DEBUG_LOG(L"PROCESS %lu WILL BE TERMINATED", pid);
		RPC_MESSAGE(ExitChildProcess, detach) // Let it be part of detach timer for now
		writer.WriteU32(pid);
		writer.WriteU32(ExitStatus);
		writer.Flush();
	}

	return True_NtTerminateProcess(trueProcess, ExitStatus);
}

NTSTATUS Detoured_NtCreateJobObject(HANDLE* handle, ACCESS_MASK access, const OBJECT_ATTRIBUTES* attr)
{
	NTSTATUS res = True_NtCreateJobObject(handle, access, attr);
	auto g = MakeGuard([&]() { DEBUG_LOG_TRUE(L"NtCreateJobObject", L"%llu -> %s", u64(*handle), ToString(res).data); });
	if (res != STATUS_SUCCESS)
		return res;

	auto fo = new FileObject();
	auto dh = new DetouredHandle(HandleType_Job, *handle);
	dh->fileObject = fo;
	*handle = makeDetouredHandle(dh);
	return STATUS_SUCCESS;
}

NTSTATUS Detoured_NtSetInformationJobObject(HANDLE handle, JOBOBJECTINFOCLASS infoClass, void* info, ULONG len)
{
	UBA_ASSERT(isDetouredHandle(handle));
	DetouredHandle& dh = asDetouredHandle(handle);
	HANDLE trueHandle = dh.trueHandle;

	NTSTATUS res = True_NtSetInformationJobObject(trueHandle, infoClass, info, len);
	
	const tchar* prefix = TC("");(void)prefix;
	auto g = MakeGuard([&]() { DEBUG_LOG_TRUE(L"NtSetInformationJobObject", L"%s(%u) -> %s", prefix, infoClass, ToString(res).data); });
	
	if (res != STATUS_SUCCESS)
		return res;

	bool killOnJobClose = false;
	if (infoClass == JobObjectBasicLimitInformation)
	{
		auto info2 = *(JOBOBJECT_BASIC_LIMIT_INFORMATION*)info;
		killOnJobClose = (info2.LimitFlags & JOB_OBJECT_LIMIT_KILL_ON_JOB_CLOSE) != 0;
	}
	if (infoClass == JobObjectExtendedLimitInformation)
	{
		auto info2 = *(JOBOBJECT_EXTENDED_LIMIT_INFORMATION*)info;
		killOnJobClose = (info2.BasicLimitInformation.LimitFlags & JOB_OBJECT_LIMIT_KILL_ON_JOB_CLOSE) != 0;
	}
	if (!killOnJobClose)
		return res;

	SCOPED_WRITE_LOCK(g_jobsLock, lock);
	JobInfo& jobInfo = g_jobs[dh.fileObject];
	jobInfo.shouldKill = true;
	prefix = TC("(KillOnClose) ");
	return res;
}

NTSTATUS Detoured_NtAssignProcessToJobObject(HANDLE job, HANDLE process)
{
	UBA_ASSERT(isDetouredHandle(job));
	DetouredHandle& dh = asDetouredHandle(job);
	HANDLE trueJobHandle = dh.trueHandle;

	HANDLE processHandle = process;
	if (isDetouredHandle(process))
		processHandle = asDetouredHandle(process).trueHandle;

	NTSTATUS res = True_NtAssignProcessToJobObject(trueJobHandle, processHandle);
	DEBUG_LOG_DETOURED(L"NtAssignProcessToJobObject", L"Process %llu assigned to %llu -> %s", u64(process), u64(job), ToString(res).data);
	if (res != STATUS_SUCCESS)
		return res;

	SCOPED_WRITE_LOCK(g_jobsLock, lock);
	JobInfo& jobInfo = g_jobs[dh.fileObject];
	jobInfo.processes.push_back(process);
	return res;
}

NTSTATUS Detoured_NtTerminateJobObject(HANDLE handle, NTSTATUS status)
{
	UBA_ASSERT(!isDetouredHandle(handle));
	auto res = True_NtTerminateJobObject(handle, status);
	DEBUG_LOG_DETOURED(L"NtTerminateJobObject", L"%llu -> %s", u64(handle), ToString(res).data);
	return res;
}


NTSTATUS NTAPI Detoured_NtCreateSection(PHANDLE SectionHandle, ACCESS_MASK DesiredAccess, POBJECT_ATTRIBUTES ObjectAttributes, PLARGE_INTEGER MaximumSize, ULONG SectionPageProtection, ULONG AllocationAttributes, HANDLE FileHandle)
{
	DETOURED_CALL(NtCreateSection);
	if (isDetouredHandle(FileHandle))
	{
		DetouredHandle& h = asDetouredHandle(FileHandle);
		FileHandle = h.trueHandle;
		UBA_ASSERTF(FileHandle != INVALID_HANDLE_VALUE, L"NtCreateSection does not support detoured handles without backing file (%s)", HandleToName(FileHandle));
	}
	return True_NtCreateSection(SectionHandle, DesiredAccess, ObjectAttributes, MaximumSize, SectionPageProtection, AllocationAttributes, FileHandle);
}

VOID NTAPI Detoured_RtlExitUserProcess(NTSTATUS ExitStatus)
{
	// Can't log this one
	DETOURED_CALL(ExitProcess);
	//DEBUG_LOG_TRUE(L"RtlExitUserProcess", L"(%u)", uExitCode);

	CloseProcess(u32(ExitStatus));
	True_RtlExitUserProcess(ExitStatus);
}

bool g_checkRtlHeap = true;

SIZE_T Detoured_RtlSizeHeap(HANDLE HeapPtr, ULONG Flags, PVOID Ptr)
{
#if UBA_USE_MIMALLOC
	if (g_checkRtlHeap && IsInMiMalloc(Ptr))
		return mi_usable_size(Ptr);
#endif
	return True_RtlSizeHeap(HeapPtr, Flags, Ptr);
}

BOOLEAN Detoured_RtlFreeHeap(PVOID HeapHandle, ULONG Flags, PVOID BaseAddress)
{
#if UBA_USE_MIMALLOC
	if (g_checkRtlHeap && IsInMiMalloc(BaseAddress))
	{
		mi_free(BaseAddress);
		return true;
	}
#endif
	return True_RtlFreeHeap(HeapHandle, Flags, BaseAddress);
}

NTSTATUS Detoured_RtlAnsiStringToUnicodeString(PUNICODE_STRING DestinationString, PCANSI_STRING SourceString, BOOLEAN AllocateDestinationString)
{
#if UBA_USE_MIMALLOC
	if (AllocateDestinationString && g_useMiMalloc)
	{
		DestinationString->MaximumLength = (USHORT)((SourceString->Length+1) * 2);
		DestinationString->Buffer = (wchar_t*)mi_malloc(DestinationString->MaximumLength);
		AllocateDestinationString = false;
	}
#endif
	auto res = True_RtlAnsiStringToUnicodeString(DestinationString, SourceString, AllocateDestinationString);
	return res;
}

NTSTATUS Detoured_RtlUnicodeStringToAnsiString(PANSI_STRING DestinationString, PCUNICODE_STRING SourceString, BOOLEAN AllocateDestinationString)
{
#if UBA_USE_MIMALLOC
	if (AllocateDestinationString && g_useMiMalloc)
	{
		DWORD ret;
		RtlUnicodeToMultiByteSize( &ret, SourceString->Buffer, SourceString->Length );
		DWORD len = ret + 1;
		DestinationString->MaximumLength = (USHORT)len;
		DestinationString->Buffer = (char*)mi_malloc(len);
		AllocateDestinationString = false;
	}
#endif
	return True_RtlUnicodeStringToAnsiString(DestinationString, SourceString, AllocateDestinationString);
}

VOID Detoured_RtlFreeAnsiString(PANSI_STRING AnsiString)
{
#if UBA_USE_MIMALLOC
	if (AnsiString && AnsiString->Buffer && g_checkRtlHeap && IsInMiMalloc(AnsiString->Buffer))
	{
		mi_free(AnsiString->Buffer);
		return;
	}
#endif
	return True_RtlFreeAnsiString(AnsiString);
}

BOOLEAN Detoured_RtlIsPackageSid(PSID Sid)
{
	DETOURED_CALL(RtlIsPackageSid);
	DEBUG_LOG_TRUE(L"RtlIsPackageSid", L"");
	return FALSE;//True_RtlIsPackageSid(Sid);
}

BOOLEAN Detoured_RtlIsCapabilitySid(PSID sid)
{
	DETOURED_CALL(RtlIsCapabilitySid);
	DEBUG_LOG_TRUE(L"RtlIsCapabilitySid", L"");
	return FALSE;
}

NTSTATUS NTAPI Local_NtCreateFile(bool IsCreateFunc, PHANDLE hFileHandle, ACCESS_MASK DesiredAccess, POBJECT_ATTRIBUTES ObjectAttributes, PIO_STATUS_BLOCK IoStatusBlock, PLARGE_INTEGER AllocationSize, ULONG FileAttributes,
	ULONG ShareAccess, ULONG CreateDisposition, ULONG CreateOptions, PVOID EaBuffer, ULONG EaLength)
{
#if 0
	if (IsContentWrite(DesiredAccess, CreateDisposition))
	{
		StringBuffer<> b;
		b.Append(ObjectAttributes->ObjectName->Buffer, ObjectAttributes->ObjectName->Length / 2);
		if (!b.Contains(L"\\Device\\") && !b.EndsWith(L"\\nul"))
			Rpc_WriteLogf(L"[%ls] WRITTEN: %ls", g_rulesIndex ? GetApplicationRules()[g_rulesIndex].app : wcsrchr(g_virtualApplication.data, '\\') + 1, ObjectAttributes->ObjectName->Buffer);
	}
#endif

	TimerScope ts(g_kernelStats.createFile);

	constexpr u32 retryCount = 15;
	u32 retriesLeft = retryCount;
	while (true)
	{
		//if (!Contains(ObjectAttributes->ObjectName->Buffer, L".dll") && !Contains(ObjectAttributes->ObjectName->Buffer, L".mui"))
		//Rpc_WriteLogf(L"NtCreateFile: %ls", ObjectAttributes->ObjectName->Buffer);
		auto res = True_NtCreateFile(hFileHandle, DesiredAccess, ObjectAttributes, IoStatusBlock, AllocationSize, FileAttributes, ShareAccess, CreateDisposition, CreateOptions, EaBuffer, EaLength);

		UBA_ASSERTF(res != STATUS_SUCCESS || u64(*hFileHandle) < DetouredHandleStart - 10000, L"Normal handle range is closing in on detoured. Bump detour range (normal: %llu, detour start: %llu) (%.*s)", u64(*hFileHandle), DetouredHandleStart, ObjectAttributes->ObjectName->Length / 2, ObjectAttributes->ObjectName->Buffer);

		// I have no idea why we get this sometimes when trying to open pch for read after recently being written..
		// All scenarios I've seen this succeeds after 1 second.
		// Only theory I have is some antivirus or something. 
		if (res == STATUS_SHARING_VIOLATION)
		{
			if (!--retriesLeft)
				return res;

			StringView fileName(ObjectAttributes->ObjectName->Buffer, ObjectAttributes->ObjectName->Length / 2);
			if (!fileName.EndsWith(TCV(".pch")))
				return res;

			#if UBA_DEBUG
			StringBuffer<> b;
			b.Appendf(L"Got access denied trying to open %.*s. Retrying in one second", ObjectAttributes->ObjectName->Length / 2, ObjectAttributes->ObjectName->Buffer);
			Rpc_WriteLog(b.data, b.count, true, false);
			#endif
			Sleep(1000);
			continue;
		}

		#if UBA_DEBUG
		if (retriesLeft != retryCount)
		{
			StringBuffer<> b;
			b.Appendf(L"SUCCEEDED to open %.*s after %u retries.", ObjectAttributes->ObjectName->Length / 2, ObjectAttributes->ObjectName->Buffer, retryCount - retriesLeft);
			Rpc_WriteLog(b.data, b.count, true, true);
		}
		#endif

		return res;
	}
}

NTSTATUS NTAPI Shared_NtCreateFile(bool IsCreateFunc, PHANDLE hFileHandle, ACCESS_MASK DesiredAccess, POBJECT_ATTRIBUTES ObjectAttributes, PIO_STATUS_BLOCK IoStatusBlock, PLARGE_INTEGER AllocationSize, ULONG FileAttributes,
	ULONG ShareAccess, ULONG CreateDisposition, ULONG CreateOptions, PVOID EaBuffer, ULONG EaLength)
{
	*hFileHandle = INVALID_HANDLE_VALUE;

	#if UBA_DEBUG_LOG_ENABLED
	const wchar_t* funcName = IsCreateFunc ? L"NtCreateFile" : L"NtOpenFile"; (void)funcName;
	#endif

	StringView ntPath(ObjectAttributes->ObjectName->Buffer, ObjectAttributes->ObjectName->Length / 2);
	bool allowLocal = true;(void)allowLocal;

	auto trueFunc = [&]()
		{
			UBA_ASSERT(allowLocal);
			NTSTATUS res = Local_NtCreateFile(IsCreateFunc, hFileHandle, DesiredAccess, ObjectAttributes, IoStatusBlock, AllocationSize, FileAttributes, ShareAccess, CreateDisposition, CreateOptions, EaBuffer, EaLength);
			DEBUG_LOG_TRUE(funcName, L"(SUPPRESSDETOUR) %llu (%.*s) -> %s", uintptr_t(*hFileHandle), ntPath.count, ntPath.data, ToString(res).data);
			return res;
		};

	
	if (t_disallowCreateFileDetour || t_disallowDetour)
		return trueFunc();
	if (ntPath.count && ntPath[1] != '?') // we can have an empty ntPath but RootDirectory set
		return trueFunc();

	HANDLE rootDir = ObjectAttributes->RootDirectory;
	StringBuffer<> fileName;

	if (ntPath.StartsWith(TCV("\\??\\")))
	{
		UBA_ASSERT(!ObjectAttributes->RootDirectory);
		StringView dosPath = ntPath.Skip(4);

		if (dosPath.count < 2)
			return trueFunc();

		if (dosPath[1] != ':') // No ':' means it is a device
		{
			if (dosPath.Equals(TCV("NUL")))
			{
				auto res = trueFunc();
				g_nullFile = *hFileHandle;
				return res;
			}
			if (dosPath.StartsWith(TCV("pipe\\")))
			{
				if (g_rules->IsThrowAway(dosPath, g_runningRemote))
				{
					//allowDirectoryCache = false;
					//fileName.Append(TCV("\\\\.\\")).Append(dosPath);
					DEBUG_LOG_DETOURED(funcName, L"(THROWAWAY) (%.*s) -> STATUS_OBJECT_NAME_NOT_FOUND", dosPath.count, dosPath.data);
					return STATUS_OBJECT_NAME_NOT_FOUND;
				}
			}
			else if (dosPath[dosPath.count - 1] == '$')
			{
				constexpr const StringView stdStr[] = { TCV("conerr$"), TCV("conout$"), TCV("conin$") };
				for (u32 i = 0; i != 3; ++i)
				{
					if (!dosPath.EndsWith(stdStr[i]))
						continue;
					if (g_isDetachedProcess)
					{
						IoStatusBlock->Status = STATUS_SUCCESS;
						IoStatusBlock->Status = FILE_OPENED;
						*hFileHandle = g_stdHandle[i];
						DEBUG_LOG_DETOURED(funcName, L"(STD) %llu (%.*s) -> %s", uintptr_t(*hFileHandle), dosPath.count, dosPath.data, ToString(STATUS_SUCCESS).data);
						return STATUS_SUCCESS;
					}
					break;
				}
			}
			return trueFunc();
		}

		if (!g_rules->CanDetour(dosPath, g_runningRemote))
			return trueFunc();

		if (!FixPath(fileName, dosPath))
		{
			DEBUG_LOG_DETOURED(funcName, L"(INVALID) (%.*s) -> STATUS_OBJECT_NAME_INVALID", dosPath.count, dosPath.data);
			return STATUS_OBJECT_NAME_INVALID;
		}
	}
	else if (ObjectAttributes->RootDirectory)
	{
		if (isDetouredHandle(ObjectAttributes->RootDirectory))
		{
			auto& dh = asDetouredHandle(ObjectAttributes->RootDirectory);
			fileName.Append(dh.fileObject->fileInfo->originalName);
			rootDir = dh.trueHandle;
		}
		else if (isListDirectoryHandle(ObjectAttributes->RootDirectory))
		{
			auto& lh = asListDirectoryHandle(ObjectAttributes->RootDirectory);
			fileName.Append(lh.originalName);
			allowLocal = false;
		}
		else
		{
			if (!ntPath.count)
				return trueFunc();
			tchar rootDirPath[1024];
			u32 rootDirLen = True_GetFinalPathNameByHandleW(rootDir, rootDirPath, 1024, 0);
			UBA_ASSERT(rootDirLen != 0 && rootDirLen < sizeof(rootDirPath));
			UBA_ASSERT(StartsWith(rootDirPath, TC("\\\\?\\")));
			fileName.Append(rootDirPath + 4, rootDirLen - 4);
		}
		if (ntPath.count)
			fileName.EnsureEndsWithSlash().Append(ntPath);
		ObjectAttributes->RootDirectory = nullptr;
	}
	else
	{
		UBA_ASSERT(false);
		return trueFunc();
	}


	DevirtualizePath(fileName);

	//UBA_ASSERT(CreateDisposition != FILE_SUPERSEDE);
	bool isDeleteOnClose = (CreateOptions & FILE_DELETE_ON_CLOSE) != 0; // clang is using CreateFile with DeleteOnClose to delete files after build errors

	bool useContent = IsContentUse(DesiredAccess, CreateDisposition);
	bool isWrite = IsWrite(DesiredAccess, CreateDisposition);
	bool isThrowAway = g_rules->IsThrowAway(fileName, g_runningRemote);
	bool keepInMemory = g_allowKeepFilesInMemory && g_rules->KeepInMemory(fileName, g_systemTemp, g_runningRemote, isWrite);
	
	keepInMemory = keepInMemory || ((isWrite || isDeleteOnClose) && g_rules->IsOutputFile(fileName, g_systemTemp) && g_allowKeepFilesInMemory) || (isWrite && isThrowAway);

	#if UBA_DEBUG_LOG_ENABLED
	const wchar_t* isWriteStr = isWrite ? L"WRITE " : L""; (void)isWriteStr;
	#endif

	bool isSystemFile = g_systemRoot.count && fileName.StartsWith(g_systemRoot.data);
	if (g_runningRemote)
	{
		if (fileName.EndsWith(TCV(".tlb")))  // We want the tbl files to be detoured
			isSystemFile = false;
		if (fileName.StartsWith(g_exeDir))
			isSystemFile = true;
	}

	//bool checkIfDir = false;
	// This is here just to avoid getting a NtQueryVolumeInformationFile to get volume information 
	if (fileName[3] == 0 && fileName[1] == ':')
	{
		isSystemFile = true;
		if (g_runningRemote)
		{
			UBA_ASSERT(g_systemRoot.count);
			// TODO: Should we create a detoured handle so name calls are correct?
			ObjectAttributes->ObjectName->Buffer[4] = g_systemRoot[0];
		}
	}

	if (isSystemFile)
	{
		ObjectAttributes->RootDirectory = rootDir;
		NTSTATUS res = Local_NtCreateFile(IsCreateFunc, hFileHandle, DesiredAccess, ObjectAttributes, IoStatusBlock, AllocationSize, FileAttributes, ShareAccess, CreateDisposition, CreateOptions, EaBuffer, EaLength);
		if (NT_ERROR(res))
			*hFileHandle = INVALID_HANDLE_VALUE;
		DEBUG_LOG_TRUE(funcName, L"(NODETOUR) %s%llu (%.*ls) -> %ls", isWriteStr, uintptr_t(*hFileHandle), ObjectAttributes->ObjectName->Length / 2, ObjectAttributes->ObjectName->Buffer, ToString(res).data);
		if (NT_ERROR(res))
			return res;
		SkipTrackInput(fileName);
		return res;
	}

	StringBuffer<> fileNameLower(fileName);
	fileNameLower.MakeLower();
	StringKey fileNameKey = ToStringKey(fileNameLower);

	DirTableOffset dirTableOffset;
	auto& dirTable = *g_directoryTable;

	// All NtCreateFile that needs existing file is checked first
	if (CreateDisposition == FILE_OPEN || CreateDisposition == FILE_OVERWRITE)
	{
		DirectoryTable::EntryInformation entryInfo;
		if (Rpc_GetEntryOffset(dirTableOffset, fileNameKey, fileName, false))
			dirTable.GetEntryInformation(entryInfo, dirTableOffset);
		if (!entryInfo.attributes)
		{
			IoStatusBlock->Status = STATUS_OBJECT_NAME_NOT_FOUND;
			IoStatusBlock->Information = FILE_DOES_NOT_EXIST;
			DEBUG_LOG_DETOURED(funcName, L"(DIRTABLE) %llu, (%ls) -> STATUS_OBJECT_NAME_NOT_FOUND", uintptr_t(*hFileHandle), fileName.data);
			return STATUS_OBJECT_NAME_NOT_FOUND;
		}
	}
	else if (const tchar* lastSeparator = fileNameLower.Last(PathSeparator))
	{
		StringView dirName(fileNameLower.data, u32(lastSeparator - fileNameLower.data));
		DirHash hash(dirName);

		SCOPED_WRITE_LOCK(dirTable.m_lookupLock, lookupLock);
		auto insres = dirTable.m_lookup.try_emplace(hash.key, g_memoryBlock);
		DirectoryTable::Directory& dir = insres.first->second;
		if (insres.second)
			if (dirTable.EntryExistsNoLock(hash.key, dirName) != DirectoryTable::Exists_No)
				Rpc_UpdateDirectory(hash.key, dirName.data, dirName.count, false);
		DirTableOffset parentDirTableOffset;

		DirectoryTable::EntryInformation parentEntryInfo;
		if (dirTable.GetLatestOffset(parentDirTableOffset, dir))
			dirTable.GetEntryInformation(parentEntryInfo, parentDirTableOffset);
		if (parentEntryInfo.attributes == 0)
		{
			IoStatusBlock->Status = STATUS_OBJECT_PATH_NOT_FOUND;
			IoStatusBlock->Information = FILE_DOES_NOT_EXIST;
			DEBUG_LOG_DETOURED(funcName, L"(DIRTABLE) %llu, (%s) -> STATUS_OBJECT_PATH_NOT_FOUND", uintptr_t(*hFileHandle), fileName.data);
			return STATUS_OBJECT_PATH_NOT_FOUND;
		}

		if (CreateDisposition == FILE_CREATE && dirTable.ContainsFile(hash.open, dir, fileNameKey))
		{
			IoStatusBlock->Information = FILE_EXISTS;
			IoStatusBlock->Status = STATUS_OBJECT_NAME_COLLISION;
			DEBUG_LOG_DETOURED(funcName, L"(DIRTABLE) %llu, (%s) -> STATUS_OBJECT_NAME_COLLISION", uintptr_t(*hFileHandle), fileName.data);
			return STATUS_OBJECT_NAME_COLLISION;
		}
	}
	else
	{
		UBA_ASSERT(CreateDisposition != FILE_CREATE); // It is extremely unlikely we end up here.
	}

	static constexpr GENERIC_MAPPING genericMapping = { FILE_GENERIC_READ, FILE_GENERIC_WRITE, FILE_GENERIC_EXECUTE, FILE_ALL_ACCESS };
	ACCESS_MASK desiredAccess = DesiredAccess;
	if (desiredAccess & MAXIMUM_ALLOWED)
		desiredAccess = FILE_ALL_ACCESS | SYNCHRONIZE;
	else
		RtlMapGenericMask(&desiredAccess, &genericMapping);

	//bool wantsWrite = (desiredAccess & (FILE_WRITE_DATA | FILE_APPEND_DATA | FILE_WRITE_ATTRIBUTES | FILE_WRITE_EA | DELETE | WRITE_DAC | WRITE_OWNER)) != 0;
	//bool wantsRead  = (desiredAccess & (FILE_READ_DATA | FILE_READ_ATTRIBUTES | FILE_READ_EA)) != 0;


	// Almost all non-writes end up here except the weird FILE_OPEN_IF with read-only
	if (!isWrite && CreateDisposition != FILE_OPEN_IF)
	{
		UBA_ASSERT(IsValidEntry(dirTableOffset));
		DirectoryTable::EntryInformation entryInfo;
		dirTable.GetEntryInformation(entryInfo, dirTableOffset);

		// File/directory has been deleted
		if (entryInfo.attributes == 0)
		{
			IoStatusBlock->Status = STATUS_OBJECT_NAME_NOT_FOUND;
			IoStatusBlock->Information = FILE_DOES_NOT_EXIST;
			DEBUG_LOG_DETOURED(funcName, L"DELETED %llu, (%ls) -> STATUS_OBJECT_NAME_NOT_FOUND", uintptr_t(*hFileHandle), fileName.data);
			return STATUS_OBJECT_NAME_NOT_FOUND;
		}


		bool isDirectory = IsDirectory(entryInfo.attributes);

		if (isDirectory && (CreateOptions & FILE_NON_DIRECTORY_FILE) != 0)
		{
			IoStatusBlock->Information = 0;
			IoStatusBlock->Status = STATUS_FILE_IS_A_DIRECTORY;
			DEBUG_LOG_DETOURED(funcName, L"%llu, (%ls) -> STATUS_FILE_IS_A_DIRECTORY", uintptr_t(*hFileHandle), fileName.data);
			return STATUS_FILE_IS_A_DIRECTORY;
		}

		if (!isDirectory && (CreateOptions & FILE_DIRECTORY_FILE) != 0)
		{
			IoStatusBlock->Information = 0;
			IoStatusBlock->Status = STATUS_NOT_A_DIRECTORY;
			DEBUG_LOG_DETOURED(funcName, L"%llu, (%ls) -> STATUS_NOT_A_DIRECTORY", uintptr_t(*hFileHandle), fileName.data);
			return STATUS_NOT_A_DIRECTORY;
		}

		// Only meta data, no content.
		if ((desiredAccess & (FILE_READ_DATA | FILE_WRITE_DATA | FILE_APPEND_DATA)) == 0)
		{
			// If file is an output file we still allow this path and accept wrong (compressed) size.
			// This is a bit hacky but we don't want to transfer and decompress file just to get the size
			if (!CouldBeCompressedFile(fileName) || g_rules->IsOutputFile(fileName, g_systemTemp))
			{
				auto dh = new DetouredHandle(HandleType_File);
				dh->fileObject = new FileObject();
				dh->fileObject->desiredAccess = DesiredAccess;
				dh->dirTableOffset = dirTableOffset;

				FileInfo* tempFileInfo = new FileInfo();
				dh->fileObject->fileInfo = tempFileInfo;
				dh->fileObject->ownsFileInfo = true;
				dh->fileObject->deleteOnClose = isDeleteOnClose;
				tempFileInfo->originalName = _wcsdup(fileName.data);
				tempFileInfo->name = L"GETATTRIBUTES";
				tempFileInfo->refCount = 1;
				*hFileHandle = makeDetouredHandle(dh);
				//SetLastError(ERROR_SUCCESS); // Don't think this is needed
				DEBUG_LOG_DETOURED(funcName, L"GETATTRIBUTES %llu, (%ls) -> Success", uintptr_t(*hFileHandle), fileName.data);
				return STATUS_SUCCESS;
			}
		}

		// We can't fail desiredAccess & FILE_LIST_DIRECTORY here, this needs to be set with the handle (FILE_READ_DATA is the same as FILE_LIST_DIRECTORY)
		if (isDirectory && (desiredAccess & FILE_LIST_DIRECTORY) != 0)
		{
			// I want to defer this because it doesn't mean that it will access the directory content
			UBA_ASSERT(fileNameLower.data[fileNameLower.count - 1] != '\\');
			DirHash hash(fileNameLower);
			UBA_ASSERT(!isDeleteOnClose);
			SCOPED_WRITE_LOCK(dirTable.m_lookupLock, lookupLock);
			auto insres = dirTable.m_lookup.try_emplace(hash.key, g_memoryBlock);
			DirectoryTable::Directory& dir = insres.first->second;
			if (insres.second)
			{
				auto existsResult = dirTable.EntryExistsNoLock(hash.key, fileNameLower);
				if (existsResult != DirectoryTable::Exists_No)
					Rpc_UpdateDirectory(hash.key, fileNameLower.data, fileNameLower.count, false);
			}
			dirTable.PopulateDirectory(hash.open, dir);

			auto listHandle = new ListDirectoryHandle{ hash.key, dir };
			listHandle->dirTableOffset = dirTableOffset;
			listHandle->it = 0;

			SCOPED_READ_LOCK(dir.lock, lock);
			listHandle->fileTableOffsets.resize(dir.files.size());
			u32 it = 0;
			for (auto& pair : dir.files)
				listHandle->fileTableOffsets[it++] = pair.second;
			lock.Leave();

			#if UBA_DEBUG_VALIDATE
			if (g_validateFileAccess)
				listHandle->validateHandle = validateHandle;
			#endif

			*hFileHandle = makeListDirectoryHandle(listHandle);

			listHandle->originalName = g_memoryBlock.Strdup(fileName).data;

			IoStatusBlock->Information = 1;
			IoStatusBlock->Pointer = nullptr;
			IoStatusBlock->Status = 0;
			DEBUG_LOG_DETOURED(funcName, L"(AS_DIRECTORY) (%ls) -> %llu", fileName.data, uintptr_t(*hFileHandle));
			return STATUS_SUCCESS;
		}
	}
	else
	{
		UBA_ASSERT((CreateOptions & FILE_DIRECTORY_FILE) == 0); // Create directory not supported atm.. will implement if needed
	}

	if (!keepInMemory || !isWrite) // we might get \\pipe\ here... 
		CHECK_PATH(fileNameLower);

	const wchar_t* lpFileName = fileName.data;
	u32 closeId = 0;

	SCOPED_WRITE_LOCK(g_mappedFileTable.m_lookupLock, lookupLock);
	auto insres = g_mappedFileTable.m_lookup.try_emplace(fileNameKey);
	FileInfo& info = insres.first->second;
	UBA_ASSERT(!info.recursiveCall || !isWrite);
	u32 lastDesiredAccess = info.lastDesiredAccess; // TODO: REVISIT lastDesiredAccess.. I don't like this one.. with overlays maybe it should simply be removed
	if (insres.second || info.recursiveCall)
	{
		u64 size = InvalidValue;
		info.originalName = g_memoryBlock.Strdup(fileName).data;
		info.name = info.originalName;

		u8 access = GetFileAccessFlags(DesiredAccess, CreateDisposition);
		wchar_t newFileName[512];
		Rpc_CreateFileW(fileName, fileNameKey, access, newFileName, sizeof_array(newFileName), size, closeId, false);
		info.name = g_memoryBlock.Strdup(newFileName);
		lpFileName = info.name;

		info.size = size;
		info.fileNameKey = fileNameKey;
		info.lastDesiredAccess = DesiredAccess;
	}
	else
	{
		if (!info.originalName)
			info.originalName = g_memoryBlock.Strdup(fileName).data;
		if (isWrite)
		{
			// THIS IS MEGA HACKY.. the whole KeepInMemory does not work well in process hierarchies with different rules.
			// We should probably change so all files that are written are in memory or something
			if (info.name[0] == WrittenMemoryHandleChar)
				keepInMemory = true;

			//bool lastWasWrite = IsContentWrite(info.lastDesiredAccess, 0);
			bool shouldReport = true;//!lastWasWrite || isDeleteOnClose;
			shouldReport = shouldReport && !keepInMemory;

			if (info.isFileMap)
			{
				info.fileMapMem = nullptr;
				info.fileMapMemSize = 0;
				info.isFileMap = false;
				shouldReport = true;
			}

			if (shouldReport)
			{
				u64 size = InvalidValue;
				wchar_t newFileName[1024];
				u8 access = GetFileAccessFlags(DesiredAccess, CreateDisposition);
				Rpc_CreateFileW(fileName, fileNameKey, access, newFileName, sizeof_array(newFileName), size, closeId, false);
				info.name = g_memoryBlock.Strdup(newFileName);
				//info.size = size; // TODO: Should this be set?
				lpFileName = info.name;
			}
			bool lastUseContent = IsContentUse(info.lastDesiredAccess, 0);
			if (!useContent || !lastUseContent)
				lpFileName = info.name;
			info.lastDesiredAccess |= DesiredAccess;
		}
		else
		{
			if (IsFreeableMemoryHandle(info.name))
				Rpc_CheckRemapping(fileName, fileNameKey);
			lpFileName = info.name;
		}
	}

	UBA_ASSERT(*lpFileName);

	auto TrackFileInput = [&]()
		{
			if (!keepInMemory && useContent && !isWrite)
			{
				if (!info.tracked && !IsOverlayOffset(dirTableOffset))
				{
					info.tracked = true;
					TrackInput(fileName);
				}
			}
			else
			{
				SkipTrackInput(fileName);
			}
		};

	auto CreateFileHandle = [&](HANDLE th = INVALID_HANDLE_VALUE)
		{
			IoStatusBlock->Status = STATUS_SUCCESS;
			IoStatusBlock->Information = 1;
			auto fo = new FileObject();
			fo->desiredAccess = DesiredAccess;
			fo->closeId = closeId;
			fo->fileInfo = &info;
			InterlockedIncrement(&info.refCount);
			fo->deleteOnClose = isDeleteOnClose;
			auto dh = new DetouredHandle(HandleType_File, th);
			dh->dirTableOffset = dirTableOffset;
			dh->fileObject = fo;
			if (isWrite)
				Rpc_RegisterFileForWrite(fileName, fileNameKey);
			return makeDetouredHandle(dh);
		};

	if (lpFileName[0] == '$')
	{
		lookupLock.Leave();

		UBA_ASSERT(!lpFileName[2]);

		bool isDir = lpFileName[1] == 'd';
		if (isDir && useContent)
		{
			DEBUG_LOG_DETOURED(funcName, L"%s (%s) -> Error (STATUS_FILE_IS_A_DIRECTORY)", lpFileName, (fileName.data != lpFileName ? fileName.data : L""));
			return STATUS_FILE_IS_A_DIRECTORY;
		}
		MemoryFile& mf = g_emptyMemoryFile;
		info.memoryFile = &mf;

		UBA_ASSERT(!isDeleteOnClose);
		*hFileHandle = CreateFileHandle();

		TrackFileInput();

		DEBUG_LOG_DETOURED(funcName, L"(EMPTY) %llu (%ls) (%ls)", uintptr_t(*hFileHandle), lpFileName, (fileName.data != lpFileName ? fileName.data : L""));
		return STATUS_SUCCESS;
	}

	if (IsMemoryHandle(lpFileName)) // It is a HANDLE from session process
	{
		u64 mappingHandle;
		u64 mappingOffset;
		GetMappingHandleAndOffset(lpFileName, mappingHandle, mappingOffset);

		if (mappingHandle & 0x8000'0000'0000'0000)
		{
			mappingHandle &= ~0x8000'0000'0000'0000;

			HANDLE fileMapHandle;
			if (!True_DuplicateHandle(g_hostProcess, (HANDLE)mappingHandle, GetCurrentProcess(), &fileMapHandle, 0, FALSE, DUPLICATE_SAME_ACCESS))
			{
				Rpc_WriteLogf(L"Can't duplicate handle 0x%llx (%ls) for file %ls (Error %u)", uintptr_t(mappingHandle), lpFileName, info.originalName, GetLastError());
				UBA_ASSERTF(fileMapHandle, L"Can't duplicate handle 0x%llx (%ls) for file %ls (Error %u)", uintptr_t(mappingHandle), lpFileName, info.originalName, GetLastError());
				return STATUS_ACCESS_DENIED;
			}

			u64 alignedOffsetStart = AlignUp(mappingOffset - (PageSize - 1), PageSize);
			u64 mapSize = (mappingOffset - alignedOffsetStart) + info.size;

			LARGE_INTEGER li = ToLargeInteger(alignedOffsetStart);
			u8* mem = (u8*)True_MapViewOfFile(fileMapHandle, FILE_MAP_READ, li.HighPart, li.LowPart, mapSize); // We will leak the mapping until process end
			UBA_ASSERTF(mem, TC("MapViewOfFile failed for %s (%s)"), info.originalName, LastErrorToText().data);

			info.fileMapMem = mem + mappingOffset - alignedOffsetStart;
			info.fileMapMemSize = info.size;
			info.isFileMap = true;

			CloseHandle(fileMapHandle);

			lookupLock.Leave();
			*hFileHandle = CreateFileHandle();
			TrackFileInput();

			DEBUG_LOG_DETOURED(funcName, L"(SHAREDMEMFILE) %ls%llu (%ls) (%ls) -> Success", isWriteStr, uintptr_t(*hFileHandle), lpFileName, (fileName.data != lpFileName ? fileName.data : L""));
			return STATUS_SUCCESS;
		}

		SharedMemoryHandle memoryHandle = SharedMemoryHandle::FromU64(mappingHandle);

		if (g_permanentFilesMemory && memoryHandle == g_permanentFilesHandle)
		{
			info.fileMapMem = g_permanentFilesMemory + mappingOffset;
			info.fileMapMemSize = info.size;
		}
		else if (info.trueFileMapHandle.IsValid() && (info.trueFileMapHandle != memoryHandle || info.trueFileMapOffset != mappingOffset))
		{
			// File has changed (should probably close previous)
			info.fileMapMem = nullptr;
		}

		info.trueFileMapHandle = memoryHandle;
		info.trueFileMapOffset = mappingOffset;
		info.isFileMap = true;

		lookupLock.Leave();
		*hFileHandle = CreateFileHandle();
		TrackFileInput();

		DEBUG_LOG_DETOURED(funcName, L"(SHAREDMEMFILE) %ls%llu (%ls) (%ls) -> Success", isWriteStr, uintptr_t(*hFileHandle), lpFileName, (fileName.data != lpFileName ? fileName.data : L""));
		return STATUS_SUCCESS;
	}

	if (lpFileName[0] == WrittenMemoryHandleChar) // It is a HANDLE from session process. A written file that is writable
	{
		if (!info.memoryFile)
		{
			u64 mappingHandle;
			u64 mappingOffset;
			GetMappingHandleAndOffset(lpFileName, mappingHandle, mappingOffset);
			UBA_ASSERT(mappingOffset == 0);
			auto memoryHandle = SharedMemoryHandle::FromU64(mappingHandle);
			UBA_ASSERT(info.size != InvalidValue);
			u64 mappingHandleSize = info.size;

			info.memoryFile = new MemoryFile(nullptr, false);
			MemoryFile& mf = *info.memoryFile;
			mf.writtenSize = mappingHandleSize;
			mf.committedSize = AlignUp(mappingHandleSize, g_pageSize);
			mf.mappedSize = mf.committedSize;
			mf.reserveSize = FileTypeMaxSize(fileName);

			TimerScope ts2(g_kernelStats.mapViewOfFile);

			mf.memoryHandle = memoryHandle;
			Rpc_UpdateSharedMemory(mf.memoryView, g_sharedMemoryAllocator, memoryHandle, mf.reserveSize);
			mf.baseAddress = mf.memoryView.GetMemory();
		}

		if (CreateDisposition != FILE_OPEN && CreateDisposition != FILE_OPEN_IF)
			info.memoryFile->writtenSize = 0;

		lookupLock.Leave();
		*hFileHandle = CreateFileHandle();
		TrackFileInput();
		DEBUG_LOG_DETOURED(funcName, L"(WRITTENFILE) %ls%llu (%ls) (%ls) -> Success", isWriteStr, uintptr_t(*hFileHandle), lpFileName, (fileName.data != lpFileName ? fileName.data : L""));
		return STATUS_SUCCESS;
	}
	
	if (keepInMemory || info.memoryFile)
	{
		#if UBA_DEBUG_LOG_ENABLED
		const wchar_t* memoryType = L"MEMORY";
		#endif

		if (!info.memoryFile)
		{
			bool isOutput = (isWrite || isDeleteOnClose) && g_rules->IsOutputFile(fileName, g_systemTemp);
			bool isLocal = !isOutput;
			MemoryFile* mf = new MemoryFile(isLocal, FileTypeMaxSize(fileName), isThrowAway, FileTypeCommitSize(fileName), fileName.data);
			auto mfg = MakeGuard([mf](){ delete mf; });

			if (!isThrowAway && CreateDisposition == FILE_OPEN)
			{
				if (!isWrite)
				{
					*hFileHandle = INVALID_HANDLE_VALUE;
					DEBUG_LOG_DETOURED(funcName, L"NOTEXISTS1 (%ls) -> Error", fileName.data);
					return STATUS_OBJECT_NAME_NOT_FOUND;
				}

				// We need to open file for read first and then copy content over to a memory file since this is actually a write
				// (We end up in this code path is used for incremental linking)

				DEBUG_LOG_DETOURED(funcName, L"INTERNAL READ FOR MEMORYWRITE (%ls) (%lu %lu)", fileName.data, CreateDisposition, CreateOptions);
				UBA_ASSERT(!info.recursiveCall);
				FileInfo temp = info;
				info.recursiveCall = true;
				lookupLock.Leave();
				HANDLE fileHandle;
				IO_STATUS_BLOCK ioStatusBlock;
				NTSTATUS res2 = Shared_NtCreateFile(IsCreateFunc, &fileHandle, FILE_GENERIC_READ, ObjectAttributes, &ioStatusBlock, NULL, FILE_ATTRIBUTE_NORMAL, FILE_SHARE_READ, FILE_OPEN, FILE_SEQUENTIAL_ONLY|FILE_SYNCHRONOUS_IO_NONALERT, NULL, 0);
				lookupLock.Enter();
				info = temp;

				if (res2 != STATUS_SUCCESS)
				{
					*hFileHandle = INVALID_HANDLE_VALUE;
					DEBUG_LOG_DETOURED(funcName, L"NOTEXISTS2 (%ls) -> Error", fileName.data);
					return res2;
				}

				BY_HANDLE_FILE_INFORMATION fileInfo2;
				if (!GetFileInformationByHandle(fileHandle, &fileInfo2))
				{
					UBA_ASSERTF(false, TC("GetFileInformationByHandle failed when in NtCreateFile and open file for write (%s)"), fileName.data);
					return STATUS_OBJECT_NAME_EXISTS;
				}
				u64 fileSize2 = ToLargeInteger(fileInfo2.nFileSizeHigh, fileInfo2.nFileSizeLow).QuadPart;

				DetouredHandle tempDh(HandleType_File, INVALID_HANDLE_VALUE);
				mf->EnsureCommitted(tempDh, fileSize2);
				u64 left = fileSize2;
				u8* writePos = mf->baseAddress;
				while (left)
				{
					DWORD toRead = (DWORD)Min(left, u64(~0u));
					DWORD read;
					if (!ReadFile(fileHandle, writePos, toRead, &read, NULL))
					{
						UBA_ASSERTF(false, TC("%s"), fileName.data);
						return STATUS_OBJECT_NAME_EXISTS;
					}
					writePos += read;
					left -= read;
				}
				NtClose(fileHandle);
		
				mf->writtenSize = fileSize2;
				mf->fileTime = ToLargeInteger(fileInfo2.ftCreationTime.dwHighDateTime, fileInfo2.ftCreationTime.dwLowDateTime).QuadPart;
				mf->volumeSerial = fileInfo2.dwVolumeSerialNumber;
				mf->fileIndex = ToLargeInteger(fileInfo2.nFileIndexHigh, fileInfo2.nFileIndexLow).QuadPart;
			}
			else
			{
				if (isOutput && (CreateDisposition & FILE_OPEN_IF) == 0)
				{
					UBA_ASSERTF(false, TC("Trying to open %s with openif. This is not supported"), fileName.data);
				}

				// TODO: Time should be in sync with host machine!
				FILETIME ft;
				SYSTEMTIME st;
				GetSystemTime(&st);
				SystemTimeToFileTime(&st, &ft);
				mf->fileTime = (u64&)ft;
				mf->volumeSerial = 1;
				mf->fileIndex = InterlockedDecrement(&g_memoryFileIndexCounter);
			}

			mfg.Cancel();

			info.created = true;
			info.memoryFile = mf;
		}
		else
		{
			#if UBA_DEBUG_LOG_ENABLED
			if (!info.memoryFile->isLocalOnly)
				memoryType = L"SHAREDMEMORY";
			#endif
		}

		lookupLock.Leave();

		*hFileHandle = CreateFileHandle();

		TrackFileInput();

		DEBUG_LOG_DETOURED(funcName, L"(%s) %ls%llu (%ls) (%ls) -> Success", memoryType, isWriteStr, uintptr_t(*hFileHandle), lpFileName, (fileName.data != lpFileName ? fileName.data : L""));
		return STATUS_SUCCESS;
	}

	lookupLock.Leave();

	StringView tempFileName;
	if (lpFileName[0] == '#')
		tempFileName = fileName;
	else
		tempFileName = ToView(info.name);

	StringBuffer<> temp;
	temp.Append(TCV("\\??\\"));
	if (IsUncPath(tempFileName.data))
		temp.Append(TCV("UNC")).Append(StringView(tempFileName).Skip(1));
	else
		temp.Append(tempFileName);

	UNICODE_STRING* old = ObjectAttributes->ObjectName;
	UNICODE_STRING str;
	str.Buffer = temp.data;
	str.Length = u16(temp.count * 2);
	str.MaximumLength = str.Length + 2;
	ObjectAttributes->ObjectName = &str;
	// TODO!!! THIS NEEDS TO set the ObjectAttributes->ObjectName->Buffer and ObjectAttributes->ObjectName->Length;
	//wcscpy_s(ObjectAttributes->ObjectName->Buffer + 4, ObjectAttributes->ObjectName->MaximumLength/2 - 8, lpFileName);
	//ObjectAttributes->ObjectName->Length = u16(wcslen(lpFileName)*2) + 8;

	NTSTATUS res = Local_NtCreateFile(IsCreateFunc, hFileHandle, DesiredAccess, ObjectAttributes, IoStatusBlock, AllocationSize, FileAttributes, ShareAccess, CreateDisposition, CreateOptions, EaBuffer, EaLength);

	ObjectAttributes->ObjectName = old;

	if (NT_ERROR(res))
	{
		if (closeId)
		{
			UBA_ASSERTF(res != STATUS_SHARING_VIOLATION, TC("We should never end up here.. previous directory table checking should prevent that: %s (CreateDisposition: %lu CreateOptions: %lu DesiredAccess: %lu). This would likely be a race condition between multiple processes (%s)"), tempFileName.data, CreateDisposition, CreateOptions, DesiredAccess, ToString(res).data);
			info.lastDesiredAccess = lastDesiredAccess;
			Rpc_UpdateCloseHandle(L"", closeId, false, L"", {}, 0, 0, 0, false);
		}
		DEBUG_LOG_TRUE(funcName, L"%ls(%ls) (%ls) -> %ls", isWriteStr, lpFileName, (fileName.data != lpFileName ? fileName.data : L""), ToString(res).data);
		return res;
	}

	TrackFileInput();

	HANDLE trueHandle = *hFileHandle;
	UBA_ASSERT(info.originalName);
	*hFileHandle = CreateFileHandle(trueHandle);
	DEBUG_LOG_TRUE(funcName, L"%ls%llu/%llu (%ls)%s (%lu %lu) -> %ls", isWriteStr, uintptr_t(*hFileHandle), u64(trueHandle), tempFileName.data, isDeleteOnClose ? TC(" DeleteOnClose") : TC(""), CreateDisposition, CreateOptions, ToString(res).data);
	return res;
}

NTSTATUS NTAPI Detoured_NtCreateFile(PHANDLE FileHandle, ACCESS_MASK DesiredAccess, POBJECT_ATTRIBUTES ObjectAttributes, PIO_STATUS_BLOCK IoStatusBlock, PLARGE_INTEGER AllocationSize, ULONG FileAttributes,
	ULONG ShareAccess, ULONG CreateDisposition, ULONG CreateOptions, PVOID EaBuffer, ULONG EaLength)
{
	DETOURED_CALL(NtCreateFile);
	return Shared_NtCreateFile(true, FileHandle, DesiredAccess, ObjectAttributes, IoStatusBlock, AllocationSize, FileAttributes, ShareAccess, CreateDisposition, CreateOptions, EaBuffer, EaLength);
}

thread_local bool t_ntOpenFileDisallowed;

NTSTATUS NTAPI Detoured_NtOpenFile(PHANDLE FileHandle, ACCESS_MASK DesiredAccess, POBJECT_ATTRIBUTES ObjectAttributes, PIO_STATUS_BLOCK IoStatusBlock, ULONG ShareAccess, ULONG OpenOptions)
{
	DETOURED_CALL(NtOpenFile);
	if (t_ntOpenFileDisallowed && !IsKnownSystemFile(ObjectAttributes->ObjectName->Buffer))
	{
		DEBUG_LOG_DETOURED(L"NtOpenFile", L"(DISALLOWED)(%.*s) -> STATUS_OBJECT_NAME_NOT_FOUND", ObjectAttributes->ObjectName->Length / 2, ObjectAttributes->ObjectName->Buffer);
		return STATUS_OBJECT_NAME_NOT_FOUND;
	}
	return Shared_NtCreateFile(false, FileHandle, DesiredAccess, ObjectAttributes, IoStatusBlock, (PLARGE_INTEGER)NULL, 0L, ShareAccess, FILE_OPEN, OpenOptions, (PVOID)NULL, 0L);
}

#if DETOURED_INCLUDE_DEBUG
thread_local bool t_ntReadFileCalled;
#endif

#if UBA_CATCH_NTREAD_EXCEPTIONS
int ReadFileAccessViolation(PEXCEPTION_POINTERS ep, HANDLE fileHandle, const FileInfo& fi, const void* mem, u64 size, u64 readPos, u64 bytesRead, u64 length, bool hasByteOffset)
{
	if (ep->ExceptionRecord->ExceptionCode == EXCEPTION_ACCESS_VIOLATION)
	{
		u64 readWrite = ep->ExceptionRecord->NumberParameters >= 1 ? ep->ExceptionRecord->ExceptionInformation[0] : 0;
		u64 faultAddr = ep->ExceptionRecord->NumberParameters >= 2 ? ep->ExceptionRecord->ExceptionInformation[1] : 0;
		Rpc_WriteLogf(L"NtReadFile access violation: fault=0x%llx rw=%llu handle=0x%llx original=%ls name=%ls mem=0x%llx size=%llu readPos=%llu bytesRead=%llu length=%llu hasByteOffset=%d isFileMap=%d trueFileMapHandle=0x%llx trueFileMapOffset=%llu",
			faultAddr, readWrite, uintptr_t(fileHandle),
			fi.originalName ? fi.originalName : L"(null)", fi.name ? fi.name : L"(null)",
			uintptr_t(mem), size, readPos, bytesRead, length,
			hasByteOffset ? 1 : 0, fi.isFileMap ? 1 : 0,
			fi.trueFileMapHandle.ToU64(), fi.trueFileMapOffset);

		MEMORY_BASIC_INFORMATION mbiFault = {};
		if (faultAddr && VirtualQuery((LPCVOID)faultAddr, &mbiFault, sizeof(mbiFault)))
		{
			Rpc_WriteLogf(L"  fault region: base=0x%llx allocBase=0x%llx size=%llu state=0x%x protect=0x%x type=0x%x allocProtect=0x%x",
				uintptr_t(mbiFault.BaseAddress), uintptr_t(mbiFault.AllocationBase), u64(mbiFault.RegionSize),
				u32(mbiFault.State), u32(mbiFault.Protect), u32(mbiFault.Type), u32(mbiFault.AllocationProtect));
		}
		MEMORY_BASIC_INFORMATION mbiMem = {};
		if (mem && VirtualQuery(mem, &mbiMem, sizeof(mbiMem)))
		{
			Rpc_WriteLogf(L"  mem   region: base=0x%llx allocBase=0x%llx size=%llu state=0x%x protect=0x%x type=0x%x allocProtect=0x%x",
				uintptr_t(mbiMem.BaseAddress), uintptr_t(mbiMem.AllocationBase), u64(mbiMem.RegionSize),
				u32(mbiMem.State), u32(mbiMem.Protect), u32(mbiMem.Type), u32(mbiMem.AllocationProtect));
		}

		if (g_isRunningWine)
			TerminateProcess(GetCurrentProcess(), EmergencyShutdownExitCode);
	}
	return EXCEPTION_CONTINUE_SEARCH;
}
#endif

NTSTATUS NTAPI Detoured_NtReadFile(HANDLE FileHandle, HANDLE Event, PIO_APC_ROUTINE ApcRoutine, PVOID ApcContext, PIO_STATUS_BLOCK IoStatusBlock, PVOID Buffer, ULONG Length, PLARGE_INTEGER ByteOffset, PULONG Key)
{
	DETOURED_CALL(NtReadFile);

	#if DETOURED_INCLUDE_DEBUG
	t_ntReadFileCalled = true;
	#endif

	auto trueRun = [&](HANDLE fileHandle, const tchar* prefix)
		{
			(void)prefix;
			TimerScope ts(g_kernelStats.readFile);
			NTSTATUS status = True_NtReadFile(fileHandle, Event, ApcRoutine, ApcContext, IoStatusBlock, Buffer, Length, ByteOffset, Key);
			DEBUG_LOG_TRUE(L"NtReadFile", L"%s%llu (%ls) (%llu/%llu) -> %s", prefix, uintptr_t(FileHandle), HandleToName(FileHandle), IoStatusBlock->Information, u64(Length), ToString(status).data);
			return status;
		};

	UBA_ASSERT(!isListDirectoryHandle(FileHandle));
	if (!isDetouredHandle(FileHandle))
		return trueRun(FileHandle, TC(""));

	auto& dh = asDetouredHandle(FileHandle);

	if (dh.type == HandleType_StdIn)
		return trueRun(True_GetStdHandle(STD_INPUT_HANDLE), TC("(STDIN)"));

	UBA_ASSERT(dh.type != HandleType_StdIn);
	auto& fo = *dh.fileObject;
	fo.wasUsed = true;

	FileInfo& fi = *fo.fileInfo;
	if (!fi.isFileMap && !fi.memoryFile)
		return trueRun(dh.trueHandle, TC(""));

	// TODO: Handle lpOverlapped - If a read happen and there is 0 left it should return 0 with SetLastError(ERROR_HANDLE_EOF)
	if (!EnsureMapped(dh))
	{
		DEBUG_LOG_DETOURED(L"FileHandle", L"%llu %lu (%ls) -> FAILED TO MAP", uintptr_t(FileHandle), Length, HandleToName(FileHandle));
		return FALSE;
	}
	UBA_ASSERTF(fi.fileMapMem || !fi.memoryFile->isThrowAway, TC("Trying to read throw-away file %s"), HandleToName(FileHandle));

	u8* mem = fi.fileMapMem ? fi.fileMapMem : fi.memoryFile->baseAddress;
	u64 size = fi.fileMapMem ? fi.fileMapMemSize : fi.memoryFile->writtenSize;
	u64 readPos = dh.pos;

	NTSTATUS status = STATUS_SUCCESS;
	u64 bytesRead = 0;

	if (ByteOffset != nullptr)
		readPos = ByteOffset->QuadPart;

	if (readPos < 0)
		status = STATUS_INVALID_PARAMETER;
	else if (readPos >= size)
	{
		status = STATUS_END_OF_FILE;
		bytesRead = 0;
	}
	else
	{
		u64 availableBytes = size - readPos;
		bytesRead = Min((u64)Length, availableBytes);

		#if UBA_CATCH_NTREAD_EXCEPTIONS
		__try
		#endif
		{
			memcpy(Buffer, (u8*)mem + readPos, bytesRead);
		}
		#if UBA_CATCH_NTREAD_EXCEPTIONS
		__except(ReadFileAccessViolation(GetExceptionInformation(), FileHandle, fi, mem, size, readPos, bytesRead, (u64)Length, ByteOffset != nullptr))
		{
		}
		#endif

		if (ByteOffset == nullptr)
			dh.pos = readPos + bytesRead;
	}

	if (IoStatusBlock != nullptr)
	{
		IoStatusBlock->Status = status;
		IoStatusBlock->Information = bytesRead;
	}

	if (Event != nullptr)
		SetEvent(Event);

	if (ApcRoutine != nullptr)
		ApcRoutine(ApcContext, IoStatusBlock, 0);

	DEBUG_LOG_DETOURED(L"NtReadFile", L"(MemoryFile) %llu (%ls) %llu length=%lu -> %s", uintptr_t(FileHandle), HandleToName(FileHandle), readPos, Length, ToString(status).data);
	return status;
}

#if DETOURED_INCLUDE_DEBUG
thread_local bool t_ntWriteFileCalled;
#endif

StringBuffer<32 * 1024> g_stdFile;
ReaderWriterLock& g_stdFileLock = *new ReaderWriterLock();

void WriteStdFile(LPCVOID lpBuffer, DWORD nNumberOfBytesToWrite, bool isError)
{
	if (!g_conEnabled[isError?0:1] || g_suppressLogging)
		return;

	SCOPED_WRITE_LOCK(g_stdFileLock, lock);
	u32 start = 0;
	u32 i = 0;
	auto bufferStr = (const char*)lpBuffer;
	while (i != nNumberOfBytesToWrite)
	{
		if (bufferStr[i] == '\n')
		{
			int len = i - start;
			if (len > 0 && bufferStr[i - 1] == '\r')
				--len;
			if (len)
				g_stdFile.Appendf(L"%.*hs", len, bufferStr + start);
			Rpc_WriteLog(g_stdFile.data, g_stdFile.count, false, isError);
			g_stdFile.Clear();
			start = i + 1;
		}
		++i;
	}
	if (u32 left = nNumberOfBytesToWrite - start)
		g_stdFile.Appendf(L"%.*hs", left, bufferStr + start);
}

int WriteFileAccessViolation(PEXCEPTION_POINTERS ep)
{
	// This likely is OOM in the file mapping system on wine (tmpfs running out of space or something).
	// We need to improve how uba tracks memory usage on wine to catch these before it is too late. but for now, just terminate
	if (g_isRunningWine && ep->ExceptionRecord->ExceptionCode == EXCEPTION_ACCESS_VIOLATION)
		TerminateProcess(GetCurrentProcess(), EmergencyShutdownExitCode);
	return EXCEPTION_CONTINUE_SEARCH;
}

NTSTATUS NTAPI Detoured_NtWriteFile(HANDLE FileHandle, HANDLE Event, PIO_APC_ROUTINE ApcRoutine, PVOID ApcContext, PIO_STATUS_BLOCK IoStatusBlock, PVOID Buffer, ULONG Length, PLARGE_INTEGER ByteOffset, PULONG Key)
{
	DETOURED_CALL(NtWriteFile);
	
	#if DETOURED_INCLUDE_DEBUG
	t_ntWriteFileCalled = true;
	#endif

	auto trueRun = [&](HANDLE fileHandle, const tchar* prefix)
		{
			(void)prefix;
			TimerScope ts(g_kernelStats.writeFile);
			NTSTATUS status = True_NtWriteFile(fileHandle, Event, ApcRoutine, ApcContext, IoStatusBlock, Buffer, Length, ByteOffset, Key);
			DEBUG_LOG_TRUE(L"NtWriteFile", L"%s%llu (%llu/%llu) (%ls) -> %s", prefix, uintptr_t(FileHandle), u64(IoStatusBlock->Information), u64(Length), HandleToName(FileHandle), ToString(status).data);
			g_kernelStats.writeFile.bytes += IoStatusBlock->Information;
			return status;
		};

	UBA_ASSERT(!isListDirectoryHandle(FileHandle));

	if (FileHandle == PseudoHandle)
		return trueRun(FileHandle, TC("(PseudoHandle)"));

	auto successRun = [&](const tchar* prefix)
		{
			IoStatusBlock->Information = Length;
			IoStatusBlock->Status = STATUS_SUCCESS;
			if (Event)
				SetEvent(Event);
			if (ApcRoutine != nullptr)
				ApcRoutine(ApcContext, IoStatusBlock, 0);
			DEBUG_LOG_DETOURED(L"NtWriteFile", L"%s%llu (%llu/%llu) (%ls) -> %s", prefix, uintptr_t(FileHandle), u64(IoStatusBlock->Information), u64(Length), HandleToName(FileHandle), ToString(STATUS_SUCCESS).data);
			return STATUS_SUCCESS;
		};

	if (FileHandle == g_stdHandle[1] || FileHandle == g_stdHandle[0])
	{
		//DEBUG_LOG_DETOURED(L"WriteStdFile2", L"%llu", uintptr_t(hFile));
		WriteStdFile(Buffer, Length, FileHandle == g_stdHandle[0]);
		return successRun(TC("(StdOut1)"));
	}

	if (!isDetouredHandle(FileHandle))
		return trueRun(FileHandle, TC(""));

	auto& dh = asDetouredHandle(FileHandle);

	if (dh.type >= HandleType_StdErr)
	{
		if (dh.type != HandleType_StdIn)
		{
			//DEBUG_LOG_DETOURED(L"WriteStdFile1", L"%llu", uintptr_t(hFile));
			WriteStdFile(Buffer, Length, dh.type == HandleType_StdErr);
		}
		return successRun(TC("(StdOut2)"));
	}

	auto& fo = *dh.fileObject;
	auto& fi = *fo.fileInfo;

	if (!fi.memoryFile)
	{
		UBA_ASSERTF(!fi.isFileMap, L"Trying to write to file %ls which is a filemap. This is not supported (%s)", HandleToName(FileHandle), fi.originalName);
		UBA_ASSERTF(dh.trueHandle != INVALID_HANDLE_VALUE, L"Trying to write to file %ls which does not have a valid handle (%s)", HandleToName(FileHandle), fi.originalName);
		return trueRun(dh.trueHandle, TC(""));
	}
	MemoryFile& mf = *fi.memoryFile;

	u64 offset = ByteOffset ? ByteOffset->QuadPart : dh.pos;
	u64 offsetAfterWrite = offset + Length;
	dh.pos = offsetAfterWrite;

	SCOPED_WRITE_LOCK(mf.lock, lock);
	u64 newSize = Max(mf.writtenSize, offsetAfterWrite);
	if (mf.isThrowAway)
	{
		mf.writtenSize = newSize;
		return successRun(TC("(ThrowAway)"));
	}
	mf.EnsureCommitted(dh, newSize);
	mf.writtenSize = newSize;
	mf.isReported = false;
	lock.Leave();

	__try
	{
		memcpy(mf.baseAddress + offset, Buffer, Length);
	}
	__except(WriteFileAccessViolation(GetExceptionInformation()))
	{
	}

	return successRun(TC("(Memory)"));
}

NTSTATUS NTAPI Detoured_NtFsControlFile(HANDLE FileHandle, HANDLE Event, PIO_APC_ROUTINE ApcRoutine, PVOID ApcContext, PIO_STATUS_BLOCK IoStatusBlock, ULONG FsControlCode, PVOID InputBuffer, ULONG InputBufferLength, PVOID OutputBuffer, ULONG OutputBufferLength)
{
	DETOURED_CALL(NtFsControlFile);
	HANDLE trueHandle = FileHandle;
	if (isDetouredHandle(FileHandle))
	{
		auto& dh = asDetouredHandle(FileHandle);
		trueHandle = dh.trueHandle;

		if (dh.type >= HandleType_StdOut)
			trueHandle = True_GetStdHandle(STD_OUTPUT_HANDLE);
		else if (dh.type >= HandleType_StdErr)
			trueHandle = True_GetStdHandle(STD_ERROR_HANDLE);
		else if (trueHandle == INVALID_HANDLE_VALUE)
		{
			if (FsControlCode == FSCTL_GET_REPARSE_POINT)
			{
				DEBUG_LOG_DETOURED(L"NtFsControlFile", L"(FSCTL_GET_REPARSE_POINT) %llu (%s) -> %s", uintptr_t(FileHandle), HandleToName(FileHandle), ToString(STATUS_NOT_A_REPARSE_POINT).data);
				return STATUS_NOT_A_REPARSE_POINT;
			}
			else
			{
				UBA_ASSERTF(false, L"NtFsControlFile code 0x%lx not handled (%s)", FsControlCode, HandleToName(FileHandle));
			}
		}
	}
	UBA_ASSERT(!isListDirectoryHandle(FileHandle));

	auto res = True_NtFsControlFile(trueHandle, Event, ApcRoutine, ApcContext, IoStatusBlock, FsControlCode, InputBuffer, InputBufferLength, OutputBuffer, OutputBufferLength);
	DEBUG_LOG_TRUE(L"NtFsControlFile", L"(0x%lx) %llu (%s) -> %s", FsControlCode, uintptr_t(FileHandle), HandleToName(FileHandle), ToString(res).data);
	return res;
}

NTSTATUS Detoured_NtDeviceIoControlFile(HANDLE FileHandle, HANDLE Event, PIO_APC_ROUTINE ApcRoutine, PVOID ApcContext, PIO_STATUS_BLOCK IoStatusBlock, ULONG IoControlCode, PVOID InputBuffer, ULONG InputBufferLength, PVOID OutputBuffer, ULONG OutputBufferLength)
{
	HANDLE trueHandle = FileHandle;
	if (isDetouredHandle(FileHandle))
	{
		DetouredHandle& dh = asDetouredHandle(FileHandle);
		trueHandle = dh.trueHandle;

		if (dh.type >= HandleType_StdOut)
			trueHandle = True_GetStdHandle(STD_OUTPUT_HANDLE);
		else if (dh.type >= HandleType_StdErr)
			trueHandle = True_GetStdHandle(STD_ERROR_HANDLE);
		else if (dh.trueHandle == INVALID_HANDLE_VALUE)
		{
			if (IoControlCode == FSCTL_GET_REPARSE_POINT)
			{
				DEBUG_LOG_DETOURED(L"NtDeviceIoControlFile", L"(FSCTL_GET_REPARSE_POINT) %llu (%ls) -> STATUS_NOT_A_REPARSE_POINT", uintptr_t(FileHandle), HandleToName(FileHandle));
				return STATUS_NOT_A_REPARSE_POINT;
			}
			else if (IoControlCode == FSCTL_SET_SPARSE)
			{
				UBA_ASSERTF(false, TC("FSCTL_SET_SPARSE"));
			}
			else if (IoControlCode == IOCTL_STORAGE_GET_DEVICE_NUMBER)
			{
				const wchar_t* originalName = dh.fileObject->fileInfo->originalName;
				UBA_ASSERT(originalName && originalName[0]);
				DWORD deviceNumber = originalName[0] % 32u;

				auto& number = *(STORAGE_DEVICE_NUMBER*)OutputBuffer;
				number.DeviceType = FILE_DEVICE_DISK;
				number.DeviceNumber = deviceNumber;
				number.PartitionNumber = 0;
				if (IoStatusBlock)
					IoStatusBlock->Information = sizeof(STORAGE_DEVICE_NUMBER);
				DEBUG_LOG_DETOURED(L"NtDeviceIoControlFile", L"(IOCTL_STORAGE_GET_DEVICE_NUMBER) %llu (%ls) -> Success", uintptr_t(FileHandle), HandleToName(FileHandle));
				if (IoStatusBlock)
					IoStatusBlock->Status = STATUS_SUCCESS;
				return STATUS_SUCCESS;
			}
			else
			{
				UBA_ASSERTF(false, TC("NtDeviceIoControlFile on detoured handle with code %lu not handled (%s)"), IoControlCode, HandleToName(FileHandle));
			}
		}
	}
	else if (isListDirectoryHandle(FileHandle))
	{
		if (IoControlCode == FSCTL_GET_REPARSE_POINT)
		{
			DEBUG_LOG_DETOURED(L"NtDeviceIoControlFile", L"(FSCTL_GET_REPARSE_POINT) %llu (%ls) -> STATUS_NOT_A_REPARSE_POINT", uintptr_t(FileHandle), HandleToName(FileHandle));
			return STATUS_NOT_A_REPARSE_POINT;
		}
		if (IoControlCode == 0x504000) // IOCTL_CONDRV_GET_MODE
		{
			DEBUG_LOG_DETOURED(L"NtDeviceIoControlFile", L"(FSCTL_GET_REPARSE_POINT) %llu (%ls) -> STATUS_NOT_A_REPARSE_POINT", uintptr_t(FileHandle), HandleToName(FileHandle));
			return STATUS_INVALID_DEVICE_REQUEST;
		}
		UBA_ASSERTF(false, TC("NtDeviceIoControlFile on detoured list handle with code %lu not handled (%s)"), IoControlCode, HandleToName(FileHandle));
	}

	auto res = True_NtDeviceIoControlFile(trueHandle, Event, ApcRoutine, ApcContext, IoStatusBlock, IoControlCode, InputBuffer, InputBufferLength, OutputBuffer, OutputBufferLength);

	#if UBA_DEBUG_LOG_ENABLED
	if (IoControlCode == 0x1211b) // Spammy IOCTL_AFD_RIO in RadLink.exe
		return res;
	#endif

	DEBUG_LOG_TRUE(L"NtDeviceIoControlFile", L"(0x%lx) %llu (%s) -> %s", IoControlCode, uintptr_t(FileHandle), HandleToName(FileHandle), ToString(res).data);
	return res;
}

NTSTATUS NTAPI Detoured_NtCopyFileChunk(HANDLE Source, HANDLE Dest, HANDLE Event, PIO_STATUS_BLOCK IoStatusBlock, ULONG Length, PULONG SourceOffset, PULONG DestOffset, PULONG SourceKey, PULONG DestKey, ULONG Flags)
{
	DETOURED_CALL(NtCopyFileChunk);
	HANDLE trueSourceHandle = Source;
	if (isDetouredHandle(Source))
	{
		auto& dh = asDetouredHandle(Source);
		trueSourceHandle = dh.trueHandle;
		UBA_ASSERT(trueSourceHandle != INVALID_HANDLE_VALUE);
	}
	HANDLE trueDestHandle = Dest;
	if (isDetouredHandle(Dest))
	{
		auto& dh = asDetouredHandle(Dest);
		trueDestHandle = dh.trueHandle;
		UBA_ASSERT(trueDestHandle != INVALID_HANDLE_VALUE);
	}
	return True_NtCopyFileChunk(trueSourceHandle, trueDestHandle, Event, IoStatusBlock, Length, SourceOffset, DestOffset, SourceKey, DestKey, Flags);
}

NTSTATUS NTAPI Detoured_NtClose(HANDLE handle)
{
	DETOURED_CALL(NtClose);

	if (handle == INVALID_HANDLE_VALUE || handle == PseudoHandle)
	{
		TimerScope ts(g_kernelStats.closeHandle);
		return True_NtClose(handle);
	}

	if (isListDirectoryHandle(handle))
	{
		auto& listHandle = asListDirectoryHandle(handle);

#if UBA_DEBUG_VALIDATE
		if (g_validateFileAccess)
		{
			auto res = True_NtClose(listHandle.validateHandle);
			if (res != 0)
				ToInvestigate(L"NtClose failed for validate handle");
		}
#endif

		DEBUG_LOG_DETOURED(L"NtClose", L"%llu (%ls) -> %ls", uintptr_t(handle), HandleToName(handle), ToString(STATUS_SUCCESS).data);
		delete& listHandle;
		return STATUS_SUCCESS;
	}

	if (!isDetouredHandle(handle))
	{
		TimerScope ts(g_kernelStats.closeHandle);
		auto res = True_NtClose(handle);
#if !defined(_M_ARM64) // For some reason this log line crashes on arm64 with access violation on internal tls variable
		DEBUG_LOG_TRUE(L"NtClose", L"%llu (%ls) -> %ls", uintptr_t(handle), HandleToName(handle), ToString(res).data);
#endif
		return res;
	}

	DetouredHandle& dh = asDetouredHandle(handle);

	HANDLE trueHandle = dh.trueHandle;
	auto closeTrueHandle = [&](bool log)
		{
			if (trueHandle == INVALID_HANDLE_VALUE)
			{
				if (!log)
					return STATUS_SUCCESS;
				DEBUG_LOG_TRUE(L"NtClose", L"%llu (%ls) -> %ls", uintptr_t(handle), HandleToName(handle), ToString(STATUS_SUCCESS).data);
				return STATUS_SUCCESS;
			}
			TimerScope ts(g_kernelStats.closeFile);
			auto res = True_NtClose(trueHandle);
			ts.Leave();
			if (!log)
				return res;
			DEBUG_LOG_TRUE(L"NtClose", L"%llu (%ls) -> %ls", uintptr_t(handle), HandleToName(handle), ToString(res).data);
			return res;
		};

	FileObject* fo = dh.fileObject;
	if (!fo)
	{
		auto res = closeTrueHandle(true);
		if (dh.type < HandleType_StdErr) // TODO: We leak std handles if duplicated.. but ignore for now
			delete& dh;
		return res;
	}

	auto foRefCount = InterlockedDecrement(&fo->refCount);
	UBA_ASSERT(foRefCount != ~u64(0));
	if (foRefCount)
	{
		auto res = closeTrueHandle(true);
		delete& dh;
		return res;
	}

	if (dh.type == HandleType_Job)
	{
		SCOPED_WRITE_LOCK(g_jobsLock, lock);
		auto findIt = g_jobs.find(dh.fileObject);
		if (findIt != g_jobs.end())
		{
			JobInfo jobInfo = std::move(findIt->second);
			g_jobs.erase(findIt);
			lock.Leave();
			TerminateProcessesInJob(std::move(jobInfo), 0xC000010A);
		}
		DEBUG_LOG_TRUE(L"NtClose", L"%llu (Job) (%ls)", uintptr_t(handle), HandleToName(handle));
		return closeTrueHandle(false);
	}

	SharedMemoryHandle memoryHandle;
	FileInfo& fi = *fo->fileInfo;
	const wchar_t* path = fi.name;
	wchar_t temp[512];

	u64 fileSize = InvalidValue;
	u64 lastWriteTime = 0;
	u32 attributes = FILE_ATTRIBUTE_NORMAL;

	// TODO: We can track file size outside of kernel calls
	if (trueHandle != INVALID_HANDLE_VALUE && (fo->closeId || !fo->newName.empty()) && !fo->deleteOnClose)
	{
		IO_STATUS_BLOCK b;
		FILE_NETWORK_OPEN_INFORMATION info;
		NTSTATUS res = True_NtQueryInformationFile(trueHandle, &b, &info, sizeof(info), FileNetworkOpenInformation);(void)res;
		UBA_ASSERTF(res == STATUS_SUCCESS, TC("Failed to get size of file %s"), path);
		attributes = info.FileAttributes;
		lastWriteTime = info.LastWriteTime.QuadPart;
		fileSize = info.EndOfFile.QuadPart;
	}

	NTSTATUS res = closeTrueHandle(false);
		
	if (MemoryFile* mf = fi.memoryFile)
	{
		if (IsWrite(fo->desiredAccess, 0))
		{
			// TODO: There are race conditions in this code. There could be other file handles accessing the same piece of memory (although unlikely)
			u64 alignedWritten = AlignUp(mf->writtenSize, PageSize);
			if (alignedWritten < mf->committedSize)
			{
				u64 decommitSize = u64(mf->committedSize - alignedWritten);
				if (mf->isLocalOnly)
				{
#pragma warning(push)
#pragma warning(disable:6250)
					if (!::VirtualFree(mf->baseAddress + alignedWritten, decommitSize, MEM_DECOMMIT))
						ToInvestigate(L"Failed to decommit memory (%u)", GetLastError());
#pragma warning(pop)
				}
				mf->committedSize = alignedWritten;
			}
		}

		memoryHandle = mf->memoryHandle;
		fileSize = mf->writtenSize;

		FILETIME ft;
		::GetSystemTimeAsFileTime(&ft);
		lastWriteTime = u64(ToLargeInteger(ft.dwHighDateTime, ft.dwLowDateTime).QuadPart);

		u32 orginalNameLen = TStrlen(fi.originalName);
		if ((fo->deleteOnClose || IsWrite(fo->desiredAccess, 0)) && g_rules->IsOutputFile({fi.originalName, orginalNameLen}, g_systemTemp) && !g_rules->IsThrowAway(StringView(fi.originalName, orginalNameLen), g_runningRemote))
		{
			// Need to report this file to host so it can be tracked in directory table
			if (!mf->isReported && !fo->closeId)
			{
				path = temp;
				mf->isReported = true;
				const wchar_t* fileName = fi.originalName;
				if (!fo->newName.empty())
					fileName = fo->newName.c_str();
				StringBuffer<> fixedName;
				FixPath(fixedName, fileName);
				StringKey fileNameKey = fi.fileNameKey;
				if (!fo->newName.empty())
					fileNameKey = ToStringKeyLower(fixedName);

				u64 size;
				Rpc_CreateFileW(fixedName, fileNameKey, AccessFlag_Write, temp, sizeof_array(temp), size, fo->closeId, true);
			}

			PROCESS_MEMORY_COUNTERS_EX pmc;
			GetProcessMemoryInfo(GetCurrentProcess(), (PROCESS_MEMORY_COUNTERS*)&pmc, sizeof(pmc));
			AtomicMax(g_stats.peakMemory, pmc.PagefileUsage);
		}
	}
	else if (fo->deleteOnClose) // We have used an optimized handle that actually never opens the file so we need to delete it manually
	{
		fileSize = 0;

		if (dh.trueHandle == INVALID_HANDLE_VALUE)
			DeleteFileW(fi.originalName);
		else if (!fo->closeId && !g_exitMessageSent)
		{
			// If file was DeleteOnClose but never got a closeId we need to create one so we can delete the file on session side.
			u64 size;
			Rpc_CreateFileW(ToView(fi.originalName), fi.fileNameKey, AccessFlag_Write, temp, sizeof_array(temp), size, fo->closeId, true);
		}
		fo->newName.clear();
	}

	if (!fo->newName.empty())
	{
		if (!fo->closeId)
		{
			// If file was DeleteOnClose but never got a closeId we need to create one so we can delete the file on session side.
			u64 size;
			Rpc_CreateFileW(ToView(fi.originalName), fi.fileNameKey, AccessFlag_Write, temp, sizeof_array(temp), size, fo->closeId, true);
		}

		UBA_ASSERTF(!fo->deleteOnClose, TC("DeleteOnClose for file rename not implemented (%s -> %s)"), fi.originalName, fo->newName.c_str());
		// It might be that same process will open it again, so we will need to update the mapping table
		StringBuffer<> fixedNewName;
		FixPath(fixedNewName, fo->newName.c_str());
		fixedNewName.MakeLower();
		StringKey fileNameKey = ToStringKey(fixedNewName);
		SCOPED_WRITE_LOCK(g_mappedFileTable.m_lookupLock, _);
		auto insres = g_mappedFileTable.m_lookup.try_emplace(fileNameKey);
		FileInfo& newInfo = insres.first->second;
		newInfo = fi;
		newInfo.originalName = g_memoryBlock.Strdup(fo->newName).data;
		UBA_ASSERT(Equals(newInfo.name, fi.name));
		if (!g_runningRemote)
			newInfo.name = newInfo.originalName;
		newInfo.fileNameKey = fileNameKey;
		u32 refCount = fi.refCount;
		fi = {};
		fi.refCount = refCount;
		fo->ownsFileInfo = false;
		//fo->newName.clear(); // Don't remove newName here since we might want to enter the update close
	}

	if (fo->closeId && !g_exitMessageSent)
	{
		DEBUG_LOG_DETOURED(L"NtClose", L"%llu (%ls) CloseId:%u, Size:%llu -> %ls", uintptr_t(handle), HandleToName(handle), fo->closeId, fileSize, ToString(res).data);
		Rpc_UpdateCloseHandle(path, fo->closeId, fo->deleteOnClose, fo->newName.c_str(), memoryHandle, fileSize, lastWriteTime, attributes, true);
	}
	else
	{
		DEBUG_LOG_DETOURED(L"NtClose", L"%llu (%ls) -> %ls", uintptr_t(handle), HandleToName(handle), ToString(res).data);
	}

	InterlockedDecrement(&fi.refCount);

	if (fo->ownsFileInfo)
	{
		UBA_ASSERT(!fi.memoryFile);
		if (fi.fileMapMem)
		{
			bool success = True_UnmapViewOfFile(fi.fileMapMem); (void)success;
			DEBUG_LOG_TRUE(L"INTERNAL UnmapViewOfFile", L"%llu (%ls) (%ls) -> %ls", uintptr_t(fi.fileMapMem), fi.name, fi.originalName, ToString(success));
		}

		free((void*)fi.originalName);
		delete& fi;
	}

	delete fo;
	delete& dh;
	return res;
}

NTSTATUS Detoured_NtQueryObject(HANDLE Handle, OBJECT_INFORMATION_CLASS ObjectInformationClass, PVOID ObjectInformation, ULONG ObjectInformationLength, PULONG ReturnLength)
{
	DETOURED_CALL(NtQueryObject);

	HANDLE trueHandle = Handle;

	// This can be other things than FILES.. Is used by GetHandleInformation
	if (isDetouredHandle(Handle))
	{
		DetouredHandle& dh = asDetouredHandle(Handle);
		trueHandle = dh.trueHandle;

		if (trueHandle == INVALID_HANDLE_VALUE)
		{
			if (ObjectInformationClass == 1) // ObjectNameInformation
			{
				auto fo = dh.fileObject;
				UBA_ASSERT(fo);
				auto& fi = *fo->fileInfo;
				UBA_ASSERT(fi.originalName);
				const wchar_t* fileName = fi.originalName;

				StringBuffer<> fixedPath;
				FixPath(fileName, g_virtualWorkingDir.data, g_virtualWorkingDir.count, fixedPath);

				StringBuffer<> buffer;

				g_directoryTable->GetFinalPath(buffer, fixedPath.data);
				VirtualizePath(buffer);

				// Remote machines will just use the native exe dir drive information
				// Since there are no better options. This will make it so we don't have to implement detouring for all kinds of queries about volume information etc.
				if (g_runningRemote)
					buffer[0] = g_exeDir[0];

				wchar_t drive[3] = { buffer[0], ':', 0 };
				wchar_t device[256];
				DWORD deviceLen = QueryDosDeviceW(drive, device, sizeof_array(device));
				UBA_ASSERT(deviceLen);
				buffer.Prepend(StringView(device, deviceLen), 2);

				u64 bufferSize = (buffer.count+1)*sizeof(tchar);
				u64 totalSize = sizeof(UNICODE_STRING) + bufferSize;

				if (ObjectInformationLength < totalSize)
				{
					DEBUG_LOG_DETOURED(L"NtQueryObject", L"(ObjectNameInformation) %s -> STATUS_BUFFER_OVERFLOW", HandleToName(Handle));
					return STATUS_BUFFER_OVERFLOW;
				}
				auto& ustr = *(PUNICODE_STRING)ObjectInformation;
				ustr.Length = u16(bufferSize - sizeof(tchar));
				ustr.MaximumLength = u16(bufferSize);
				ustr.Buffer = (wchar_t*)((&ustr) + 1);
				memcpy(ustr.Buffer, buffer.data, bufferSize);
				*ReturnLength = (ULONG)totalSize;

				DEBUG_LOG_DETOURED(L"NtQueryObject", L"(ObjectNameInformation) %llu -> Success (%s)", uintptr_t(Handle), buffer.data);
				return STATUS_SUCCESS;
			}

			UBA_ASSERTF(false, L"NtQueryObject NOT_IMPLEMENTED (class %i) (%s)", ObjectInformationClass, HandleToName(Handle));
		}
	}
	auto res = True_NtQueryObject(trueHandle, ObjectInformationClass, ObjectInformation, ObjectInformationLength, ReturnLength);
	DEBUG_LOG_TRUE(L"NtQueryObject", L"(%i) %llu -> %ls", ObjectInformationClass, uintptr_t(Handle), ToString(res).data);
	return res;
}

NTSTATUS Detoured_NtQueryInformationProcess(HANDLE ProcessHandle, PROCESSINFOCLASS ProcessInformationClass, PVOID ProcessInformation, ULONG ProcessInformationLength, PULONG ReturnLength)
{
	DETOURED_CALL(NtQueryInformationProcess);
	if (isDetouredHandle(ProcessHandle))
	{
		ProcessHandle = asDetouredHandle(ProcessHandle).trueHandle;
	}

	if (g_isRunningWine)
	{
		if (ProcessInformationClass == (PROCESSINFOCLASS)49) // ProcessConsoleHostProcess
		{
			struct PROCESS_CONSOLE_HOST_PROCESS_INFORMATION
			{
				ULONG_PTR ConsoleHostProcess;
			};

			if (ProcessInformationLength < sizeof(PROCESS_CONSOLE_HOST_PROCESS_INFORMATION))
			{
				DEBUG_LOG_TRUE(L"NtQueryInformationProcess", L"(ProcessConsoleHostProcess) %llu -> %s", uintptr_t(ProcessHandle), ToString(STATUS_INFO_LENGTH_MISMATCH).data);
				return STATUS_INFO_LENGTH_MISMATCH;
			}
			((PROCESS_CONSOLE_HOST_PROCESS_INFORMATION*)ProcessInformation)->ConsoleHostProcess = 0;
			if (ReturnLength)
				*ReturnLength = sizeof(PROCESS_CONSOLE_HOST_PROCESS_INFORMATION);
			DEBUG_LOG_TRUE(L"NtQueryInformationProcess", L"(ProcessConsoleHostProcess) %llu -> %s", uintptr_t(ProcessHandle), ToString(STATUS_SUCCESS).data);
			return STATUS_SUCCESS;
		}
	}

	NTSTATUS res = True_NtQueryInformationProcess(ProcessHandle, ProcessInformationClass, ProcessInformation, ProcessInformationLength, ReturnLength);
	DEBUG_LOG_TRUE(L"NtQueryInformationProcess", L"(class %u) %llu -> %ls", ProcessInformationClass, uintptr_t(ProcessHandle), ToString(res).data);
	return res;
}

#if DETOURED_INCLUDE_DEBUG
thread_local bool t_ntQuerySecurityObjectCalled;
#endif

NTSTATUS Detoured_NtQuerySecurityObject(HANDLE Handle, SECURITY_INFORMATION SecurityInformation, PSECURITY_DESCRIPTOR SecurityDescriptor, ULONG Length, PULONG LengthNeeded)
{
	DETOURED_CALL(NtQuerySecurityObject);
	
	#if DETOURED_INCLUDE_DEBUG
	t_ntQuerySecurityObjectCalled = true;
	#endif

	auto callTrue = [&](HANDLE handle)
		{
			NTSTATUS res = True_NtQuerySecurityObject(handle, SecurityInformation, SecurityDescriptor, Length, LengthNeeded);
			DEBUG_LOG_TRUE(L"NtQuerySecurityObject", L"%llu -> %ls", uintptr_t(Handle), ToString(res).data);
			return res;
		};

	if (!isDetouredHandle(Handle))
		return callTrue(Handle);

	HANDLE trueHandle = asDetouredHandle(Handle).trueHandle;
	if (trueHandle != INVALID_HANDLE_VALUE)
		return callTrue(trueHandle);

	BYTE sys[SECURITY_MAX_SID_SIZE], adm[SECURITY_MAX_SID_SIZE], app[SECURITY_MAX_SID_SIZE], res[SECURITY_MAX_SID_SIZE];
	DWORD s=sizeof(sys);
	CreateWellKnownSid(WinLocalSystemSid,0,sys,&s);
	s=sizeof(adm);
	CreateWellKnownSid(WinBuiltinAdministratorsSid,0,adm,&s);

	SID_IDENTIFIER_AUTHORITY auth = SECURITY_APP_PACKAGE_AUTHORITY;
	InitializeSid((PSID)app,&auth,2);
	*GetSidSubAuthority((PSID)app,0)=2;
	*GetSidSubAuthority((PSID)app,1)=1;
	InitializeSid((PSID)res,&auth,2);
	*GetSidSubAuthority((PSID)res,0)=2;
	*GetSidSubAuthority((PSID)res,1)=2;

	// get current user SID
	BYTE userSid[SECURITY_MAX_SID_SIZE];
	//DWORD userSidLen = sizeof(userSid);
	HANDLE token;
	OpenProcessToken(GetCurrentProcess(), TOKEN_QUERY, &token);
	DWORD len;
	GetTokenInformation(token, TokenUser, nullptr, 0, &len);
	BYTE* tokenBuf = (BYTE*)_alloca(len);
	GetTokenInformation(token, TokenUser, tokenBuf, len, &len);
	PSID srcUser = ((TOKEN_USER*)tokenBuf)->User.Sid;
	CopySid(sizeof(userSid), userSid, srcUser);
	CloseHandle(token);

	ULONG aclSz = sizeof(ACL) +
		sizeof(ACCESS_ALLOWED_ACE)-sizeof(DWORD)+GetLengthSid(app) +
		sizeof(ACCESS_ALLOWED_ACE)-sizeof(DWORD)+GetLengthSid(res) +
		sizeof(ACCESS_ALLOWED_ACE)-sizeof(DWORD)+GetLengthSid(userSid) +
		sizeof(ACCESS_ALLOWED_ACE)-sizeof(DWORD)+GetLengthSid(adm) +
		sizeof(ACCESS_ALLOWED_ACE)-sizeof(DWORD)+GetLengthSid(adm) +
		sizeof(ACCESS_ALLOWED_ACE)-sizeof(DWORD)+GetLengthSid(sys) +
		sizeof(ACCESS_ALLOWED_ACE)-sizeof(DWORD)+GetLengthSid(sys);

	ULONG sdSz = sizeof(SECURITY_DESCRIPTOR_RELATIVE) +
		GetLengthSid(userSid) + GetLengthSid(sys) + aclSz;

	if(LengthNeeded)*LengthNeeded=sdSz;
	if(Length<sdSz) return STATUS_BUFFER_TOO_SMALL;

	BYTE* p=(BYTE*)SecurityDescriptor;
	auto sd=(SECURITY_DESCRIPTOR_RELATIVE*)p;
	p+=sizeof(*sd);

	// owner = current user
	BYTE* owner=p;
	CopySid(GetLengthSid(userSid),owner,userSid);
	p+=GetLengthSid(userSid);

	// group = SYSTEM (same as before)
	BYTE* group=p;
	CopySid(GetLengthSid(sys),group,sys);
	p+=GetLengthSid(sys);

	ACL* acl=(ACL*)p;
	InitializeAcl(acl,aclSz,ACL_REVISION);

	AddAccessAllowedAceEx(acl,ACL_REVISION,OBJECT_INHERIT_ACE|CONTAINER_INHERIT_ACE,FILE_GENERIC_READ|FILE_GENERIC_EXECUTE,app);
	AddAccessAllowedAceEx(acl,ACL_REVISION,OBJECT_INHERIT_ACE|CONTAINER_INHERIT_ACE,FILE_GENERIC_READ|FILE_GENERIC_EXECUTE,res);

	// current user full access
	AddAccessAllowedAceEx(acl,ACL_REVISION,0,FILE_ALL_ACCESS,userSid);

	AddAccessAllowedAceEx(acl,ACL_REVISION,INHERITED_ACE,FILE_ALL_ACCESS,adm);
	AddAccessAllowedAceEx(acl,ACL_REVISION,INHERITED_ACE|OBJECT_INHERIT_ACE|CONTAINER_INHERIT_ACE|INHERIT_ONLY_ACE,FILE_ALL_ACCESS,adm);

	AddAccessAllowedAceEx(acl,ACL_REVISION,INHERITED_ACE,FILE_ALL_ACCESS,sys);
	AddAccessAllowedAceEx(acl,ACL_REVISION,INHERITED_ACE|OBJECT_INHERIT_ACE|CONTAINER_INHERIT_ACE|INHERIT_ONLY_ACE,FILE_ALL_ACCESS,sys);

	sd->Revision=SECURITY_DESCRIPTOR_REVISION;
	sd->Control=SE_SELF_RELATIVE|SE_DACL_PRESENT;
	sd->Owner=(ULONG)(owner-(BYTE*)sd);
	sd->Group=(ULONG)(group-(BYTE*)sd);
	sd->Dacl =(ULONG)((BYTE*)acl-(BYTE*)sd);

	DEBUG_LOG_DETOURED(L"NtQuerySecurityObject", L"%llu -> %ls", uintptr_t(Handle), ToString(STATUS_SUCCESS).data);
	return STATUS_SUCCESS;
}


#if DETOURED_INCLUDE_DEBUG

NTSTATUS Detoured_RtlCreateUserThread(HANDLE process, SECURITY_DESCRIPTOR* descr, BOOLEAN suspended, ULONG zero_bits, SIZE_T stack_reserve, SIZE_T stack_commit, void* start, void *param, HANDLE *handle_ptr, CLIENT_ID* id)
{
	DEBUG_LOG_TRUE(L"RtlCreateUserThread", L"");
	auto res = True_RtlCreateUserThread(process, descr, suspended, zero_bits, stack_reserve, stack_commit, start, param, handle_ptr, id);
	return res;
}

NTSTATUS Detoured_NtCreateThreadEx(PHANDLE ThreadHandle, ACCESS_MASK DesiredAccess, POBJECT_ATTRIBUTES ObjectAttributes, HANDLE ProcessHandle, USER_THREAD_START_ROUTINE* StartRoutine, PVOID Argument, ULONG CreateFlags, SIZE_T ZeroBits, SIZE_T StackSize, SIZE_T MaximumStackSize, PS_ATTRIBUTE_LIST* AttributeList)
{
	auto res = True_NtCreateThreadEx(ThreadHandle, DesiredAccess, ObjectAttributes, ProcessHandle, StartRoutine, Argument, CreateFlags, ZeroBits, StackSize, MaximumStackSize, AttributeList);
	DEBUG_LOG_TRUE(L"NtCreateThreadEx", L"(%llu) -> %s", ThreadHandle?u64(*ThreadHandle):0ull, ToString(res).data);
	return res;
}

NTSTATUS Detoured_NtOpenJobObject(HANDLE* handle, ACCESS_MASK access, const OBJECT_ATTRIBUTES *attr)
{
	UBA_ASSERT(!isDetouredHandle(handle));
	DEBUG_LOG_TRUE(L"NtOpenJobObject", L"");
	return True_NtOpenJobObject(handle, access, attr);
}

NTSTATUS Detoured_NtAllocateVirtualMemoryEx(HANDLE ProcessHandle, PVOID *BaseAddress, SIZE_T *RegionSize, ULONG AllocationType, ULONG Protect, PMEM_EXTENDED_PARAMETER ExtendedParameters, ULONG ExtendedParameterCount)
{
	UBA_ASSERT(!isDetouredHandle(ProcessHandle));
	//DEBUG_LOG_TRUE(L"NtAllocateVirtualMemoryEx", L"");
	return True_NtAllocateVirtualMemoryEx(ProcessHandle, BaseAddress, RegionSize, AllocationType, Protect, ExtendedParameters, ExtendedParameterCount);
}

NTSTATUS Detoured_NtQueryInformationJobObject(HANDLE handle, JOBOBJECTINFOCLASS infoClass, void* info, ULONG len, ULONG* retLen)
{
	UBA_ASSERT(!isDetouredHandle(handle));
	DEBUG_LOG_TRUE(L"NtQueryInformationJobObject", L"");
	return True_NtQueryInformationJobObject(handle, infoClass, info, len, retLen);
}

NTSTATUS Detoured_NtCreateIoCompletion(PHANDLE IoCompletionHandle, ACCESS_MASK DesiredAccess, POBJECT_ATTRIBUTES ObjectAttributes, ULONG Count)
{
	UBA_ASSERT(!isDetouredHandle(*IoCompletionHandle));
	auto res = True_NtCreateIoCompletion(IoCompletionHandle, DesiredAccess, ObjectAttributes, Count);
	DEBUG_LOG_TRUE(L"NtCreateIoCompletion", L"%llu -> %s", IoCompletionHandle ? u64(*IoCompletionHandle) : 0ull, ToString(res).data);
	return res;
}

NTSTATUS Detoured_NtRemoveIoCompletion(HANDLE IoCompletionHandle, PULONG_PTR CompletionKey, PULONG_PTR CompletionValue, PIO_STATUS_BLOCK IoStatusBlock, PLARGE_INTEGER Timeout)
{
	auto res = True_NtRemoveIoCompletion(IoCompletionHandle, CompletionKey, CompletionValue, IoStatusBlock, Timeout);
	DEBUG_LOG_TRUE(L"NtRemoveIoCompletion", L"%llu -> %s", u64(IoCompletionHandle), ToString(res).data);
	return res;
}

NTSTATUS NTAPI Detoured_NtFlushBuffersFileEx(HANDLE FileHandle, ULONG Flags, PVOID Parameters, ULONG ParametersSize, PIO_STATUS_BLOCK IoStatusBlock)
{
	DETOURED_CALL(NtFlushBuffersFileEx);
	UBA_ASSERT(!isDetouredHandle(FileHandle));
	DEBUG_LOG_TRUE(L"NtFlushBuffersFileEx", L"");
	return True_NtFlushBuffersFileEx(FileHandle, Flags, Parameters, ParametersSize, IoStatusBlock);
}

NTSTATUS NTAPI Detoured_NtAlpcCreatePort(PHANDLE PortHandle, POBJECT_ATTRIBUTES ObjectAttributes, PALPC_PORT_ATTRIBUTES PortAttributes)
{
	//DEBUG_LOG_TRUE(L"NtAlpcCreatePort", L"");
	return True_NtAlpcCreatePort(PortHandle, ObjectAttributes, PortAttributes);
}

NTSTATUS NTAPI Detoured_NtAlpcConnectPort(PHANDLE PortHandle, PUNICODE_STRING PortName, POBJECT_ATTRIBUTES ObjectAttributes, PALPC_PORT_ATTRIBUTES PortAttributes, DWORD ConnectionFlags, PSID RequiredServerSid, PPORT_MESSAGE ConnectionMessage, PSIZE_T ConnectMessageSize, PALPC_MESSAGE_ATTRIBUTES OutMessageAttributes, PALPC_MESSAGE_ATTRIBUTES InMessageAttributes, PLARGE_INTEGER Timeout)
{
	//DEBUG_LOG_TRUE(L"NtAlpcConnectPort", L"");
	return True_NtAlpcConnectPort(PortHandle, PortName, ObjectAttributes, PortAttributes, ConnectionFlags, RequiredServerSid, ConnectionMessage, ConnectMessageSize, OutMessageAttributes, InMessageAttributes, Timeout);
}

NTSTATUS NTAPI Detoured_NtAlpcCreatePortSection(HANDLE PortHandle, ULONG Flags, HANDLE SectionHandle, SIZE_T SectionSize, PHANDLE AlpcSectionHandle, PSIZE_T ActualSectionSize)
{
	//DEBUG_LOG_TRUE(L"NtAlpcCreatePortSection", L"");
	return True_NtAlpcCreatePortSection(PortHandle, Flags, SectionHandle, SectionSize, AlpcSectionHandle, ActualSectionSize);
}

NTSTATUS NTAPI Detoured_NtAlpcSendWaitReceivePort(HANDLE PortHandle, DWORD Flags, PPORT_MESSAGE SendMessage_, PALPC_MESSAGE_ATTRIBUTES SendMessageAttributes, PPORT_MESSAGE ReceiveMessage, PSIZE_T BufferLength, PALPC_MESSAGE_ATTRIBUTES ReceiveMessageAttributes, PLARGE_INTEGER Timeout)
{
	//u64 size = 0;
	//if (SendMessage_)
	//	size = SendMessage_->u1.s1.DataLength;
	//DEBUG_LOG_TRUE(L"NtAlpcSendWaitReceivePort", L"%llu", size);
	return True_NtAlpcSendWaitReceivePort(PortHandle, Flags, SendMessage_, SendMessageAttributes, ReceiveMessage, BufferLength, ReceiveMessageAttributes, Timeout);
}

NTSTATUS NTAPI Detoured_NtAlpcDisconnectPort(HANDLE PortHandle, ULONG Flags)
{
	//DEBUG_LOG_TRUE(L"NtAlpcDisconnectPort", L"");
	return True_NtAlpcDisconnectPort(PortHandle, Flags);
}

NTSTATUS NTAPI Detoured_ZwQueryDirectoryFile(HANDLE FileHandle, HANDLE Event, PIO_APC_ROUTINE ApcRoutine, PVOID ApcContext, PIO_STATUS_BLOCK IoStatusBlock, PVOID FileInformation, ULONG Length,
	FILE_INFORMATION_CLASS FileInformationClass, BOOLEAN ReturnSingleEntry, PUNICODE_STRING FileName, BOOLEAN RestartScan)
{
	DETOURED_CALL(ZwQueryDirectoryFile);
	DEBUG_LOG_TRUE(L"ZwQueryDirectoryFile", L"(%ls)", HandleToName(FileHandle));
	UBA_ASSERT(!isDetouredHandle(FileHandle));
	return True_ZwQueryDirectoryFile(FileHandle, Event, ApcRoutine, ApcContext, IoStatusBlock, FileInformation, Length, FileInformationClass, ReturnSingleEntry, FileName, RestartScan);
}

//NTSTATUS NTAPI Detoured_ZwCreateFile(PHANDLE FileHandle, ACCESS_MASK DesiredAccess, POBJECT_ATTRIBUTES ObjectAttributes, PIO_STATUS_BLOCK IoStatusBlock, PLARGE_INTEGER AllocationSize, ULONG FileAttributes, 
//									 ULONG ShareAccess, ULONG CreateDisposition, ULONG CreateOptions, PVOID EaBuffer, ULONG EaLength)
//{
//	DETOURED_CALL(ZwCreateFile);
//	DEBUG_LOG_TRUE(L"ZwCreateFile", L"");
//	UBA_ASSERT(!isDetouredHandle(FileHandle));
//	return True_ZwCreateFile(FileHandle, DesiredAccess, ObjectAttributes, IoStatusBlock, AllocationSize, FileAttributes, ShareAccess, CreateDisposition, CreateOptions, EaBuffer, EaLength);
//}

//NTSTATUS NTAPI Detoured_ZwOpenFile(PHANDLE FileHandle, ACCESS_MASK DesiredAccess, POBJECT_ATTRIBUTES ObjectAttributes, PIO_STATUS_BLOCK IoStatusBlock, ULONG ShareAccess, ULONG OpenOptions)
//{
//	DETOURED_CALL(ZwOpenFile);
//	DEBUG_LOG_TRUE(L"ZwOpenFile", L"");
//	UBA_ASSERT(!isDetouredHandle(FileHandle));
//	return True_ZwCreateFile(FileHandle, DesiredAccess, ObjectAttributes, IoStatusBlock, (PLARGE_INTEGER)NULL, 0L, ShareAccess, FILE_OPEN, OpenOptions, (PVOID)NULL, 0L);
//}

NTSTATUS NTAPI Detoured_ZwSetInformationFile(HANDLE FileHandle, PIO_STATUS_BLOCK IoStatusBlock, PVOID FileInformation, ULONG Length, FILE_INFORMATION_CLASS FileInformationClass)
{
	DETOURED_CALL(ZwSetInformationFile);
	DEBUG_LOG_TRUE(L"ZwSetInformationFile", L"%llu (%ls)", uintptr_t(FileHandle), HandleToName(FileHandle));
	UBA_ASSERT(!isDetouredHandle(FileHandle));
	return True_ZwSetInformationFile(FileHandle, IoStatusBlock, FileInformation, Length, FileInformationClass);
}

NTSTATUS Detoured_RtlGetVersion(PRTL_OSVERSIONINFOW lpVersionInformation)
{
	NTSTATUS res = True_RtlGetVersion(lpVersionInformation);
	DEBUG_LOG_TRUE(L"RtlGetVersion", L"%lu.%lu.%lu -> %s", lpVersionInformation->dwMajorVersion, lpVersionInformation->dwMinorVersion, lpVersionInformation->dwBuildNumber, ToString(res).data);
	//lpVersionInformation->dwBuildNumber = 26052;
	return res;
}

PVOID Detoured_RtlAllocateHeap(PVOID HeapHandle, ULONG Flags, SIZE_T Size)
{
	//if (Flags & HEAP_ZERO_MEMORY)
	//	return mi_zalloc(Size);
	//else
	//	return mi_malloc(Size);
	return True_RtlAllocateHeap(HeapHandle, Flags, Size);
}

PVOID Detoured_RtlReAllocateHeap(PVOID HeapHandle, ULONG Flags, PVOID BaseAddress, SIZE_T Size)
{
#if UBA_USE_MIMALLOC
	if (g_checkRtlHeap && IsInMiMalloc(BaseAddress))
	{
		return mi_realloc(BaseAddress, Size);
		//Rpc_WriteLogf(L"ERROR: RtlReAllocateHeap - This is not implemented");
		//return 0;
	}
#endif
	//if (Flags & HEAP_ZERO_MEMORY)
	//	return mi_realloc(BaseAddress, Size);
	//else
	//	return mi_realloc(BaseAddress, Size);
	return True_RtlReAllocateHeap(HeapHandle, Flags, BaseAddress, Size);
}

BOOLEAN Detoured_RtlValidateHeap(HANDLE HeapPtr, ULONG Flags, PVOID Block)
{
	//return true;
	return True_RtlValidateHeap(HeapPtr, Flags, Block);
}

#endif
