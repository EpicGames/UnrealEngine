// Copyright Epic Games, Inc. All Rights Reserved.

#if UBA_SUPPORT_MSPDBSRV

RPC_STATUS Detoured_RpcStringBindingComposeW(RPC_WSTR ObjUuid, RPC_WSTR Protseq, RPC_WSTR NetworkAddr, RPC_WSTR Endpoint, RPC_WSTR Options, RPC_WSTR* StringBinding)
{
	RPC_STATUS res = True_RpcStringBindingComposeW(ObjUuid, Protseq, NetworkAddr, Endpoint, Options, StringBinding);

	DEBUG_LOG_TRUE(L"RpcStringBindingComposeW", L"%s %s %s To %s", Protseq, NetworkAddr, Endpoint, *(wchar_t**)StringBinding);
	return res;
}

UnorderedSet<void*> g_rpcBindings;

RPC_STATUS Detoured_RpcBindingFromStringBindingW(RPC_WSTR StringBinding, RPC_BINDING_HANDLE *Binding)
{
	/*
	if (g_runningRemote && Contains((wchar_t*)StringBinding, L"mspdb"))
	{
		SCOPED_FUTEX(g_communicationLock, pcs);
		BinaryWriter writer;
		writer.WriteByte(MessageType_RpcCommunication);
		writer.WriteByte(1);
		writer.Flush();
		BinaryReader reader;
		*Binding = new char();
		g_rpcBindings.insert(*Binding);
		return 0;
	}
	*/

	auto res = True_RpcBindingFromStringBindingW(StringBinding, Binding);

	g_rpcBindings.insert(*Binding);

	DEBUG_LOG_TRUE(L"RpcBindingFromStringBindingW", L"%s TO %p", StringBinding, *Binding);
	return res;
}

RPC_STATUS Detoured_RpcBindingSetAuthInfoExW(RPC_BINDING_HANDLE Binding, RPC_WSTR ServerPrincName, unsigned long AuthnLevel, unsigned long AuthnSvc, RPC_AUTH_IDENTITY_HANDLE AuthIdentity, unsigned long AuthzSvc, RPC_SECURITY_QOS *SecurityQOS)
{
	//if (g_rpcBindings.find(Binding) != g_rpcBindings.end())
	//{
	//	return 0;
	//}

	auto res = True_RpcBindingSetAuthInfoExW(Binding, ServerPrincName, AuthnLevel, AuthnSvc, AuthIdentity, AuthzSvc, SecurityQOS);
	DEBUG_LOG_TRUE(L"RpcBindingSetAuthInfoExW", L"%p", Binding);
	return res;
}

extern "C" 
{
	void* True2_NdrClientCall2 = nullptr;

	ULONG_PTR __cdecl Detoured_NdrClientCall2(PMIDL_STUB_DESC pStubDescriptor, PFORMAT_STRING  pFormat, ...);

	ULONG_PTR Local_NdrClientCall2(PMIDL_STUB_DESC pStubDescriptor, PFORMAT_STRING pFormat, void** pStack)
	{
		void* returnAddress = *(pStack - 3);(void)returnAddress;

		//const WCHAR* CallingModule = FindModuleByAddress(returnAddress);
		//if (!CallingModule)
		//    CallingModule = L"unknown";
		RPC_BINDING_HANDLE* Binding = nullptr;

		unsigned char handleType = 255;
		if (pFormat) {
			handleType = pFormat[0];
			if (!handleType) { // explicit handle
				// pStack[0] explicit handle
				// pStack[1] Arg0
				// pStack[2] Arg1
				// ...
				// pStack[n+1] ArgN
				Binding = (RPC_BINDING_HANDLE*)&pStack[0]; // first argument in the var arg list is the explicit handle
			}
			else { // implicit handle
				// pStack[0] Arg0
				// pStack[1] Arg1
				// ...
				// pStack[n] ArgN
				Binding = (RPC_BINDING_HANDLE*)&pStubDescriptor->IMPLICIT_HANDLE_INFO.pPrimitiveHandle;
			}
		}

		if (Binding && *Binding == nullptr)
			Binding = nullptr;

		if (PRPC_CLIENT_INTERFACE rpcInterface = (PRPC_CLIENT_INTERFACE)pStubDescriptor->RpcInterfaceInformation)
		{
			//WCHAR interfaceID[48];
			//Sbie_StringFromGUID(&rpcInterface->InterfaceId.SyntaxGUID, interfaceID);

			//Sbie_snwprintf(text, 512, L"Calling %s UUID = %s}, %d.%d, BindingHandle = 0x%X (%X), caller = '%s'", Function, interfaceID,
			//    rpcInterface->InterfaceId.SyntaxVersion.MajorVersion, rpcInterface->InterfaceId.SyntaxVersion.MinorVersion, Binding ? *Binding : nullptr, handleType, CallingModule);
		}
		else
		{
			//Sbie_snwprintf(text, 512, L"Calling %s caller = '%s'", Function, CallingModule);
		}

		if (g_rpcBindings.find(*Binding) != g_rpcBindings.end())
			printf("");
		//SbieApi_MonitorPutMsg(MONITOR_RPC | MONITOR_TRACE, text);

		DEBUG_LOG_TRUE(L"NdrClientCall2", L"%p", *Binding);
		return 0;
	}

}
/*
ULONG_PTR __cdecl Detoured_NdrClientCall2(PMIDL_STUB_DESC pStubDescriptor, PFORMAT_STRING  pFormat, ...)
{
	RPC_BINDING_HANDLE* Binding = nullptr;

	unsigned char handleType = 255;
	if (pFormat) {
		handleType = pFormat[0];
		if (!handleType) { // explicit handle
			// pStack[0] explicit handle
			// pStack[1] Arg0
			// pStack[2] Arg1
			// ...
			// pStack[n+1] ArgN
			//Binding = (RPC_BINDING_HANDLE*)&pStack[0]; // first argument in the var arg list is the explicit handle
		}
		else { // implicit handle
			// pStack[0] Arg0
			// pStack[1] Arg1
			// ...
			// pStack[n] ArgN
			Binding = (RPC_BINDING_HANDLE*)&pStubDescriptor->IMPLICIT_HANDLE_INFO.pPrimitiveHandle;
		}
	}

	if (Binding && *Binding == nullptr)
		Binding = nullptr;
	DEBUG_LOG_TRUE(L"NdrClientCall2", L"%p", *Binding);
	return 0;
}
*/
#endif
