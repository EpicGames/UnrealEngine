// Copyright Epic Games, Inc. All Rights Reserved.

HRESULT Detoured_SHGetKnownFolderPath(REFKNOWNFOLDERID rfid, DWORD dwFlags, HANDLE hToken, PWSTR* ppszPath)
{
	if (!g_runningRemote)
	{
		SuppressDetourScope _;
		HRESULT res = True_SHGetKnownFolderPath(rfid, dwFlags, hToken, ppszPath);
		DEBUG_LOG_TRUE(L"SHGetKnownFolderPath", L"(%ls) -> %ls", *ppszPath, ToString(res == S_OK));
		return res;
	}

	UBA_ASSERT(hToken == NULL);
	RPC_MESSAGE(SHGetKnownFolderPath, getFullFileName)
	writer.WriteBytes(&rfid, sizeof(KNOWNFOLDERID));
	writer.WriteU32(dwFlags);
	BinaryReader reader = writer.Flush();
	HRESULT res = reader.ReadU32();
	*ppszPath = NULL;
	if (res == S_OK)
	{
		StringBuffer<> path;
		reader.ReadString(path);
		u32 memSize = (path.count+1)*2;
		void* mem = CoTaskMemAlloc(memSize);
		UBA_ASSERT(mem);
		if (!mem)
			return E_FAIL;
		memcpy(mem, path.data, memSize);
		*ppszPath = (PWSTR)mem;
	}
	DEBUG_LOG_DETOURED(L"SHGetKnownFolderPath", L"(%ls) -> %ls", *ppszPath, ToString(res == S_OK));
	return res;
}


HRESULT Detoured_SHGetFolderPathAndSubDirW(HWND hwnd, int csidl, HANDLE hToken, DWORD dwFlags, LPCWSTR pszSubDir, LPWSTR pszPath)
{
	if (!g_runningRemote)
	{
		SuppressDetourScope _;
		HRESULT res = True_SHGetFolderPathAndSubDirW(hwnd, csidl, hToken, dwFlags, pszSubDir, pszPath);
		DEBUG_LOG_TRUE(L"SHGetFolderPathAndSubDirW", L"(%s) -> %s", pszPath, ToString(res == S_OK));
		return res;
	}

	UBA_ASSERT(hwnd == NULL);
	UBA_ASSERT(hToken == NULL);

	RPC_MESSAGE(HostRun, getFullFileName);
	u16& size = *(u16*)writer.AllocWrite(2);
	u64 pos = writer.GetPosition();
	writer.WriteU32(csidl);
	writer.WriteU32(dwFlags);
	writer.WriteBool(pszSubDir != nullptr);
	if (pszSubDir)
		writer.WriteString(pszSubDir);
	size = u16(writer.GetPosition() - pos);
	BinaryReader reader = writer.Flush();
	u32 res = reader.ReadU32();
	reader.ReadString(pszPath, MAX_PATH);

	DEBUG_LOG_DETOURED(L"SHGetFolderPathAndSubDirW", L"%s", pszPath);
	return res;
}

#if UBA_DEBUG
HRESULT Detoured_SHGetFolderPathW(HWND hwnd, int csidl, HANDLE hToken, DWORD dwFlags, LPWSTR pszPath)
{
	SuppressDetourScope _;
	HRESULT res = True_SHGetFolderPathW(hwnd, csidl, hToken, dwFlags, pszPath);
	DEBUG_LOG_TRUE(L"SHGetFolderPathW", L"(%s) -> %s", pszPath, ToString(res == S_OK));
	return res;
}

HRESULT Detoured_SHGetKnownFolderIDList(REFKNOWNFOLDERID rfid, DWORD flags, HANDLE token, PIDLIST_ABSOLUTE *pidl)
{
	HRESULT res = True_SHGetKnownFolderIDList(rfid, flags, token, pidl);
	DEBUG_LOG_TRUE(L"SHGetKnownFolderIDList", L"-> %s", ToString(res == S_OK));
	return res;

}
#endif