// Copyright Epic Games, Inc. All Rights Reserved.

wchar_t* Detoured__wgetcwd(wchar_t* buffer, int maxlen)
{
	DETOURED_CALL(_wgetcwd);
	//wchar_t* res = True__wgetcwd(buffer, maxlen);
	//return res;
	UBA_ASSERT(g_virtualWorkingDir.count < u32(maxlen));
	memcpy(buffer, g_virtualWorkingDir.data, u64(g_virtualWorkingDir.count) * 2);
	buffer[g_virtualWorkingDir.count - 1] = 0; // Remove last slash

#if UBA_DEBUG_VALIDATE
	if (!g_runningRemote)
	{
		//wchar_t* res = True__wgetcwd(buffer, maxlen);
	}
#endif

	DEBUG_LOG_DETOURED(L"_wgetcwd", L"(%ls)", buffer);
	return buffer;
}

wchar_t* Detoured__wfullpath(wchar_t* absPath, const wchar_t* relPath, size_t maxLength)
{
	DETOURED_CALL(_wfullpath);
	u64 relLen = 0;
	if (relPath)
		relLen = wcslen(relPath);

	wchar_t* res = nullptr;
	if (!absPath)
	{
		maxLength = relLen + g_virtualWorkingDir.count + 2;
		if (g_isRunningWine)
			res = (wchar_t*)malloc(maxLength * sizeof(WCHAR)); // Same as wine implementation
		else
			res = (wchar_t*)_malloc_base(maxLength * sizeof(WCHAR));
		FixPath2(relPath, ~0u, g_virtualWorkingDir.data, g_virtualWorkingDir.count, res, maxLength, nullptr);
		UBA_ASSERT(res && wcslen(res) < maxLength - 1);
	}
	else if ((relPath && relPath[1] == ':' && relLen < maxLength) || (relLen + g_virtualWorkingDir.count <= maxLength))
	{
		FixPath2(relPath, ~0u, g_virtualWorkingDir.data, g_virtualWorkingDir.count, absPath, maxLength, nullptr);
		UBA_ASSERT(wcslen(absPath) < maxLength);
		res = absPath;
	}

	//DEBUG_LOG_DETOURED(L"_wfullpath", L"(%ls) -> %ls", relPath, res); // This is very spammy

#if UBA_DEBUG_VALIDATE
	if (g_validateFileAccess && !g_runningRemote)
	{
		wchar_t otherBuf[1024] = {};
		SuppressDetourScope _;
		wchar_t* test = True__wfullpath(otherBuf, relPath, maxLength); (void)test;
		UBA_ASSERT((res != nullptr) == (test != nullptr) && (!res || (Equals(res, test, false) || wcschr(test, '~'))));
	}
#endif
	return res;
}

// Needed for ltcg builds on wine
char* Detoured__fullpath(char* absPath, const char* relPath, size_t maxLength)
{
	DETOURED_CALL(_fullpath);
	u64 relLen = 0;
	if (relPath)
		relLen = strlen(relPath);

	StringBuffer<> wideRelPath;
	wideRelPath.Appendf(L"%hs", relPath);

	char* res = nullptr;
	if (!absPath)
	{
		maxLength = relLen + g_virtualWorkingDir.count + 1;

		if (g_isRunningWine)
			res = (char*)malloc(maxLength); // Same as wine implementation
		else
			res = (char*)_malloc_base(maxLength);
		if (!res)
			return nullptr;

		StringBuffer<> wres;
		FixPath(wres, wideRelPath.data);
		sprintf_s(res, maxLength, "%ls", wres.data);
		UBA_ASSERT(res && wideRelPath.count < maxLength - 1);
	}
	else if ((relPath && relPath[1] == ':' && relLen < maxLength) || (relLen + g_virtualWorkingDir.count <= maxLength))
	{
		StringBuffer<> wres;
		FixPath(wres, wideRelPath.data);
		sprintf_s(absPath, maxLength, "%ls", wres.data);
		UBA_ASSERT(wres.count < maxLength);
		res = absPath;
	}

	DEBUG_LOG_DETOURED(L"_fullpath", L"(%hs) -> %hs", relPath, res);

#if UBA_DEBUG_VALIDATE
	if (g_validateFileAccess && !g_runningRemote)
	{
		char otherBuf[1024] = {};
		SuppressDetourScope _;
		char* test = True__fullpath(otherBuf, relPath, maxLength); (void)test;
		UBA_ASSERT((res != nullptr) == (test != nullptr) && (!res || (strcmp(res, test) == 0 || strchr(test, '~'))));
	}
#endif
	return res;
}

errno_t Detoured__get_wpgmptr(wchar_t** pValue)
{
	DETOURED_CALL(_get_wpgmptr);
	DEBUG_LOG_DETOURED(L"_get_wpgmptr", L"(%ls)", g_virtualApplication.data);
	*pValue = (wchar_t*)g_virtualApplication.data;
	return 0;
}

static WCHAR* g_fake_wpgmptr = g_virtualApplication.data;

WCHAR** Detoured___p__wpgmptr()
{
	DETOURED_CALL(__p__wpgmptr);
	DEBUG_LOG_DETOURED(L"__p__wpgmptr", L"(%ls)", g_virtualApplication.data);
	return &g_fake_wpgmptr;
}

__time64_t Detoured__time64(__time64_t* destTime)
{
	DETOURED_CALL(_time64);
	if (g_rules->Time64AlwaysReturnZero())
	{
		if (destTime)
			*destTime = 0;
		return 0;
	}
	return True__time64(destTime);
}

#if UBA_DEBUG_LOG_ENABLED
const wchar_t* WaccessResultToString(errno_t res)
{
	switch (res)
	{
	case 0:
		return L"Success";
	case EACCES:
		return L"AccessDenied";
	case ENOENT:
		return L"NotFound";
	case EINVAL:
		return L"InvalidParameter.";
	default:
		return L"UnknownError.";
	}
}
#endif

errno_t Detoured__waccess_s(const wchar_t* path, int mode)
{
	DETOURED_CALL(_waccess_s);

	StringBuffer<> fixedPath;
	FixPath(fixedPath, path);

	const wchar_t* originalPath = path;(void)originalPath;
	path = fixedPath.data;

	if (g_runningRemote && fixedPath.StartsWith(g_exeDir.data))
	{
		StringBuffer<> temp;
		temp.Append(g_virtualApplicationDir).Append(fixedPath.data + g_exeDir.count);
		fixedPath.Clear().Append(temp);
	}


	if (!CanDetour(fixedPath))
	{
		auto res = True__waccess_s(fixedPath.data, mode);
		DEBUG_LOG_TRUE(L"_waccess_s", L"(NODETOUR) %ls %i -> %ls", originalPath, mode, WaccessResultToString(res));
		return res;
	}

	DevirtualizePath(fixedPath);

	FileAttributes attr;
	Shared_GetFileAttributes(attr, fixedPath);

	SetLastError(attr.lastError);

	errno_t res = attr.exists ? 0 : ENOENT;
	DEBUG_LOG_DETOURED(L"_waccess_s", L"%ls %i -> %ls", originalPath, mode, WaccessResultToString(res));
	return res;
}

//int Detoured__wsopen_s(int* pfh, const wchar_t *filename, int oflag, int shflag, int pmode)
//{
//	DEBUG_LOG_DETOURED(L"_wsopen_s", L"(%ls)", filename);
//	int ret = True__wsopen_s(pfh, filename, oflag, shflag, pmode);
//	return ret;
//}

//int Detoured__fileno(FILE *stream)
//{
//	if (stream == stdin)
//		return 0;
//	if (stream == stdout)
//		return 1;
//	if (stream == stderr)
//		return 3;
//	return True__fileno(stream);
//}

intptr_t Detoured__get_osfhandle(int fd)
{
	DETOURED_CALL(_get_osfhandle);
	if (g_isDetachedProcess)
	{
		int fdTemp = fd;
		if (fd == StdOutFd)
			fdTemp = 1;
		if (fdTemp >= 0 && fdTemp <= 2)
		{
			intptr_t res = intptr_t(g_stdHandle[fdTemp]);
			DEBUG_LOG_DETOURED(L"_get_osfhandle", L"(%i) -> %llu", fd, u64(res));
			return res;
		}
	}

	auto res = True__get_osfhandle(fd);
	//DEBUG_LOG_TRUE(L"_get_osfhandle", L"(%i) -> %llu", fd, u64(res)); // Spam
	return res;
}

#if !defined(_M_ARM64)
int Detoured__isatty(int fd)
{
	DETOURED_CALL(_isatty);
	if (fd == StdOutFd && g_isDetachedProcess) // Tell process stdout should go to file
		return 1;
	int res = True__isatty(fd);
	// DEBUG_LOG_TRUE(L"_isatty", L"(%i) -> %i", fd, res); // Spam
	return res;
}
#endif

int Detoured__write(int fd, const void* buffer, unsigned int count)
{
	DETOURED_CALL(_write);
	if (fd == StdOutFd && g_isDetachedProcess)
	{
		if (g_suppressLogging)
			return count;

		const char* str = (const char*)buffer;
		const char* end = str + count;
		const char* line = str;
		do
		{
			const char* lineEnd = strchr(line, '\n');
			if (!lineEnd)
				lineEnd = end;
			Rpc_WriteLogf(L"%.*hs", u32(lineEnd - line), line);
			line = lineEnd + 1;

		} while (line < end);

		return count;
	}
	int res = True__write(fd, buffer, count);
	// DEBUG_LOG_TRUE(L"_write", L"(%i) -> %i", fd, res); // Spam
	return res;
}

int Detoured__wspawnl(int mode, const wchar_t* cmdname, const wchar_t* arg0, const wchar_t* arg1, ...)
{
	DETOURED_CALL(_wspawnl);
	StringBuffer<32*1024> cmdLine; // Need long commandline for cmake/ninja

	UBA_ASSERTF(mode == 0 /*_P_WAIT*/, L"_wspawnl called with unsupported mode %i (only _P_WAIT handled)", mode);
	UBA_ASSERTF(arg0 != nullptr, L"_wspawnl called with null arg0");

	if (cmdname == nullptr)
		cmdname = arg0;

	// Unfortunately lib.exe is using _wpgmptr which means it will return a path that is bad
	StringBuffer<260> cmdNameTemp;
	if (g_runningRemote && StartsWith(cmdname, g_exeDir.data))
	{
		cmdNameTemp.Append(g_virtualApplicationDir);
		cmdNameTemp.Append(cmdname + g_exeDir.count);
		cmdname = cmdNameTemp.data;
	}

	// TODO: Not sure what the rule is here.. very confusing with CreateProcess having application in two places
	cmdLine.Append('\"').Append(arg0).Append('\"');

	// arg1..argN are appended raw, separated by a single space. Callers (e.g. dumpbin) hand us pre-joined command-line
	// fragments that already contain spaces, so we must not quote/escape them here.
	if (arg1)
		cmdLine.Append(' ').Append(arg1);
	va_list args;
	va_start(args, arg1);
	while (wchar_t* argstr = va_arg(args, wchar_t*))
		cmdLine.Append(' ').Append(argstr);
	va_end(args);

	DEBUG_LOG_DETOURED(L"_wspawnl", L"(%ls %ls)", cmdname, cmdLine.data);

	STARTUPINFO si;
	ZeroMemory(&si, sizeof(si));
	si.cb = sizeof(si);
	si.dwFlags = STARTF_USESTDHANDLES;
	si.hStdInput = Detoured_GetStdHandle(STD_INPUT_HANDLE); // A console stdin is redirected to NUL inside Detoured_CreateProcessW
	si.hStdOutput = Detoured_GetStdHandle(STD_OUTPUT_HANDLE);
	si.hStdError = Detoured_GetStdHandle(STD_ERROR_HANDLE);
	PROCESS_INFORMATION pi;
	ZeroMemory(&pi, sizeof(pi));

	DWORD flags = CREATE_NO_WINDOW;
	if (!Detoured_CreateProcessW(cmdname, cmdLine.data, NULL, NULL, true, flags, NULL, NULL, &si, &pi))
	{
		UBA_ASSERT(false);
		return -1;
	}

	auto g = MakeGuard([&]() { CloseHandle(pi.hProcess); CloseHandle(pi.hThread); });

	DWORD res = Detoured_WaitForSingleObject(pi.hProcess, INFINITE);
	if (res != WAIT_OBJECT_0)
	{
		UBA_ASSERT(false);
		return -1;
	}

	DWORD exitCode = 0;
	if (!Detoured_GetExitCodeProcess(pi.hProcess, &exitCode))
	{
		UBA_ASSERT(false);
		return -1;
	}
	return exitCode;
}

#if !defined(__clang__)
errno_t Detoured__wsplitpath_s(const wchar_t* path, wchar_t* drive, size_t driveNumberOfElements, wchar_t* dir, size_t dirNumberOfElements, wchar_t* fname, size_t nameNumberOfElements, wchar_t* ext, size_t extNumberOfElements)
{
	DETOURED_CALL(_wsplitpath_s);

	g_rules->RepairMalformedLibPath(path);
	auto res = True__wsplitpath_s(path, drive, driveNumberOfElements, dir, dirNumberOfElements, fname, nameNumberOfElements, ext, extNumberOfElements);
	//DEBUG_LOG_TRUE(L"_wsplitpath_s", L"%ls %ls %ls %ls %ls", path, drive, dir, fname, ext);
	return res;
}
#endif

#if DETOURED_INCLUDE_DEBUG
int Detoured__wcsnicoll_l(const wchar_t* string1, const wchar_t* string2, size_t count, _locale_t locale)
{
	DETOURED_CALL(_wcsnicoll_l);
	return True__wcsnicoll_l(string1, string2, count, locale);
	/*
	//DETOURED_SCOPE;
	auto res = _wcsnicmp(string1, string2, count);
	if (res < 0)
		res = -1;
	else if (res > 0)
		res = 1;
#if UBA_DEBUG_VALIDATE
	auto res2 = True__wcsnicoll_l(string1, string2, count, locale);
	if (res2 < 0)
		res2 = -1;
	else if (res2 > 0)
		res2 = 1;
	UBA_ASSERT (res == res2);
#endif
	return res;
	*/
}

wchar_t* Detoured__wgetenv(const wchar_t* varname)
{
	DETOURED_CALL(_wgetenv);
	auto res = True__wgetenv(varname);
	DEBUG_LOG_DETOURED(L"_wgetenv", L"%ls -> %ls", varname, res);
	return res;
}

errno_t Detoured__wgetenv_s(size_t* pReturnValue, wchar_t* buffer, size_t numberOfElements, const wchar_t* varname)
{
	DETOURED_CALL(_wgetenv);
	auto res = True__wgetenv_s(pReturnValue, buffer, numberOfElements, varname);
	DEBUG_LOG_DETOURED(L"_wgetenv_s", L"%ls -> %ls", varname, buffer);
	return res;
}

char* Detoured_getenv(const char* varname)
{
	DETOURED_CALL(getenv);
	auto res = True_getenv(varname);
	DEBUG_LOG_DETOURED(L"getenv", L"%hs -> %hs", varname, res);
	return res;
}

errno_t Detoured_getenv_s(size_t* pReturnValue, char* buffer, size_t numberOfElements, const char* varname)
{
	DETOURED_CALL(getenv_s);
	auto res = True_getenv_s(pReturnValue, buffer, numberOfElements, varname);
	DEBUG_LOG_DETOURED(L"getenv_s", L"%hs -> %hs", varname, buffer);
	return res;
}

errno_t Detoured__wmakepath_s(wchar_t* path, size_t sizeInWords, const wchar_t* drive, const wchar_t* dir, const wchar_t* fname, const wchar_t* ext)
{
	DETOURED_CALL(_wmakepath_s);
	auto res = True__wmakepath_s(path, sizeInWords, drive, dir, fname, ext);
	DEBUG_LOG_TRUE(L"_wmakepath_s", L"%ls %ls %ls %ls %ls", path, drive, dir, fname, ext);
	return res;
}
char* Detoured__getcwd(char* buffer, int maxlen)
{
	DETOURED_CALL(_getcwd);
	auto res = True__getcwd(buffer, maxlen); // We know this calls GetCommandLine for both wine and windows
	DEBUG_LOG_TRUE(L"_getcwd", L"%hs", res);
	return res;
}

int Detoured_fputs(const char* str, FILE* stream)
{
	// This code is not working properly on wine helpers.
	// When using iwyu on remote wine helper we get a fno that matches stderr but it is not a stderr.. it is supposed to write to a file
	// Unfortunately I can't remember the initial reason this code was added so I can't change it and know if I broke something or not
	// - Decided to remove this function because I think it was written before the refactoring where detoured processes spawned their own children
	// and pipes became a huge mess
	/*
	if (!g_runningRemote || !g_isRunningWine)
	{
	int fno = _fileno(stream); (void)fno;
	int errfno = _fileno(stderr);
	if (fno == errfno || fno == _fileno(stdout))
	{
	//Shared_WriteConsole(str, u32(strlen(str)), fno == errfno);
	//return 1;
	}
	}
	*/
	return True_fputs(str, stream);
}


#endif // DETOURED_INCLUDE_DEBUG
