// Copyright Epic Games, Inc. All Rights Reserved.

#define UBA_IS_DETOURED_INCLUDE 1

#include "UbaDetoursFunctionsWin.h"
#include "UbaDetoursFileMappingTable.h"
#include "UbaDetoursApi.h"

#if !defined(UBA_USE_MIMALLOC)
#define True_malloc malloc
#else
// Taken from mimalloc/types.h
#define MI_ZU(x)  x##ULL
#define MI_SEGMENT_MASK                   ((uintptr_t)(MI_SEGMENT_ALIGN - 1))
#define MI_SEGMENT_ALIGN                  MI_SEGMENT_SIZE
#define MI_SEGMENT_SIZE                   (MI_ZU(1)<<MI_SEGMENT_SHIFT)
#define MI_SEGMENT_SHIFT                  ( 9 + MI_SEGMENT_SLICE_SHIFT)  // 32MiB
#define MI_SEGMENT_SLICE_SHIFT            (13 + MI_INTPTR_SHIFT)
#define MI_INTPTR_SHIFT (3)
#endif

#include <ntstatus.h>
#define WIN32_NO_STATUS

#include <malloc.h>

NTSTATUS NtCopyFileChunk(HANDLE Source, HANDLE Dest, HANDLE Event, PIO_STATUS_BLOCK IoStatusBlock, ULONG Length, PULONG SourceOffset, PULONG DestOffset, PULONG SourceKey, PULONG DestKey, ULONG Flags) { return 0; }
#define NtCurrentProcess() ((HANDLE)-1)

#undef DETOURED_FUNCTION_UNKNOWN
#define DETOURED_FUNCTION_UNKNOWN(Func) decltype(Func)* True_##Func;

#define DETOURED_FUNCTION(Func) decltype(Func)* True_##Func = ::Func; 
DETOURED_FUNCTIONS
DETOURED_FUNCTIONS_MEMORY
DETOURED_FUNCTIONS_MEMORY_NON_WINE
#undef DETOURED_FUNCTION

#undef DETOURED_FUNCTION_UNKNOWN
#define DETOURED_FUNCTION_UNKNOWN(Func) DETOURED_FUNCTION(Func)

#if !DETOURED_INCLUDE_DEBUG
#define True_GetEnvironmentVariableW GetEnvironmentVariableW
#endif

#include "UbaBinaryParser.h"
#include "UbaDefinitions.h"
#include "UbaDirectoryTable.h"
#include "UbaList.h"
#include "UbaProcessStats.h"
#include "UbaProtocol.h"
#include "UbaDetoursPayload.h"
#include "UbaApplicationRules.h"
#include "UbaDetoursShared.h"
#include "UbaDetoursUtilsWin.h"
#include "UbaFileMapping.h"

#include "Shlwapi.h"
#include <detours/detours.h>
#include <stdio.h>

#define UBA_LOG_COW_STATS 0

#if UBA_USE_MIMALLOC
extern "C" bool _mi_prim_random_buf(void* buf, size_t buf_len);
#endif

namespace uba
{
	extern HANDLE g_hostProcess;
	SharedMemoryAllocatorHandle g_sharedMemoryAllocator;
	u8* g_sharedMemory;
	FileMappingBackend g_sharedMemoryFileMappingBackend;

	SharedMemoryHandle g_permanentFilesHandle;
	u8* g_permanentFilesMemory;


bool g_useMiMalloc;
constexpr u64  g_pageSize = 64*1024;

thread_local u32 t_disallowCreateFileDetour	 = 0; // Set this to 1 to disallow file detour.. note that this will prevent directory cache from properly being updated

// Beautiful! cl.exe needs an exact address in that range to be able to map in pch file
// So we'll reserve a bigger range than will be requested and give it back when needed.
constexpr uintptr_t g_clExeBaseAddress = 0x6bb00000000;
constexpr u64 g_clExeBaseAddressSize = 0x400000000;
void* g_clExeBaseReservedMemory = 0;

HANDLE PseudoHandle = (HANDLE)0xfffffffffffffffe;
constexpr int StdOutFd = -2;

#if UBA_DEBUG_LOG_ENABLED
bool g_debugFileFlushOnWrite = false;

void WriteDebugInner(const void* data, u32 dataLen)
{
	if (g_debugFile == InvalidFileHandle) // Re-check under g_debugLogMutex — closes the reuse handle-swap race.
		return;
	auto readMem = (const u8*)data;
	ULONG toWrite = (ULONG)dataLen;
	DWORD lastError = GetLastError();
	while (toWrite)
	{
		IO_STATUS_BLOCK ioStatusBlock;
		if (True_NtWriteFile((HANDLE)g_debugFile, NULL, NULL, NULL, &ioStatusBlock, (void*)readMem, toWrite, NULL, NULL) != STATUS_SUCCESS)
			break;
		readMem += ioStatusBlock.Information;
		toWrite -= (ULONG)ioStatusBlock.Information;
	}
	if (g_debugFileFlushOnWrite)
		True_FlushFileBuffers((HANDLE)g_debugFile);
	SetLastError(lastError);
}
void FlushDebugLog()
{
	if (isLogging())
		True_FlushFileBuffers((HANDLE)g_debugFile);
}
#endif

//#define UBA_PROFILE_DETOURED_CALLS

#if defined(UBA_PROFILE_DETOURED_CALLS)
#define DETOURED_FUNCTION(name) Timer timer##name;
DETOURED_FUNCTIONS
DETOURED_FUNCTIONS_MEMORY
DETOURED_FUNCTIONS_MEMORY_NON_WINE
#undef DETOURED_FUNCTION
#define DETOURED_CALL(name) TimerScope _(timer##name)
#else
#define DETOURED_CALL(name) //DEBUG_LOG(TC(#name));
#endif

bool g_isDetachedProcess;
bool g_isRunningWine;
bool g_useLazyCow;
int g_uiLanguage;
u32 g_processId;

u8 g_emptyMemoryFileMem;
MemoryFile& g_emptyMemoryFile = *new MemoryFile(&g_emptyMemoryFileMem, true);

constexpr u64 DetouredHandleMaxCount = 200*1024; // ~200000 handles enough?
constexpr u64 DetouredHandleStart = 600000; // Let's hope noone uses the handles starting at 600000! :)
constexpr u64 DetouredHandleEnd = DetouredHandleStart + DetouredHandleMaxCount;
constexpr u64 DetouredHandlesMemReserve = AlignUp(DetouredHandleMaxCount*sizeof(DetouredHandle), 64*1024);
constexpr u64 DetouredHandlesMemStart = 0;
MemoryBlock& g_detouredHandleMemoryBlock = *new MemoryBlock(TC("DetouredHandles"), DetouredHandlesMemReserve, (void*)DetouredHandlesMemStart);
u64 g_detouredHandlesStart = u64(g_detouredHandleMemoryBlock.memory);
u64 g_detouredHandlesEnd = g_detouredHandlesStart + g_detouredHandleMemoryBlock.reserveSize;
BlockAllocator<DetouredHandle> g_detouredHandleAllocator(g_detouredHandleMemoryBlock);
void* DetouredHandle::operator new(size_t size) { return g_detouredHandleAllocator.Allocate(); }
void DetouredHandle::operator delete(void* p) { g_detouredHandleAllocator.Free(p); }

inline bool isDetouredHandle(HANDLE h)
{
	return u64(h) >= DetouredHandleStart && u64(h) < DetouredHandleEnd;
}

inline HANDLE makeDetouredHandle(DetouredHandle* p)
{
	u64 index = (u64(p) - g_detouredHandlesStart) / sizeof(DetouredHandle);
	UBA_ASSERT(index < DetouredHandleMaxCount);
	return HANDLE(DetouredHandleStart + index);
}

inline DetouredHandle& asDetouredHandle(HANDLE h)
{
	u64 index = u64(h) - DetouredHandleStart;
	u64 p = (index * sizeof(DetouredHandle)) + g_detouredHandlesStart;
	return *(DetouredHandle*)p;
}

HANDLE g_stdHandle[3];
HANDLE g_nullFile;

struct ListDirectoryHandle
{
	void* operator new(size_t size);
	void operator delete(void* p);

	StringKey dirNameKey;
	DirectoryTable::Directory& dir;
	DirTableOffset dirTableOffset;
	int it;
	Vector<DirTableOffset> fileTableOffsets;
	HANDLE validateHandle;
	TString wildcard;
	const wchar_t* originalName = nullptr;
};

constexpr u64 ListDirHandlesRange = 4*1024*1024;
MemoryBlock& g_listDirHandleMemoryBlock = *new MemoryBlock(TC("DetouredDirHandles"), ListDirHandlesRange);
constexpr u64 ListDirHandlesStart = DetouredHandleEnd;
constexpr u64 ListDirHandlesEnd = ListDirHandlesStart + ListDirHandlesRange/sizeof(ListDirectoryHandle);
BlockAllocator<ListDirectoryHandle>& g_listDirectoryHandleAllocator = *new BlockAllocator<ListDirectoryHandle>(g_listDirHandleMemoryBlock);
void* ListDirectoryHandle::operator new(size_t size) { return g_listDirectoryHandleAllocator.Allocate(); }
void ListDirectoryHandle::operator delete(void* p) { g_listDirectoryHandleAllocator.Free(p); }
inline bool isListDirectoryHandle(HANDLE h) { return (u64)h >= ListDirHandlesStart && (u64)h < ListDirHandlesEnd; }
inline HANDLE makeListDirectoryHandle(ListDirectoryHandle* p) { return HANDLE(ListDirHandlesStart + (p - (ListDirectoryHandle*)g_listDirHandleMemoryBlock.memory)); }
inline ListDirectoryHandle& asListDirectoryHandle(HANDLE h) { return *(((ListDirectoryHandle*)g_listDirHandleMemoryBlock.memory) + (u64(h) - ListDirHandlesStart)); }

ReaderWriterLock& g_loadedModulesLock = *new ReaderWriterLock();
UnorderedMap<HMODULE, TString>& g_loadedModules = *new UnorderedMap<HMODULE, TString>();
u64 g_memoryFileIndexCounter = ~u64(0) - 1000000; // I really hope this will not collide with anything
bool g_filesCouldBeCompressed;

bool CouldBeCompressedFile(const StringView& fileName)
{
	return g_filesCouldBeCompressed && g_globalRules.FileCanBeCompressed(fileName);
}

bool CanDetour(StringView path)
{
	if (t_disallowDetour || !path.count)
		return false;

	if (path[0] == '\\')
	{
		if (path[1] == '\\')
		{
			if (path[2] == '.' && path[3] == '\\') // \\.\ - Win32 namespace for files and devices
			{
				if (path[5] != ':') // Not file
					return false;
				path = path.Skip(4);
			}
			else if (path[2] == '?') // \\?\ - Win32 path prefix to send through unmodified to nt layer
			{
				if (path[3] != '\\')
					return false;
				path = path.Skip(4);
			}
			else
			{
				if (path[2] == '\\' || path[2] == '/' || path[2] == ':' || path[2] == '*' || path[2] == '?' || path[2] == '\"' || path[2] == '<' || path[2] == '>' || path[2] == '|')
					return false; // Unknown
			}
		}
		else if (path[1] == '?' && path[2] == '?' && path[3] == '\\')
		{
			if (path[4] == 'U' && path[5] == 'N' && path[6] == 'C') // All network paths ok?
				return true;
			if (path[5] == ':')
				path = path.Skip(4);
			else
				return false; // Unknown
		}
	}
	if (path.Equals(TC("nul")))
		return false;

	return g_rules->CanDetour(path, g_runningRemote);
}

struct SuppressCreateFileDetourScope
{
	SuppressCreateFileDetourScope() { ++t_disallowCreateFileDetour; }
	~SuppressCreateFileDetourScope() { --t_disallowCreateFileDetour; }
};

const wchar_t* HandleToName(const DetouredHandle& dh)
{
	if (dh.fileObject)
		if (const wchar_t* name = dh.fileObject->fileInfo->name)
			return name;
	return L"Unknown";
}

BlockAllocator<FileObject>& g_fileObjectAllocator = *new BlockAllocator<FileObject>(g_memoryBlock);

MemoryFile::MemoryFile(u8* data, bool localOnly)
:	memoryView(g_sharedMemoryFileMappingBackend)
,	baseAddress(data)
,	isLocalOnly(localOnly)
{
}

MemoryFile::MemoryFile(bool localOnly, u64 reserveSize_, bool isThrowAway_, u32 commitStepSize_, const tchar* fileName)
:	commitStepSize(commitStepSize_)
,	memoryView(g_sharedMemoryFileMappingBackend)
,	isLocalOnly(localOnly)
,	isThrowAway(isThrowAway_)
{
	if (!isThrowAway)
		Reserve(reserveSize_, fileName);
}

void MemoryFile::Reserve(u64 reserveSize_, const tchar* fileName)
{
	UBA_ASSERT(!isThrowAway);
	reserveSize = reserveSize_;
	if (isLocalOnly)
	{
		baseAddress = (u8*)VirtualAlloc(NULL, reserveSize, MEM_RESERVE, PAGE_READWRITE);
		if (!baseAddress)
			FatalError(1354, L"VirtualAlloc failed trying to reserve %llu for %s. (Error code: %u)", reserveSize, fileName, GetLastError());
		mappedSize = reserveSize;
	}
	else
	{
		memoryHandle = Rpc_CreateSharedMemory(memoryView, g_sharedMemoryAllocator, reserveSize, 0, fileName);
		mappedSize = reserveSize;
		baseAddress = memoryView.GetMemory();
	}
}

void MemoryFile::Unreserve()
{
	if (isLocalOnly)
	{
		VirtualFree(baseAddress, 0, MEM_RELEASE);
	}
	else
	{
		memoryView.Reset();
	}
	baseAddress = nullptr;
	committedSize = 0;
}

void MemoryFile::Write(DetouredHandle& handle, LPCVOID lpBuffer, u64 nNumberOfBytesToWrite, const tchar* hint)
{
	u64 newPos = handle.pos + nNumberOfBytesToWrite;

	if (isThrowAway)
	{
		writtenSize = newPos;
		return;
	}

	EnsureCommitted(handle, newPos);
	memcpy(baseAddress + handle.pos, lpBuffer, nNumberOfBytesToWrite);
	handle.pos += nNumberOfBytesToWrite;
	if (writtenSize < newPos)
	{
		writtenSize = newPos;
		isReported = false;
	}
}

void MemoryFile::EnsureCommitted(const DetouredHandle& handle, u64 size)
{
	if (isThrowAway)
		return;

	if (committedSize >= size)
		return;
	if (size > reserveSize)
	{
		UBA_ASSERTF(false, L"Reserved size of %ls is smaller than what is requested to be. ReserveSize: %llu Written: %llu Requested: %llu", HandleToName(handle), reserveSize, writtenSize, size);
		FatalError(1347, L"Reserved size of %ls is smaller than what is requested to be. ReserveSize: %llu Written: %llu Requested: %llu", HandleToName(handle), reserveSize, writtenSize, size);
	}
	/*
	if (size > mappedSize)
	{
		if (shouldRemap)
		{
			memoryHandle = Rpc_CreateSharedMemory(memoryView, g_sharedMemoryAllocator, reserveSize, 0, TC(""));
			mappedSize = reserveSize;
			baseAddress = memoryView.GetMemory();
		}
	}
	*/
	{
		u64 toCommit = Min(reserveSize, AlignUp(size - committedSize, g_pageSize));

		if (!isLocalOnly)
		{
			toCommit = AlignUp(toCommit, u64(commitStepSize));
			Rpc_CommitSharedMemory(memoryView, g_sharedMemoryAllocator, memoryHandle, toCommit);
		}
		else
		{
			if (!VirtualAlloc(baseAddress + committedSize, toCommit, MEM_COMMIT, PAGE_READWRITE))
				FatalError(1347, L"Failed to ensure virtual memory for %ls trying to commit %llu at %llx. MappedSize: %llu, CommittedSize: %llu RequestedSize: %llu. (%u)", HandleToName(handle), toCommit, baseAddress + committedSize, mappedSize, committedSize, size, GetLastError());
			if (g_allowDiscardVirtualMemory)
				DiscardVirtualMemory(baseAddress + committedSize, toCommit);
		}
		committedSize += toCommit;
	}
}

void ToInvestigate(const wchar_t* format, ...)
{
#if UBA_DEBUG_LOG_ENABLED
	va_list arg;
	va_start (arg, format);
	StringBuffer<> buffer;
	buffer.Append(format, arg);
	va_end (arg);
	DEBUG_LOG(buffer.data);
	FlushDebugLog();
	Rpc_WriteLogf(L"%ls\n", buffer.data);
#endif
}

UBA_NOINLINE void UbaAssert(const tchar* text, const char* file, u32 line, const char* expr, bool allowTerminate, u32 terminateCode, void* context, u32 skipCallstackCount)
{
	SuppressDetourScope _;
	static CriticalSection cs;
	ScopedCriticalSection s(cs);

	#if UBA_DEBUG_LOG_ENABLED
	FlushDebugLog();
	#endif

	static auto& sb = *new StringBuffer<16*1024>();
	WriteAssertInfo(sb.Clear(), text, file, line, expr);
	Rpc_ResolveCallstack(sb, 3 + skipCallstackCount, context);
	Rpc_WriteLog(sb.data, sb.count, true, true);

	#if UBA_DEBUG_LOG_ENABLED
	FlushDebugLog();
	#endif

	#if UBA_ASSERT_MESSAGEBOX
	StringBuffer<> title;
	title.Appendf(L"Assert %ls - pid %u", GetApplicationShortName(), GetCurrentProcessId());
	int ret = MessageBoxW(GetConsoleWindow(), sb.data, title.data, MB_ABORTRETRYIGNORE|MB_SYSTEMMODAL);
	if (ret == IDABORT)
		ExitProcess(terminateCode);
	else if (ret == IDRETRY && IsDebuggerPresent())
		DebugBreak();
	#else
	if (allowTerminate)
		ExitProcess(terminateCode);
	#endif
}

LastErrorToText::LastErrorToText(u32 lastError)
{
	SuppressDetourScope _;
	size_t size = ::FormatMessageW(FORMAT_MESSAGE_FROM_SYSTEM | FORMAT_MESSAGE_IGNORE_INSERTS, NULL, lastError, MAKELANGID(LANG_ENGLISH, SUBLANG_ENGLISH_US), data, capacity, NULL);
	if (!size)
		AppendValue(lastError);
	else
		Resize(size - 2);
}

const wchar_t* ToString(BOOL b) { return b ? L"Success" : L"Error"; }

const wchar_t* HandleToName(HANDLE handle)
{
	if (handle == INVALID_HANDLE_VALUE)
		return L"INVALID";
	if (isListDirectoryHandle(handle))
	{
		#if UBA_DEBUG
		return asListDirectoryHandle(handle).originalName;
		#else
		return L"DIRECTORY";
		#endif
	}
	if (!isDetouredHandle(handle))
		return L"UNKNOWN";
	DetouredHandle& dh = asDetouredHandle(handle);
	if (FileObject* fo = dh.fileObject)
		if (FileInfo* fi = fo->fileInfo)
			if (const wchar_t* name = fi->name)
				return name;
	return L"DETOURED";
}


u64 FileTypeMaxSize(StringView file) { return g_rules->FileTypeMaxSize(file, file.StartsWith(g_systemTemp)); }
u32 FileTypeCommitSize(StringView file) { return g_rules->FileTypeCommitSize(file); }

bool EnsureMapped(DetouredHandle& handle, DWORD dwFileOffsetHigh = 0, DWORD dwFileOffsetLow = 0, SIZE_T numberOfBytesToMap = 0, void* baseAddress = nullptr, SharedMemoryMapType type = SharedMemoryMapType_ReadOnly)
{
	FileInfo& info = *handle.fileObject->fileInfo;
	
	if (info.memoryFile)
		return true;
	if (info.fileMapMem && type == SharedMemoryMapType_ReadOnly)
		return true;

	u64 offset = ToLargeInteger(dwFileOffsetHigh, dwFileOffsetLow).QuadPart;
	if (!numberOfBytesToMap)
	{
		UBA_ASSERTF(info.size && info.size != InvalidValue || (info.isFileMap && info.size == 0), L"FileInfo file size is bad: %llu", info.size);
		numberOfBytesToMap = info.size;
	}

	offset += info.trueFileMapOffset;
	u64 alignedOffsetStart = AlignUp(offset - (PageSize - 1), PageSize);
	u64 mapSize = (offset - alignedOffsetStart) + numberOfBytesToMap;

	SharedMemoryView view(g_sharedMemoryFileMappingBackend);
	Rpc_GetSharedMemory(view, g_sharedMemoryAllocator, g_sharedMemory, info.trueFileMapHandle, alignedOffsetStart, mapSize, info.originalName, type);

	info.fileMapMem = view.DetachMemory() + (offset - alignedOffsetStart);
	info.fileMapMemSize = info.size;

	DEBUG_LOG_TRUE(L"INTERNAL MapViewOfFileEx", L"(%ls) (size: %llu) %s (%ls) -> 0x%llx", info.name, numberOfBytesToMap, (type == SharedMemoryMapType_ReadOnly ? L"ReadOnly" : L"CoW"), info.originalName, uintptr_t(info.fileMapMem));
	return true;
}

struct LazyCowRange
{
	u8* base;
	u64 size;

	#if UBA_LOG_COW_STATS
	u64 cowSize;
	#endif
};
ReaderWriterLock& g_lazyCowLock = *new ReaderWriterLock();
Vector<LazyCowRange>& g_lazyCowRanges = *new Vector<LazyCowRange>(); // sorted ascending by base
PVOID g_lazyCowHandler = nullptr;

u64 LazyCowUpperBound(uintptr_t addr)
{
	u64 lo = 0, hi = g_lazyCowRanges.size();
	while (lo < hi)
	{
		u64 mid = lo + (hi - lo) / 2;
		if ((uintptr_t)g_lazyCowRanges[mid].base <= addr)
			lo = mid + 1;
		else
			hi = mid;
	}
	return lo;
}

LONG CALLBACK LazyCowExceptionHandler(PEXCEPTION_POINTERS ep)
{
	auto* r = ep->ExceptionRecord;
	if (r->ExceptionCode != EXCEPTION_ACCESS_VIOLATION || r->NumberParameters < 2)
		return EXCEPTION_CONTINUE_SEARCH;
	if (r->ExceptionInformation[0] != 1) // 1 == write access (0 read, 8 execute)
		return EXCEPTION_CONTINUE_SEARCH;
	u64 addr = (u64)r->ExceptionInformation[1];

	SCOPED_READ_LOCK(g_lazyCowLock, lock);
	u64 ub = LazyCowUpperBound(addr);
	if (ub == 0)
		return EXCEPTION_CONTINUE_SEARCH;
	LazyCowRange& rg = g_lazyCowRanges[ub - 1]; // greatest base <= addr
	u64 end = (u64)rg.base + rg.size;
	if (addr >= end)
		return EXCEPTION_CONTINUE_SEARCH;

	#if UBA_LOG_COW_STATS
	rg.cowSize += PageSize;
	#endif

	lock.Leave();

	u64 page = addr & ~u64(PageSize - 1);
	u64 size = Min(PageSize, end - page);
	DWORD oldProtect;
	if (!VirtualProtect((void*)page, size, PAGE_WRITECOPY, &oldProtect)) // VirtualProtect is not detoured - safe to call here
		return EXCEPTION_CONTINUE_SEARCH;
	return EXCEPTION_CONTINUE_EXECUTION; // retry the faulting write; the kernel now copies-on-write this one page
}

void RegisterLazyCowRange(u8* base, u64 size)
{
	SCOPED_WRITE_LOCK(g_lazyCowLock, lock);
	u64 ub = LazyCowUpperBound((uintptr_t)base);
	if (ub > 0 && g_lazyCowRanges[ub - 1].base == base) // same view remapped - keep the larger size
	{
		if (size > g_lazyCowRanges[ub - 1].size)
			g_lazyCowRanges[ub - 1].size = size;
		return;
	}
	g_lazyCowRanges.insert(g_lazyCowRanges.begin() + ub, { base, size });
}

void UnregisterLazyCowRange(u8* base)
{
	SCOPED_WRITE_LOCK(g_lazyCowLock, lock);
	u64 ub = LazyCowUpperBound((uintptr_t)base);
	if (ub <= 0 || g_lazyCowRanges[ub - 1].base != base)
		return;

	#if UBA_LOG_COW_STATS
	auto& t = g_lazyCowRanges[ub - 1];
	DEBUG_LOG(L"COW %llu/%llu", t.cowSize, t.size);
	#endif

	g_lazyCowRanges.erase(g_lazyCowRanges.begin() + (ub - 1));
}

ReaderWriterLock& g_longPathNameCacheLock = *new ReaderWriterLock();
using LongPathMap = GrowingUnorderedMap<const wchar_t*, const wchar_t*, HashString, EqualString>;
LongPathMap& g_longPathNameCache = *new LongPathMap(g_memoryBlock);

void Rpc_AllocFailed(const wchar_t* allocType, u32 error)
{
	RPC_MESSAGE(VirtualAllocFailed, virtualAllocFailed)
	writer.WriteString(allocType);
	writer.WriteU32(error);
	writer.Flush();
	Sleep(5*1000);
}

void CloseCaches()
{
	#if UBA_LOG_COW_STATS
	for (auto& t : g_lazyCowRanges)
		DEBUG_LOG(L"COW %llu/%llu", t.cowSize, t.size);
	#endif

	for (auto& it : g_mappedFileTable.m_lookup)
	{
		FileInfo& info = it.second;
		if (info.fileMapMem)
		{
			// TODO: Most files here are in the big shared memory block and can't be unmapped this way
			//BOOL res = True_UnmapViewOfFile(info.fileMapMem);
			//DEBUG_LOG_TRUE(L"INTERNAL UnmapViewOfFile", L"0x%llx (%ls) (%ls) -> %s", uintptr_t(info.fileMapMem), info.name, info.originalName, ToString(res));
		}
		if (info.trueFileMapHandle.IsValid())
		{
			// TODO
			//DEBUG_LOG_TRUE(L"INTERNAL CloseHandle", L"%llu (%ls) (%ls)", info.trueFileMapHandle.ToU64(), info.name, info.originalName);
			//CloseHandle(info.trueFileMapHandle);
		}

		// Let them leak
		if (info.memoryFile && !info.memoryFile->isLocalOnly)
			info.memoryFile->memoryView.DetachMemory();
		//if (info.memoryFile && info.memoryFile != &g_emptyMemoryFile)
		//	delete info.memoryFile;
	}
}

bool g_exitMessageSent;

void SendExitMessage(DWORD exitCode, u64 startTime);
void OnModuleLoaded(HMODULE moduleHandle, const StringView& name);

// Variables used to communicate state from kernelbase functions to ntdll functions
thread_local const wchar_t* t_renameFileNewName;

#include "UbaDetoursFunctionsMiMalloc.inl"
#include "UbaDetoursFunctionsNtDll.inl"
#include "UbaDetoursFunctionsKernelBase.inl"
#include "UbaDetoursFunctionsUcrtBase.inl"
#include "UbaDetoursFunctionsImagehlp.inl"
#include "UbaDetoursFunctionsDbgHelp.inl"
#include "UbaDetoursFunctionsShell32.inl"
#include "UbaDetoursFunctionsRpcrt4.inl"
#include "UbaDetoursFunctionsCombase.inl"

extern u32 g_consoleStringIndex;

void SendExitMessage(DWORD exitCode, u64 startTime)
{
	if (g_exitMessageSent)
		return;
	g_exitMessageSent = true;

	PROCESS_MEMORY_COUNTERS_EX pmc;
	GetProcessMemoryInfo(GetCurrentProcess(), (PROCESS_MEMORY_COUNTERS*)&pmc, sizeof(pmc));
	AtomicMax(g_stats.peakMemory, pmc.PagefileUsage);

	FILETIME creationTime, exitTimeFt, kernelTime, userTime;
	if (GetProcessTimes(GetCurrentProcess(), &creationTime, &exitTimeFt, &kernelTime, &userTime))
	{
		ULARGE_INTEGER k, u;
		k.LowPart = kernelTime.dwLowDateTime;
		k.HighPart = kernelTime.dwHighDateTime;
		u.LowPart = userTime.dwLowDateTime;
		u.HighPart = userTime.dwHighDateTime;
		AtomicMax(g_stats.cpuTime, u64(k.QuadPart + u.QuadPart));
	}

	if (g_consoleStringIndex)
		Shared_WriteConsole(L"\n", 1, 0);

	#if UBA_USE_MIMALLOC
	#if TRACK_UCRT_ALLOC_ENABLED
	StringBuffer<256> str;
	str.Appendf(L"Alloc: %llu Realloc: %llu Free: %llu\n", g_allocCount.load(), g_reallocCount.load(), g_freeCount.load());
	Shared_WriteConsole(str.data, str.count, 0);
	#endif
	#endif

	SendInput();

	g_stats.detoursMemory = g_memoryBlock.writtenSize; // + g_directoryTable.m_memorySize + g_mappedFileTable.m_memPosition;

	RPC_MESSAGE(Exit, log)
	writer.WriteU32(exitCode);
	writer.WriteString(g_logName);

	g_stats.detach.time += GetTime() - startTime;
	g_stats.detach.count = 1;

	g_stats.Write(writer);
	g_kernelStats.Write(writer);

	// We must flush here if this is a child because,
	// if there is a parent process waiting for this to finish,
	// the parent might move on before Exit message has been processed on session side
	writer.Flush(g_isChild);
}

void* FindIATEntryByName(HMODULE moduleBase, const char* importDllName, const char* importFuncName)
{
	u8* base = reinterpret_cast<u8*>(moduleBase);
	auto* dos = reinterpret_cast<IMAGE_DOS_HEADER*>(base);
	if (dos->e_magic != IMAGE_DOS_SIGNATURE)
		return nullptr;
	auto* nt = reinterpret_cast<IMAGE_NT_HEADERS64*>(base + dos->e_lfanew);
	if (nt->Signature != IMAGE_NT_SIGNATURE)
		return nullptr;
	const auto& dir = nt->OptionalHeader.DataDirectory[IMAGE_DIRECTORY_ENTRY_IMPORT];
	if (!dir.VirtualAddress || !dir.Size)
		return nullptr;
	for (auto* imp = reinterpret_cast<IMAGE_IMPORT_DESCRIPTOR*>(base + dir.VirtualAddress); imp->Name; ++imp)
	{
		const char* dllName = reinterpret_cast<const char*>(base + imp->Name);
		if (_stricmp(dllName, importDllName) != 0)
			continue;
		auto* origThunk = reinterpret_cast<IMAGE_THUNK_DATA64*>(base + imp->OriginalFirstThunk);
		auto* firstThunk = reinterpret_cast<IMAGE_THUNK_DATA64*>(base + imp->FirstThunk);
		if (!imp->OriginalFirstThunk) // Some binaries may have OriginalFirstThunk == 0; then use FirstThunk for names too.
			origThunk = firstThunk;
		for (; origThunk->u1.AddressOfData; ++origThunk, ++firstThunk)
		{
			if (origThunk->u1.Ordinal & IMAGE_ORDINAL_FLAG64)
				continue; // ordinal import
			auto* importByName = reinterpret_cast<IMAGE_IMPORT_BY_NAME*>(base + origThunk->u1.AddressOfData);
			const char* funcName = reinterpret_cast<const char*>(importByName->Name);
			if (strcmp(funcName, importFuncName) == 0)
				return &firstThunk->u1.Function; // Return address of the IAT slot (the pointer you must overwrite)
		}
	}
	return nullptr;
}

// Returns true if function is imported from outside dlls, false if module(exe) itself is handling the import
bool IsFunctionImportedFromAnyDll(HMODULE moduleBase, const char* importFuncName)
{
	u8* base = reinterpret_cast<u8*>(moduleBase);
	auto* dos = reinterpret_cast<IMAGE_DOS_HEADER*>(base);
	if (dos->e_magic != IMAGE_DOS_SIGNATURE)
		return false;
	auto* nt = reinterpret_cast<IMAGE_NT_HEADERS64*>(base + dos->e_lfanew);
	if (nt->Signature != IMAGE_NT_SIGNATURE)
		return false;
	const auto& dir = nt->OptionalHeader.DataDirectory[IMAGE_DIRECTORY_ENTRY_IMPORT];
	if (!dir.VirtualAddress || !dir.Size)
		return false;
	for (auto* imp = reinterpret_cast<IMAGE_IMPORT_DESCRIPTOR*>(base + dir.VirtualAddress); imp->Name; ++imp)
	{
		auto* origThunk = reinterpret_cast<IMAGE_THUNK_DATA64*>(base + imp->OriginalFirstThunk);
		if (!imp->OriginalFirstThunk)
			origThunk = reinterpret_cast<IMAGE_THUNK_DATA64*>(base + imp->FirstThunk);
		for (; origThunk->u1.AddressOfData; ++origThunk)
		{
			if (origThunk->u1.Ordinal & IMAGE_ORDINAL_FLAG64)
				continue;
			auto* importByName = reinterpret_cast<IMAGE_IMPORT_BY_NAME*>(base + origThunk->u1.AddressOfData);
			if (strcmp(reinterpret_cast<const char*>(importByName->Name), importFuncName) == 0)
				return true;
		}
	}
	return false;
}

bool PatchIAT(HMODULE exeBase, const char* importDllName, const char* importFuncName, void* newFunc, void** oldFuncOut = nullptr)
{
	void* slot = FindIATEntryByName(exeBase, importDllName, importFuncName);
	if (!slot)
		return false;
	DWORD oldProt = 0;
	if (!VirtualProtect(slot, sizeof(void*), PAGE_READWRITE, &oldProt))
		return false;
	void* old = *reinterpret_cast<void**>(slot);
	*reinterpret_cast<void**>(slot) = newFunc;
	VirtualProtect(slot, sizeof(void*), oldProt, &oldProt);
	FlushInstructionCache(GetCurrentProcess(), slot, sizeof(void*));
	if (oldFuncOut)
		*oldFuncOut = old;
	return true;
}

static void IdentifyPatcher(StringBufferBase& out, void* fn)
{
	const u8* p = reinterpret_cast<const u8*>(fn);
	void* target = nullptr;

	if (p[0] == 0xE9)	// jmp rel32
		target = (void*)(p + 5 + *reinterpret_cast<const int*>(p + 1));
	else if (p[0] == 0xFF && p[1] == 0x25)	// jmp [rip+disp32]
	{
		void** slot = (void**)(p + 6 + *reinterpret_cast<const int*>(p + 2));
		target = *slot;
	}
	else if (p[0] == 0x48 && p[1] == 0xB8 && p[10] == 0xFF && p[11] == 0xE0) // mov rax,imm64; jmp rax
		target = *reinterpret_cast<void* const*>(p + 2);
	else if (p[0] == 0x68 && p[5] == 0xC3)	// push imm32; ret
		target = (void*)(uintptr_t)*reinterpret_cast<const u32*>(p + 1);

	//Rpc_WriteLogf(L"prologue: %02x %02x %02x %02x %02x %02x", p[0], p[1], p[2], p[3], p[4], p[5]);
	if (!target)
	{
		out.Append(TCV("Function referenced is too small to be detoured."));
		return;
	}

	HMODULE mod = nullptr;
	if (GetModuleHandleExW(GET_MODULE_HANDLE_EX_FLAG_FROM_ADDRESS | GET_MODULE_HANDLE_EX_FLAG_UNCHANGED_REFCOUNT, reinterpret_cast<LPCWSTR>(target), &mod) && mod)
	{
		wchar_t path[MAX_PATH] = {};
		GetModuleFileNameW(mod, path, MAX_PATH);
		out.Appendf(L"Function already hooked by: %s (target %p)", path, target);
	}
	else
		out.Appendf(L"Function has hook target %p in unknown/private memory", target);
}

void DetourAttachFunction(void** trueFunc, void* detouredFunc, const char* funcName)
{
	if (!*trueFunc)
	{
		//Rpc_WriteLogf(L"GetProcAddress Failed to find %hs", funcName);
		return;
	}
	auto error = DetourAttach(trueFunc, detouredFunc);
	if (error == NO_ERROR)
		return;
	StringBuffer<1024> errorString;
	switch (error)
	{
	case ERROR_INVALID_BLOCK: IdentifyPatcher(errorString, *trueFunc); break;
	case ERROR_INVALID_HANDLE: errorString = TCV("The ppPointer parameter is NULL or points to a NULL pointer."); break;
	case ERROR_INVALID_OPERATION: errorString = TCV("No pending transaction exists."); break;
	case ERROR_NOT_ENOUGH_MEMORY: errorString = TCV("Not enough memory exists to complete the operation."); break;
	default: errorString = TCV("Unknown error");
	}
	Rpc_WriteLogf(L"Failed to detour %hs (%s)", funcName, errorString.data);
	ExitProcess(error);
}

void DetourDetachFunction(void** trueFunc, void* detouredFunc, const char* funcName)
{
	if (!*trueFunc)
		return;
	auto error = DetourDetach(trueFunc, detouredFunc);
	if (error == NO_ERROR)
		return;
	Rpc_WriteLogf(L"Failed to detach detoured %hs", funcName);
}

void DetourTransactionBegin()
{
	LONG error = ::DetourTransactionBegin();
	if (error != NO_ERROR)
		FatalError(1357, L"DetourTransactionBegin failed (%ld)", error);
	error = ::DetourUpdateThread(GetCurrentThread());
	if (error != NO_ERROR)
		FatalError(1358, L"DetourUpdateThread failed (%ld)", error);
}

void DetourTransactionCommit()
{
	LONG error = ::DetourTransactionCommit();
	if (error != NO_ERROR)
		FatalError(1343, L"DetourTransactionCommit failed (%ld)", error);
}

int DetourAttachFunctions(bool runningRemote)
{
	DetourTransactionBegin();

	#define DETOURED_FUNCTION(Func) True_##Func = (decltype(True_##Func))(void*)GetProcAddress(moduleHandle, #Func);

	if (HMODULE moduleHandle = GetModuleHandleW(L"kernelbase.dll"))
	{
		DETOURED_FUNCTIONS_KERNELBASE
	}

	if (HMODULE moduleHandle = GetModuleHandleW(L"kernel32.dll"))
	{
		DETOURED_FUNCTIONS_KERNEL32

		if (IsRunningWine())
		{
			// Old wine versions do not have this implemented. Call to another function that shows a message
			if (!GetProcAddress(moduleHandle, "InitializeSynchronizationBarrier"))
				PatchIAT(GetModuleHandleW(nullptr), "KERNEL32.dll", "InitializeSynchronizationBarrier", (void*)&Detoured_InitializeSynchronizationBarrier);

			// DeviceIoControl and GetProcAddress exists as a separately implementation on wine
			#if DETOURED_INCLUDE_DEBUG
			void* ptr1 = GetProcAddress(moduleHandle, "DeviceIoControl");
			DetourAttachFunction(&ptr1, Detoured_DeviceIoControl, "DeviceIoControl");
			void* ptr2 = GetProcAddress(moduleHandle, "GetProcAddress");
			DetourAttachFunction(&ptr2, Detoured_GetProcAddress, "GetProcAddress");
			#endif
		}
	}

	if (HMODULE moduleHandle = GetModuleHandleW(L"ntdll.dll"))
	{
		DETOURED_FUNCTIONS_NTDLL
	}

	if (HMODULE moduleHandle = GetModuleHandleW(L"ucrtbase.dll"))
	{
		DETOURED_FUNCTIONS_UCRTBASE
		if (g_useMiMalloc)
		{
			DETOURED_FUNCTIONS_MEMORY
			if (!g_isRunningWine)
			{
				DETOURED_FUNCTIONS_MEMORY_NON_WINE
			}
		}
	}

	if (HMODULE moduleHandle = GetModuleHandleW(L"shlwapi.dll"))
	{
		DETOURED_FUNCTIONS_SHLWAPI
	}

	if (HMODULE moduleHandle = GetModuleHandleW(L"shell32.dll"))
	{
		DETOURED_FUNCTIONS_SHELL32
	}

	#if UBA_SUPPORT_MSPDBSRV
	if (HMODULE moduleHandle = GetModuleHandleW(L"rpcrt4.dll"))
	{
		DETOURED_FUNCTIONS_RPCRT4
	}
	#endif

#undef DETOURED_FUNCTION

	// Can't attach to these when running through debugger with some vs extensions (Microsoft child process debugging)
#if UBA_DEBUG
	if (IsDebuggerPresent())
	{
		True_CreateProcessW = nullptr;
		#if DETOURED_INCLUDE_DEBUG
		True_CreateProcessA = nullptr;
		True_CreateProcessAsUserW = nullptr;
		#endif
	}
#endif

	#define DETOURED_FUNCTION(Func) DetourAttachFunction((PVOID*)&True_##Func, Detoured_##Func, #Func);
	DETOURED_FUNCTIONS
	if (g_useMiMalloc)
	{
		DETOURED_FUNCTIONS_MEMORY
		if (!g_isRunningWine)
		{
			DETOURED_FUNCTIONS_MEMORY_NON_WINE
		}
	}
	#undef DETOURED_FUNCTION


	DetourTransactionCommit();

	#if UBA_SUPPORT_MSPDBSRV
	True2_NdrClientCall2 = True_NdrClientCall2;
	#endif

	return 0;
}

void OnModuleLoaded(HMODULE moduleHandle, const StringView& name)
{
	// SymLoadModuleExW do something bad that cause remote wine to fail everything after this call.. TODO: Revisit
	if (g_isRunningWine && !True_SymLoadModuleExW && name.Contains(L"dbghelp.dll"))
	{
		True_SymLoadModuleExW = (SymLoadModuleExWFunc*)(void*)GetProcAddress(moduleHandle, "SymLoadModuleExW");
		UBA_ASSERT(True_SymLoadModuleExW);
		DetourTransactionBegin();
		DetourAttachFunction((PVOID*)&True_SymLoadModuleExW, Detoured_SymLoadModuleExW, "SymLoadModuleExW");
		DetourTransactionCommit();
	}

	// ImageGetDigestStream is buggy in wine so we have to detour it for ShaderCompileWorker
	if (g_isRunningWine && !True_ImageGetDigestStream && name.Contains(L"imagehlp.dll"))
	{
		True_ImageGetDigestStream = (ImageGetDigestStreamFunc*)(void*)GetProcAddress(moduleHandle, "ImageGetDigestStream");
		UBA_ASSERT(True_ImageGetDigestStream);
		DetourTransactionBegin();
		DetourAttachFunction((PVOID*)&True_ImageGetDigestStream, Detoured_ImageGetDigestStream, "ImageGetDigestStream");
		DetourTransactionCommit();
	}

	// SHGetKnownFolderPath is used by Metal.exe and must always execute on host
	if (!True_SHGetKnownFolderPath && name.Contains(L"shell32.dll"))
	{
		DetourTransactionBegin();
		#define DETOURED_FUNCTION(Func) True_##Func = (decltype(True_##Func))(void*)GetProcAddress(moduleHandle, #Func);
		DETOURED_FUNCTIONS_SHELL32
		#undef DETOURED_FUNCTION
		#define DETOURED_FUNCTION(Func) DetourAttachFunction((PVOID*)&True_##Func, Detoured_##Func, #Func);
		DETOURED_FUNCTIONS_SHELL32
		#undef DETOURED_FUNCTION
		DetourTransactionCommit();
	}
}

int DetourDetachFunctions()
{
	if (g_directoryTable->m_memory)
		True_UnmapViewOfFile(g_directoryTable->m_memory);

	if (g_mappedFileTable.m_mem)
		True_UnmapViewOfFile(g_mappedFileTable.m_mem);

	//assert(g_wantsOnCloseLookup.empty());
	//UBA_ASSERT(g_mappedFileTable.m_memLookup.empty());

	CloseCaches();

	#define DETOURED_FUNCTION(Func) DetourDetachFunction((PVOID*)&True_##Func, Detoured_##Func, #Func);
	if (g_useMiMalloc)
	{
		DETOURED_FUNCTIONS_MEMORY
		if (!g_isRunningWine)
		{
			DETOURED_FUNCTIONS_MEMORY_NON_WINE
		}
	}
	DETOURED_FUNCTIONS
	#undef DETOURED_FUNCTION
	return 0;
}

extern bool g_reportAllExceptions;

void PreInit(const DetoursPayload& payload)
{
	#if UBA_USE_MIMALLOC
	//mi_option_enable(mi_option_large_os_pages);
	#endif

	InitSharedVariables();

	g_reportAllExceptions = payload.reportAllExceptions;

	g_rulesIndex = payload.rulesIndex;
	g_rules = GetApplicationRules()[payload.rulesIndex].rules;
	g_useMiMalloc = payload.useCustomAllocator;

	g_runningRemote = payload.runningRemote;
	g_isChild = payload.isChild;
	g_allowKeepFilesInMemory = payload.allowKeepFilesInMemory;
	g_allowOutputFiles = g_allowKeepFilesInMemory && payload.allowOutputFiles;
	g_allowDiscardVirtualMemory = payload.allowDiscardVirtualMemory;
	g_suppressLogging = payload.suppressLogging;
	g_isDetachedProcess = g_rules->AllowDetach();
	g_isRunningWine = payload.isRunningWine;
	g_uiLanguage = payload.uiLanguage;
	g_useLazyCow = payload.allowLazyCopyOnWrite && !g_isRunningWine;

	#if UBA_DEBUG_VALIDATE
	if (g_runningRemote)
		g_validateFileAccess = false;
	#endif

	if (!payload.logFile.IsEmpty())
	{
		g_logName.Append(payload.logFile);
		HANDLE debugFile = CreateFileW(payload.logFile.data, GENERIC_WRITE, FILE_SHARE_READ, NULL, CREATE_ALWAYS, FILE_ATTRIBUTE_NORMAL, NULL);
		#if UBA_DEBUG_LOG_ENABLED
		g_debugFile = (FileHandle)(u64)debugFile;
		#else
		if (debugFile != INVALID_HANDLE_VALUE)
		{
			const char str[] = "Run in debug to get this file populated";
			DWORD written = 0;
			WriteFile(debugFile, str, sizeof(str), &written, NULL);
			CloseHandle(debugFile);
		}
		#endif
	}

	if (g_useMiMalloc)
	{
		// If mimalloc exists in the binary, then we don't want to detour because it is likely an optimization that is made for that specific binary
		if (!IsFunctionImportedFromAnyDll(GetModuleHandleW(nullptr), "malloc"))
		{
			g_useMiMalloc = false;
			DEBUG_LOG(L"Disabling mimalloc detour: host exe imports no `malloc` (statically-linked allocator, e.g. rpmalloc)");
		}
	}

	if (g_runningRemote)
	{
		ULONG languageCount = 1;
		wchar_t languageBuffer[6];
		swprintf_s(languageBuffer, 6, L"%04x", g_uiLanguage);
		languageBuffer[5] = 0;
		if (!SetProcessPreferredUILanguages(MUI_LANGUAGE_ID, languageBuffer, &languageCount))
		{
			DEBUG_LOG(L"Failed to set locale");
		}
	}

	{
		wchar_t exeFullName[256];
		if (!GetModuleFileNameW(NULL, exeFullName, sizeof_array(exeFullName)))
			FatalError(1350, L"GetModuleFileNameW failed (%u)", GetLastError());
		wchar_t* lastSlash = wcsrchr(exeFullName, '\\');
		*lastSlash = 0;
		FixPath(g_exeDir, exeFullName);
		g_exeDir.Append('\\');
	}

	// Special cl.exe handling..  this is needed for compiles using pch files where this address _must_ be available.
	if (payload.rulesIndex == SpecialRulesIndex_ClExe)
	{
		g_clExeBaseReservedMemory = VirtualAlloc((void*)g_clExeBaseAddress, g_clExeBaseAddressSize, MEM_RESERVE, PAGE_READWRITE);
		DEBUG_LOG(L"Reserving %llu bytes at 0x%llx for cl.exe", g_clExeBaseAddressSize, g_clExeBaseAddress);
		if (!g_clExeBaseReservedMemory)
			FatalError(1349, L"Failed to reserve memory for cl.exe (%u)", GetLastError());
	}

	
	#if UBA_USE_MIMALLOC
	_mi_prim_random_buf(nullptr, 0); // Mimalloc loads bcrypt and can deadlock processes, let's call this to initialize static variable
	#endif

	if (const tchar* const* preloads = g_rules->LibrariesToPreload())
		for (auto it = preloads; *it; ++it)
		{
			DEBUG_LOG(L"Preloading %s", *it);
			if (!LoadLibraryExW(*it, NULL, LOAD_LIBRARY_SEARCH_SYSTEM32))
				FatalError(1351, L"Failed to preload %s (%u)", *it, GetLastError());
		}
}

struct PatchImportInfo
{
	const char* funcName;
	void* replacement;
};

static bool PatchImportByName(HMODULE module, const char* importedDll, PatchImportInfo* infos, u32 infoCount)
{
	auto base = (unsigned char*)module;
	auto dos  = (IMAGE_DOS_HEADER*)base;
	if (dos->e_magic != IMAGE_DOS_SIGNATURE)
		return false;

	auto nt = (IMAGE_NT_HEADERS*)(base + dos->e_lfanew);
	if (nt->Signature != IMAGE_NT_SIGNATURE)
		return false;

	auto& dir = nt->OptionalHeader.DataDirectory[IMAGE_DIRECTORY_ENTRY_IMPORT];
	if (!dir.VirtualAddress || !dir.Size)
		return false;

	auto imp = (IMAGE_IMPORT_DESCRIPTOR*)(base + dir.VirtualAddress);
	for (; imp->Name; ++imp)
	{
		const char* dllName = (const char*)(base + imp->Name);
		if (_stricmp(dllName, importedDll) == 0)
			break;
	}
	if (!imp->Name)
		return false;

	auto thunk     = (IMAGE_THUNK_DATA*)(base + imp->FirstThunk);
	auto origThunk = (IMAGE_THUNK_DATA*)(base + imp->OriginalFirstThunk);

	u32 patchCount = 0;
	for (; origThunk->u1.AddressOfData; ++origThunk, ++thunk)
	{
		if (IMAGE_SNAP_BY_ORDINAL(origThunk->u1.Ordinal))
			continue;

		auto ibn = (IMAGE_IMPORT_BY_NAME*)(base + origThunk->u1.AddressOfData);

		for (PatchImportInfo* i=infos, *e=i+infoCount; i!=e; ++i)
		{
			if (strcmp((const char*)ibn->Name, i->funcName) != 0)
				continue;

			DWORD oldProt;
			if (!VirtualProtect(&thunk->u1.Function, sizeof(void*), PAGE_READWRITE, &oldProt))
				return false;

			thunk->u1.Function = (ULONG_PTR)i->replacement;

			DWORD dummy;
			VirtualProtect(&thunk->u1.Function, sizeof(void*), oldProt, &dummy);
			FlushInstructionCache(GetCurrentProcess(), nullptr, 0);
			++patchCount;
			break;
		}
	}
	return patchCount == infoCount;
}

void Init(const DetoursPayload& payload, u64 startTime)
{
	AddExceptionHandler();
	InitMemory();
	
	if (g_isRunningWine && g_runningRemote)
	{
		// Wine (11.2) does not have RtlIsPackageSid and RtlIsCapabilitySid implemented and route them to a function that terminates the process
		HMODULE exe = GetModuleHandleW(nullptr);
		PatchImportInfo infos[] = { { "RtlIsPackageSid", (void*)Detoured_RtlIsPackageSid }, { "RtlIsCapabilitySid", (void*)Detoured_RtlIsCapabilitySid }, };
		PatchImportByName(exe, "ntdll.dll", infos, sizeof_array(infos));
	}

	DetourAttachFunctions(g_runningRemote);

	if (!g_isDetachedProcess)
	{
		// If GetStdHandle returns 0 it is likely that there is a parent process and that one has detached process set (which means now conhost is created)
		// .. solve this by detaching this process too
		HANDLE stdoutHandle = True_GetStdHandle(STD_OUTPUT_HANDLE);
		if (stdoutHandle == 0)
		{
			g_isDetachedProcess = true;
			DEBUG_LOG(L"Detached: true (stdout == 0)");
		}
		else
		{
			HANDLE stderrHandle = True_GetStdHandle(STD_ERROR_HANDLE);
			g_stdHandle[0] = True_GetFileType(stderrHandle) == FILE_TYPE_CHAR ? stderrHandle : 0;
			g_stdHandle[1] = True_GetFileType(stdoutHandle) == FILE_TYPE_CHAR ? stdoutHandle : 0;

			ULONG neededSize = 0;

			u8 buffer0[256];
			auto& objName0 = *reinterpret_cast<UNICODE_STRING*>(buffer0);
			if (!g_stdHandle[0])
				objName0.Buffer = const_cast<tchar*>(L"Unset");
			else if (NT_SUCCESS(True_NtQueryObject(g_stdHandle[0], ObjectNameInformation, buffer0, sizeof(buffer0), &neededSize)) && objName0.Buffer)
				g_conEnabled[0] = !Equals(objName0.Buffer, L"\\Device\\Null");
			else
				objName0.Buffer = const_cast<tchar*>(L"???");

			u8 buffer1[256];
			auto& objName1 = *reinterpret_cast<UNICODE_STRING*>(buffer1);
			if (!g_stdHandle[1])
				objName1.Buffer = const_cast<tchar*>(L"Unset");
			else if (NT_SUCCESS(True_NtQueryObject(g_stdHandle[1], ObjectNameInformation, buffer1, sizeof(buffer1), &neededSize)) && objName1.Buffer)
				g_conEnabled[1] = !Equals(objName1.Buffer, L"\\Device\\Null");
			else
				objName1.Buffer = const_cast<tchar*>(L"???");

			DEBUG_LOG(L"Detached: false (stderr: %s, stdout: %s)", objName0.Buffer, objName1.Buffer);
		}
	}
	else
	{
		DEBUG_LOG(L"Detached: true");
	}

	if (g_isDetachedProcess)
	{
		g_stdHandle[0] = makeDetouredHandle(new DetouredHandle(HandleType_StdErr)); // STD_ERR
		g_stdHandle[1] = makeDetouredHandle(new DetouredHandle(HandleType_StdOut)); // STD_OUT
		g_stdHandle[2] = makeDetouredHandle(new DetouredHandle(HandleType_StdIn)); // STD_IN
	}

	// These can actually not exist in certain programs
	g_systemRoot.count = True_GetEnvironmentVariableW(L"SystemRoot", g_systemRoot.data, g_systemRoot.capacity);
	g_systemRoot.MakeLower();
	wchar_t systemTemp[256];
	DWORD tempLen = True_GetEnvironmentVariableW(L"TEMP", systemTemp, sizeof_array(systemTemp));
	if (tempLen)
		FixPath(g_systemTemp, systemTemp);

	StringBuffer<512> applicationBuffer;
	StringBuffer<512> workingDirBuffer;

	bool trackInputs = false;
	SharedMemoryAllocatorHandle sharedMemoryAllocatorHandle;
	SharedMemoryHandle permanentFilesHandle;
	FileMappingHandle directoryTableHandle;
	DirTableSize directoryTableSize;
	u32 directoryTableCount;
	FileMappingHandle mappedFileTableHandle;
	u32 mappedFileTableSize;
	u32 mappedFileTableCount;
	FileMappingHandle directoryTableSnapshotHandle;
	u64 directoryTableSnapshotAddress;
	u32 directoryTableSnapshotSize;
	FileMappingHandle overlayFilesHandle;

	{
		RPC_MESSAGE(Init, init)
		BinaryReader reader = writer.Flush();

		g_processId = reader.ReadU32();
		g_isChild = reader.ReadBool();
		trackInputs = reader.ReadBool();

		reader.ReadString(applicationBuffer);
		reader.ReadString(workingDirBuffer);

		sharedMemoryAllocatorHandle = FileMappingHandle::FromU64(reader.ReadU64());
		permanentFilesHandle = SharedMemoryHandle::FromU64(reader.ReadU64());
		directoryTableHandle = FileMappingHandle::FromU64(reader.ReadU64());
		directoryTableSize = FromU64(reader.ReadU64());
		directoryTableCount = reader.ReadU32();
		mappedFileTableHandle = FileMappingHandle::FromU64(reader.ReadU64());
		mappedFileTableSize = reader.ReadU32();
		mappedFileTableCount = reader.ReadU32();

		directoryTableSnapshotHandle = FileMappingHandle::FromU64(reader.ReadU64());
		directoryTableSnapshotAddress = reader.ReadU64();
		directoryTableSnapshotSize = reader.ReadU32();

		overlayFilesHandle = FileMappingHandle::FromU64(reader.ReadU64());

		if (u16 vfsSize = reader.ReadU16())
		{
			BinaryReader vfsReader(reader.GetPositionData(), 0, vfsSize);
			PopulateVfs(vfsReader);
		}
		DEBUG_LOG_PIPE(L"Init", L"");
	}

	if (trackInputs)
		InitTrackInput();

	TrackInput(applicationBuffer);

	VirtualizePath(applicationBuffer);
	VirtualizePath(workingDirBuffer);
	VirtualizePath(g_exeDir);

	Shared_SetCurrentDirectory(workingDirBuffer.data);

	{
		FixPath(applicationBuffer.data, g_virtualWorkingDir.data, g_virtualWorkingDir.count, g_virtualApplication);

		if (const wchar_t* lastBackslash = g_virtualApplication.Last('\\'))
			g_virtualApplicationDir.Append(g_virtualApplication.data, (lastBackslash + 1 - g_virtualApplication.data));
		else
			FatalError(4444, L"What the heck: %s (%s)", g_virtualApplication.data, applicationBuffer.data);
	}

	#if UBA_DEBUG_LOG_ENABLED
	if (isLogging())
	{
		StringView sv(ToView(True_GetCommandLineW()));
		LogHeader(sv);
		LogVfsInfo();
	}
	#endif

	if (g_isRunningWine)
	{
		// TODO: This should probably check a variable sent through payload instead
		if (HMODULE m = GetModuleHandleW(L"UbaWine.dll"))
		{
			Logger logger;
			bool res = FileMapping_GetWineBackend(g_sharedMemoryFileMappingBackend, logger, m);
			UBA_ASSERT(res);(void)res;
		}
	}

	if (g_useLazyCow)
		g_lazyCowHandler = True_AddVectoredExceptionHandler(1, LazyCowExceptionHandler);

	g_sharedMemoryAllocator = g_sharedMemoryFileMappingBackend.DuplicateFromHost(sharedMemoryAllocatorHandle, "SharedMem");
	UBA_ASSERT(g_sharedMemoryAllocator.IsValid());

	TimerScope ts1(g_stats.fileTable);
	u8* mappedFileTableMem = FileMapping_MapFromHost(FileMapping_DuplicateFromHost(mappedFileTableHandle, "FileTable"), FileMappingTableMaxSize, 0, false, "FileTable");
	g_mappedFileTable.Init(mappedFileTableMem, mappedFileTableCount, mappedFileTableSize);
	ts1.Leave();

	TimerScope ts2(g_stats.dirTable);
	u8* directoryTableMem = FileMapping_MapFromHost(FileMapping_DuplicateFromHost(directoryTableHandle, "DirTable"), DirTableMaxSize, 0, false, "DirTable");
	u8* overlayTableMem = FileMapping_MapFromHost(FileMapping_DuplicateFromHost(overlayFilesHandle, "OverlayTable"), OverlayTableMaxSize, 0, false, "OverlayTable");
	if (directoryTableSnapshotHandle.IsValid())
	{
		MemoryMapType mapType = g_useLazyCow ? MemoryMapType_LazyCopyOnWrite : MemoryMapType_CopyOnWrite;
		u8* directoryTableSnapshotMem = FileMapping_MapFromHost2(FileMapping_DuplicateFromHost(directoryTableSnapshotHandle, "FileTable"), directoryTableSnapshotSize, 0, mapType, (void*)directoryTableSnapshotAddress, "FileTable");

		RPC_MESSAGE(SnapshotDuplicated, init)
		writer.Flush();
		RPC_MESSAGE_END;

		if (u64(directoryTableSnapshotMem) == directoryTableSnapshotAddress)
		{
			if (g_useLazyCow)
				RegisterLazyCowRange(directoryTableSnapshotMem, directoryTableSnapshotSize);
			g_directoryTable = (DirectoryTable*)directoryTableSnapshotMem;
			g_directoryTable->m_memory = directoryTableMem;
			g_directoryTable->ParseDirectoryTable({directoryTableSize.main, 0});
			DEBUG_LOG(L"Using dir table snapshot with size %u", g_directoryTable->m_memorySize);
		}
		else
		{
			DEBUG_LOG(L"Failed to map snapshot (Needed address 0x%llx but got 0x%llx)", directoryTableSnapshotAddress, u64(directoryTableSnapshotMem));
		}
	}
	
	if (!g_directoryTable)
	{
		g_directoryTable = new DirectoryTable(g_memoryBlock);
		g_directoryTable->Init(directoryTableMem, directoryTableCount, directoryTableSize.main);
	}
	g_directoryTable->InitOverlay(overlayTableMem, directoryTableSize.overlay);
	ts2.Leave();

	if (permanentFilesHandle.IsValid())
	{
		SharedMemoryView permanentFilesView(g_sharedMemoryFileMappingBackend);
		Rpc_GetSharedMemory(permanentFilesView, g_sharedMemoryAllocator, nullptr, permanentFilesHandle, 0, 0, TC("PermanentFiles"));
		g_permanentFilesHandle = permanentFilesHandle;
		g_permanentFilesMemory = permanentFilesView.DetachMemory();
	}

	if (g_isChild)
		Rpc_GetWrittenFiles();

	g_stats.attach.time += GetTime() - startTime;
	g_stats.attach.count = 1;

	g_filesCouldBeCompressed = payload.readIntermediateFilesCompressed && g_rules->CanDependOnCompressedFiles();
}

void Deinit(u64 startTime)
{
	if (g_isRunningWine) // mt.exe etc fails if detaching is not done during shutdown
	{
		DetourTransactionBegin();
		DetourDetachFunctions();
		::DetourTransactionCommit(); // Ignore errors
	}

	#if defined(UBA_PROFILE_DETOURED_CALLS)
	#define DETOURED_FUNCTION(name) if (timer##name.count != 0) { char sb[1024]; sprintf_s(sb, sizeof(sb), "%s: %u %llu\n", #name, timer##name.count.load(), TimeToMs(timer##name.time.load())); WriteDebug(sb); }
	DETOURED_FUNCTIONS
	DETOURED_FUNCTIONS_MEMORY
	DETOURED_FUNCTIONS_MEMORY_NON_WINE
	#undef DETOURED_FUNCTION
	#endif

	DWORD exitCode = STILL_ACTIVE;
	if (!True_GetExitCodeProcess(GetCurrentProcess(), &exitCode))
		exitCode = STILL_ACTIVE;

	if (!g_exitMessageSent)
		SendExitMessage(exitCode, startTime); // This should never happen? ExitProcess is always called after main function
}

void PostDeinit()
{
	DEBUG_LOG(L"Finished");
	#if UBA_DEBUG_LOG_ENABLED
	if (isLogging())
	{
		FlushDebugLog();
		HANDLE debugFile = (HANDLE)g_debugFile;
		g_debugFile = InvalidFileHandle;
		CloseHandle(debugFile);
	}
	#endif
}

} // namespace uba

extern "C"
{
	using namespace uba;

	UBA_DETOURED_API u32 UbaSendCustomMessage(const void* send, u32 sendSize, void* recv, u32 recvCapacity)
	{
		RPC_MESSAGE(Custom, log)
		writer.WriteU32(sendSize);
		writer.WriteBytes(send, sendSize);
		BinaryReader reader = writer.Flush();
		u32 recvSize = reader.ReadU32();
		UBA_ASSERT(recvSize < recvCapacity);
		reader.ReadBytes(recv, recvSize);
		return recvSize;
	}

	UBA_DETOURED_API bool UbaFlushWrittenFiles()
	{
		RPC_MESSAGE(FlushWrittenFiles, log)
		BinaryReader reader = writer.Flush();
		return reader.ReadBool();
	}

	UBA_DETOURED_API bool UbaUpdateEnvironment(const wchar_t* reason, bool resetStats)
	{
		{
			RPC_MESSAGE(UpdateEnvironment, log)
			writer.WriteString(reason ? reason : L"");
			writer.WriteBool(resetStats);
			BinaryReader reader = writer.Flush();
			if (!reader.ReadBool())
				return false;
		}
		Rpc_UpdateTables();
		return true;
	}

	UBA_DETOURED_API bool UbaRunningRemote()
	{
		return g_runningRemote;
	}

	UBA_DETOURED_API bool UbaRequestNextProcess2(u32 prevExitCode, wchar_t* outArguments, u32 outArgumentsCapacity, u32 timeOutMs, bool* outShouldExit)
	{
		#if UBA_DEBUG_LOG_ENABLED
		FlushDebugLog();
		#endif

		*outArguments = 0;
		bool newProcess;
		DirTableSize directoryTableSize;
		{
			RPC_MESSAGE(GetNextProcess, log)
			writer.WriteU32(prevExitCode);
			writer.WriteU32(timeOutMs);
			g_stats.Write(writer);
			g_kernelStats.Write(writer);

			BinaryReader reader = writer.Flush();
			newProcess = reader.ReadBool();
			*outShouldExit = reader.ReadBool();
			if (newProcess)
			{
				reader.ReadString(outArguments, outArgumentsCapacity);
				reader.SkipString(); // workingDir
				reader.SkipString(); // description
				reader.ReadString(g_logName.Clear());
				directoryTableSize = FromU64(reader.ReadU64());
			}
		}

		g_stats = {};
		g_kernelStats = {};

		if (newProcess)
		{
			#if UBA_DEBUG_LOG_ENABLED
			SuppressCreateFileDetourScope scope;
			{
				// Serialize the handle swap against concurrent loggers. isLogging()/WriteDebug read g_debugFile
				// under g_debugLogMutex; without this a logging thread sees InvalidFileHandle (dropped line) or
				// writes through the handle being closed. Scope closes before LogHeader, which self-locks.
				SCOPED_FUTEX(g_debugLogMutex, l);
				HANDLE debugFile = (HANDLE)g_debugFile;
				g_debugFile = InvalidFileHandle;
				CloseHandle(debugFile);
				debugFile = CreateFileW(g_logName.data, GENERIC_WRITE, FILE_SHARE_READ, NULL, CREATE_ALWAYS, FILE_ATTRIBUTE_NORMAL, NULL);
				g_debugFile = (FileHandle)(u64)debugFile;
			}
			LogHeader(ToView(outArguments));
			#endif

			auto& dirTable = *g_directoryTable;
			const u8* overlayMemory = dirTable.m_overlayMemory;
			dirTable.DeinitOverlay();
			dirTable.m_overlayMemory = overlayMemory;

			if (u32 overlaySize = directoryTableSize.overlay)
			{
				directoryTableSize.overlay = 0; // Want to just parse normal directory first
				dirTable.ParseDirectoryTableNoLock(directoryTableSize);
				dirTable.InitOverlay(overlayMemory, overlaySize);
			}
		}

		Rpc_UpdateTables();
		return newProcess;
	}

	UBA_DETOURED_API bool UbaRequestNextProcess(u32 prevExitCode, wchar_t* outArguments, u32 outArgumentsCapacity)
	{
		bool shouldExit;
		return UbaRequestNextProcess2(prevExitCode, outArguments, outArgumentsCapacity, 0, &shouldExit);
	}

}
