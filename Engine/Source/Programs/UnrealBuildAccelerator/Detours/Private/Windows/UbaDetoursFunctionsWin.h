// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaPlatform.h"
#include "UbaWinTypes.h"
#include <tlhelp32.h>
#include <psapi.h>
#include <io.h>
#include <aclapi.h>
#include <mbstring.h>
#include <Shlwapi.h>
#include <wchar.h>
#include <stdio.h>

#if UBA_DEBUG
#define DETOURED_INCLUDE_DEBUG 1
#else
#define DETOURED_INCLUDE_DEBUG 0
#endif

#if DETOURED_INCLUDE_DEBUG
#include <direct.h>
#endif


#define DETOURED_FUNCTIONS \
	DETOURED_FUNCTIONS_KERNELBASE \
	DETOURED_FUNCTIONS_KERNEL32 \
	DETOURED_FUNCTIONS_NTDLL \
	DETOURED_FUNCTIONS_SHLWAPI \
	DETOURED_FUNCTIONS_SHELL32 \
	DETOURED_FUNCTIONS_UCRTBASE \
	DETOURED_FUNCTIONS_RPCRT4 \
	DETOURED_FUNCTIONS_COMBASE \

#if defined(_M_ARM64)
#define DETOURED_FUNCTION_X64(x)
#else
#define DETOURED_FUNCTION_X64(x) DETOURED_FUNCTION(x)
#endif

#define DETOURED_FUNCTIONS_KERNELBASE \
	DETOURED_FUNCTION(GetCommandLineW) \
	DETOURED_FUNCTION(GetSystemInfo) \
	DETOURED_FUNCTION(GetCurrentDirectoryW) \
	DETOURED_FUNCTION(GetCurrentDirectoryA) \
	DETOURED_FUNCTION(SetCurrentDirectoryW) \
	DETOURED_FUNCTION(DuplicateHandle) \
	DETOURED_FUNCTION(CreateDirectoryW) \
	DETOURED_FUNCTION(RemoveDirectoryW) \
	DETOURED_FUNCTION(LockFile) \
	DETOURED_FUNCTION(LockFileEx) \
	DETOURED_FUNCTION(UnlockFile) \
	DETOURED_FUNCTION(UnlockFileEx) \
	DETOURED_FUNCTION(FlushFileBuffers) \
	DETOURED_FUNCTION(SetFileTime) \
	DETOURED_FUNCTION(GetFileType) \
	DETOURED_FUNCTION(GetLongPathNameW) \
	DETOURED_FUNCTION(GetFullPathNameW) \
	DETOURED_FUNCTION(GetFullPathNameA) \
	DETOURED_FUNCTION(GetVolumePathNameW) \
	DETOURED_FUNCTION(GetModuleFileNameW) \
	DETOURED_FUNCTION(GetModuleFileNameExW) \
	DETOURED_FUNCTION(GetModuleFileNameA) \
	DETOURED_FUNCTION(GetModuleFileNameExA) \
	DETOURED_FUNCTION(GetModuleHandleExW) \
	DETOURED_FUNCTION(GetFileAttributesW) \
	DETOURED_FUNCTION(SetFileAttributesW) \
	DETOURED_FUNCTION(GetFileAttributesExW) \
	DETOURED_FUNCTION(CopyFileW) \
	DETOURED_FUNCTION(CopyFileExW) \
	DETOURED_FUNCTION(CreateHardLinkW) \
	DETOURED_FUNCTION(DeleteFileW) \
	DETOURED_FUNCTION(MoveFileWithProgressW) \
	DETOURED_FUNCTION(MoveFileExW) \
	DETOURED_FUNCTION(FindFirstFileW) \
	DETOURED_FUNCTION(FindFirstFileExW) \
	DETOURED_FUNCTION(FindNextFileW) \
	DETOURED_FUNCTION(FindFirstFileA) \
	DETOURED_FUNCTION(FindNextFileA) \
	DETOURED_FUNCTION(FindClose) \
	DETOURED_FUNCTION(SetFileInformationByHandle) \
	DETOURED_FUNCTION(CreateFileMappingW) \
	DETOURED_FUNCTION(OpenFileMappingW) \
	DETOURED_FUNCTION(MapViewOfFile) \
	DETOURED_FUNCTION(MapViewOfFile3) \
	DETOURED_FUNCTION(MapViewOfFileEx) \
	DETOURED_FUNCTION(FlushViewOfFile) \
	DETOURED_FUNCTION(UnmapViewOfFile) \
	DETOURED_FUNCTION(UnmapViewOfFileEx) \
	DETOURED_FUNCTION(GetFinalPathNameByHandleW) \
	DETOURED_FUNCTION(CreateProcessW) \
	DETOURED_FUNCTION(CreateProcessA) \
	DETOURED_FUNCTION(SearchPathW) \
	DETOURED_FUNCTION(LoadLibraryExW) \
	DETOURED_FUNCTION(GetStdHandle) \
	DETOURED_FUNCTION(SetStdHandle) \
	DETOURED_FUNCTION(GetConsoleMode) \
	DETOURED_FUNCTION(SetConsoleMode) \
	DETOURED_FUNCTION(GetDriveTypeW) \
	DETOURED_FUNCTION(GetDiskFreeSpaceExW) \
	DETOURED_FUNCTION(GetFileInformationByHandleEx) \
	DETOURED_FUNCTION(GetFileInformationByHandle) \
	DETOURED_FUNCTION_UNKNOWN(GetFileInformationByName) \
	DETOURED_FUNCTION(GetVolumeInformationByHandleW) \
	DETOURED_FUNCTION(GetVolumeInformationW) \
	DETOURED_FUNCTION(GetUserDefaultUILanguage) \
	DETOURED_FUNCTION(GetThreadPreferredUILanguages) \
	DETOURED_FUNCTION_X64(GetConsoleTitleW) \
	DETOURED_FUNCTION(WaitForSingleObject) \
	DETOURED_FUNCTION(WaitForSingleObjectEx) \
	DETOURED_FUNCTION(WaitForMultipleObjects) \
	DETOURED_FUNCTION(WaitForMultipleObjectsEx) \
	DETOURED_FUNCTION(WriteConsoleA) \
	DETOURED_FUNCTION(WriteConsoleW) \
	DETOURED_FUNCTION(ReadConsoleW) \
	DETOURED_FUNCTION(VirtualAlloc) \
	DETOURED_FUNCTION(AddVectoredExceptionHandler) \
	DETOURED_FUNCTION(PathSearchAndQualifyW) \
	DETOURED_FUNCTIONS_KERNELBASE_DEBUG \

#define DETOURED_FUNCTIONS_KERNEL32 \
	DETOURED_FUNCTION(RegisterWaitForSingleObject) \
	DETOURED_FUNCTION(CreateFileMappingA) \
	DETOURED_FUNCTION(GetExitCodeProcess) \
	DETOURED_FUNCTION(CreateTimerQueueTimer) \
	DETOURED_FUNCTION(DeleteTimerQueueTimer) \
	DETOURED_FUNCTION(CreateToolhelp32Snapshot) \
	DETOURED_FUNCTIONS_KERNEL32_DEBUG \

#define DETOURED_FUNCTIONS_NTDLL \
	DETOURED_FUNCTION(RtlDosPathNameToNtPathName_U_WithStatus) \
	DETOURED_FUNCTION(RtlDosPathNameToRelativeNtPathName_U_WithStatus) \
	DETOURED_FUNCTION(RtlGetFullPathName_U) \
	DETOURED_FUNCTION(RtlGetFullPathName_UEx) \
	DETOURED_FUNCTION(RtlDosPathNameToRelativeNtPathName_U) \
	DETOURED_FUNCTION(NtClose) \
	DETOURED_FUNCTION(NtCreateFile) \
	DETOURED_FUNCTION(NtOpenFile) \
	DETOURED_FUNCTION(NtReadFile) \
	DETOURED_FUNCTION(NtWriteFile) \
	DETOURED_FUNCTION(NtFsControlFile) \
	DETOURED_FUNCTION(NtDeviceIoControlFile) \
	DETOURED_FUNCTION(NtCopyFileChunk) \
	DETOURED_FUNCTION(NtQueryVolumeInformationFile) \
	DETOURED_FUNCTION(NtQueryInformationFile) \
	DETOURED_FUNCTION(NtQueryDirectoryFile) \
	DETOURED_FUNCTION(NtQueryFullAttributesFile) \
	DETOURED_FUNCTION(NtQueryObject) \
	DETOURED_FUNCTION(NtQueryInformationProcess) \
	DETOURED_FUNCTION(NtQuerySecurityObject) \
	DETOURED_FUNCTION(NtSetInformationFile) \
	DETOURED_FUNCTION(NtSetInformationObject) \
	DETOURED_FUNCTION(NtRemoveIoCompletionEx) \
	DETOURED_FUNCTION(NtTerminateProcess) \
	DETOURED_FUNCTION(NtCreateJobObject) \
	DETOURED_FUNCTION(NtSetInformationJobObject) \
	DETOURED_FUNCTION(NtAssignProcessToJobObject) \
	DETOURED_FUNCTION(NtTerminateJobObject) \
	DETOURED_FUNCTION(NtCreateSection) \
	DETOURED_FUNCTION(RtlExitUserProcess) \
	DETOURED_FUNCTION(RtlSizeHeap) \
	DETOURED_FUNCTION(RtlFreeHeap) \
	DETOURED_FUNCTION(RtlAnsiStringToUnicodeString) \
	DETOURED_FUNCTION(RtlUnicodeStringToAnsiString) \
	DETOURED_FUNCTION(RtlFreeAnsiString) \
	DETOURED_FUNCTIONS_NTDLL_DEBUG \

#define DETOURED_FUNCTIONS_SHLWAPI \
	DETOURED_FUNCTIONS_SHLWAPI_DEBUG

#define DETOURED_FUNCTIONS_SHELL32 \
	DETOURED_FUNCTION(SHGetFolderPathAndSubDirW) \
	DETOURED_FUNCTION(SHGetKnownFolderPath) \
	DETOURED_FUNCTIONS_SHELL32_DEBUG

#if !defined(__clang__)
#define DETOURED_WSPLITPATH DETOURED_FUNCTION(_wsplitpath_s)
#else
#define DETOURED_WSPLITPATH
#endif

#define DETOURED_FUNCTIONS_UCRTBASE \
	DETOURED_FUNCTION(_wgetcwd) \
	DETOURED_FUNCTION(_wfullpath) \
	DETOURED_FUNCTION(_fullpath) \
	DETOURED_FUNCTION(_get_wpgmptr) \
	DETOURED_FUNCTION(__p__wpgmptr) \
	DETOURED_FUNCTION(_time64) \
	DETOURED_FUNCTION(_waccess_s) \
	DETOURED_FUNCTION(_wspawnl) \
	DETOURED_FUNCTION(_get_osfhandle) \
	DETOURED_FUNCTION(_write) \
	DETOURED_FUNCTION_X64(_isatty) \
	DETOURED_WSPLITPATH \
	DETOURED_FUNCTIONS_UCRTBASE_DEBUG \

#if UBA_SUPPORT_MSPDBSRV
#define DETOURED_FUNCTIONS_RPCRT4 \
	DETOURED_FUNCTION(RpcStringBindingComposeW) \
	DETOURED_FUNCTION(RpcBindingSetAuthInfoExW) \
	DETOURED_FUNCTION(RpcBindingFromStringBindingW) \
	DETOURED_FUNCTION(NdrClientCall2) \

#else
#define DETOURED_FUNCTIONS_RPCRT4
#endif

#if UBA_USE_MIMALLOC
#define DETOURED_FUNCTIONS_MEMORY \
	DETOURED_FUNCTION(malloc) \
	DETOURED_FUNCTION(calloc) \
	DETOURED_FUNCTION(_recalloc) \
	DETOURED_FUNCTION(realloc) \
	DETOURED_FUNCTION(_expand) \
	DETOURED_FUNCTION(_msize) \
	DETOURED_FUNCTION(free) \
	DETOURED_FUNCTION(_strdup) \
	DETOURED_FUNCTION(_wcsdup) \
	DETOURED_FUNCTION(_mbsdup) \
	DETOURED_FUNCTION(_aligned_malloc) \
	DETOURED_FUNCTION(_aligned_realloc) \
	DETOURED_FUNCTION(_aligned_recalloc) \
	DETOURED_FUNCTION(_aligned_free) \
	DETOURED_FUNCTION(_aligned_offset_malloc) \
	DETOURED_FUNCTION(_aligned_offset_realloc) \
	DETOURED_FUNCTION(_aligned_offset_recalloc) \
	DETOURED_FUNCTION(_dupenv_s) \
	DETOURED_FUNCTION(_wdupenv_s) \
	DETOURED_FUNCTION(_free_base) \
	DETOURED_FUNCTIONS_MEMORY_DEBUG \

// All these are calling above functions on wine
#define DETOURED_FUNCTIONS_MEMORY_NON_WINE \
	DETOURED_FUNCTION(_malloc_base) \
	DETOURED_FUNCTION(_calloc_base) \
	DETOURED_FUNCTION(_realloc_base) \
	DETOURED_FUNCTION(_expand_base) \
	DETOURED_FUNCTION(_msize_base) \
	DETOURED_FUNCTION(_recalloc_base) \

#else
#define DETOURED_FUNCTIONS_MEMORY
#define DETOURED_FUNCTIONS_MEMORY_NON_WINE
#endif

#define DETOURED_FUNCTIONS_COMBASE \
	DETOURED_FUNCTION(CoCreateInstance)

#if DETOURED_INCLUDE_DEBUG

#define DETOURED_FUNCTIONS_KERNELBASE_DEBUG \
	DETOURED_FUNCTION(ExitProcess) \
	DETOURED_FUNCTION(GetCommandLineA) \
	DETOURED_FUNCTION(CommandLineToArgvW) \
	DETOURED_FUNCTION(GetProcAddress) \
	DETOURED_FUNCTION(FreeLibrary) \
	DETOURED_FUNCTION(RegOpenKeyW) \
	DETOURED_FUNCTION(RegOpenKeyExW) \
	DETOURED_FUNCTION(RegCreateKeyExW) \
	DETOURED_FUNCTION_X64(SetLastError) \
	DETOURED_FUNCTION_X64(GetLastError) \
	DETOURED_FUNCTION(RegOpenKeyExA) \
	DETOURED_FUNCTION(RegCloseKey) \
	DETOURED_FUNCTION(IsValidCodePage) \
	DETOURED_FUNCTION(GetACP) \
	DETOURED_FUNCTION(GetConsoleWindow) \
	DETOURED_FUNCTION(SetConsoleCursorPosition) \
	DETOURED_FUNCTION(GetConsoleScreenBufferInfo) \
	DETOURED_FUNCTION(ScrollConsoleScreenBufferW) \
	DETOURED_FUNCTION(FillConsoleOutputAttribute) \
	DETOURED_FUNCTION(FillConsoleOutputCharacterW) \
	DETOURED_FUNCTION(FlushConsoleInputBuffer) \
	DETOURED_FUNCTION(SetConsoleTextAttribute) \
	DETOURED_FUNCTION(SetConsoleTitleW) \
	DETOURED_FUNCTION(CreateConsoleScreenBuffer) \
	DETOURED_FUNCTION(OpenProcess) \
	DETOURED_FUNCTION(RemoveVectoredExceptionHandler) \
	DETOURED_FUNCTION(RaiseFailFastException) \
	DETOURED_FUNCTION(SetErrorMode) \
	DETOURED_FUNCTION(CreateProcessAsUserW) \
	DETOURED_FUNCTION(SetConsoleCtrlHandler) \
	DETOURED_FUNCTION(GetConsoleOutputCP) \
	DETOURED_FUNCTION(ReadConsoleInputA) \
	DETOURED_FUNCTION(GetLocaleInfoEx) \
	DETOURED_FUNCTION(GetUserDefaultLocaleName) \
	DETOURED_FUNCTION(DeviceIoControl) \
	DETOURED_FUNCTION(GetDiskFreeSpaceExA) \
	DETOURED_FUNCTION(GetLongPathNameA) \
	DETOURED_FUNCTION(GetVolumePathNameA) \
	DETOURED_FUNCTION(CreateFileW) \
	DETOURED_FUNCTION(CreateFileA) \
	DETOURED_FUNCTION(GetFileSize) \
	DETOURED_FUNCTION(GetFileSizeEx) \
	DETOURED_FUNCTION(GetFileTime) \
	DETOURED_FUNCTION(SetFilePointer) \
	DETOURED_FUNCTION(SetFilePointerEx) \
	DETOURED_FUNCTION(SetEndOfFile) \
	DETOURED_FUNCTION(GetFileAttributesA) \
	DETOURED_FUNCTION(GetFileAttributesExA) \
	DETOURED_FUNCTION_X64(LoadLibraryW) \
	DETOURED_FUNCTION(GetModuleBaseNameA) \
	DETOURED_FUNCTION(GetModuleBaseNameW) \
	DETOURED_FUNCTION(SetUnhandledExceptionFilter) \
	DETOURED_FUNCTION(FlushInstructionCache) \
	DETOURED_FUNCTION(CreateFile2) \
	DETOURED_FUNCTION(CreateFileTransactedW) \
	DETOURED_FUNCTION(OpenFile) \
	DETOURED_FUNCTION(ReOpenFile) \
	DETOURED_FUNCTION(ReadFile) \
	DETOURED_FUNCTION(ReadFileEx) \
	DETOURED_FUNCTION(ReadFileScatter) \
	DETOURED_FUNCTION(WriteFile) \
	DETOURED_FUNCTION(WriteFileEx) \
	DETOURED_FUNCTION(SetFileValidData) \
	DETOURED_FUNCTION(ReplaceFileW) \
	DETOURED_FUNCTION(CreateHardLinkA) \
	DETOURED_FUNCTION(DeleteFileA) \
	DETOURED_FUNCTION(GetShortPathNameW) \
	DETOURED_FUNCTION(NeedCurrentDirectoryForExePathW) \
	DETOURED_FUNCTION(ReadDirectoryChangesW) \
	DETOURED_FUNCTION(SetCurrentDirectoryA) \
	DETOURED_FUNCTION(CreateSymbolicLinkW) \
	DETOURED_FUNCTION(CreateSymbolicLinkA) \
	DETOURED_FUNCTION(SetEnvironmentVariableW) \
	DETOURED_FUNCTION(GetEnvironmentVariableW) \
	DETOURED_FUNCTION(GetEnvironmentVariableA) \
	DETOURED_FUNCTION(GetEnvironmentStringsW) \
	DETOURED_FUNCTION(GetTempPathW) \
	DETOURED_FUNCTION(GetTempPath2W) \
	DETOURED_FUNCTION(ExpandEnvironmentStringsW) \
	DETOURED_FUNCTION(FindFirstFileNameW) \
	DETOURED_FUNCTION(GetTempFileNameW) \
	DETOURED_FUNCTION(GetTempPathA) \
	DETOURED_FUNCTION(GetTempFileNameA) \
	DETOURED_FUNCTION(CreateDirectoryExW) \
	DETOURED_FUNCTION(RemoveDirectoryA) \
	DETOURED_FUNCTION(CreateEventW) \
	DETOURED_FUNCTION(CreateEventExW) \
	DETOURED_FUNCTION(CreateMutexExW) \
	DETOURED_FUNCTION(CreateWaitableTimerExW) \
	DETOURED_FUNCTION(CreateIoCompletionPort) \
	DETOURED_FUNCTION(CreateRemoteThread) \
	DETOURED_FUNCTION(CancelIo) \
	DETOURED_FUNCTION(CancelIoEx) \
	DETOURED_FUNCTION(SetFileCompletionNotificationModes) \
	DETOURED_FUNCTION(PostQueuedCompletionStatus) \
	DETOURED_FUNCTION(GetQueuedCompletionStatusEx) \
	DETOURED_FUNCTION(CreatePipe) \
	DETOURED_FUNCTION(SetHandleInformation) \
	DETOURED_FUNCTION(CreateNamedPipeW) \
	DETOURED_FUNCTION(CallNamedPipeW ) \
	DETOURED_FUNCTION(PeekNamedPipe) \
	DETOURED_FUNCTION(GetKernelObjectSecurity) \
	DETOURED_FUNCTION(ImpersonateNamedPipeClient) \
	DETOURED_FUNCTION(TransactNamedPipe) \
	DETOURED_FUNCTION(SetNamedPipeHandleState) \
	DETOURED_FUNCTION(GetNamedPipeInfo) \
	DETOURED_FUNCTION(GetNamedPipeHandleStateW) \
	DETOURED_FUNCTION(GetNamedPipeServerProcessId) \
	DETOURED_FUNCTION(GetNamedPipeServerSessionId) \
	DETOURED_FUNCTION(OpenFileById) \
	DETOURED_FUNCTION(OpenFileMappingA) \
	DETOURED_FUNCTION(GetMappedFileNameW) \
	DETOURED_FUNCTION(IsProcessorFeaturePresent) \
	DETOURED_FUNCTION(VirtualAlloc2) \
	DETOURED_FUNCTION(VirtualAlloc2FromApp) \
	DETOURED_FUNCTION(UnmapViewOfFile2) \
	DETOURED_FUNCTION(GetModuleHandleW) \
	DETOURED_FUNCTION(GetStartupInfoW) \
	DETOURED_FUNCTION(GetSystemTimeAsFileTime) \
	DETOURED_FUNCTION(GetLargePageMinimum) \
	DETOURED_FUNCTION(UrlCreateFromPathW) \
	DETOURED_FUNCTION(PathCreateFromUrlW) \
	//DETOURED_FUNCTION(VirtualFree) \
	//DETOURED_FUNCTION(BaseThreadInitThunk) \
	//DETOURED_FUNCTION(VirtualAllocEx) \
	//DETOURED_FUNCTION(CryptCreateHash) \
	//DETOURED_FUNCTION(CryptHashData) \
	//DETOURED_FUNCTION(CreateFileMapping2) \
	//DETOURED_FUNCTION(CreateFileMappingNumaW)

#define DETOURED_FUNCTIONS_KERNEL32_DEBUG \
	DETOURED_FUNCTION(SetDllDirectoryW) \
	DETOURED_FUNCTION(GetDllDirectoryW) \
	DETOURED_FUNCTION(TerminateJobObject) \
	DETOURED_FUNCTION(GetComputerNameA) \
	DETOURED_FUNCTION(GetShortPathNameA) \

#define DETOURED_FUNCTIONS_NTDLL_DEBUG \
	DETOURED_FUNCTION(RtlGetVersion) \
	DETOURED_FUNCTION(RtlAllocateHeap) \
	DETOURED_FUNCTION(RtlReAllocateHeap) \
	DETOURED_FUNCTION(RtlValidateHeap) \
	DETOURED_FUNCTION(RtlCreateUserThread) \
	DETOURED_FUNCTION(NtCreateThreadEx) \
	DETOURED_FUNCTION(NtOpenJobObject) \
	DETOURED_FUNCTION(NtAllocateVirtualMemoryEx) \
	DETOURED_FUNCTION(NtQueryInformationJobObject) \
	DETOURED_FUNCTION(NtCreateIoCompletion) \
	DETOURED_FUNCTION(NtRemoveIoCompletion) \
	DETOURED_FUNCTION(NtFlushBuffersFileEx) \
	DETOURED_FUNCTION(NtAlpcCreatePort) \
	DETOURED_FUNCTION(NtAlpcConnectPort) \
	DETOURED_FUNCTION(NtAlpcCreatePortSection) \
	DETOURED_FUNCTION(NtAlpcSendWaitReceivePort) \
	DETOURED_FUNCTION(NtAlpcDisconnectPort) \
	DETOURED_FUNCTION(ZwSetInformationFile) \
	DETOURED_FUNCTION(ZwQueryDirectoryFile) \
	//DETOURED_FUNCTION(ZwCreateFile) \
	//DETOURED_FUNCTION(ZwOpenFile) \
	//DETOURED_FUNCTION(RtlAllocateHeap)

#define DETOURED_FUNCTIONS_SHLWAPI_DEBUG \
	DETOURED_FUNCTION(PathFindFileNameW) \
	DETOURED_FUNCTION(PathIsRelativeW) \
	DETOURED_FUNCTION(PathIsDirectoryEmptyW) \
	DETOURED_FUNCTION(SHCreateStreamOnFileW) \
	DETOURED_FUNCTION(PathFileExistsW) \

#define DETOURED_FUNCTIONS_SHELL32_DEBUG \
	DETOURED_FUNCTION(SHGetKnownFolderIDList) \
	DETOURED_FUNCTION(SHGetFolderPathW) \

#define DETOURED_FUNCTIONS_UCRTBASE_DEBUG \
	DETOURED_FUNCTION(_wcsnicoll_l) \
	DETOURED_FUNCTION(_wgetenv) \
	DETOURED_FUNCTION(_wgetenv_s) \
	DETOURED_FUNCTION(getenv) \
	DETOURED_FUNCTION(getenv_s) \
	DETOURED_FUNCTION(_wmakepath_s) \
	DETOURED_FUNCTION(_getcwd) \
	DETOURED_FUNCTION(fputs) \
	//DETOURED_FUNCTION(_wsopen_s) \
	//DETOURED_FUNCTION(_fileno)

// Advapi32.dll
/*
DETOURED_FUNCTION(GetSecurityInfo) \
DETOURED_FUNCTION(DecryptFileW) \
DETOURED_FUNCTION(DecryptFileA) \
DETOURED_FUNCTION(EncryptFileW) \
DETOURED_FUNCTION(EncryptFileA) \
DETOURED_FUNCTION(OpenEncryptedFileRawW) \
DETOURED_FUNCTION(OpenEncryptedFileRawA) \
*/

#if UBA_USE_MIMALLOC
#define DETOURED_FUNCTIONS_MEMORY_DEBUG \
	DETOURED_FUNCTION(_aligned_msize) \
	//DETOURED_FUNCTION(_free_dbg)

#endif

#else
#define DETOURED_FUNCTIONS_KERNELBASE_DEBUG
#define DETOURED_FUNCTIONS_KERNEL32_DEBUG
#define DETOURED_FUNCTIONS_NTDLL_DEBUG
#define DETOURED_FUNCTIONS_SHLWAPI_DEBUG
#define DETOURED_FUNCTIONS_SHELL32_DEBUG
#define DETOURED_FUNCTIONS_UCRTBASE_DEBUG
#define DETOURED_FUNCTIONS_MEMORY_DEBUG
#endif

extern "C" {
	using PALPC_PORT_ATTRIBUTES = void*;
	using PALPC_MESSAGE_ATTRIBUTES = void*;
	using PPORT_MESSAGE = void*;
	NTSTATUS NTAPI NtQueryVolumeInformationFile(HANDLE FileHandle, PIO_STATUS_BLOCK IoStatusBlock, PVOID FsInformation, ULONG Length, FS_INFORMATION_CLASS FsInformationClass);
	NTSTATUS NTAPI NtQueryFullAttributesFile(POBJECT_ATTRIBUTES ObjectAttributes, PVOID Attributes);
	NTSTATUS NTAPI NtQueryInformationFile(HANDLE FileHandle, PIO_STATUS_BLOCK IoStatusBlock, PVOID FileInformation, ULONG Length, FILE_INFORMATION_CLASS FileInformationClass);
	NTSTATUS NTAPI NtQueryDirectoryFile(HANDLE FileHandle, HANDLE Event, PIO_APC_ROUTINE ApcRoutine, PVOID ApcContext, PIO_STATUS_BLOCK IoStatusBlock, PVOID FileInformation, ULONG Length, FILE_INFORMATION_CLASS FileInformationClass, BOOLEAN ReturnSingleEntry, PUNICODE_STRING FileName, BOOLEAN RestartScan);
	NTSTATUS NTAPI NtDeviceIoControlFile(HANDLE FileHandle, HANDLE Event, PIO_APC_ROUTINE ApcRoutine, PVOID ApcContext, PIO_STATUS_BLOCK IoStatusBlock, ULONG IoControlCode, PVOID InputBuffer, ULONG InputBufferLength, PVOID OutputBuffer, ULONG OutputBufferLength);
	NTSTATUS NTAPI NtFsControlFile(HANDLE FileHandle, HANDLE Event, PIO_APC_ROUTINE ApcRoutine, PVOID ApcContext, PIO_STATUS_BLOCK IoStatusBlock, ULONG FsControlCode, PVOID InputBuffer, ULONG InputBufferLength, PVOID OutputBuffer, ULONG OutputBufferLength);
	NTSTATUS NTAPI NtCopyFileChunk(HANDLE Source, HANDLE Dest, HANDLE Event, PIO_STATUS_BLOCK IoStatusBlock, ULONG Length, PULONG SourceOffset, PULONG DestOffset, PULONG SourceKey, PULONG DestKey, ULONG Flags);
	NTSTATUS NTAPI NtFlushBuffersFileEx(HANDLE FileHandle, ULONG Flags, PVOID Parameters, ULONG ParametersSize, PIO_STATUS_BLOCK IoStatusBlock);
	NTSTATUS NTAPI NtReadFile(HANDLE FileHandle, HANDLE Event, PIO_APC_ROUTINE ApcRoutine, PVOID ApcContext, PIO_STATUS_BLOCK IoStatusBlock, PVOID Buffer, ULONG Length, PLARGE_INTEGER ByteOffset, PULONG Key);
	NTSTATUS NTAPI NtWriteFile(HANDLE FileHandle, HANDLE Event, PIO_APC_ROUTINE ApcRoutine, PVOID ApcContext, PIO_STATUS_BLOCK IoStatusBlock, PVOID Buffer, ULONG Length, PLARGE_INTEGER ByteOffset, PULONG Key);
	NTSTATUS NTAPI NtTerminateProcess(HANDLE ProcessHandle, NTSTATUS ExitStatus);
	NTSTATUS NTAPI NtSetInformationFile(HANDLE FileHandle, PIO_STATUS_BLOCK IoStatusBlock, PVOID FileInformation, ULONG Length, FILE_INFORMATION_CLASS FileInformationClass);
	NTSTATUS NTAPI NtSetInformationObject(HANDLE ObjectHandle, OBJECT_INFORMATION_CLASS ObjectInformationClass, PVOID ObjectInformation, ULONG Length);
	NTSTATUS NTAPI RtlCreateUserThread(HANDLE process, SECURITY_DESCRIPTOR* descr, BOOLEAN suspended, ULONG zero_bits, SIZE_T stack_reserve, SIZE_T stack_commit, void* start, void *param, HANDLE *handle_ptr, CLIENT_ID* id);
	NTSTATUS NTAPI NtCreateThreadEx(PHANDLE ThreadHandle, ACCESS_MASK DesiredAccess, POBJECT_ATTRIBUTES ObjectAttributes, HANDLE ProcessHandle, USER_THREAD_START_ROUTINE* StartRoutine, PVOID Argument, ULONG CreateFlags, SIZE_T ZeroBits, SIZE_T StackSize, SIZE_T MaximumStackSize, PS_ATTRIBUTE_LIST* AttributeList);
	NTSTATUS NTAPI NtCreateJobObject(HANDLE* handle, ACCESS_MASK access, const OBJECT_ATTRIBUTES* attr);
	NTSTATUS NTAPI NtOpenJobObject(HANDLE* handle, ACCESS_MASK access, const OBJECT_ATTRIBUTES *attr);
	NTSTATUS NTAPI NtQueryInformationJobObject(HANDLE handle, JOBOBJECTINFOCLASS infoClass, void* info, ULONG len, ULONG* retLen);
	NTSTATUS NTAPI NtSetInformationJobObject(HANDLE handle, JOBOBJECTINFOCLASS infoClass, void* info, ULONG len);
	NTSTATUS NTAPI NtAssignProcessToJobObject(HANDLE job, HANDLE process);
	NTSTATUS NTAPI NtTerminateJobObject(HANDLE handle, NTSTATUS status);
	NTSTATUS NTAPI NtCreateSection(PHANDLE SectionHandle, ACCESS_MASK DesiredAccess, POBJECT_ATTRIBUTES ObjectAttributes, PLARGE_INTEGER MaximumSize, ULONG SectionPageProtection, ULONG AllocationAttributes, HANDLE FileHandle);
	NTSTATUS NTAPI NtCreateIoCompletion(PHANDLE IoCompletionHandle, ACCESS_MASK DesiredAccess, POBJECT_ATTRIBUTES ObjectAttributes, ULONG Count);
	NTSTATUS NTAPI NtRemoveIoCompletion(HANDLE IoCompletionHandle, PULONG_PTR CompletionKey, PULONG_PTR CompletionValue, PIO_STATUS_BLOCK IoStatusBlock, PLARGE_INTEGER Timeout);
	NTSTATUS NTAPI NtRemoveIoCompletionEx(HANDLE IoCompletionHandle, FILE_IO_COMPLETION_INFORMATION* info, ULONG count, ULONG *written, LARGE_INTEGER *timeout, BOOLEAN alertable);
	NTSTATUS NTAPI NtAllocateVirtualMemoryEx(HANDLE ProcessHandle, PVOID *BaseAddress, SIZE_T *RegionSize, ULONG AllocationType, ULONG Protect, PMEM_EXTENDED_PARAMETER ExtendedParameters, ULONG ExtendedParameterCount);
	NTSTATUS NTAPI NtQuerySecurityObject(HANDLE Handle, SECURITY_INFORMATION SecurityInformation, PSECURITY_DESCRIPTOR SecurityDescriptor, ULONG Length, PULONG LengthNeeded);
	NTSTATUS NTAPI NtAlpcCreatePort(PHANDLE PortHandle, POBJECT_ATTRIBUTES ObjectAttributes, PALPC_PORT_ATTRIBUTES PortAttributes);
	NTSTATUS NTAPI NtAlpcConnectPort(PHANDLE PortHandle, PUNICODE_STRING PortName, POBJECT_ATTRIBUTES ObjectAttributes, PALPC_PORT_ATTRIBUTES PortAttributes, DWORD ConnectionFlags, PSID RequiredServerSid, PPORT_MESSAGE ConnectionMessage, PSIZE_T ConnectMessageSize, PALPC_MESSAGE_ATTRIBUTES OutMessageAttributes, PALPC_MESSAGE_ATTRIBUTES InMessageAttributes, PLARGE_INTEGER Timeout);
	NTSTATUS NTAPI NtAlpcCreatePortSection(HANDLE PortHandle, ULONG Flags, HANDLE SectionHandle, SIZE_T SectionSize, PHANDLE AlpcSectionHandle, PSIZE_T ActualSectionSize);
	NTSTATUS NTAPI NtAlpcSendWaitReceivePort(HANDLE PortHandle, DWORD Flags, PPORT_MESSAGE SendMessage_, PALPC_MESSAGE_ATTRIBUTES SendMessageAttributes, PPORT_MESSAGE ReceiveMessage, PSIZE_T BufferLength, PALPC_MESSAGE_ATTRIBUTES ReceiveMessageAttributes, PLARGE_INTEGER Timeout);
	NTSTATUS NTAPI NtAlpcDisconnectPort(HANDLE PortHandle, ULONG Flags);
	NTSTATUS NTAPI NtExtendSection(HANDLE, PLARGE_INTEGER);
	NTSTATUS NTAPI ZwCreateFile(PHANDLE FileHandle, ACCESS_MASK DesiredAccess, POBJECT_ATTRIBUTES ObjectAttributes, PIO_STATUS_BLOCK IoStatusBlock, PLARGE_INTEGER AllocationSize, ULONG FileAttributes, ULONG ShareAccess, ULONG CreateDisposition, ULONG CreateOptions, PVOID EaBuffer, ULONG EaLength);
	NTSTATUS NTAPI ZwOpenFile(PHANDLE FileHandle, ACCESS_MASK DesiredAccess, POBJECT_ATTRIBUTES ObjectAttributes, PIO_STATUS_BLOCK IoStatusBlock, ULONG ShareAccess, ULONG OpenOptions);
	NTSTATUS NTAPI ZwClose(HANDLE Handle);
	NTSTATUS NTAPI ZwMapViewOfSection(HANDLE SectionHandle, HANDLE ProcessHandle, PVOID* BaseAddress, ULONG_PTR ZeroBits, SIZE_T CommitSize, PLARGE_INTEGER SectionOffset, PSIZE_T ViewSize, DWORD InheritDisposition, ULONG AllocationType, ULONG Win32Protect);
	NTSTATUS NTAPI ZwCreateSection(PHANDLE SectionHandle, ACCESS_MASK DesiredAccess, POBJECT_ATTRIBUTES ObjectAttributes, PLARGE_INTEGER MaximumSize, ULONG SectionPageProtection, ULONG AllocationAttributes, HANDLE FileHandle);
	NTSTATUS NTAPI ZwQueryDirectoryFile(HANDLE FileHandle, HANDLE Event, PIO_APC_ROUTINE ApcRoutine, PVOID ApcContext, PIO_STATUS_BLOCK IoStatusBlock, PVOID FileInformation, ULONG Length, FILE_INFORMATION_CLASS FileInformationClass, BOOLEAN ReturnSingleEntry, PUNICODE_STRING FileName, BOOLEAN RestartScan);
	NTSTATUS NTAPI ZwSetInformationFile(HANDLE FileHandle,PIO_STATUS_BLOCK IoStatusBlock, PVOID FileInformation, ULONG Length, FILE_INFORMATION_CLASS FileInformationClass);
	NTSTATUS NTAPI LdrLockLoaderLock(ULONG Flags, ULONG* Disposition, PVOID* Cookie);
	NTSTATUS NTAPI LdrUnlockLoaderLock(ULONG Flags, PVOID Cookie);
	NTSTATUS NTAPI LdrAddRefDll(ULONG Flags, PVOID BaseAddress);
	PVOID WINAPI ResolveDelayLoadedAPI(PVOID ParentModuleBase, PCIMAGE_DELAYLOAD_DESCRIPTOR DelayloadDescriptor, void* FailureDllHook, void* FailureSystemHook, PIMAGE_THUNK_DATA ThunkAddress, ULONG Flags);
	void WINAPI RtlExitUserThread(ULONG);
	NTSTATUS NTAPI RtlGetVersion(PRTL_OSVERSIONINFOW lpVersionInformation);
	BOOLEAN NTAPI RtlFreeHeap(PVOID HeapHandle, ULONG Flags, PVOID HeapBase);
	BOOLEAN NTAPI RtlValidateHeap(HANDLE HeapPtr, ULONG Flags, PVOID Block);
	NTSTATUS NTAPI RtlDosPathNameToNtPathName_U_WithStatus(PCWSTR dos_path, PUNICODE_STRING ntpath, PWSTR* file_part, VOID* reserved);
	NTSTATUS NTAPI RtlDosPathNameToRelativeNtPathName_U_WithStatus(PCWSTR DosFileName, PUNICODE_STRING NtFileName, PWSTR *FilePart, RTL_RELATIVE_NAME_U* RelativeName);
	DWORD WINAPI RtlGetFullPathName_U(const WCHAR* name, ULONG size, WCHAR* buffer, WCHAR** file_part);
	BOOLEAN NTAPI RtlDosPathNameToRelativeNtPathName_U(PCWSTR DosName, PUNICODE_STRING NtName, PWSTR *FilePart, RTL_RELATIVE_NAME_U* RelativeName);
	ULONG NTAPI WINAPI RtlGetFullPathName_UEx(const WCHAR* name, ULONG size, WCHAR* buffer, WCHAR** file_part, RTL_PATH_TYPE* type);
	VOID NTAPI RtlExitUserProcess(NTSTATUS ExitStatus);
	VOID NTSYSAPI RtlMapGenericMask(PACCESS_MASK AccessMask, const GENERIC_MAPPING *GenericMapping);
	PVOID NTAPI RtlAllocateHeap( PVOID HeapHandle, ULONG Flags, SIZE_T Size);
	PVOID NTAPI RtlReAllocateHeap(PVOID HeapHandle, ULONG Flags, PVOID BaseAddress, SIZE_T Size);
	SIZE_T NTAPI RtlSizeHeap(HANDLE HeapPtr, ULONG Flags, PVOID Ptr);
	void NTAPI BaseThreadInitThunk(ULONG Unknown,LPTHREAD_START_ROUTINE StartAddress,PVOID ThreadParameter);
	void* _expand_base(void* memblock, size_t size);
	HRESULT SHGetFolderPathAndSubDirW(HWND hwnd, int csidl, HANDLE hToken, DWORD dwFlags, LPCWSTR pszSubDir, LPWSTR pszPath);
	HRESULT SHGetKnownFolderPath(REFKNOWNFOLDERID rfid, DWORD dwFlags, HANDLE hToken, PWSTR* ppszPath);
	HRESULT SHGetKnownFolderIDList(REFKNOWNFOLDERID rfid, DWORD flags, HANDLE token, PIDLIST_ABSOLUTE *pidl);
	HRESULT SHGetFolderPathW(HWND hwnd, int csidl, HANDLE hToken, DWORD dwFlags, LPWSTR pszPath);
	enum FILE_INFO_BY_NAME_CLASS{};
	BOOL WINAPI GetFileInformationByName(PCWSTR FileName, FILE_INFO_BY_NAME_CLASS FileInformationClass, PVOID FileInfoBuffer, ULONG FileInfoBufferSize);
	DWORD WINAPI GetTempPath2W(DWORD  BufferLength, LPWSTR Buffer);
	__time64_t WINAPI _time64(__time64_t* destTime);

}

#define DETOURED_FUNCTION_UNKNOWN(Func) DETOURED_FUNCTION(Func)

#define DETOURED_FUNCTION(Func) extern decltype(Func)* True_##Func;
DETOURED_FUNCTIONS
#undef DETOURED_FUNCTION

namespace uba
{
	struct DetoursPayload;

	void PreInit(const DetoursPayload& payload);
	void Init(const DetoursPayload& payload, u64 startTime);
	void Deinit(u64 startTime);
	void PostDeinit();
}