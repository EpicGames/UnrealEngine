// Copyright Epic Games, Inc. All Rights Reserved.

#if defined( __clang_analyzer__ )
#include <corecrt.h>
#undef __DEFINE_CPP_OVERLOAD_SECURE_FUNC_SPLITPATH
#define __DEFINE_CPP_OVERLOAD_SECURE_FUNC_SPLITPATH(_ReturnType, _FuncName, _DstType, _Src)
#endif

#include "UbaProtocol.h"
#include "UbaProcessStats.h"
#include "UbaDefinitions.h"
#include "UbaDetoursPayload.h"
#include "UbaDetoursShared.h"
#include "UbaDetoursFunctionsWin.h"
#include "UbaFileMapping.h"
#include <detours/detours.h>
#include <stdio.h>

namespace uba
{
	struct Bottleneck { Bottleneck(...) {} };
	struct BottleneckScope { BottleneckScope(Bottleneck&, Timer&) {} };
}
#include "UbaFileMappingWin.inl"

namespace uba
{
	HANDLE g_writeEvent;
	HANDLE g_readEvent;
	HANDLE g_cancelEvent;
	u8* g_writeMessageMappingMem;
	bool g_useSignalObjectAndWait;

	bool WasCancelled()
	{
		return WaitForSingleObject(g_cancelEvent, 0) != WAIT_TIMEOUT;
	}

	int Connect(DetoursPayload& payload)
	{
		g_hostProcess = payload.hostProcess;

		if (!DuplicateHandle(g_hostProcess, payload.cancelEvent, GetCurrentProcess(), &g_cancelEvent, SYNCHRONIZE, false, 0))
		{
			UBA_ASSERT(!WasCancelled());
			return -4;
		}

		// Sync primitive for writing message to host process
		if (!DuplicateHandle(g_hostProcess, payload.readEvent, GetCurrentProcess(), &g_writeEvent, EVENT_MODIFY_STATE, false, 0))
			return -2;

		// Sync primitive for reading messages from host process
		if (!DuplicateHandle(g_hostProcess, payload.writeEvent, GetCurrentProcess(), &g_readEvent, SYNCHRONIZE, false, 0))
			return -3;

		FileMappingHandle comHandle = FileMapping_DuplicateFromHost(FileMappingHandle::FromU64(payload.communicationHandle), "Communication");
		u8* mem = FileMapping_MapFromHost(comHandle, CommunicationMemSize, payload.communicationOffset, true, "Communication");
		if (!mem)
		{
			return -5;
		}
		
		g_useSignalObjectAndWait = !IsRunningWine(); // There seems to be a bug in wine (as of 11.0) for using this function. It can return before it should
		g_writeMessageMappingMem = mem;

		SetMemoryWorkingSet(CommunicationMemSize*2);

		VirtualLock(mem, CommunicationMemSize);

		return 0;
	}

	int Disconnect()
	{
		//::UnmapViewOfFile(g_writeMessageMappingMem); // Could be linux backend
		//::CloseHandle(comHandle.internal); // Could be linux backend
		::CloseHandle(g_cancelEvent);
		::CloseHandle(g_readEvent);
		::CloseHandle(g_writeEvent);

		::CloseHandle(g_hostProcess);
		return 0;
	}

	ANALYSIS_NORETURN void TerminateCurrentProcess(u32 exitCode)
	{
		if (True_NtTerminateProcess)
			True_NtTerminateProcess(GetCurrentProcess(), exitCode);
		else
			TerminateProcess(GetCurrentProcess(), exitCode);
	}

	BinaryWriter::BinaryWriter(ProcessCommunication)
	{
		m_begin = g_writeMessageMappingMem;
		m_pos = m_begin;
		m_end = m_begin + CommunicationMemSize;// / 2;

		#if UBA_PROTOCOL_GUARD
		WriteU32(0xdeadbeef);
		#endif
	}

	BinaryReader BinaryWriter::Flush(bool waitOnResponse)
	{
		#if UBA_PROTOCOL_GUARD
		if (BinaryReader(g_writeMessageMappingMem, 0, CommunicationMemSize).ReadU32() != 0xdeadbeef)
			ExitProcess(2345);
		#endif

		if (!waitOnResponse)
		{
			SetEvent(g_writeEvent);
			return BinaryReader(nullptr, 0, 0);
		}

		TimerScope ts(g_stats.waitOnResponse);
		DWORD res;
		
		if (g_useSignalObjectAndWait)
			res = SignalObjectAndWait(g_writeEvent, g_readEvent, 1000, FALSE);
		else
		{
			if (!SetEvent(g_writeEvent))
				ExitProcess(1339);
			res = True_WaitForSingleObject(g_readEvent, 1000);
		}

		do
		{
			if (res == WAIT_OBJECT_0)
				break;

			if (res != WAIT_TIMEOUT)
				TerminateCurrentProcess(1337);

			// Using True_NtQueryInformationProcess to get rid of spam in logs
			PROCESS_BASIC_INFORMATION pbi;
			if (True_NtQueryInformationProcess(g_hostProcess, ProcessBasicInformation, &pbi, sizeof(pbi), NULL) != STATUS_SUCCESS)
				TerminateCurrentProcess(1338);
			auto exitCode = (NTSTATUS)(ULONG_PTR)pbi.Reserved1;
			if (exitCode != STATUS_PENDING)
				TerminateCurrentProcess(1339);

			DWORD cancelRes = True_WaitForSingleObject(g_cancelEvent, 0);
			if (cancelRes == WAIT_OBJECT_0)
				ExitProcess(1339);
			else if (cancelRes != WAIT_TIMEOUT)
				TerminateCurrentProcess(1353);

			res = True_WaitForSingleObject(g_readEvent, 500);

		} while (true);

		BinaryReader reader(m_begin, 0, CommunicationMemSize);
	
		#if UBA_PROTOCOL_GUARD
		u32 baadfood = reader.ReadU32();
		if (baadfood != 0xbaadf00d)
			ExitProcess(2345);
		BinaryWriter(g_writeMessageMappingMem, 0, CommunicationMemSize).WriteU32(0xcafebabe);
		#endif

		m_begin = nullptr;
		m_pos = 0;
		m_end = m_begin;
		return reader;
	}
}

BOOL WINAPI DllMain(HINSTANCE hinst, DWORD dwReason, LPVOID reserved)
{
	using namespace uba;


	u64 startTime = GetTime();

	if (DetourIsHelperProcess())
		return TRUE;

	if (dwReason == DLL_PROCESS_ATTACH)
	{
		SetThreadErrorMode(GetThreadErrorMode() | SEM_FAILCRITICALERRORS | SEM_NOOPENFILEERRORBOX, nullptr);

		if (!DetourRestoreAfterWith())
			TerminateCurrentProcess(1344);

		DetoursPayload* payload = (DetoursPayload*)DetourFindPayloadEx(DetoursPayloadGuid, nullptr);
		if (!payload)
			TerminateCurrentProcess(1342);
		if (payload->version != ProcessMessageVersion)
			TerminateCurrentProcess(1398);

		PreInit(*payload);

		int result = Connect(*payload);
		if (result != 0)
		{
			Disconnect();
			TerminateCurrentProcess(result);
		}

		Init(*payload, startTime);
	}
	else if (dwReason == DLL_THREAD_ATTACH)
	{
		//DEBUG_LOG_TRUE(TC("DllMain"), TC("THREAD_ATTACH %u"), GetCurrentThreadId());
		SetThreadErrorMode(GetThreadErrorMode() | SEM_FAILCRITICALERRORS | SEM_NOOPENFILEERRORBOX, nullptr);
	}
	else if (dwReason == DLL_THREAD_DETACH)
	{
		//DEBUG_LOG_TRUE(TC("DllMain"), TC("THREAD_DETACH %u"), GetCurrentThreadId());
	}
	else if (dwReason == DLL_PROCESS_DETACH)
	{
		Deinit(startTime);
		Disconnect();
		PostDeinit();
	}

	return TRUE;
}
