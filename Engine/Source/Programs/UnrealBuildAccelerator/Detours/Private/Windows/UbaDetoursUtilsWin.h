// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaDefinitions.h"
#include "UbaDetoursFunctionsWin.h"
#include "UbaDetoursFileMappingTable.h"
#include "UbaSharedMemoryView.h"

namespace uba
{
	struct FileObject
	{
		void* operator new(size_t size);
		void operator delete(void* p);
		FileInfo* fileInfo = nullptr;
		u32 refCount = 1;
		u32 closeId = 0;
		u32 desiredAccess = 0;
		bool deleteOnClose = false;
		bool ownsFileInfo = false;

		// Is set to true once content of file has been accessed (ReadFile or MapViewOfFile)
		bool wasUsed = false;


		TString newName;
	};
	extern BlockAllocator<FileObject>& g_fileObjectAllocator;
	inline void* FileObject::operator new(size_t size) { return g_fileObjectAllocator.Allocate(); }
	inline void FileObject::operator delete(void* p) { g_fileObjectAllocator.Free(p); }


	enum HandleType
	{
		HandleType_File,
		HandleType_FileMapping,
		HandleType_Process,
		HandleType_Job,
		HandleType_StdErr,
		HandleType_StdOut,
		HandleType_StdIn,
		// Std handle types must be last
	};


	struct DetouredHandle
	{
		void* operator new(size_t size);
		void operator delete(void* p);

		DetouredHandle(HandleType t, HANDLE th = INVALID_HANDLE_VALUE) : trueHandle(th), type(t) {}

		HANDLE trueHandle;
		DirTableOffset dirTableOffset;
		HandleType type;

		// Only for files
		FileObject* fileObject = nullptr;
		u64 pos = 0;
	};


	struct MemoryFile
	{
		MemoryFile(u8* data = nullptr, bool localOnly = true);
		MemoryFile(bool localOnly, u64 reserveSize, bool isThrowAway, u32 commitStepSize, const tchar* fileName = nullptr);

		void Reserve(u64 reserveSize_, const tchar* fileName = nullptr);
		void Unreserve();
		void Write(struct DetouredHandle& handle, LPCVOID lpBuffer, u64 nNumberOfBytesToWrite, const tchar* hint);
		void EnsureCommitted(const struct DetouredHandle& handle, u64 size);
		void Remap(const struct DetouredHandle& handle, u64 size);

		u64 fileIndex = ~u64(0);
		u64 fileTime = ~u64(0);
		u32 volumeSerial = 0;
		u32 commitStepSize = 1024*1024;

		SharedMemoryHandle memoryHandle;
		SharedMemoryView memoryView;

		u8* baseAddress = nullptr;
		u64 reserveSize = 0;
		u64 mappedSize = 0;
		u64 committedSize = 0;
		u64 writtenSize = 0;
		ReaderWriterLock lock;
		bool isLocalOnly;
		bool isReported = false;
		bool isThrowAway = false;
	};
}