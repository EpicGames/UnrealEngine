// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(UBA_IS_DETOURED_INCLUDE)
#define UBA_DETOURED_API __declspec(dllexport)
#else
#define UBA_DETOURED_API __declspec(dllimport)
#endif

extern "C"
{
	// Send custom message from detoured process all the way out to callback registered with RegisterCustomService
	// Handles remote processes as well and network the message
	UBA_DETOURED_API unsigned int UbaSendCustomMessage(const void* send, unsigned int sendSize, void* recv, unsigned int recvCapacity);

	// Make sure all written files are flushed to destination (lots of files are kept in memory until process ends otherwise)
	UBA_DETOURED_API bool UbaFlushWrittenFiles();

	// Update environment. Making sure caches are up-to-date in detoured process.
	// reason ends up in the visualizer for the next block of process if resetStats is true (if resetStats is not true, this will just update the environment within same process)
	UBA_DETOURED_API bool UbaUpdateEnvironment(const wchar_t* reason, bool resetStats);

	// Returns true if process is running on a remote machine
	UBA_DETOURED_API bool UbaRunningRemote();

	// This function will automatically flush written files before requesting next process and will also update environment if new process is retrieved
	// prevExitCode should contain exit code of "finished" process.
	// Will write arguments into outArguments. outArgumentsCapacity should be in characters (not bytes)
	UBA_DETOURED_API bool UbaRequestNextProcess(unsigned int prevExitCode, wchar_t* outArguments, unsigned int outArgumentsCapacity);

	// Same as above but with a timeout how long it can wait before giving up to get next process and a "should exit" variable telling the calling process to exit
	UBA_DETOURED_API bool UbaRequestNextProcess2(unsigned int prevExitCode, wchar_t* outArguments, unsigned int outArgumentsCapacity, unsigned int timeOutMs, bool* outShouldExit);
}


// Above functions can be found and used using code looking like this:

#if 0
// Windows
static HMODULE UbaDetoursModule = GetModuleHandleW(L"UbaDetours.dll");
if (UbaDetoursModule)
{
	using UbaFlushWrittenFilesFunc = bool();
	using UbaRequestNextProcessFunc = bool(unsigned int prevExitCode, const wchar_t* outArguments, unsigned int outArgumentsCapacity);
	using UbaUpdateEnvironmentFunc = bool(const wchar_t* reason, bool resetStats);

	static UbaFlushWrittenFilesFunc* flushWrittenFiles = (UbaFlushWrittenFilesFunc*)(void*)GetProcAddress(UbaDetoursModule, "UbaFlushWrittenFiles");
	static UbaRequestNextProcessFunc* requestNextProcess = (UbaRequestNextProcessFunc*)(void*)GetProcAddress(UbaDetoursModule, "UbaRequestNextProcess");
	static UbaUpdateEnvironmentFunc* updateEnvironment = (UbaUpdateEnvironmentFunc*)(void*)GetProcAddress(UbaDetoursModule, "UbaUpdateEnvironment");

	// ...
}

// Linux/Mac not implemented yet
...
#endif