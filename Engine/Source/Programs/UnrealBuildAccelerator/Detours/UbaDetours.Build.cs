// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaDetours : ModuleRules
{
	public UbaDetours(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		StaticAnalyzerDisabledCheckers.Clear();
		bUseUnity = false;

		bRequiresPlatformSDK = true;

		PrivateDependencyModuleNames.AddRange(new string[] {
			"UbaCore",
			"OodleDataCompression",
		});

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
				"Detours",
			});

			PublicSystemLibraries.AddRange(new string[] {
				"ntdll.lib",
				"onecore.lib"
			});

			// This is to handle mspdbsrv.exe ... not supporting arm atm
			bool supportMspdbSrv = false; // Target.Architecture == UnrealArch.X64 && Target.Configuration == UnrealTargetConfiguration.Debug
			
			if (supportMspdbSrv)
			{
				PrivateDefinitions.Add("UBA_SUPPORT_MSPDBSRV=1");
				PublicAdditionalLibraries.Add(Path.Combine(ModuleDirectory, "Lib", "UbaAsmX64.obj"));
			}
			else
			{
				PrivateDefinitions.Add("UBA_SUPPORT_MSPDBSRV=0");
			}
		}
		else if (Target.Platform.IsInGroup(UnrealPlatformGroup.Linux))
		{
			PublicSystemLibraries.Add("dl");
		}
	}
}
