#!/usr/bin/env bash
#
# Compile UbaStaticStub.S and produce the raw blob binary.
#
# Output lands at:
#   Engine/Binaries/Linux/UnrealBuildAccelerator/UbaStaticStub.bin
#
# UbaStaticPatcher reads this file at runtime, finds the sentinel
# 0xDEADBEEFCAFEBABE in it, and embeds the bytes into the target ELF
# after patching the sentinel to the target's original e_entry.
#
# Run manually whenever the .S source changes. The .bin is checked in
# to p4 alongside the other UBA Linux binaries.
#
# Invoke from Linux (or WSL):
#   bash Build.sh
#
# Environment overrides:
#   CLANG, CLANGXX, LD, OBJCOPY, LLVMNM  — toolchain paths
#     (default: AOSP prebuilt clang under ~/git/android/prebuilts/clang/;
#      set to system clang/lld/llvm binaries for a clean-machine build —
#      see Readme.txt).

set -euo pipefail

script_dir=$(cd -- "$(dirname -- "${BASH_SOURCE[0]}")" && pwd)
engine_root=$(cd -- "$script_dir/../../../../.." && pwd)

src_s="$script_dir/UbaStaticStub.S"
src_cpp="$script_dir/UbaStaticStubCore.cpp"
src_filemap="$script_dir/UbaStubFileMapping.cpp"
# Linker script routes GlibcHandlerTable into .text.uba_handlers; patcher locates it via kGlibcHandlerTableMagic.
src_glibc="$script_dir/UbaGlibc.cpp"
core_private="$engine_root/Engine/Source/Programs/UnrealBuildAccelerator/Core/Private"
detours_private="$engine_root/Engine/Source/Programs/UnrealBuildAccelerator/Detours/Private"
src_hash="$core_private/UbaHash.cpp"
src_shared_core=(
    "$core_private/UbaMemory.cpp"
    "$core_private/UbaParkingLot.cpp"
    "$core_private/UbaStringBuffer.cpp"
    "$core_private/UbaSynchronization.cpp"
    "$detours_private/UbaDetoursFileMappingTable.cpp"
    "$detours_private/UbaDetoursShared.cpp"
    "$detours_private/PosixOS/UbaDetoursSharedPosix.cpp"
)
# Intentionally excluded: UbaSharedMemoryView.cpp (Linux stub), UbaEvent.cpp (pulls pthread types).
# BLAKE3 recompiled portable-only: prebuilt .a's CPUID dispatcher reads g_cpu_features in .data; our linker script packs .data into .text
# and objcopy --only-section=.text extracts only .text, so the merge fails address-dependently at high stub_base addresses.
blake3_inc="$engine_root/Engine/Source/ThirdParty/BLAKE3/1.3.1/c"
blake3_srcs=(
    "$blake3_inc/blake3.c"
    "$blake3_inc/blake3_dispatch.c"
    "$blake3_inc/blake3_portable.c"
)
BLAKE3_NO_SIMD=(
    -DBLAKE3_NO_AVX512
    -DBLAKE3_NO_AVX2
    -DBLAKE3_NO_SSE41
    -DBLAKE3_NO_SSE2
    -DBLAKE3_USE_NEON=0
)
out_dir="$engine_root/Engine/Binaries/Linux/UnrealBuildAccelerator"
out_bin="$out_dir/UbaStaticStub.bin"

CLANG=${CLANG:-/home/honk/git/android/prebuilts/clang/host/linux-x86/clang-r547379/bin/clang}
CLANGXX=${CLANGXX:-/home/honk/git/android/prebuilts/clang/host/linux-x86/clang-r547379/bin/clang++}
LD=${LD:-/home/honk/git/android/prebuilts/clang/host/linux-x86/clang-r547379/bin/ld.lld}
OBJCOPY=${OBJCOPY:-/home/honk/git/android/prebuilts/clang/host/linux-x86/clang-r547379/bin/llvm-objcopy}
LLVMNM=${LLVMNM:-/home/honk/git/android/prebuilts/clang/host/linux-x86/clang-r547379/bin/llvm-nm}

for tool in "$CLANG" "$CLANGXX" "$LD" "$OBJCOPY"; do
    if [[ ! -x "$tool" ]]; then
        echo "ERROR: missing tool: $tool" >&2
        exit 1
    fi
done

mkdir -p "$out_dir"

tmp=$(mktemp -d)
if [[ "${STUB_KEEP_TMP:-0}" != "1" ]]; then
    trap 'rm -rf "$tmp"' EXIT
else
    echo "=== keeping tmp dir: $tmp ==="
fi

obj_s="$tmp/stub_entry.o"
obj_cpp="$tmp/stub_core.o"
obj_filemap="$tmp/stub_filemap.o"
obj_hash="$tmp/uba_hash.o"
obj_glibc="$tmp/uba_glibc.o"
elf="$tmp/stub.elf"

CFLAGS_COMMON=(
    -O2 -nostdlib -ffreestanding -fPIC -fno-stack-protector
    -mno-red-zone -fno-asynchronous-unwind-tables -fno-unwind-tables
    -fvisibility=hidden -fno-plt -Wa,--noexecstack
    # Go syscall bridge re-enters the stub with rsp not 16-byte aligned; without this clang assumes 16-byte and GPFs on movaps.
    -mstackrealign
    -DUBA_STUB_BUILD=1
    -DUBA_DEBUG_LOG_ENABLED=1
    # Bridge return-path sanity check (SIGABRT if rsp lands in alt-stack range).
    -DUBA_BRIDGE_SANITY=1
    # Without NDEBUG, BLAKE3's asserts emit callq *<GOT slot>; our linker script drops .got so the slot reads 0 and the call jumps to 0.
    -DNDEBUG=1
)

# Disable auto-vectorization: clang otherwise emits SSE movdqu over .rodata which faults inside stub-patched Go binaries early in
# DetourCore (vectorized prologue + Go's stack/register state).
CXXFLAGS_STUB=(
    -std=c++20 -nostdlib++ -fno-exceptions -fno-rtti
    -fno-threadsafe-statics
    -fno-vectorize -fno-slp-vectorize
)

echo "=== assemble entry ==="
"$CLANG" "${CFLAGS_COMMON[@]}" -c "$src_s" -o "$obj_s"

echo "=== compile core (c++) ==="
UBA_INCLUDES=(
    -DPLATFORM_LINUX=1 -DPLATFORM_WINDOWS=0 -DPLATFORM_MAC=0
    -I"$engine_root/Engine/Source/Programs/UnrealBuildAccelerator/Core/Public"
    -I"$engine_root/Engine/Source/Programs/UnrealBuildAccelerator/Detours/Private"
    -I"$blake3_inc"
)
"$CLANGXX" "${CFLAGS_COMMON[@]}" "${CXXFLAGS_STUB[@]}" "${UBA_INCLUDES[@]}" -Wall -Werror -c "$src_cpp" -o "$obj_cpp"

echo "=== compile UbaStubFileMapping.cpp (memfd FileMapping_MapFromHost) ==="
"$CLANGXX" "${CFLAGS_COMMON[@]}" "${CXXFLAGS_STUB[@]}" "${UBA_INCLUDES[@]}" -Wall -Werror -c "$src_filemap" -o "$obj_filemap"

echo "=== compile UbaHash.cpp (shared) ==="
"$CLANGXX" "${CFLAGS_COMMON[@]}" "${CXXFLAGS_STUB[@]}" "${UBA_INCLUDES[@]}" -Wall -Werror -c "$src_hash" -o "$obj_hash"

echo "=== compile UbaGlibc.cpp (consolidated handlers + table + fd map) ==="
"$CLANGXX" "${CFLAGS_COMMON[@]}" "${CXXFLAGS_STUB[@]}" "${UBA_INCLUDES[@]}" -I"$script_dir" -Wall -Werror -c "$src_glibc" -o "$obj_glibc"

shared_core_objs=()
for src in "${src_shared_core[@]}"; do
    base=$(basename "$src" .cpp)
    out="$tmp/${base}.o"
    echo "=== compile $base.cpp (shared core) ==="
    "$CLANGXX" "${CFLAGS_COMMON[@]}" "${CXXFLAGS_STUB[@]}" "${UBA_INCLUDES[@]}" -Wall -c "$src" -o "$out"
    shared_core_objs+=("$out")
done

blake3_objs=()
for src in "${blake3_srcs[@]}"; do
    base=$(basename "$src" .c)
    out="$tmp/blake3_${base}.o"
    echo "=== compile BLAKE3 $base.c (portable-only) ==="
    "$CLANG" "${CFLAGS_COMMON[@]}" "${BLAKE3_NO_SIMD[@]}" -I"$blake3_inc" -Wall -c "$src" -o "$out"
    blake3_objs+=("$out")
done

echo "=== link (self-contained shared + custom script) ==="
"$LD" \
    -shared -Bsymbolic --no-dynamic-linker -nostdlib -z noexecstack -z defs \
    -e 0 \
    -T "$script_dir/UbaStaticStub.ld" \
    -o "$elf" "$obj_s" "$obj_cpp" "$obj_filemap" "$obj_hash" "$obj_glibc" \
    "${shared_core_objs[@]}" "${blake3_objs[@]}"

echo "=== verify DetourInit at offset 0 within .text ==="
# Dump symtab once: `awk { exit }` in a pipeline + `set -euo pipefail` is fragile (SIGPIPE kills the script on some host awk/binutils combos).
nm_dump="$tmp/stub_nm.txt"
"$LLVMNM" "$elf" > "$nm_dump"

if [[ -x "$LLVMNM" ]]; then
    init_off=$(awk '$3 == "DetourInit" { print $1 }' "$nm_dump" | head -1)
    echo "  DetourInit virtual address: 0x${init_off}"
    echo "=== unresolved externs in final ELF ==="
    "$LLVMNM" -u "$elf" || true
fi

echo "=== extract raw .text bytes ==="
"$OBJCOPY" -O binary --only-section=.text "$elf" "$out_bin"

blob_size=$(stat -c '%s' "$out_bin")
echo "=== blob size: $blob_size bytes → $out_bin ==="

sentinel_hex="bebafecaefbeadde"  # 0xDEADBEEFCAFEBABE little-endian
sentinel_off=$(python3 - <<PY
import sys
data = open("$out_bin", "rb").read()
target = bytes.fromhex("$sentinel_hex")
idx = data.find(target)
print(idx)
PY
)

if [[ "$sentinel_off" == "-1" ]]; then
    echo "ERROR: sentinel 0xDEADBEEFCAFEBABE not found in blob" >&2
    exit 2
fi

echo "=== sentinel at byte offset $sentinel_off ==="

# Post-link GlibcHandlerTable offset stamping. entries[i].offset is 0 at compile time. We stamp:
#   HandlerHook (kind=0): Glibc<CamelName> VA → offset within blob.
#   ImportSlot  (kind=1): writable slot for patcher to stamp target's symbol VA (e.g. __errno_location -> g_glibc_errno_hook_fn).
# Steps: scan blob for kGlibcHandlerTableMagic; read each symbol's VA via llvm-nm; subtract .text base VA (objcopy maps text_vaddr to byte 0); patch u32 offsets.

echo "=== stamp GlibcHandlerTable handler offsets ==="

# .text vaddr from the linked ELF is the base objcopy used when extracting .text into the flat .bin.
LLVMREADELF=${LLVMREADELF:-/home/honk/git/android/prebuilts/clang/host/linux-x86/clang-r547379/bin/llvm-readelf}
if [[ ! -x "$LLVMREADELF" ]]; then
    LLVMREADELF=$(dirname "$LLVMNM")/llvm-readelf
fi

readelf_dump="$tmp/stub_readelf.txt"
"$LLVMREADELF" -S --wide "$elf" > "$readelf_dump"
text_vaddr_hex=$(awk '/^[[:space:]]*\[[[:space:]]*[0-9]+\][[:space:]]+\.text[[:space:]]/ { print $5; exit }' "$readelf_dump")

if [[ -z "$text_vaddr_hex" ]]; then
    echo "ERROR: could not determine .text virtual address of $elf" >&2
    exit 3
fi

echo "  .text vaddr in ELF: 0x$text_vaddr_hex"

sym_names_to_lookup=(
    # Real handlers (mapped to libc wrappers via kGlibcAliasTable).
    GlibcOpen GlibcOpenat GlibcRead GlibcWrite
    GlibcClose GlibcLseek GlibcFxstat GlibcMmap
    GlibcAccess GlibcExit
    # Abort-on-reach handlers (order mirrors entries[] in UbaGlibc.cpp).
    GlibcStat GlibcLstat GlibcFstatat GlibcStatx
    GlibcUnlink GlibcUnlinkat
    GlibcRename GlibcRenameat GlibcRenameat2
    GlibcLink GlibcLinkat
    GlibcSymlink GlibcSymlinkat
    GlibcReadlink GlibcReadlinkat
    GlibcTruncate GlibcFtruncate
    GlibcMkdir GlibcMkdirat GlibcRmdir
    GlibcFork GlibcVfork
    GlibcClone GlibcClone3
    GlibcExecve GlibcExecveat
    GlibcDup GlibcDup2 GlibcDup3
    GlibcPipe GlibcPipe2
    GlibcOpendir GlibcFdopendir
    GlibcReaddir GlibcClosedir GlibcGetdents64
    GlibcChdir GlibcFchdir
    # Import slot (patcher stamps target's VA here).
    g_glibc_errno_hook_fn
)

sym_map=""
for sym in "${sym_names_to_lookup[@]}"; do
    # Read from dump file (avoids SIGPIPE-killing the script on some host awks under set -euo pipefail).
    line=$(awk -v s="$sym" '$3 == s { print $1; exit }' "$nm_dump")
    if [[ -z "$line" ]]; then
        echo "ERROR: missing symbol $sym in $elf" >&2
        exit 4
    fi
    sym_map+="${sym}=${line};"
done

OUT_BIN="$out_bin" \
TEXT_VADDR_HEX="$text_vaddr_hex" \
SYM_MAP="$sym_map" \
python3 - <<'PY'
import os, struct, sys

bin_path       = os.environ["OUT_BIN"]
elf_text_vaddr = int(os.environ["TEXT_VADDR_HEX"], 16)
sym_map_raw    = os.environ["SYM_MAP"]

sym_to_vaddr = {}
for kv in sym_map_raw.split(";"):
    if not kv:
        continue
    name, vhex = kv.split("=", 1)
    sym_to_vaddr[name] = int(vhex, 16)

MAGIC = 0x5448424C47414255  # 'UBAGLBHT' little-endian

with open(bin_path, "rb") as f:
    blob = bytearray(f.read())

table_off = -1
for i in range(0, len(blob) - 8, 8):
    if struct.unpack_from("<Q", blob, i)[0] == MAGIC:
        table_off = i
        break

if table_off < 0:
    sys.stderr.write("ERROR: GlibcHandlerTable magic not found in {}\n".format(bin_path))
    sys.exit(5)

# Layout: u64 magic, u32 count, u32 reserved, entries[] (32 bytes each: char name[24], u32 offset, u32 kind).
count = struct.unpack_from("<I", blob, table_off + 8)[0]

entries_off = table_off + 16
ENTRY_SZ    = 32
NAME_SZ     = 24
NAME_OFF    = 0
OFF_OFF     = 24
KIND_OFF    = 28

KIND_HANDLER_HOOK = 0
KIND_IMPORT_SLOT  = 1

# ImportSlot resolution is bespoke per-name; only __errno_location is wired today. HandlerHook uses Glibc<CamelName>.
IMPORT_SYM_FOR_NAME = {
    "__errno_location": "g_glibc_errno_hook_fn",
}

print("  table at blob offset 0x{:x} (count={})".format(table_off, count))
print("  stamped offsets:")
for i in range(count):
    e_off  = entries_off + i * ENTRY_SZ
    e_name_raw = bytes(blob[e_off + NAME_OFF : e_off + NAME_OFF + NAME_SZ])
    e_name = e_name_raw.split(b"\x00", 1)[0].decode("ascii")
    e_kind = struct.unpack_from("<I", blob, e_off + KIND_OFF)[0]

    if e_kind == KIND_HANDLER_HOOK:
        sym = "Glibc" + e_name[:1].upper() + e_name[1:]
        kind_label = "hook"
    elif e_kind == KIND_IMPORT_SLOT:
        sym = IMPORT_SYM_FOR_NAME.get(e_name)
        if sym is None:
            sys.stderr.write("ERROR: ImportSlot entry {} ({}) has no symbol mapping in IMPORT_SYM_FOR_NAME\n"
                             .format(i, e_name))
            sys.exit(7)
        kind_label = "import"
    else:
        sys.stderr.write("ERROR: entry {} ({}) has unknown kind {}\n".format(i, e_name, e_kind))
        sys.exit(7)

    if sym not in sym_to_vaddr:
        sys.stderr.write("ERROR: entry {} ({}) needs symbol {} but it was not collected from the ELF\n"
                         .format(i, e_name, sym))
        sys.exit(7)

    sym_vaddr = sym_to_vaddr[sym]
    sym_offset = sym_vaddr - elf_text_vaddr
    if sym_offset < 0 or sym_offset >= len(blob):
        sys.stderr.write("ERROR: entry {} ({}) -> {} offset 0x{:x} outside blob (size 0x{:x})\n"
                         .format(i, e_name, sym, sym_offset, len(blob)))
        sys.exit(8)
    struct.pack_into("<I", blob, e_off + OFF_OFF, sym_offset)
    print("    {:18s} [{}] -> {:24s} +0x{:x}".format(e_name, kind_label, sym, sym_offset))

with open(bin_path, "wb") as f:
    f.write(bytes(blob))
PY

echo "=== done ==="
