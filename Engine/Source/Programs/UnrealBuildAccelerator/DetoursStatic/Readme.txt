This folder contains code to build UbaStaticStub.bin, a freestanding ELF blob that UbaStaticPatcher injects into statically-linked Linux binaries (Go compile/link, AOSP build tools, glibc-static clang, etc.) so UBA can intercept their I/O without going through ld.so.

To build the stub blob, just run Build.sh from wsl or a linux machine

Note, you might need to run these first

sudo apt update
sudo apt install --yes clang lld llvm python3
CLANG=$(command -v clang) CLANGXX=$(command -v clang++) LD=$(command -v ld.lld) OBJCOPY=$(command -v llvm-objcopy) LLVMNM=$(command -v llvm-nm) bash Build.sh

Without the env vars, Build.sh defaults to the AOSP prebuilt clang under ~/git/android/prebuilts/clang/host/linux-x86/clang-r547379/. That toolchain is what produced the target binaries we inject into, so it is the most-tested path — but any recent clang with lld and llvm-objcopy will work.

Output lands at ../../../../Binaries/Linux/UnrealBuildAccelerator/UbaStaticStub.bin and is checked in to p4 alongside the other UBA Linux binaries.
