// Copyright Epic Games, Inc. All Rights Reserved.
// glibc-static detour bridge implementation. See UbaGlibc.h for the contract.

#include <stddef.h>

#include "UbaGlibc.h"
#include "UbaDetoursShared.h"
#include "UbaDirectoryTable.h"
#include "UbaDetoursFileMappingTable.h"
#include "PosixOS/UbaDetoursSharedPosix.h"
#include "PosixOS/UbaDetoursRealPosix.h"

using namespace uba;

extern "C" u32 g_initDone; // stub-init flag; handlers passthrough pre-init (dirtable empty)

// NOT weak — stub linker drops .got so weak-ref guard would read 0.
extern "C" int* __errno_location(void);

namespace
{
	__attribute__((noreturn, noinline, unused))
	void NotReached(const char* what, size_t whatLen)
	{
		static const char pre[] __attribute__((section(".text"))) = "[uba_stub] glibc-handler not-reached: ";
		static const char nl[]  __attribute__((section(".text"))) = "\n";
		RawSyscallDirect(SYS_write, 2, (long)pre,  sizeof(pre) - 1);
		RawSyscallDirect(SYS_write, 2, (long)what, (long)whatLen);
		RawSyscallDirect(SYS_write, 2, (long)nl,   sizeof(nl) - 1);
		RawSyscallDirect(SYS_exit_group, 134, 0, 0); // SIGABRT-ish
		__builtin_unreachable();
	}

	__attribute__((noreturn, noinline, unused))
	void AbortUnsupported(const char* op, size_t opLen)
	{
		static const char pre[] __attribute__((section(".text"))) = "[uba_stub] unsupported glibc call: ";
		static const char post[] __attribute__((section(".text"))) = " — add a handler or teach the patcher to skip it\n";
		RawSyscallDirect(SYS_write, 2, (long)pre,  sizeof(pre) - 1);
		RawSyscallDirect(SYS_write, 2, (long)op,   (long)opLen);
		RawSyscallDirect(SYS_write, 2, (long)post, sizeof(post) - 1);
		RawSyscallDirect(SYS_exit_group, 134, 0, 0);
		__builtin_unreachable();
	}
}

#define STUB_NOT_REACHED(label) do {                                           \
		static const char _stub_msg[] __attribute__((section(".text"))) = label; \
		NotReached(_stub_msg, sizeof(_stub_msg) - 1);                            \
	} while (0)

#define STUB_ABORT_UNSUPPORTED(op) do {                                        \
		static const char _stub_op[] __attribute__((section(".text"))) = #op;    \
		AbortUnsupported(_stub_op, sizeof(_stub_op) - 1);                        \
	} while (0)

static auto StubTrueOpen = [](const char* p, int fl, int m) { return Real_openat(errno, AT_FDCWD, p, fl, m); };

extern "C" __attribute__((used))
int GlibcOpen(const char* path, int flags, int mode)
{
	if (!path || !__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE))
		return Real_openat(errno, AT_FDCWD, path, flags, mode);
	return Shared_open("open", path, flags, mode, StubTrueOpen);
}

// AT_FDCWD or absolute → Shared_open; dirfd-relative paths go kernel-direct (UBA doesn't track dirfds in patched targets).
extern "C" __attribute__((used))
int GlibcOpenat(int dirfd, const char* path, int flags, int mode)
{
	bool ubaHandles = (dirfd == AT_FDCWD) || IsAbsolutePath(path);
	if (!path || !ubaHandles || !__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE))
		return Real_openat(errno, dirfd, path, flags, mode);
	return Shared_open("openat", path, flags, mode, StubTrueOpen);
}

extern "C" __attribute__((used))
long GlibcRead(int fd, void* buf, unsigned long n)
{
	long r = Real_read(errno, fd, buf, n);
	DEBUG_LOG(TC("__libc_read: fd=%d n=%lu -> %lld"), fd, (unsigned long)n, (long long)r);
	return r;
}

extern "C" __attribute__((used))
long GlibcWrite(int fd, const void* buf, unsigned long n)
{
	long r = Real_write(errno, fd, buf, n);
	DEBUG_LOG(TC("__libc_write: fd=%d n=%lu -> %lld"), fd, (unsigned long)n, (long long)r);
	return r;
}

extern "C" __attribute__((used))
int GlibcClose(int fd)
{
	if (!__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE))
		return Real_close(errno, fd);

	auto trueClose = [&]() { return Real_close(errno, fd); };

	auto trueSnapshot = [](int targetFd, u64& outSize, u64& outLastWritten, u32& outAttrs) -> bool
	{
		int dupFd = Real_dup(errno, targetFd);
		if (dupFd < 0)
			return false;
		struct stat sb;
		int r = Real_fstat(errno, dupFd, &sb);
		Real_close(errno, dupFd);
		if (r != 0)
			return false;
		outAttrs       = sb.st_mode;
		outSize        = (u64)sb.st_size;
		outLastWritten = (u64)sb.st_mtim.tv_sec * 10000000ull + (u64)sb.st_mtim.tv_nsec / 100;
		return true;
	};

	int rc = Shared_close("close", fd, trueClose, trueSnapshot);
	DEBUG_LOG(TC("__libc_close: fd=%d -> %d"), fd, rc);
	return rc;
}

extern "C" __attribute__((used))
long GlibcLseek(int fd, long off, int whence)
{
	long r = Real_lseek(errno, fd, off, whence);
	DEBUG_LOG(TC("__libc_lseek: fd=%d off=%lld whence=%d -> %lld"), fd, (long long)off, whence, (long long)r);
	return r;
}

// TODO: synth from dirtable via fd→path instead of SYS_fstat passthrough.
extern "C" __attribute__((used))
int GlibcFxstat(int ver, int fd, void* statbuf)
{
	(void)ver;
	int r = Real_fstat(errno, fd, (struct stat*)statbuf);
	DEBUG_LOG(TC("__fxstat: ver=%d fd=%d -> %d (passthrough)"), ver, fd, r);
	return r;
}

extern "C" __attribute__((used))
void* GlibcMmap(void* addr, unsigned long len, int prot, int flags, int fd, long off)
{
	auto r = RawSyscallDirect(SYS_mmap, (long)addr, (long)len, (long)prot, (long)flags, (long)fd, off);
	void* result;
	if (r.r1 < 0 && r.r1 >= -4095)
	{
		*__errno_location() = (int)-r.r1;
		result = MAP_FAILED;
	}
	else
		result = (void*)r.r1;
	DEBUG_LOG(TC("__libc_mmap: addr=%p len=%lu prot=0x%x flags=0x%x fd=%d off=%lld -> %p"),
		addr, (unsigned long)len, (unsigned)prot, (unsigned)flags, (int)fd, (long long)off, result);
	return result;
}

extern "C" __attribute__((used))
int GlibcAccess(const char* path, int mode)
{
	if (path && __atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE))
	{
		u64 plen = 0;
		while (path[plen] && plen < 4096) ++plen;
		if (plen > 0)
		{
			StringView view(path, (u32)plen);
			StringKey  key  = ToStringKeyNoCheck(path, plen);
			DirectoryTableEntryInformation info;
			if (Rpc_GetEntryInformation(info, key, view, false))
			{
				DEBUG_LOG(TC("__libc_access: %s mode=0x%x -> 0 (UBA exists, attr=0x%x)"),
					path, (unsigned)mode, (u32)info.attributes);
				return 0;
			}
		}
	}
	int r = Real_access(errno, path, mode);
	DEBUG_LOG(TC("__libc_access: %s mode=0x%x -> %d (passthrough)"), path ? path : "(null)", (unsigned)mode, r);
	return r;
}

extern "C" __attribute__((noreturn, used))
void GlibcExit(int status)
{
	DEBUG_LOG(TC("__libc_exit: status=%d"), status);
	if (__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE))
		Shared_SendExit((u32)status);
	for (;;) // loop so compiler can't fall through to adjacent code
		RawSyscallDirect(SYS_exit_group, status, 0, 0);
}

// stat/lstat/fstatat: mirror SYS_newfstatat via SharedStatPath (FixPath/Devirt/dirtable; ENOENT on miss).
// Legacy `__xstat(int ver, ...)` vs modern `stat(...)` is autodetected by LooksLikeXstatVer.

// _STAT_VER is 0..3 so no real char* lives there — discriminates __xstat(ver,...) from stat(...).
namespace { inline bool LooksLikeXstatVer(long maybeVer) { return maybeVer >= 0 && maybeVer < 16; } }

extern "C" __attribute__((used))
int GlibcStat(long arg1, void* arg2, void* arg3)
{
	const char* path; struct stat* sb;
	if (LooksLikeXstatVer(arg1)) { path = (const char*)arg2; sb = (struct stat*)arg3; }
	else                         { path = (const char*)arg1; sb = (struct stat*)arg2; }
	if (!__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE))
		return Real_fstatat(errno, AT_FDCWD, path, sb, 0);
	return SharedStatPath("__xstat", path, sb, true);
}

extern "C" __attribute__((used))
int GlibcLstat(long arg1, void* arg2, void* arg3)
{
	const char* path; struct stat* sb;
	if (LooksLikeXstatVer(arg1)) { path = (const char*)arg2; sb = (struct stat*)arg3; }
	else                         { path = (const char*)arg1; sb = (struct stat*)arg2; }
	if (!__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE))
		return Real_fstatat(errno, AT_FDCWD, path, sb, AT_SYMLINK_NOFOLLOW);
	return SharedStatPath("__lxstat", path, sb, false);
}

extern "C" __attribute__((used))
int GlibcFstatat(long arg1, long arg2, void* arg3, void* arg4, int arg5)
{
	int dirfd; const char* path; struct stat* sb; int flags;
	if (LooksLikeXstatVer(arg1)) // __fxstatat(ver, dirfd, path, buf, flags)
		{ dirfd = (int)arg2; path = (const char*)arg3; sb = (struct stat*)arg4; flags = arg5; }
	else                         // fstatat(dirfd, path, buf, flags)
		{ dirfd = (int)arg1; path = (const char*)arg2; sb = (struct stat*)arg3; flags = (int)(long)arg4; }

	if (!path || !sb || !__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE))
		return Real_fstatat(errno, dirfd, path, sb, flags);

	StringBuffer<> resolvedTemp;
	const char* resolved = ResolvePath(dirfd, path, resolvedTemp);
	if (!resolved)
		return Real_fstatat(errno, dirfd, path, sb, flags);

	return SharedStatPath("__fxstatat", resolved, sb, (flags & AT_SYMLINK_NOFOLLOW) == 0);
}

// unlinkat / renameat / mkdirat dispatch to Shared_<libcname> in .inl (shared with LD_PRELOAD UBA_WRAPPER + stub Go-bridge).

static bool InitDone() { return __atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE) != 0; }

extern "C" __attribute__((used))
int GlibcUnlink(const char* path)
{
	if (!InitDone()) return Real_unlinkat(errno, AT_FDCWD, path, 0);
	return Shared_unlinkat("__unlink", AT_FDCWD, path, 0);
}

extern "C" __attribute__((used))
int GlibcUnlinkat(int dirfd, const char* path, int flags)
{
	if (!InitDone()) return Real_unlinkat(errno, dirfd, path, flags);
	return Shared_unlinkat("__unlinkat", dirfd, path, flags);
}

extern "C" __attribute__((used))
int GlibcRename(const char* oldpath, const char* newpath)
{
	if (!InitDone()) return Real_renameat(errno, AT_FDCWD, oldpath, AT_FDCWD, newpath);
	return Shared_renameat("__rename", AT_FDCWD, oldpath, AT_FDCWD, newpath);
}

extern "C" __attribute__((used))
int GlibcRenameat(int olddirfd, const char* oldpath, int newdirfd, const char* newpath)
{
	if (!InitDone()) return Real_renameat(errno, olddirfd, oldpath, newdirfd, newpath);
	return Shared_renameat("__renameat", olddirfd, oldpath, newdirfd, newpath);
}

extern "C" __attribute__((used))
int GlibcRenameat2(int olddirfd, const char* oldpath, int newdirfd, const char* newpath, unsigned int flags)
{
	(void)flags; // mirroring LD_PRELOAD — flags ignored
	if (!InitDone()) return Real_renameat(errno, olddirfd, oldpath, newdirfd, newpath);
	return Shared_renameat("renameat2", olddirfd, oldpath, newdirfd, newpath);
}

extern "C" __attribute__((used))
int GlibcMkdir(const char* path, unsigned int mode)
{
	if (!InitDone()) return Real_mkdirat(errno, AT_FDCWD, path, mode);
	return Shared_mkdirat("__mkdir", AT_FDCWD, path, mode);
}

extern "C" __attribute__((used))
int GlibcMkdirat(int dirfd, const char* path, unsigned int mode)
{
	if (!InitDone()) return Real_mkdirat(errno, dirfd, path, mode);
	return Shared_mkdirat("__mkdirat", dirfd, path, mode);
}

// x86-64 has no rmdir syscall; route through unlinkat with AT_REMOVEDIR.
extern "C" __attribute__((used))
int GlibcRmdir(const char* path)
{
	if (!InitDone()) return Real_unlinkat(errno, AT_FDCWD, path, AT_REMOVEDIR);
	return Shared_unlinkat("__rmdir", AT_FDCWD, path, AT_REMOVEDIR);
}

// Loud failure beats silent dirtable divergence for any libc op we don't model.
#define UBA_GLIBC_ABORT(camelName, libcName)                                   \
	extern "C" __attribute__((noreturn, used))                                   \
	void Glibc##camelName(void) { STUB_ABORT_UNSUPPORTED(libcName); }

UBA_GLIBC_ABORT(Statx, statx) // TODO: add when on the test path

UBA_GLIBC_ABORT(Link, link)
UBA_GLIBC_ABORT(Linkat, linkat)
UBA_GLIBC_ABORT(Symlink, symlink)
UBA_GLIBC_ABORT(Symlinkat, symlinkat)
UBA_GLIBC_ABORT(Readlink, readlink)
UBA_GLIBC_ABORT(Readlinkat, readlinkat)
UBA_GLIBC_ABORT(Truncate, truncate)
UBA_GLIBC_ABORT(Ftruncate, ftruncate)
UBA_GLIBC_ABORT(Fork, fork)
UBA_GLIBC_ABORT(Vfork, vfork)
UBA_GLIBC_ABORT(Clone, clone)
UBA_GLIBC_ABORT(Clone3, clone3)
UBA_GLIBC_ABORT(Execve, execve)
UBA_GLIBC_ABORT(Execveat, execveat)
UBA_GLIBC_ABORT(Dup, dup)
UBA_GLIBC_ABORT(Dup2, dup2)
UBA_GLIBC_ABORT(Dup3, dup3)
UBA_GLIBC_ABORT(Pipe, pipe)
UBA_GLIBC_ABORT(Pipe2, pipe2)
UBA_GLIBC_ABORT(Opendir, opendir)
UBA_GLIBC_ABORT(Fdopendir, fdopendir)
UBA_GLIBC_ABORT(Readdir, readdir)
UBA_GLIBC_ABORT(Closedir, closedir)
UBA_GLIBC_ABORT(Getdents64, getdents64)

#undef UBA_GLIBC_ABORT

// libc-going chdir() must reach Shared_chdir or it'd hit the unsupported-op abort.
extern "C" __attribute__((used))
int GlibcChdir(const char* path)
{
	DEBUG_LOG(TC("__libc_chdir: %s"), path ? path : "(null)");
	if (!path || !*path) { errno = ENOENT; return -1; }
	int err = 0;
	int rc = Shared_chdir("chdir", path, err);
	if (rc != 0) errno = err ? err : ENOENT;
	return rc;
}

extern "C" __attribute__((used))
int GlibcFchdir(int fd)
{
	DEBUG_LOG(TC("__libc_fchdir: fd=%d"), fd);
	int err = 0;
	int rc = Shared_fchdir(fd, err);
	if (rc != 0) errno = err ? err : EBADF;
	return rc;
}

// .text.uba_handlers keeps it early in the blob; patcher scans for kGlibcHandlerTableMagic at u64
// alignment and stamps `offset` post-link via Build.sh.
extern "C" __attribute__((section(".text.uba_handlers"), used))
const GlibcHandlerTable g_glibcHandlerTable = {
	/* magic    */ kGlibcHandlerTableMagic,
	/* count    */ 49,
	/* reserved */ 0,
	/* entries  */ {
		// --- Real handlers (mapped via kGlibcAliasTable in the patcher) ---
		{ "open",              0, GlibcEntry_HandlerHook },
		{ "openat",            0, GlibcEntry_HandlerHook },
		{ "read",              0, GlibcEntry_HandlerHook },
		{ "write",             0, GlibcEntry_HandlerHook },
		{ "close",             0, GlibcEntry_HandlerHook },
		{ "lseek",             0, GlibcEntry_HandlerHook },
		{ "fxstat",            0, GlibcEntry_HandlerHook },
		{ "mmap",              0, GlibcEntry_HandlerHook },
		{ "access",            0, GlibcEntry_HandlerHook },
		{ "exit",              0, GlibcEntry_HandlerHook },

		// --- Abort handlers for unsupported operations ---
		{ "stat",              0, GlibcEntry_HandlerHook },
		{ "lstat",             0, GlibcEntry_HandlerHook },
		{ "fstatat",           0, GlibcEntry_HandlerHook },
		{ "statx",             0, GlibcEntry_HandlerHook },
		{ "unlink",            0, GlibcEntry_HandlerHook },
		{ "unlinkat",          0, GlibcEntry_HandlerHook },
		{ "rename",            0, GlibcEntry_HandlerHook },
		{ "renameat",          0, GlibcEntry_HandlerHook },
		{ "renameat2",         0, GlibcEntry_HandlerHook },
		{ "link",              0, GlibcEntry_HandlerHook },
		{ "linkat",            0, GlibcEntry_HandlerHook },
		{ "symlink",           0, GlibcEntry_HandlerHook },
		{ "symlinkat",         0, GlibcEntry_HandlerHook },
		{ "readlink",          0, GlibcEntry_HandlerHook },
		{ "readlinkat",        0, GlibcEntry_HandlerHook },
		{ "truncate",          0, GlibcEntry_HandlerHook },
		{ "ftruncate",         0, GlibcEntry_HandlerHook },
		{ "mkdir",             0, GlibcEntry_HandlerHook },
		{ "mkdirat",           0, GlibcEntry_HandlerHook },
		{ "rmdir",             0, GlibcEntry_HandlerHook },
		{ "fork",              0, GlibcEntry_HandlerHook },
		{ "vfork",             0, GlibcEntry_HandlerHook },
		{ "clone",             0, GlibcEntry_HandlerHook },
		{ "clone3",            0, GlibcEntry_HandlerHook },
		{ "execve",            0, GlibcEntry_HandlerHook },
		{ "execveat",          0, GlibcEntry_HandlerHook },
		{ "dup",               0, GlibcEntry_HandlerHook },
		{ "dup2",              0, GlibcEntry_HandlerHook },
		{ "dup3",              0, GlibcEntry_HandlerHook },
		{ "pipe",              0, GlibcEntry_HandlerHook },
		{ "pipe2",             0, GlibcEntry_HandlerHook },
		{ "opendir",           0, GlibcEntry_HandlerHook },
		{ "fdopendir",         0, GlibcEntry_HandlerHook },
		{ "readdir",           0, GlibcEntry_HandlerHook },
		{ "closedir",          0, GlibcEntry_HandlerHook },
		{ "getdents64",        0, GlibcEntry_HandlerHook },
		{ "chdir",             0, GlibcEntry_HandlerHook },
		{ "fchdir",            0, GlibcEntry_HandlerHook },

		// --- Import slot (patcher stamps target's VA at blob+offset) ---
		{ "__errno_location",  0, GlibcEntry_ImportSlot  },
	},
};
