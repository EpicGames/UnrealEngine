// Copyright Epic Games, Inc. All Rights Reserved.
//
// glibc-static detour bridge contract. Patcher locates g_glibcHandlerTable by scanning for kGlibcHandlerTableMagic, then for each entry writes a 5-byte
// JMP rel32 over the libc wrapper's prologue pointing at Glibc<CamelName>, or stamps a function pointer at an ImportSlot offset. Handlers use SysV
// AMD64 directly (no asm shim). Fd tracking uses shared g_fileHandles via uba::CreateFileTableEntry/Shared_close. Unsupported libc ops abort loudly.

#pragma once

#include "UbaBase.h"

namespace uba
{
	using ::uba::u32;
	using ::uba::u64;
}

// HandlerHook: 5-byte JMP rel32 at kGlibcAliasTable[name] → blob_base + offset.
// ImportSlot:  resolve symbol `name`, stamp its 8-byte VA at blob + offset (only __errno_location).

static constexpr uba::u64 kGlibcHandlerTableMagic      = 0x5448424C47414255ULL; // 'UBAGLBHT' LE
static constexpr uba::u32 kGlibcHandlerTableMaxEntries = 64;

enum GlibcEntryKind : uba::u32
{
	GlibcEntry_HandlerHook = 0,
	GlibcEntry_ImportSlot  = 1,
};

struct GlibcHandlerEntry
{
	char       name[24];   // HandlerHook: stripped libc name (e.g. "open"). ImportSlot: exact target symbol name.
	uba::u32   offset;     // bytes from blob base (stamped post-link by Build.sh)
	uba::u32   kind;       // GlibcEntryKind
};

static_assert(sizeof(GlibcHandlerEntry) == 32, "GlibcHandlerEntry must be exactly 32 bytes");

struct GlibcHandlerTable
{
	uba::u64           magic;
	uba::u32           count;
	uba::u32           reserved;
	GlibcHandlerEntry  entries[kGlibcHandlerTableMaxEntries];
};

static_assert(sizeof(GlibcHandlerTable) == 16 + 32 * kGlibcHandlerTableMaxEntries,
	"GlibcHandlerTable must be packed: 16-byte header + 32-byte entries");

extern "C" int   GlibcOpen(const char* path, int flags, int mode);
extern "C" int   GlibcOpenat(int dirfd, const char* path, int flags, int mode);
extern "C" long  GlibcRead(int fd, void* buf, unsigned long n);
extern "C" long  GlibcWrite(int fd, const void* buf, unsigned long n);
extern "C" int   GlibcClose(int fd);
extern "C" long  GlibcLseek(int fd, long off, int whence);
extern "C" int   GlibcFxstat(int ver, int fd, void* statbuf);
extern "C" void* GlibcMmap(void* addr, unsigned long len, int prot, int flags, int fd, long off);
extern "C" int   GlibcAccess(const char* path, int mode);
extern "C" __attribute__((noreturn)) void GlibcExit(int status);

// Abort-on-reach handlers: stderr + exit_group(134). Patcher skips entries the target lacks.
// stat/lstat/fstatat ABI varies (__xstat 3-arg vs stat 2-arg); handler autodetects via LooksLikeXstatVer.
extern "C" int  GlibcStat(long arg1, void* arg2, void* arg3);
extern "C" int  GlibcLstat(long arg1, void* arg2, void* arg3);
extern "C" int  GlibcFstatat(long arg1, long arg2, void* arg3, void* arg4, int arg5);
extern "C" __attribute__((noreturn)) void GlibcStatx(void);
extern "C" int  GlibcUnlink(const char* path);
extern "C" int  GlibcUnlinkat(int dirfd, const char* path, int flags);
extern "C" int  GlibcRename(const char* oldpath, const char* newpath);
extern "C" int  GlibcRenameat(int olddirfd, const char* oldpath, int newdirfd, const char* newpath);
extern "C" int  GlibcRenameat2(int olddirfd, const char* oldpath, int newdirfd, const char* newpath, unsigned int flags);
extern "C" __attribute__((noreturn)) void GlibcLink(void);
extern "C" __attribute__((noreturn)) void GlibcLinkat(void);
extern "C" __attribute__((noreturn)) void GlibcSymlink(void);
extern "C" __attribute__((noreturn)) void GlibcSymlinkat(void);
extern "C" __attribute__((noreturn)) void GlibcReadlink(void);
extern "C" __attribute__((noreturn)) void GlibcReadlinkat(void);
extern "C" __attribute__((noreturn)) void GlibcTruncate(void);
extern "C" __attribute__((noreturn)) void GlibcFtruncate(void);
extern "C" int  GlibcMkdir(const char* path, unsigned int mode); // `unsigned int` avoids pulling in <sys/types.h>; ABI matches libc mode_t.
extern "C" int  GlibcMkdirat(int dirfd, const char* path, unsigned int mode);
extern "C" int  GlibcRmdir(const char* path);
extern "C" __attribute__((noreturn)) void GlibcFork(void);
extern "C" __attribute__((noreturn)) void GlibcVfork(void);
extern "C" __attribute__((noreturn)) void GlibcClone(void);
extern "C" __attribute__((noreturn)) void GlibcClone3(void);
extern "C" __attribute__((noreturn)) void GlibcExecve(void);
extern "C" __attribute__((noreturn)) void GlibcExecveat(void);
extern "C" __attribute__((noreturn)) void GlibcDup(void);
extern "C" __attribute__((noreturn)) void GlibcDup2(void);
extern "C" __attribute__((noreturn)) void GlibcDup3(void);
extern "C" __attribute__((noreturn)) void GlibcPipe(void);
extern "C" __attribute__((noreturn)) void GlibcPipe2(void);
extern "C" __attribute__((noreturn)) void GlibcOpendir(void);
extern "C" __attribute__((noreturn)) void GlibcFdopendir(void);
extern "C" __attribute__((noreturn)) void GlibcReaddir(void);
extern "C" __attribute__((noreturn)) void GlibcClosedir(void);
extern "C" __attribute__((noreturn)) void GlibcGetdents64(void);
extern "C" int GlibcChdir(const char* path);
extern "C" int GlibcFchdir(int fd);

