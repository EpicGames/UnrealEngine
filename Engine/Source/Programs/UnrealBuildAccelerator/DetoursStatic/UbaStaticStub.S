// Copyright Epic Games, Inc. All Rights Reserved.
//
// UbaStaticStub.S — freestanding stub entry point embedded into
// statically-linked ELF binaries by UbaStaticPatcher.
//
// The kernel hands control here via the patched e_entry. We save the
// initial register state (which the target binary's real _start
// depends on), compute envp from the initial stack layout, call into
// C to do the work, restore registers, and jump to the target's
// *original* e_entry (patched in as a 64-bit immediate).
//
// Initial stack layout at process entry (Linux x86-64 ABI):
//   [rsp]          = argc              (Elf64_Xword)
//   [rsp+8]        = argv[0]
//   [rsp+8+8*argc] = NULL
//   [rsp+8+8*argc+8] = envp[0]
//   ...            = envp[N] = NULL
//   ...            = auxv entries
//
// envp pointer = &argv + (argc + 1) = rsp + 8*(argc + 1) + 8
//             = rsp + 8 + 8*argc + 8
//
// The sentinel 0xDEADBEEFCAFEBABE is rewritten by UbaStaticPatcher to
// the target's original e_entry before the blob is embedded.

    // Placed in .text.entry so the linker script keeps this first within
    // the consolidated output .text section (blob offset 0).
    .section .text.entry,"ax",@progbits
    .globl DetourInit
    .hidden DetourInit
    .type DetourInit, @function
DetourInit:
    // Earliest diagnostic: append marker line to /tmp/uba_entry.log proving we reached the patched
    // e_entry. Uses scratch regs only — clobbers restored by the canonical save below.
    pushq %rax                          // save initial regs the kernel set
    pushq %rcx
    pushq %rdx
    pushq %rsi
    pushq %rdi
    pushq %r10
    pushq %r11
    // openat(AT_FDCWD=-100, path, O_WRONLY|O_CREAT|O_APPEND=0x441, 0644=0x1a4)
    movq  $257, %rax                    // SYS_openat
    movq  $-100, %rdi                   // AT_FDCWD
    leaq  EntryPath(%rip), %rsi
    movq  $0x441, %rdx                  // O_WRONLY|O_CREAT|O_APPEND
    movq  $0x1a4, %r10                  // mode 0644
    syscall
    cmpq  $0, %rax
    js    .Luba_entry_skip              // skip on error (rax negative)
    movq  %rax, %r11                    // r11 = fd
    // write(fd, msg, msglen)
    movq  $1, %rax                      // SYS_write
    movq  %r11, %rdi
    leaq  EntryMsg(%rip), %rsi
    movq  $EntryMsgEnd - EntryMsg, %rdx
    syscall
    // close(fd)
    movq  $3, %rax                      // SYS_close
    movq  %r11, %rdi
    syscall
.Luba_entry_skip:
    popq  %r11
    popq  %r10
    popq  %rdi
    popq  %rsi
    popq  %rdx
    popq  %rcx
    popq  %rax

    // Preserve caller-saved regs across the C call. Odd push count restores 16-byte alignment for
    // the System V AMD64 ABI (entry has no return-address slot).
    pushq %rax
    pushq %rcx
    pushq %rdx
    pushq %rsi
    pushq %rdi
    pushq %r8
    pushq %r9
    pushq %r10
    pushq %r11

    // Compute envp from the pre-push rsp value (which sat at argc).
    // After 9 pushes we're 72 bytes lower; the original rsp is at +72.
    movq 72(%rsp), %rcx           // rcx = argc
    leaq 72(%rsp), %rax           // rax = &argc  (original rsp)
    leaq 16(%rax, %rcx, 8), %rdi  // rdi = envp = &argc + 8 + 8*argc + 8

    // call DetourCore(envp)
    call DetourCore

    popq %r11
    popq %r10
    popq %r9
    popq %r8
    popq %rdi
    popq %rsi
    popq %rdx
    popq %rcx
    popq %rax

    // Jump to original e_entry. The 64-bit immediate is the sentinel
    // UbaStaticPatcher rewrites to the real address.
    movabsq $0xDEADBEEFCAFEBABE, %r10
    jmpq *%r10

    .size DetourInit, . - DetourInit

// ---------------------------------------------------------------------------
// SyscallBridge — entered by JMP (not CALL) from a patched
// internal/runtime/syscall.Syscall6 prologue. Registers follow Go's
// register ABI:
//     AX = trap (syscall number)
//     BX = arg1, CX = arg2, DI = arg3, SI = arg4, R8 = arg5, R9 = arg6
//
// We save every GPR (15 regs; rsp is implicit), set up System V AMD64
// ABI to call SyscallDispatch(trap), restore registers, then jmp
// through the trampoline. The trampoline contains the displaced 14
// bytes of Syscall6's prologue followed by a jmp back to Syscall6+14
// so the rest runs unchanged.
//
    .text
    .globl SyscallBridge
    .hidden SyscallBridge
    .type SyscallBridge, @function
SyscallBridge:
    // Multi-threaded: per-call saved mask goes on Go stack; per-call saved rsp on alt-stack itself.
    // Signals MUST be blocked before the spin-acquire — a signal handler invoking Syscall6 would
    // recursively enter this bridge and spin forever waiting for the outer's lock.

    // 0. Bypass syscalls UBA doesn't track. Going through the alt-stack on hot Go-runtime
    //    paths (futex/epoll) opens a Go stack grow/shrink race → AL2023 stack-split panic.
    //    rax holds Go's trap number; do not clobber other regs (still Syscall6 caller state).
    cmpq $202, %rax               // SYS_futex
    je   .Lbypass_to_trampoline
    cmpq $233, %rax               // SYS_epoll_ctl
    je   .Lbypass_to_trampoline
    cmpq $281, %rax               // SYS_epoll_pwait
    je   .Lbypass_to_trampoline
    cmpq $232, %rax               // SYS_epoll_wait (older variant)
    je   .Lbypass_to_trampoline
    cmpq $291, %rax               // SYS_epoll_create1
    je   .Lbypass_to_trampoline
    cmpq $290, %rax               // SYS_eventfd2
    je   .Lbypass_to_trampoline
    // More Go-runtime-heavy syscalls — bridge overhead scales badly under goroutine concurrency.
    cmpq $35,  %rax               // SYS_nanosleep
    je   .Lbypass_to_trampoline
    cmpq $230, %rax               // SYS_clock_nanosleep
    je   .Lbypass_to_trampoline
    cmpq $24,  %rax               // SYS_sched_yield
    je   .Lbypass_to_trampoline
    cmpq $39,  %rax               // SYS_getpid
    je   .Lbypass_to_trampoline
    cmpq $186, %rax               // SYS_gettid
    je   .Lbypass_to_trampoline
    cmpq $228, %rax               // SYS_clock_gettime
    je   .Lbypass_to_trampoline
    cmpq $234, %rax               // SYS_tgkill (Go preemption)
    je   .Lbypass_to_trampoline

    // 1. Save regs the first rt_sigprocmask will clobber: rax/rcx/rdx/rdi/rsi/r10/r11 (7 pushes).
    pushq %rax                    // Go trap
    pushq %rcx                    // Go a2
    pushq %rdx                    // Go ABI scratch (closure ctx in some paths)
    pushq %rdi                    // Go a3
    pushq %rsi                    // Go a4
    pushq %r10                    // Go scratch
    pushq %r11                    // syscall clobbers r11

    // 2. Reserve 8-byte slot on Go stack for the per-call saved signal mask (no global, no race).
    subq $8, %rsp

    // ------------------------------------------------------------------
    // 3. rt_sigprocmask(SIG_SETMASK, &g_fullMask, &savedMask_on_stack, 8)
    //    Blocks all signals for the duration of the bridge body. After
    //    this returns we're safe to spin without deadlock risk.
    // ------------------------------------------------------------------
    movq $14, %rax                // SYS_rt_sigprocmask
    movl $2, %edi                 // SIG_SETMASK
    leaq g_fullMask(%rip), %rsi
    movq %rsp, %rdx               // oldset → Go stack slot at [rsp]
    movl $8, %r10d                // sigsetsize
    syscall

    // 4. Acquire bridge spinlock with stale-lock recovery (~10s timeout — under shorter
    //    timeouts AL2023 preemption can force-clear under live holder → alt-stack reuse → panic).
    //    cmpxchgl uses %eax implicitly; r11 is safe scratch (just clobbered by rt_sigprocmask).
    xorl %r11d, %r11d
.Lspin_acquire:
    xorl %eax, %eax
    movl $1, %ecx
    lock cmpxchgl %ecx, g_bridgeLock(%rip)
    je   .Lgot_lock
    pause
    incl %r11d
    cmpl $0x80000000, %r11d
    jb   .Lspin_acquire
    // Stale-lock recovery: force-clear and retry.
    movl $0, g_bridgeLock(%rip)
    xorl %r11d, %r11d
    jmp  .Lspin_acquire
.Lgot_lock:

    // 5. Switch to alt-stack; reserve [saved Go rsp] and [handled-flag] slots.
    //    Spinlock guarantees only one thread uses alt-stack. rax is scratch (reloaded in step 6).
    movq %rsp, %rax               // rax = Go rsp (pointing at the saved mask slot)
    movq g_altStackTop(%rip), %rsp
    pushq %rax                    // alt-stack: saved Go rsp
    subq $8, %rsp                 // alt-stack: handled-flag slot (written after call)

    // 6. Reload Go ABI regs from Go stack so the 15-reg push below captures real caller state.
    //    Layout at %rax: +0 mask, +8 r11, +16 r10, +24 rsi, +32 rdi, +40 rdx, +48 rcx, +56 rax (Go trap).
    movq  8(%rax), %r11
    movq 16(%rax), %r10
    movq 24(%rax), %rsi
    movq 32(%rax), %rdi
    movq 40(%rax), %rdx
    movq 48(%rax), %rcx
    movq 56(%rax), %rax           // rax = Go trap

    // 7. Save all 15 GPRs on alt-stack. handled-flag lives in its dedicated slot from step 5 —
    //    callee-saved (r14/r15/rbp) must round-trip with Go-caller values intact.
    pushq %rax
    pushq %rbx
    pushq %rcx
    pushq %rdx
    pushq %rdi
    pushq %rsi
    pushq %rbp
    pushq %r8
    pushq %r9
    pushq %r10
    pushq %r11
    pushq %r12
    pushq %r13
    pushq %r14
    pushq %r15
    subq $8, %rsp                 // 16-byte align for the C call

    // 8. Call SyscallDispatch(saved). saved[0] = alignment pad (ignored), saved[1]=r15 … saved[15]=rax.
    movq %rsp, %rdi
    call SyscallDispatch
    // rax = handled flag (0 = passthrough, 1 = substituted return).
    // Stay at the alignment pad — step 9 addresses the handled slot
    // off this rsp, and step 10 discards the pad just before popping.

    // ------------------------------------------------------------------
    // 9. Stash the handled flag into the dedicated handled-flag slot
    //    we reserved on the alt-stack in step 5. After the align pad
    //    is discarded and all 15 regs are popped, rsp lands on this
    //    slot, and we pop it into %r8 (caller-saved in Go's ABI, so
    //    clobbering it across syscall.Syscall6 is safe). Go's r8 value
    //    was already restored by the pop-r8 inside the 15-reg sequence
    //    — we discard it here because the caller of Syscall6 cannot
    //    rely on r8 being preserved across the call anyway. We MUST
    //    NOT overwrite r15's saved slot: r15 is CALLEE-SAVED in Go's
    //    internal ABI (closure-context register), and closure-heavy
    //    programs like soong_zip crash with synthesized SIGSEGV if
    //    r15 is clobbered.
    //
    //    Alt-stack layout at this point (top-to-bottom):
    //       [saved Go rsp]
    //       [handled-flag slot]   ← we write handled here via 128(%rsp)
    //       [15 saved Go regs]    120 bytes
    //       [align pad]           8 bytes  ← rsp currently
    // ------------------------------------------------------------------
    movq %rax, 128(%rsp)          // write handled into the dedicated slot

    // ------------------------------------------------------------------
    // 10. Discard the alignment pad and pop the 15 saved regs.
    //     Every Go ABI reg (including r14 = g and r15 = closure ctx)
    //     gets its caller-original value back here.
    // ------------------------------------------------------------------
    addq $8, %rsp                 // discard alignment pad
    popq %r15
    popq %r14
    popq %r13
    popq %r12
    popq %r11
    popq %r10
    popq %r9
    popq %r8
    popq %rbp
    popq %rsi
    popq %rdi
    popq %rdx
    popq %rcx
    popq %rbx
    popq %rax                     // rax = Go trap, or synthesized r1 if handled

    // ------------------------------------------------------------------
    // 11. Pop handled flag into %r11. Earlier this step popped into %r8
    //     on the assumption that R8 was caller-saved across Syscall6.
    //     That's wrong: R8 is an INPUT to the syscall body itself
    //     (e.g. mmap's fd is passed in R8). Clobbering it with the
    //     handled flag broke any goroutine syscall whose 5th argument
    //     mattered — observed as Go compile mmap'ing import .a files
    //     with fd=0 and getting ENODEV ("could not import fmt: no such
    //     device"). R11 is safe: SYSCALL itself clobbers R11 with
    //     rflags, and Go's Syscall6 prologue never reads it, so the
    //     caller cannot have a live R11 value across the call.
    // ------------------------------------------------------------------
    popq %r11                     // r11 = handled flag (was: r8)
    popq %rsp                     // switch to Go stack (= &saved_mask_slot)
    movl $0, g_bridgeLock(%rip)   // release (plain store; x86 is TSO)

    // ------------------------------------------------------------------
    // 12. rt_sigprocmask(SIG_SETMASK, &savedMask_on_stack, NULL, 8).
    //     Save the regs the syscall will touch (rcx clobbered by SYSCALL
    //     return-rip; r11 clobbered by SYSCALL rflags AND now carrying
    //     our handled flag). After this, signals are restored. R8 is
    //     no longer pushed — it holds Go's a5 (just restored by step 10)
    //     and SYSCALL preserves it.
    // ------------------------------------------------------------------
    pushq %rax
    pushq %rcx
    pushq %rdx
    pushq %rdi
    pushq %rsi
    pushq %r10
    pushq %r11                    // save handled flag across SYSCALL

    movq $14, %rax                // SYS_rt_sigprocmask
    movl $2, %edi                 // SIG_SETMASK
    leaq 56(%rsp), %rsi           // &saved_mask is 56 bytes above post-push rsp (7 pushes)
    xorl %edx, %edx               // oldset = NULL
    movl $8, %r10d                // sigsetsize
    syscall

    popq %r11                     // restore handled flag
    popq %r10
    popq %rsi
    popq %rdi
    popq %rdx
    popq %rcx
    popq %rax

    // ------------------------------------------------------------------
    // 13. Deallocate the 64 bytes we used on the Go stack up front
    //     (7 saved regs + 8-byte savedMask slot). The regs' values are
    //     already in the real registers — this just frees the memory.
    // ------------------------------------------------------------------
    addq $64, %rsp

    // ------------------------------------------------------------------
    // 14. Dispatch on r11 (handled flag).
    //
    //   r11 == 0 (default passthrough): CALL (not JMP) the trampoline
    //   so we regain control after the real syscall. The trampoline
    //   contains the displaced 14 bytes of Syscall6's prologue followed
    //   by a jmp back to Syscall6+14, so the rest of the original
    //   function runs unchanged (incl. CALL runtime.entersyscall in the
    //   prologue and CALL runtime.exitsyscall in the epilogue) and rets
    //   back here. rax/rbx/rcx already hold Go's ABI return values at
    //   that point. R8 holds Go's a5 (preserved through steps 10-13),
    //   which the trampoline reads for syscalls like mmap (fd argument).
    //
    //   r11 == 1 (synthesized): the pre-hook wrote rax=r1, rbx=r2,
    //   rcx=errno into saved[] (already popped in step 10). Skip the
    //   trampoline entirely and ret straight to Go's caller. Used for
    //   stat-family syscalls where UBA's directory table is the source
    //   of truth.
    //
    //   r11 == 2 (trampoline + post-hook): run the real syscall via the
    //   trampoline (so Go's entersyscall/exitsyscall fire normally),
    //   then call SyscallPostHook BEFORE returning. The post-hook
    //   calls Rpc_UpdateTables to refresh the local dirtable from the
    //   host. Used for wait4/waitid: a child has just been reaped and
    //   its outputs may now be visible in the host's overlay. We MUST
    //   preserve rax (Go r1) and rcx (Go err); rbx (Go r2) survives the
    //   C call automatically because rbx is callee-saved in SysV ABI.
    //
    //   Stack alignment: at this point rsp = Syscall6 caller's return
    //   address (rsp ≡ 8 mod 16). Two 8-byte pushes leave rsp ≡ 8 mod 16
    //   still; the extra `subq $8` aligns to 0 mod 16 so the post-hook
    //   sees rsp ≡ 8 mod 16 at function entry as required by SysV.
    // ------------------------------------------------------------------
    cmpq  $1, %r11
    je    .Lhandled                  // r11 == 1: skip trampoline
    cmpq  $2, %r11
    je    .Lpost_hook                // r11 == 2: trampoline + post-hook

    // r11 == 0 (default): trampoline only.
    callq *g_syscall6Trampoline(%rip)
    jmp   .Lhandled

.Lpost_hook:
    callq *g_syscall6Trampoline(%rip)
    pushq %rax                       // preserve Go r1
    pushq %rcx                       // preserve Go err
    subq  $8, %rsp                   // align to 16 for the C call
    callq SyscallPostHook
    addq  $8, %rsp
    popq  %rcx
    popq  %rax
    // fall through to .Lhandled

.Lhandled:
#if UBA_BRIDGE_SANITY
    // Bridge-return sanity: (1) rsp NOT in alt-stack range, (2) rsp in [g.stack.lo, g.stack.hi).
    // Failures here mean the next Go prologue's morestack check will misfire as the
    // AL2023 "stack split at bad time" panic. r10/rax are caller-saved scratch.
    pushq %r10
    pushq %rax

    // (1) rsp NOT in alt-stack
    movq g_altStackBase(%rip), %r10
    cmpq %r10, %rsp
    jb   .Lsp_check_g            // rsp < base → safe so far, fall through to check (2)
    movq g_altStackTop(%rip), %r10
    cmpq %r10, %rsp
    jae  .Lsp_check_g            // rsp >= top → safe so far, fall through to check (2)
    jmp  .Lsp_bad

.Lsp_check_g:
    // (2) rsp ∈ [g.stack.lo, g.stack.hi). g layout: stack.lo=0(%r14), stack.hi=8(%r14).
    movq 0(%r14), %r10            // r10 = gp.stack.lo
    cmpq %r10, %rsp
    jb   .Lsp_bad                 // rsp < stack.lo → bad
    movq 8(%r14), %r10            // r10 = gp.stack.hi
    cmpq %r10, %rsp
    ja   .Lsp_bad                 // rsp > stack.hi → bad
    // Both checks passed.
    popq %rax
    popq %r10
    jmp  .Lsp_ok

.Lsp_bad:
    // SIGABRT via raw kill(getpid, SIGABRT) for clean coredump.
    movq $39, %rax                // SYS_getpid
    syscall
    movq %rax, %rdi               // pid
    movl $6,   %esi               // SIGABRT
    movq $62,  %rax               // SYS_kill
    syscall
    ud2

.Lsp_ok:
#endif
    // Go's amd64 register ABI requires XMM15 = 0 at function-call
    // boundaries. Go's compiler emits `MOVUPS XMM15, [mem]` to zero
    // 16-byte memory regions, relying on this invariant. Our C code
    // (notably BLAKE3 SIMD inside ToStringKey) clobbers XMM15 with
    // its ROT16 shuffle constant 0x0504070601000302... — if we don't
    // reset XMM15 before returning, Go's next 16-byte zero-store
    // writes that constant into a slice header / map header,
    // producing 'index out of range [N] with length N' panics in
    // downstream Go code (observed as esbuild lazybuf corruption).
    // vzeroupper clears upper YMM bits (AVX libc leakage corrupts Go's 32-byte zero-stores).
    vzeroupper
    pxor  %xmm15, %xmm15
    cld   // SysV ABI: DF=0 at call boundaries (rep movs/stos)
    ret

.Lbypass_to_trampoline:
    // JMP into the trampoline. Caller-state untouched; Go sees no bridge frame on the chain.
    jmp *g_syscall6Trampoline(%rip)

    .size SyscallBridge, . - SyscallBridge

// ---------------------------------------------------------------------------
// ExitBridge — entered by JMP from the patched runtime.exit.abi0
// prologue. The stack layout at entry is identical to what the original
// function sees: return address on top, caller-allocated stack args above.
// Go's runtime.exit.abi0 takes the exit code as its first stack arg:
//     [rsp]     = return address
//     [rsp+8]   = int32 code (the ABI passes it in a 64-bit slot)
//
// We call ExitCore(code) which sends MessageType_Exit via the shared
// shm/event plumbing, then issues SYS_exit_group itself. Does not return.
//
    .text
    .globl ExitBridge
    .hidden ExitBridge
    .type ExitBridge, @function
ExitBridge:
    movl 8(%rsp), %edi            // exit code → arg0 (SysV ABI)
    // Align the stack down to 16 bytes before the call. Go's runtime arrives
    // here with rsp at address+0 mod 16 - 8 (the implicit return addr bump).
    andq $-16, %rsp
    call ExitCore
    // Not reached — ExitCore never returns.
    ud2

    .size ExitBridge, . - ExitBridge

// VforkBridge — JMP target from patched syscall.rawVforkSyscall.abi0.
// Switches to alt-stack (Go goroutine stack is too small for Shared_fork's RPC + DEBUG_LOG),
// calls StaticForkForBridge (which runs Shared_fork: ForkProcess RPC + SYS_fork + StartProcess),
// returns to Go ABI ((r1,err) at rsp+40/+48). Plain SYS_fork — vfork shared-memory + chdir would corrupt parent state.
// Sigprocmask block during the C call keeps Go's signal handler off the alt-stack.
//
// Stack on entry: rsp+0=retaddr, rsp+8..32=args (ignored), rsp+40=r1 slot, +48=err slot.
    .text
    .globl VforkBridge
    .hidden VforkBridge
    .type VforkBridge, @function
VforkBridge:
    pushq %r12
    pushq %rbx

    leaq 56(%rsp), %r12                // r12 = &caller.r1_slot
    subq $8, %rsp                      // sigmask slot at (%rsp)

    // Block signals.
    movq $14, %rax                     // SYS_rt_sigprocmask
    movl $2, %edi                      // SIG_SETMASK
    leaq g_fullMask(%rip), %rsi
    movq %rsp, %rdx
    movl $8, %r10d
    syscall

    // Acquire spinlock with stale-lock recovery (~10s timeout; mirrors syscall6 bridge).
    xorl %r11d, %r11d
.Lvfork_spin_acquire:
    xorl %eax, %eax
    movl $1, %ecx
    lock cmpxchgl %ecx, g_bridgeLock(%rip)
    je   .Lvfork_got_lock
    pause
    incl %r11d
    cmpl $0x80000000, %r11d
    jb   .Lvfork_spin_acquire
    movl $0, g_bridgeLock(%rip)
    xorl %r11d, %r11d
    jmp  .Lvfork_spin_acquire
.Lvfork_got_lock:

    // Switch to alt-stack.
    movq %rsp, %rbx
    movq g_altStackTop(%rip), %rsp
    andq $-16, %rsp
    subq $8, %rsp

    call StaticForkForBridge

    movq %rbx, %rsp

    // Release lock.
    movl $0, g_bridgeLock(%rip)

    // Restore sigmask in BOTH parent and child. execve preserves the mask, so leaving the
    // bridge's full-block in the child starves every descendant of SIGTERM (py Pool.terminate).
    pushq %rax
    movq $14, %rax
    movl $2, %edi
    leaq 8(%rsp), %rsi
    xorl %edx, %edx
    movl $8, %r10d
    syscall
    popq %rax

    addq $8, %rsp                      // drop sigmask slot

    // Translate pid to Go's (r1, err) and write to caller's slots.
    cmpq $0xfffffffffffff001, %rax
    jbe  .Lvfork_ok

    movq $-1, (%r12)
    negq %rax
    movq %rax, 8(%r12)
    jmp  .Lvfork_done

.Lvfork_ok:
    movq %rax, (%r12)
    movq $0, 8(%r12)

.Lvfork_done:
    popq %rbx
    popq %r12
#if UBA_BRIDGE_SANITY
    // Mirror .Lhandled: rsp must NOT be in alt-stack on return (else AL2023 stack-split panic).
    pushq %r10
    pushq %rax
    movq g_altStackBase(%rip), %r10
    cmpq %r10, %rsp
    jb   .Lvfork_sp_ok
    movq g_altStackTop(%rip), %r10
    cmpq %r10, %rsp
    jae  .Lvfork_sp_ok
    movq $39, %rax                // SYS_getpid
    syscall
    movq %rax, %rdi
    movl $6,   %esi               // SIGABRT
    movq $62,  %rax               // SYS_kill
    syscall
    ud2
.Lvfork_sp_ok:
    popq %rax
    popq %r10
#endif
    // Go ABI: XMM15=0 + clean upper YMMs at call boundaries (C code clobbers them).
    // Without this, Go's next zero-store corrupts m.stackguard0 → latent stack-split panic.
    vzeroupper
    pxor  %xmm15, %xmm15
    cld   // SysV ABI: DF=0
    ret

    .size VforkBridge, . - VforkBridge

// ---------------------------------------------------------------------------
// CrashRestorer — sa_restorer trampoline for the rt_sigaction(2) calls
// installed by InstallCrashHandlers (UbaStaticStubCore.cpp). The kernel
// transfers control here when the signal handler returns; this function MUST
// not return — its only job is to invoke SYS_rt_sigreturn (15), which restores
// the saved register context and resumes the interrupted code.
//
// Placing it in .text so the linker script keeps it inside the relocatable
// blob alongside the rest of the stub.
//
    .text
    .globl CrashRestorer
    .hidden CrashRestorer
    .type CrashRestorer, @function
CrashRestorer:
    movq $15, %rax              // SYS_rt_sigreturn
    syscall
    ud2                         // unreachable; sigreturn never returns to caller
    .size CrashRestorer, . - CrashRestorer

// ---------------------------------------------------------------------------
// Read-only data used by the early entry diagnostic at DetourInit.
// Keep in .text so the linker script consolidates it into the blob.
//
    .section .text.entry,"ax",@progbits
    .balign 1
EntryPath:
    .ascii "/tmp/uba_entry.log\0"
EntryMsg:
    .ascii "[uba-entry] DetourInit reached\n"
EntryMsgEnd:

    .section .note.GNU-stack,"",@progbits
