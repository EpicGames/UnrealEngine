/* Linker script for the UBA static-detour stub.
 *
 * Packs .text + .rodata + .data + .bss into a single output .text
 * section so `objcopy -O binary --only-section=.text` produces a
 * contiguous blob whose %rip-relative addressing is intact.
 *
 * .bss is materialised as zero bytes in the blob (normally NOBITS) so
 * the stub's writable globals have real backing memory when the blob
 * is injected into a target binary.
 *
 * Section ordering:
 *   1. .text.entry              — DetourInit MUST sit at offset 0
 *                                  within the blob (the patcher rewrites
 *                                  e_entry to point here).
 *   2. .text.uba_handlers       — Phase 0 glibc-static handler table
 *                                  (g_glibcHandlerTable). Placed early so
 *                                  it's near the head of the blob, but
 *                                  NOT pinned at a fixed offset — the
 *                                  patcher resolves it by scanning for
 *                                  kGlibcHandlerTableMagic. Going via the
 *                                  magic-search route lets us keep the
 *                                  linker script trivial; if a future
 *                                  agent wants a fixed offset they can
 *                                  use BYTE(0) padding to pin it.
 *   3. .text .text.*            — everything else (handlers, helpers,
 *                                  shared UBA core).
 *   4. .rodata / .data / .bss   — folded into .text so the single-
 *                                  section objcopy captures them all.
 */
SECTIONS
{
    /DISCARD/ : { *(.note.*) *(.eh_frame*) *(.gcc_except_table*) *(.comment) }

    .text : {
        KEEP(*(.text.entry))
        KEEP(*(.text.uba_handlers))
        *(.text .text.*)
        *(.rodata .rodata.*)
        *(.data .data.*)
        *(.bss .bss.*)
    }
}
INSERT AFTER .dynstr;
