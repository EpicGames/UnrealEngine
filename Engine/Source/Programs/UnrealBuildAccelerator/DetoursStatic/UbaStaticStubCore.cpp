// Copyright Epic Games, Inc. All Rights Reserved.
//
// Freestanding C++ core for the static-detour stub. Called from DetourInit (asm).
// Constraints: no libc, raw SYSCALL only. String literals via STUB_STR so `objcopy -O binary --only-section=.text` captures them.

#include <stddef.h>

#include "UbaBase.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaDirectoryTable.h"
#include "UbaEvent.h"
#include "UbaFileMapping.h"
#include "UbaLinuxRawSyscall.h"
#include "UbaMemory.h"
#include "UbaProcessStats.h"
#include "UbaProtocol.h"
#include "UbaDetoursFileMappingTable.h"
#include "PosixOS/UbaDetoursSharedPosix.h"
#include "PosixOS/UbaDetoursRealPosix.h"
using namespace uba;

#include <new>

typedef unsigned long uintptr_t;

// Non-private futex: UBA crosses process boundaries (server + stubbed child share the shm mapping).
#ifndef FUTEX_WAIT
#define FUTEX_WAIT 0
#endif
#ifndef FUTEX_WAKE
#define FUTEX_WAKE 1
#endif

#define PAGE_SIZE			0x1000
#define UBA_COMM_MEM_SIZE	(64u * 1024u * 2u) // matches CommunicationMemSize in UbaProtocol.h

#define STUB_STR(name, lit) static const char name[] __attribute__((section(".text"), used)) = lit

// Init-time chatter spams stderr across thousands of static-detoured invocations per build. Build.sh `-DUBA_STUB_VERBOSE=1` re-enables.
#ifndef UBA_STUB_VERBOSE
#define UBA_STUB_VERBOSE 0
#endif
#if UBA_STUB_VERBOSE
#define STUB_INIT_LOG(buf, len) StubWrite(2, (buf), (len))
#else
// (void)sizeof so the args stay "used" — keeps -Werror=unused-variable happy.
#define STUB_INIT_LOG(buf, len) ((void)sizeof((buf)), (void)sizeof((len)))
#endif

STUB_STR(s_envPrefix,    "UBA_COMID=");
STUB_STR(s_envLogPrefix, "UBA_LOGFILE=");
STUB_STR(s_envCwdPrefix, "UBA_CWD=");
STUB_STR(s_envRemotePrefix, "UBA_REMOTE=");
STUB_STR(s_envFileMapDirPrefix, "UBA_FILE_MAPPING_DIR=");
STUB_STR(s_envMemfdPrefix, "UBA_FILE_MAPPING_MEMFD=");

// Backing the stub attaches through, matching the host. FmBackingDir/FmUseMemfd read these (stub getenv()
// is a nullptr stub); set by the envp scan before the first FileMapping_MapFromHost.
namespace uba { const char* g_ubaFileMappingDir = nullptr; bool g_ubaUseMemfd = false; }
STUB_STR(s_msgNoEnv,    "[uba_stub] no UBA_COMID; pass-through\n");
STUB_STR(s_msgBadVal,   "[uba_stub] UBA_COMID malformed (expected <uid>+<offset>)\n");
STUB_STR(s_msgOpenFail, "[uba_stub] shm openat failed\n");
STUB_STR(s_msgMapFail,  "[uba_stub] mmap failed\n");
STUB_STR(s_msgAttached,  "[uba_stub] attached to ");
STUB_STR(s_msgNoSyscall, "[uba_stub] Syscall6 address not baked; passing through\n");
STUB_STR(s_msgSyscall,   "[uba_stub] Syscall6 @ 0x");
STUB_STR(s_msgTrampFail, "[uba_stub] trampoline mmap failed\n");
STUB_STR(s_msgMprotFail, "[uba_stub] mprotect failed\n");
STUB_STR(s_msgHooked,    "[uba_stub] Syscall6 hook installed\n");
STUB_STR(s_msgEventTags, "[uba_stub] shm event tags: cancel=");
STUB_STR(s_msgShmPtr,    "[uba_stub] shm ptr=0x");
STUB_STR(s_msgSyscallN,  "[syscall ");
STUB_STR(s_msgInitSent,  "[uba_stub] Init RPC sent\n");
STUB_STR(s_msgInitResp,  "[uba_stub] Init resp: ");
STUB_STR(s_msgInitTimeout, "[uba_stub] Init response timeout\n");
STUB_STR(s_logPayload,     "Hello from UbaStaticStub");
STUB_STR(s_nl,           "\n");

// Patcher bake slots: looks up the target symbol's VA and rewrites .addr.
// Each {sentinel, guard} pair is searched as 16 bytes — unique pattern in the blob.
// volatile prevents the compiler constant-folding references back into immediates.

struct UbaStubSyscall6Slot { volatile u64 addr; volatile u64 guard; };
static UbaStubSyscall6Slot g_syscall6 = { 0xDEADBEEFC0FFEE00ULL, 0xFEEDFACE12345678ULL };
#define g_syscall6Addr (g_syscall6.addr)

// runtime.exit.abi0 issues SYS_exit_group inline (bypasses Syscall6). Choke point so we can send MessageType_Exit first.
struct UbaStubExitSlot { volatile u64 addr; volatile u64 guard; };
static UbaStubExitSlot g_runtimeExit = { 0xDEADBEEFEEEEE500ULL, 0xCAFEBABE87654321ULL };
#define g_runtimeExitAddr (g_runtimeExit.addr)

// vfork redirected to plain SYS_fork (CoW) so child doesn't share parent pages.
struct UbaStubVforkSlot { volatile u64 addr; volatile u64 guard; };
static UbaStubVforkSlot g_vforkSyscall = { 0xDEADBEEF7AC0FFEEULL, 0xBADCAFE0BADCAFE1ULL };
#define g_vforkSyscallAddr (g_vforkSyscall.addr)
#define g_vforkSyscallSentinel (0xDEADBEEF7AC0FFEEULL)

// RawSyscall0/1/3/6 live in UbaLinuxRawSyscall.h (cross-arch via RawSyscallDirect).

static void StubWrite(int fd, const char* s, size_t n)
{
	RawSyscall3(SYS_write, (s64)fd, (s64)s, (s64)n);
}

// Routes to UBA_LOGFILE if set, drops otherwise. Use for per-syscall traces. Errors still go to stderr via StubWrite(2, ...).
static void StubDbgWrite(const char* s, size_t n)
{
	if ((int)g_debugFile < 0) return;
	RawSyscall3(SYS_write, (s64)g_debugFile, (s64)s, (s64)n);
}

// Freestanding replacements: clang lowers to memcpy/memset/etc. even in -ffreestanding.
extern "C" void* memcpy(void* dst, const void* src, size_t n)
{
	u8* d = (u8*)dst; const u8* s = (const u8*)src;
	for (size_t i = 0; i < n; ++i) d[i] = s[i];
	return dst;
}
extern "C" void* memset(void* dst, int c, size_t n)
{
	u8* d = (u8*)dst;
	for (size_t i = 0; i < n; ++i) d[i] = (u8)c;
	return dst;
}
extern "C" void* memmove(void* dst, const void* src, size_t n)
{
	u8* d = (u8*)dst; const u8* s = (const u8*)src;
	if (d < s) for (size_t i = 0; i < n; ++i) d[i] = s[i];
	else       for (size_t i = n; i-- > 0; ) d[i] = s[i];
	return dst;
}
extern "C" int memcmp(const void* a, const void* b, size_t n)
{
	const u8* x = (const u8*)a; const u8* y = (const u8*)b;
	for (size_t i = 0; i < n; ++i) { if (x[i] != y[i]) return x[i] - y[i]; }
	return 0;
}
extern "C" size_t strlen(const char* s)
{
	const char* p = s;
	while (*p) ++p;
	return (size_t)(p - s);
}
// __asm__("name") bypasses glibc's <string.h> const-char C++ overload inlines so we map to the char* symbol-table entries.
extern "C" char* __stub_strchr(const char* s, int c) __asm__("strchr");
extern "C" char* __stub_strchr(const char* s, int c)
{
	for (; *s; ++s) if (*s == (char)c) return (char*)s;
	return (c == 0) ? (char*)s : nullptr;
}
extern "C" char* __stub_strrchr(const char* s, int c) __asm__("strrchr");
extern "C" char* __stub_strrchr(const char* s, int c)
{
	const char* last = nullptr;
	for (; *s; ++s) if (*s == (char)c) last = s;
	if (c == 0) return (char*)s;
	return (char*)last;
}
extern "C" int strcmp(const char* a, const char* b)
{
	while (*a && *a == *b) { ++a; ++b; }
	return (unsigned char)*a - (unsigned char)*b;
}
extern "C" char* __stub_strstr(const char* h, const char* n) __asm__("strstr");
extern "C" char* __stub_strstr(const char* h, const char* n)
{
	if (!*n) return (char*)h;
	for (; *h; ++h)
	{
		const char* hh = h; const char* nn = n;
		while (*hh && *nn && *hh == *nn) { ++hh; ++nn; }
		if (!*nn) return (char*)h;
	}
	return nullptr;
}

// Forward decl — flushes open write fds before abort so we don't leave InvalidValue placeholders on the host.
static void StubFlushHandlesAtExit(const char* reason);

__attribute__((noreturn, noinline))
static void StubNotReached(const char* what, size_t whatLen)
{
	static const char pre[] __attribute__((section(".text"))) = "[uba_stub] not-reached: ";
	static const char nl[]  __attribute__((section(".text"))) = "\n";
	StubWrite(2, pre, sizeof(pre) - 1);
	StubWrite(2, what, whatLen);
	StubWrite(2, nl, 1);
	StubFlushHandlesAtExit("StubNotReached");
	RawSyscall1(231 /* SYS_exit_group */, 1);
	__builtin_unreachable();
}
#define STUB_NOT_REACHED(label) do { static const char _stub_msg[] __attribute__((section(".text"))) = label; StubNotReached(_stub_msg, sizeof(_stub_msg) - 1); } while (0)

// UbaParkingLot::Table::Create calls aligned_alloc for the Futex lock table — abort-on-reach would kill every Go binary using UBA's Futex.
extern "C" void* aligned_alloc(size_t alignment, size_t size) { return StubAllocatorAllocate(size, alignment); }
extern "C" void free(void*) {} // bump allocator is process-lifetime

// Freestanding libc-like wrappers (we build with -nostdlib; shared UBA core calls these by POSIX names).
extern "C" void* mmap(void* addr, size_t len, int prot, int flags, int fd, long off)
{
	// raw mmap returns -errno in -4096..-1; collapse to MAP_FAILED.
	s64 r = RawSyscall6(SYS_mmap, (s64)addr, (s64)len, (s64)prot, (s64)flags, (s64)fd, (s64)off);
	return (r < 0 && r >= -4096) ? MAP_FAILED : (void*)r;
}
extern "C" int munmap(void* addr, size_t len) { return (int)RawSyscall6(11 /* SYS_munmap */, (s64)addr, (s64)len, 0, 0, 0, 0); }
extern "C" int close(int fd)                  { return (int)RawSyscall1(SYS_close, fd); }

extern "C" long syscall(long num, ...)
{
	va_list ap; va_start(ap, num);
	s64 a1 = va_arg(ap, s64); s64 a2 = va_arg(ap, s64); s64 a3 = va_arg(ap, s64);
	s64 a4 = va_arg(ap, s64); s64 a5 = va_arg(ap, s64); s64 a6 = va_arg(ap, s64);
	va_end(ap);
	return (long)RawSyscall6(num, a1, a2, a3, a4, a5, a6);
}
extern "C" int clock_gettime(int clk_id, struct timespec* tp) { return (int)RawSyscall3(228 /* SYS_clock_gettime */, (s64)clk_id, (s64)tp, 0); }

// Glibc-static targets have their own per-thread __errno_location returning a TLS int. Our glibc handlers (UbaGlibc.cpp) must
// set THAT int, not our own — but the target's address is unknown at the stub's link time. Patcher writes the target's
// __errno_location VA into g_glibcErrnoHookFn via the ImportSlot mechanism. Not const so it lands in .data (folded into .text by the linker script).
extern "C" int* (*g_glibc_errno_hook_fn)(void) __attribute__((used)) = nullptr;

extern "C" int* __errno_location()
{
	if (auto fn = __atomic_load_n(&g_glibc_errno_hook_fn, __ATOMIC_ACQUIRE))
		return fn();
	static int errval = 0;
	return &errval;
}
// Stub-side __tls_get_addr. clang emits callq __tls_get_addr for thread_locals (GD model) but we have no ld.so. We claim %gs
// (unused by Go/glibc; they use %fs) and install per-thread blocks tagged with the owner tid. Mismatch = fork-inherited %gs;
// allocate fresh. Separate arena from StubAllocator so TLS allocation can't recurse into anything needing TLS.
#define ARCH_SET_GS 0x1001
#define ARCH_GET_GS 0x1004

namespace
{
	constexpr u32 kStubTlsBlockSize  = 256;
	constexpr u32 kStubTlsHeaderSize = 8;
	constexpr u64 kStubTlsArenaSize  = 128 * 1024;
	static u8*              g_stubTlsArenaBase = nullptr;
	static std::atomic<u32> g_stubTlsArenaPos { 0 };
	alignas(64) static u8   g_stubTlsFallback[kStubTlsBlockSize] = {}; // arch_prctl-blocked sandboxes fall here; shared, wrong with >1 thread
}

static u8* StubTlsArenaAlloc()
{
	u8* base = __atomic_load_n(&g_stubTlsArenaBase, __ATOMIC_ACQUIRE);
	if (!base)
	{
		u8* fresh = (u8*)mmap(nullptr, kStubTlsArenaSize, PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANONYMOUS, -1, 0);
		if (fresh == MAP_FAILED) return nullptr;
		u8* expected = nullptr;
		if (__atomic_compare_exchange_n(&g_stubTlsArenaBase, &expected, fresh, false, __ATOMIC_ACQ_REL, __ATOMIC_ACQUIRE))
			base = fresh;
		else { munmap(fresh, kStubTlsArenaSize); base = expected; }
	}
	u32 off = g_stubTlsArenaPos.fetch_add(kStubTlsBlockSize, std::memory_order_acq_rel);
	if (off + kStubTlsBlockSize > kStubTlsArenaSize) return nullptr;
	return base + off;
}

extern "C" void* __tls_get_addr(void* arg)
{
	struct TlsIndex { unsigned long mod; unsigned long off; };
	auto* idx = (TlsIndex*)arg;
	const u64 off = idx ? idx->off : 0;
	if (off + kStubTlsHeaderSize >= kStubTlsBlockSize)
		STUB_NOT_REACHED("stub __tls_get_addr: TLS offset overflow");

	const u32 tid = (u32)RawSyscall0(186 /* SYS_gettid */);

	u64 gs_base = 0;
	s64 r = RawSyscall3(158 /* SYS_arch_prctl */, (s64)ARCH_GET_GS, (s64)&gs_base, 0);
	if (r >= 0 && gs_base != 0)
	{
		u32 owner = __atomic_load_n((u32*)gs_base, __ATOMIC_ACQUIRE);
		if (owner == tid)
			return (void*)(gs_base + kStubTlsHeaderSize + off);
	}

	u8* block = StubTlsArenaAlloc();
	if (!block)
		return &g_stubTlsFallback[kStubTlsHeaderSize + off];

	for (u32 i = 0; i < kStubTlsBlockSize; ++i) block[i] = 0;
	__atomic_store_n((u32*)block, tid, __ATOMIC_RELEASE);

	s64 setRet = RawSyscall3(158, (s64)ARCH_SET_GS, (s64)block, 0);
	if (setRet < 0)
		return &g_stubTlsFallback[kStubTlsHeaderSize + off];

	return (void*)(block + kStubTlsHeaderSize + off);
}

extern "C" int __cxa_thread_atexit(void(*)(void*), void*, void*) { return 0; } // we never exit threads cleanly

// Strong def so libc/libstdc++/BLAKE3 assert() calls don't go through the GOT (linker script drops .got → call to 0 = SIGSEGV).
__attribute__((noreturn))
extern "C" void __assert_fail(const char* expr, const char* file, unsigned int line, const char* fn)
{
	static const char pre[] __attribute__((section(".text"))) = "[uba_stub] assert: ";
	static const char nl[]  __attribute__((section(".text"))) = "\n";
	auto slen = [](const char* s){ if(!s) return 0; int n=0; while(s[n]) ++n; return n; };
	RawSyscall3(1, 2, (s64)pre, sizeof(pre)-1);
	if (expr) RawSyscall3(1, 2, (s64)expr, slen(expr));
	RawSyscall3(1, 2, (s64)" at ", 4);
	if (file) RawSyscall3(1, 2, (s64)file, slen(file));
	if (fn)   { RawSyscall3(1, 2, (s64)" in ", 4); RawSyscall3(1, 2, (s64)fn, slen(fn)); }
	RawSyscall3(1, 2, (s64)nl, sizeof(nl)-1);
	(void)line;
	RawSyscall1(231 /* SYS_exit_group */, 134);
	__builtin_unreachable();
}

extern "C" char* strerror(int)            { return (char*)"(stub: strerror not linked)"; }
extern "C" char* getenv(const char*)      { return nullptr; } // only FixPath2's "~" path uses this
// GetHugePageCount (dead path) — abort if reached so we notice rather than silently faking success.
extern "C" FILE*   fopen(const char*, const char*) { STUB_NOT_REACHED("fopen"); }
extern "C" ssize_t write(int, const void*, size_t) { STUB_NOT_REACHED("write"); }
extern "C" int     fclose(FILE*)                   { STUB_NOT_REACHED("fclose"); }
extern "C" int     __isoc23_fscanf(...)            { STUB_NOT_REACHED("__isoc23_fscanf"); } // glibc 2.38+ fscanf with -std=c++20

// Minimal vsnprintf for DEBUG_LOG_*. Supports %s %d %u %x %llu %lld %p %c %% with width (%5d), zero-pad (%04x), precision (%.*s), and ll/z modifiers.
static int StubItoaU(unsigned long long v, char* buf, int base)
{
	static const char* d = "0123456789abcdef";
	char tmp[32]; int n = 0;
	if (v == 0) { buf[0] = '0'; return 1; }
	while (v) { tmp[n++] = d[v % base]; v /= base; }
	for (int i = 0; i < n; ++i) buf[i] = tmp[n-1-i];
	return n;
}
extern "C" int vsnprintf(char* out, size_t cap, const char* fmt, va_list ap)
{
	size_t o = 0;
	auto put = [&](char c){ if (o + 1 < cap) out[o] = c; ++o; };
	auto puts_n = [&](const char* s, size_t n){ for (size_t i = 0; i < n; ++i) put(s[i]); };
	while (*fmt)
	{
		if (*fmt != '%') { put(*fmt++); continue; }
		++fmt;
		int pad = 0; bool zero = false; int prec = -1;
		bool left = false;
		if (*fmt == '-') { left = true; ++fmt; }
		if (*fmt == '0') { zero = true; ++fmt; }
		while (*fmt >= '0' && *fmt <= '9') { pad = pad * 10 + (*fmt - '0'); ++fmt; }
		if (*fmt == '.') { ++fmt; prec = 0; if (*fmt == '*') { prec = va_arg(ap, int); ++fmt; }
		else while (*fmt >= '0' && *fmt <= '9') { prec = prec * 10 + (*fmt - '0'); ++fmt; } }
		int longs = 0;
		if (*fmt == 'l') { ++longs; ++fmt; if (*fmt == 'l') { ++longs; ++fmt; } }
		else if (*fmt == 'z') { longs = 2; ++fmt; }
		char spec = *fmt++;
		char nbuf[32]; int nlen = 0;
		const char* str = nullptr; size_t slen = 0;
		switch (spec)
		{
		case 's': str = va_arg(ap, const char*);
			if (!str) str = "(null)";
			slen = 0; while (str[slen] && (prec < 0 || slen < (size_t)prec)) ++slen;
			if (pad > (int)slen && !left) for (int i = 0; i < pad - (int)slen; ++i) put(' ');
			puts_n(str, slen);
			if (pad > (int)slen && left) for (int i = 0; i < pad - (int)slen; ++i) put(' ');
			break;
		case 'c': put((char)va_arg(ap, int)); break;
		case '%': put('%'); break;
		case 'd': case 'i': {
			long long v = (longs == 2) ? va_arg(ap, long long)
						: (longs == 1) ? va_arg(ap, long)
										: va_arg(ap, int);
			bool neg = v < 0; unsigned long long u = neg ? (unsigned long long)-v : (unsigned long long)v;
			nlen = StubItoaU(u, nbuf, 10);
			int total = nlen + (neg ? 1 : 0);
			if (pad > total && !zero && !left) for (int i = 0; i < pad - total; ++i) put(' ');
			if (neg) put('-');
			if (pad > total && zero) for (int i = 0; i < pad - total; ++i) put('0');
			puts_n(nbuf, nlen);
			if (pad > total && left) for (int i = 0; i < pad - total; ++i) put(' ');
			break;
		}
		case 'o': case 'u': case 'x': case 'X': case 'p': {
			unsigned long long u;
			if (spec == 'p') { u = (unsigned long long)va_arg(ap, void*); put('0'); put('x'); }
			else if (longs == 2) u = va_arg(ap, unsigned long long);
			else if (longs == 1) u = va_arg(ap, unsigned long);
			else u = va_arg(ap, unsigned int);
			int base = (spec == 'u') ? 10 : (spec == 'o') ? 8 : 16;
			nlen = StubItoaU(u, nbuf, base);
			if (pad > nlen && !left) for (int i = 0; i < pad - nlen; ++i) put(zero ? '0' : ' ');
			puts_n(nbuf, nlen);
			if (pad > nlen && left) for (int i = 0; i < pad - nlen; ++i) put(' ');
			break;
		}
		default: put('%'); put(spec); break; // unknown — passthrough
		}
	}
	if (cap > 0) out[o < cap ? o : cap - 1] = 0;
	return (int)o;
}
extern "C" int snprintf(char* out, size_t cap, const char* fmt, ...)
{
    va_list ap; va_start(ap, fmt);
    int r = vsnprintf(out, cap, fmt, ap);
    va_end(ap);
    return r;
}
// Placement new + array/aligned variants → StubAllocator (mmap bump + per-size-class freelist). We leak; process-lifetime.
extern "C" void* _Znwm(size_t n)                          { return StubAllocatorAllocate(n, 16); }
extern "C" void  _ZdlPvm(void*, size_t) {}
extern "C" void  _ZdlPv(void*) {}
extern "C" void* _Znam(size_t n)                          { return StubAllocatorAllocate(n, 16); }
extern "C" void  _ZdaPv(void*) {}
extern "C" void* _ZnwmSt11align_val_t(size_t n, size_t a) { return StubAllocatorAllocate(n, a); }
extern "C" void  _ZdlPvmSt11align_val_t(void*, size_t, size_t) {}

namespace uba {

// Mirrors UbaSharedMemoryView.cpp:IsMemoryHandle. MappedFileTable::ParseNoLock's remap branch is essentially dead for static-Go but must link.
bool IsMemoryHandle(const tchar* name)
{
	return *name == MemoryHandleChar || *name == FreeableMemoryHandleChar;
}

// Dead path: stub's 1 GiB reservation always succeeds under Linux over-commit. Abort loudly if reached.
u32 GetLastError() { STUB_NOT_REACHED("GetLastError"); }

u64 GetTime()
{
	struct { s64 sec; s64 nsec; } ts = {0,0};
	RawSyscall3(228 /* SYS_clock_gettime */, 1 /* CLOCK_MONOTONIC */, (s64)&ts, 0);
	return (u64)ts.sec * 1'000'000'000ull + (u64)ts.nsec;
}

u64 GetFrequency() { return 1'000'000'000ull; }

void WriteDebugInner(const void* data, u32 dataLen) // Raw writer; UbaDetoursShared.cpp's locked WriteDebug wraps this.
{
	if ((int)g_debugFile < 0) return;
	RawSyscall3(1, (s64)g_debugFile, (s64)data, (s64)dataLen);
}

void FlushDebugLog()
{
	if ((int)g_debugFile < 0) return;
	RawSyscall1(74 /* SYS_fsync */, (s64)g_debugFile);
}


// UbaDetoursShared.cpp's FatalError pulls Rpc_WriteLog + exit() which we don't have freestanding.
ANALYSIS_NORETURN void FatalError(u32 code, const tchar* format, ...)
{
	(void)code;
	const char prefix[] = "[uba_stub] FatalError: ";
	StubWrite(2, prefix, sizeof(prefix) - 1);
	size_t fl = 0; while (format[fl] && fl < 512) ++fl;
	StubWrite(2, format, fl);
	StubWrite(2, "\n", 1);
	::StubFlushHandlesAtExit("FatalError");
	RawSyscall1(231 /* SYS_exit_group */, 1);
}

// Backing for stub-side Allocator<T>::allocate. UnorderedMap's rehash() deallocates the old bucket array so we can't bump-and-leak everything
// (memory doubles each rehash). Design: pow2-rounded size classes with per-class freelists; oversize allocs bypass freelist and leak.
// Arena is a chain of mmap'd chunks grown on demand; old chunks stay mmap'd so existing pointers remain valid after the active chunk advances.
struct StubArena { u8* base = 0; u64 used = 0; u64 size = 0; u64 grownBytes = 0; };
static StubArena g_stubArena;

constexpr u64 kArenaChunkBytes = 16ull * 1024 * 1024;

struct StubFreeBlock { StubFreeBlock* next; };
constexpr u64 kStubMinSizeClassBits = 4;   // 16 B
constexpr u64 kStubMaxSizeClassBits = 20;  // 1 MB
constexpr u64 kStubNumSizeClasses   = kStubMaxSizeClassBits - kStubMinSizeClassBits + 1;
static StubFreeBlock* g_stubFreeLists[kStubNumSizeClasses] = {nullptr};

// Returns size-class index, or -1 if too big to track. Class size is (1 << kStubMinSizeClassBits) << index.
static int StubSizeClass(u64 bytes)
{
	if (bytes == 0) bytes = 1;
	u64 v = (1ull << kStubMinSizeClassBits);
	int idx = 0;
	while (v < bytes && idx < (int)kStubNumSizeClasses) { v <<= 1; ++idx; }
	if (idx >= (int)kStubNumSizeClasses) return -1;
	return idx;
}

static bool StubArenaGrow(u64 minBytes, u64 alignment)
{
	u64 needed = minBytes + alignment;
	u64 chunkSize = (needed + kArenaChunkBytes - 1) & ~(kArenaChunkBytes - 1);
	if (chunkSize < kArenaChunkBytes) chunkSize = kArenaChunkBytes;
	void* m = mmap(nullptr, chunkSize, PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANONYMOUS, -1, 0);
	if (m == MAP_FAILED) return false;
	g_stubArena.base = (u8*)m;
	g_stubArena.used = 0;
	g_stubArena.size = chunkSize;
	g_stubArena.grownBytes += chunkSize;
	return true;
}

void* StubAllocatorAllocate(u64 bytes, u64 alignment)
{
	int sc = StubSizeClass(bytes);
	if (sc >= 0 && g_stubFreeLists[sc])
	{
		StubFreeBlock* b = g_stubFreeLists[sc];
		g_stubFreeLists[sc] = b->next;
		return (void*)b;
	}

	// Round up to size-class size so future frees can put it on the right freelist.
	u64 actualBytes = (sc >= 0) ? ((u64)1 << (sc + kStubMinSizeClassBits)) : bytes;

	if (!g_stubArena.base)
	{
		if (!StubArenaGrow(actualBytes, alignment))
		{
			const char msg[] = "[uba_stub] StubArena initial mmap failed\n";
			StubWrite(2, msg, sizeof(msg) - 1);
			RawSyscall1(231, 1);
		}
	}
	u64 mask = alignment - 1;
	u64 pos = (g_stubArena.used + mask) & ~mask;
	if (pos + actualBytes > g_stubArena.size)
	{
		// Old chunk's allocations remain valid; we never munmap.
		if (!StubArenaGrow(actualBytes, alignment))
		{
			const char msg[] = "[uba_stub] StubArena grow failed\n";
			StubWrite(2, msg, sizeof(msg) - 1);
			RawSyscall1(231, 1);
		}
		pos = (g_stubArena.used + mask) & ~mask;
	}
	void* p = g_stubArena.base + pos;
	g_stubArena.used = pos + actualBytes;
	return p;
}

void StubAllocatorFree(void* p, u64 bytes, u64 alignment)
{
	if (!p || bytes == 0) return;
	(void)alignment;
	int sc = StubSizeClass(bytes);
	if (sc < 0) return; // too big to track on any freelist; leak it
	StubFreeBlock* b = (StubFreeBlock*)p;
	b->next = g_stubFreeLists[sc];
	g_stubFreeLists[sc] = b;
}

} // namespace uba

// futex non-private (cross-process event memory). Spurious wakes apply; callers loop on their own condition.
__attribute__((unused)) static inline s64 FutexWait(volatile u32* word, u32 expected) { return RawSyscall6(SYS_futex, (s64)word, FUTEX_WAIT, (s64)expected, 0, 0, 0); }
__attribute__((unused)) static inline s64 FutexWake(volatile u32* word, int count)    { return RawSyscall6(SYS_futex, (s64)word, FUTEX_WAKE, (s64)count, 0, 0, 0); }

// EventFutex view (matches UbaEventFutex.inl). Tag at offset 0: 1=EventPThread, 2=SharedEventSemaphore, 3=EventFutex.
// Static-detoured children use tag 3 so we can speak the protocol via futex syscalls alone.
#define UBA_EVENT_TAG_FUTEX			3u
#define UBA_RESET_AUTO				0u
#define UBA_RESET_MANUAL			1u
#define UBA_EVENT_STATE_RESET		0u
#define UBA_EVENT_STATE_SIGNALED	2u
#define UBA_EVENT_STATE_DESTROYED	3u
#define UBA_EVENT_OFFSET_STATE		12
#define UBA_EVENT_OFFSET_AUTOCOUNT	16
#define UBA_EVENT_SIZE				104

__attribute__((unused))
static void EventWakeAuto(u8* ev)
{
	volatile u32* ac = (volatile u32*)(ev + UBA_EVENT_OFFSET_AUTOCOUNT);
	__atomic_add_fetch(ac, 1, __ATOMIC_RELEASE);
	FutexWake(ac, 1);
}

// Returns 1 on signal, 0 on destroyed/timeout. 0xffffffff = infinite.
__attribute__((unused))
static int EventWaitAuto(u8* ev, u32 timeoutMs = 0xffffffffu)
{
	volatile u32* st = (volatile u32*)(ev + UBA_EVENT_OFFSET_STATE);
	volatile u32* ac = (volatile u32*)(ev + UBA_EVENT_OFFSET_AUTOCOUNT);
	for (;;)
	{
		if (__atomic_load_n(st, __ATOMIC_ACQUIRE) == UBA_EVENT_STATE_DESTROYED) return 0;
		u32 c = __atomic_load_n(ac, __ATOMIC_ACQUIRE);
		if (c != 0)
		{
			u32 expected = c;
			if (__atomic_compare_exchange_n(ac, &expected, c - 1, 0, __ATOMIC_ACQUIRE, __ATOMIC_RELAXED)) return 1;
			continue;
		}
		if (timeoutMs == 0) return 0;
		struct timespec ts = { 0, 0 };
		const struct timespec* tsPtr = 0;
		if (timeoutMs != 0xffffffffu) { ts.tv_sec = (long)(timeoutMs / 1000u); ts.tv_nsec = (long)((timeoutMs % 1000u) * 1000000u); tsPtr = &ts; }
		RawSyscall6(SYS_futex, (s64)ac, FUTEX_WAIT, 0, (s64)tsPtr, 0, 0);
		// Re-check once after timed wait — server may have signalled right before the kernel returned.
		if (tsPtr && timeoutMs != 0xffffffffu)
			if (__atomic_load_n(ac, __ATOMIC_ACQUIRE) == 0 && __atomic_load_n(st, __ATOMIC_ACQUIRE) != UBA_EVENT_STATE_DESTROYED) return 0;
	}
}

static int StubStartsWith(const char* s, const char* prefix)
{
	while (*prefix) { if (*s == 0 || *s != *prefix) return 0; ++s; ++prefix; }
	return 1;
}

// Parse ASCII decimal digits at *p into out_val; advance *p to first non-digit. Returns 1 if any digit consumed.
static int ParseU64Dec(const char** p, u64* out_val)
{
	const char* s = *p;
	u64 v = 0;
	int any = 0;
	while (*s >= '0' && *s <= '9') { v = v * 10u + (u64)(*s - '0'); ++s; ++any; }
	*p = s;
	if (!any) return 0;
	*out_val = v;
	return 1;
}

// 16 hex digits (0-padded, MSB first). Does NOT match ValueToString's byte-swapped format.
static void StubHex16(char* out, u64 v)
{
	static const char hex[] __attribute__((section(".text"), used)) = "0123456789abcdef";
	for (int i = 15; i >= 0; --i) { out[i] = hex[v & 0xf]; v >>= 4; }
}

// `out` volatile: prevents auto-vectorization of the reverse-copy. SSE punpckhbw/pshuflw + movdqu stores fault inside stub-patched Go binaries
// (RIP 0x686198 in clang++ wrapper, fault_addr=NULL via a register-allocator artifact). Scalar byte stores avoid it.
static u32 StubDecU64(char* out_, u64 v)
{
	volatile char* out = out_;
	if (v == 0) { out[0] = '0'; return 1; }
	char tmp[24]; int tl = 0;
	while (v) { tmp[tl++] = (char)('0' + (v % 10)); v /= 10; }
	for (int i = 0; i < tl; ++i) out[i] = tmp[tl - 1 - i];
	return (u32)tl;
}

// Stub state lives in .bss; linker script collapses .text/.rodata/.data/.bss into one output and materialises .bss as zero bytes.
namespace uba { int g_comFd = -1; }
static void* g_shmMem = 0;

namespace uba
{
	void SetLastError(u32) {}

	ANALYSIS_NORETURN void UbaAssert(const tchar* text, const char* file, u32 line, const char* expr, bool /*allowTerminate*/, u32 terminateCode, void* /*context*/, u32 /*skipCallstackCount*/)
	{
		const char tag[] = "[UBA-STUB-ASSERT] ";
		RawSyscallDirect(1 /*SYS_write*/, 2, (long)tag, sizeof(tag) - 1);
		if (expr) { auto n = [](const char* s){ unsigned long k=0; while (s[k]) ++k; return (long)k; }(expr); RawSyscallDirect(1, 2, (long)expr, n); }
		const char nl[] = "\n";
		RawSyscallDirect(1, 2, (long)nl, 1);
		(void)text; (void)file; (void)line;
		RawSyscallDirect(231 /*SYS_exit_group*/, 9000 + (long)terminateCode, 0, 0);
		__builtin_unreachable();
	}
}

void* g_syscall6Trampoline __attribute__((used)) = 0;

// Go goroutines enter syscall.Syscall6 with small per-goroutine stacks. Telemetry sync.Once in `compile` leaves ~2 KB to the guard page;
// ToStringKeyNoCheck alone is 1952 bytes. SyscallBridge switches to g_altStackTop via rip-relative memory moves (no scratch register
// needed, so Go's register-ABI args survive). Single alt-stack — safe only while one M routes through the hook.
// Alt-stack is mmap'd at DetourCore startup (see kAltStackBytes below)
// instead of living in .bss — keeps the blob small even for a 256 KB stack,
// and avoids hitting the linker script's .bss→.text packing limit. Only
// g_altStackTop is exposed to the asm bridge; the actual buffer base isn't
// needed at runtime.
extern "C" u8* g_altStackTop __attribute__((used)) = 0;

// Lowest valid alt-stack address (= mmap result + guard-page size). Used by
// the bridge entry's stack-pressure check (SyscallDispatch) to compute current
// usage and update the peak watermark. The pre-guard page sits BELOW this and
// is mapped PROT_NONE — overflow walks into it and faults deterministically
// at the guard boundary instead of silently corrupting an adjacent mmap.
extern "C" u8* g_altStackBase __attribute__((used)) = 0;

// Peak alt-stack usage (g_altStackTop - lowest_observed_rsp), updated on every
// bridge entry by SyscallDispatch. Diagnostic only — surfaces in DEBUG_LOG when
// it grows. If this ever approaches kAltStackBytes we have a real risk and
// should size up; if it stays in the low tens of KB we're comfortable.
extern "C" u64 g_altStackPeakBytes __attribute__((used)) = 0;

// Spinlock serializing bridge entries across OS threads (Go Ms). Held from
// just after the first sigprocmask (signals blocked) to just before the
// second sigprocmask (signals about to be unblocked), covering the alt-stack
// use and the SyscallDispatch call. Without it, two Go Ms calling
// syscall.Syscall6 concurrently race on the single shared alt-stack buffer,
// corrupting each other's saved frames and eventually crashing Go with
// "rip == r14 == <heap addr>" — a dispatched jump through a smashed
// register. The saved-mask and saved-Go-rsp slots are per-call (Go-stack
// and alt-stack locals respectively) so they don't need the lock, but the
// alt-stack buffer itself is genuinely shared, hence the lock. Signals are
// blocked BEFORE the spin-acquire so a nested bridge entry (from a signal
// handler that invokes syscall.Syscall6) can't self-deadlock.
extern "C" u32 g_bridgeLock  __attribute__((used)) = 0;
// 1 MB: belt-and-suspenders sizing. PopulateDirectoryRecursive used to
// declare a 192 KB u32 array on the stack (now 4 KB inline, with heap
// fallback). Other deep call chains under Rpc_GetEntryInformation /
// Rpc_CheckRemapping descend through BLAKE3 (~2 KB ToStringKey frame)
// plus several StringBuffer<> locals (~520 bytes each). The sysmon-frame
// Go panic ("stack split at bad time") that recurs on AOSP builds is
// consistent with alt-stack pressure as the most likely culprit.
// 1 MB; mmap is zero-filled and demand-paged so essentially free until touched.
static constexpr u64 kAltStackBytes = 1 * 1024 * 1024;

// BLAKE3 forward decls for direct bisect calls from DetourCore.
extern "C" void blake3_hasher_init(void*);
extern "C" void blake3_hasher_update(void*, const void*, size_t);
extern "C" void blake3_hasher_finalize(const void*, void*, size_t);

// Full signal mask blocked around the hook by SyscallBridge. Without this, Go's SIGURG-with-
// SA_ONSTACK handler sees rsp on our alt-stack, panics with a synthetic SIGSEGV trace.
// Read-only; per-call old-mask is saved on the Go stack so concurrent Ms don't collide.
extern "C" u64 g_fullMask __attribute__((used)) = ~0ull;

// Per-process debug log fd. Opened from UBA_LOGFILE env var when
// DetourCore runs; -1 means "not configured, log to stderr only".
static int g_logFd = -1;

// Non-zero after Init RPC succeeds. Bridge gates RPC on this. extern "C" so glibc handlers in UbaGlibc.cpp gate on the same flag.
extern "C" u32 g_initDone __attribute__((used)) = 0;

// Init-response state lives in shared globals (g_processId/g_isChild/g_virtualWorkingDir) populated by Shared_Init in UbaDetoursSharedPosix.cpp.

// LD_PRELOAD heap-allocates this (Rpc_Init); stub has no operator new so we placement-new into a VariableMem slot baked into the blob.
static VariableMem<DirectoryTable> s_directoryTableObj;

namespace uba {
// Session-bound state populated in DetourCore after shm attach. LD_PRELOAD defines these in UbaDetoursMainPosix.cpp; we don't link that TU.
u8*          g_messageMappingMem = nullptr;
SharedEvent* g_writeEvent  = nullptr;
SharedEvent* g_readEvent   = nullptr;
SharedEvent* g_cancelEvent = nullptr;
int          g_sessionPid  = 0;
bool         g_isCancelled = false;
u32          g_processId   = 0;

// Called by Shared_Init in UbaDetoursSharedPosix.cpp — replaces the LD_PRELOAD path's `new DirectoryTable(mb)`.
DirectoryTable* StubAllocDirectoryTable(MemoryBlock& mb)
{
	s_directoryTableObj.Create(mb);
	return (DirectoryTable*)s_directoryTableObj.data;
}
}

// Hand-written bridge in UbaStaticStub.S — not C++-mangled (the .S references it by plain name).
extern "C" void SyscallBridge(void);

// Post-trampoline hook. Called when SyscallDispatch set handled=2 (wait4/waitid). REQUIRED per memory rule
// "Rpc_UpdateTables MUST fire after a successful child wait". Bridge restored Go's rsp before calling us — we run on the
// goroutine stack (~8 KB), NOT the alt-stack. DO NOT call DEBUG_LOG: its 32KB StringBuffer corrupts adjacent goroutine
// stacks (surfaces later as "runtime: stack split at bad time" in sysmon→netpoll). PostHookMarker uses 128 bytes.
static void PostHookMarker(const char* msg, u32 msglen)
{
	char buf[128];
	char* p = buf;
	s64 pid = RawSyscall0(39 /* SYS_getpid */);
	s64 tid = RawSyscall0(186 /* SYS_gettid */);
	*p++ = 'P'; *p++ = '=';
	char tmp[24]; int tl = 0;
	u64 v = (u64)pid; if (v == 0) tmp[tl++] = '0';
	while (v) { tmp[tl++] = '0' + (char)(v % 10); v /= 10; }
	while (tl--) *p++ = tmp[tl];
	*p++ = ' '; *p++ = 'T'; *p++ = '=';
	tl = 0; v = (u64)tid; if (v == 0) tmp[tl++] = '0';
	while (v) { tmp[tl++] = '0' + (char)(v % 10); v /= 10; }
	while (tl--) *p++ = tmp[tl];
	*p++ = ' ';
	for (u32 i = 0; i < msglen && (p - buf) < (s64)sizeof(buf) - 2; ++i) *p++ = msg[i];
	*p++ = '\n';
	s64 fd = RawSyscall6(257 /* SYS_openat */, -100, (s64)"/tmp/uba_post_hook.log", 0x441 /* O_WRONLY|O_CREAT|O_APPEND */, 0644, 0, 0);
	if (fd < 0) return;
	RawSyscall3(1, fd, (s64)buf, (s64)(p - buf));
	RawSyscall1(3, fd);
}

extern "C" __attribute__((used, visibility("hidden")))
void SyscallPostHook(void)
{
	static const char kEnterFn[] = "post_hook ENTERED FUNCTION";
	PostHookMarker(kEnterFn, sizeof(kEnterFn) - 1);
	if (!__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE)) {
		static const char kInitNotDone[] = "post_hook EXIT — g_initDone=0";
		PostHookMarker(kInitNotDone, sizeof(kInitNotDone) - 1);
		return;
	}
	static const char kBeforeRpc[] = "post_hook before Rpc_UpdateTables";
	PostHookMarker(kBeforeRpc, sizeof(kBeforeRpc) - 1);
	Rpc_UpdateTables();
	static const char kAfterRpc[] = "post_hook after Rpc_UpdateTables";
	PostHookMarker(kAfterRpc, sizeof(kAfterRpc) - 1);
}

// 14-byte non-register-clobbering JMP: FF 25 00 00 00 00 + 8-byte absolute target = jmpq *0(%rip).
static void WriteJmp14(u8* dst, u64 target)
{
	dst[0] = 0xFF; dst[1] = 0x25; dst[2] = 0x00; dst[3] = 0x00; dst[4] = 0x00; dst[5] = 0x00;
	for (int i = 0; i < 8; ++i) dst[6 + i] = (u8)(target >> (i * 8));
}

#define SYS_nr_read			0
#define SYS_nr_write		1
#define SYS_nr_close		3
#define SYS_nr_fstat		5
#define SYS_nr_lseek		8
#define SYS_nr_mmap			9
#define SYS_nr_pread64		17
#define SYS_nr_pwrite64		18
#define SYS_nr_dup			32
#define SYS_nr_dup2			33
#define SYS_nr_dup3			292
#define SYS_nr_nanosleep	35
#define SYS_nr_execve		59
#define SYS_nr_getcwd		79
#define SYS_nr_chdir		80
#define SYS_nr_fchdir		81
#define SYS_nr_openat		257
#define SYS_nr_mkdirat		258
#define SYS_nr_newfstatat	262
#define SYS_nr_unlinkat		263
#define SYS_nr_renameat		264
#define SYS_nr_getdents64	217
#define SYS_nr_readlinkat	267
#define SYS_nr_fchmodat		268
#define SYS_nr_faccessat	269
#define SYS_nr_utimensat	280
#define SYS_nr_renameat2	316
#define SYS_nr_statx		332
#define SYS_nr_faccessat2	439
#define SYS_nr_exit			60
#define SYS_nr_clone		56
#define SYS_nr_clone3		435
#define SYS_nr_exit_group	231
#define SYS_nr_wait4		61
#define SYS_nr_waitid		247
#define SYS_nr_pivot_root	155
#define SYS_nr_chroot		161
#define SYS_nr_mount		165
#define SYS_nr_umount2		166

// Writes one byte; advances *out. Caller ensures capacity.
static void BufChar(char** out, char c) { *(*out)++ = c; }

static void BufAppend(char** out, const char* s, size_t n)
{
    for (size_t i = 0; i < n; ++i) *(*out)++ = s[i];
}

// Append decimal representation of v (unsigned).
static void BufDec(char** out, u64 v)
{
    char tmp[24];
    int n = 0;
    if (v == 0) { BufChar(out, '0'); return; }
    while (v)
    {
        tmp[n++] = (char)('0' + (v % 10));
        v /= 10;
    }
    while (n--) BufChar(out, tmp[n]);
}

// Append a C-string (reading from the target's memory). Stops at NUL
// or `max` bytes. Caller caps the total emitted count.
__attribute__((unused))
static void BufCStr(char** out, const char* s, size_t max)
{
    if (!s) { BufAppend(out, "(null)", 6); return; }
    BufChar(out, '"');
    for (size_t i = 0; i < max; ++i)
    {
        char c = s[i];
        if (!c) break;
        if (c == '\\' || c == '"') { BufChar(out, '\\'); BufChar(out, c); }
        else if (c == '\n')         { BufChar(out, '\\'); BufChar(out, 'n'); }
        else if (c < 32 || c > 126) { BufChar(out, '.'); }
        else                        { BufChar(out, c); }
    }
    BufChar(out, '"');
}

// dup → fstat → close via raw syscalls so we don't re-enter detoured code mid-finalize.
static bool StubFstatSnapshot(int fd, u64& outSize, u64& outLastWritten, u32& outAttrs)
{
	s64 dupFd = RawSyscall6(SYS_nr_dup, (s64)fd, 0, 0, 0, 0, 0);
	if (dupFd < 0) return false;
	alignas(8) u8 sb[144] = {0};
	s64 sr = RawSyscall6(SYS_nr_fstat, dupFd, (s64)sb, 0, 0, 0, 0);
	(void)RawSyscall6(SYS_nr_close, dupFd, 0, 0, 0, 0, 0);
	if (sr != 0) return false;
	outAttrs       = *(const u32*)(sb + 24);
	outSize        = *(const u64*)(sb + 48);
	u64 mtimeSec   = *(const u64*)(sb + 88);
	u64 mtimeNsec  = *(const u64*)(sb + 96);
	outLastWritten = mtimeSec * 10000000ull + mtimeNsec / 100;
	return true;
}

// Mirrors LD_PRELOAD's Deinit flush. Without it, hard-exit via SYS_exit_group while holding a write fd leaves the closeId orphaned on the host —
// surfaces later as size=34359738367 ghost entries in DIRTABLE. "BeforeExec" name is historical; exec and exit are equivalent here.
static void StubFlushHandlesAtExit(const char* reason)
{
	if (!__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE)) return;
	FlushWriteHandlesBeforeExec(reason, StubFstatSnapshot);
}

// Linux returns r in [-4095, -1] for errno; Go's Syscall6 ABI wants r1=result/-1, r2=0, errno positive.
static inline void WriteSavedFromKernelRet(u64* saved, s64 r)
{
	if (r < 0 && r >= -4095) { saved[15] = (u64)(s64)-1; saved[14] = 0; saved[13] = (u64)(-r); }
	else                     { saved[15] = (u64)r;       saved[14] = 0; saved[13] = 0; }
}

// Called from the asm bridge on every intercepted Go syscall. extern "C" so .S can call by plain name. Signals blocked by bridge — required
// because we run on the alt-stack (outside any goroutine's known bounds) and Go's SIGURG/SA_ONSTACK handler would panic on the foreign rsp.
//
// `handled` return: 0=passthrough trampoline runs; 1=synthesized (saved[] holds result, bridge skips trampoline); 2=trampoline + SyscallPostHook
// (wait4/waitid — need real syscall then dirtable refresh). Bridge stashes this in a dedicated slot and recovers into %r11 (r8/r9 reserved for mmap).
extern "C" __attribute__((used, visibility("hidden")))
u64 SyscallDispatch(u64* saved)
{
	// INVARIANT: every char* path handed to RawSyscall6 MUST be absolute. ResolvePath/FixPath first — kernel cwd may differ from g_virtualWorkingDir.
	u64 handled = 0;

	// saved[] mirrors UbaStaticStub.S push order: rax,rbx,rcx,rdx,rdi,rsi,rbp,r8,r9,r10,r11,r12,r13,r14,r15 + alignment pad at saved[0].
	// Go's Syscall6 ABI: AX=trap, BX=a1, CX=a2, DI=a3, SI=a4, R8=a5, R9=a6.
	u64 trap = saved[15];
	u64 a1   = saved[14];
	u64 a2   = saved[13];
	u64 a3   = saved[11];
	u64 a4   = saved[10];
	u64 a5   = saved[ 8];
	u64 a6   = saved[ 7];

	// Alt-stack peak watermark. `saved` is the lowest-address local at the call boundary; sampled here at entry and again before return so the
	// diff reflects what the dispatched handler ate. Single-threaded under g_bridgeLock — no atomics needed.
	if (g_altStackBase)
	{
		u64 entryUsed = (u64)((u8*)g_altStackTop - (u8*)saved);
		if (entryUsed > g_altStackPeakBytes)
		{
			g_altStackPeakBytes = entryUsed;
			DEBUG_LOG(TC("alt-stack peak: %llu bytes (trap=%llu) at entry"), (unsigned long long)entryUsed, (unsigned long long)trap);
		}
	}

	// Per-syscall trace buffer (read/write/lseek/pread64 cases append detail). Longest format "[uba] pread64(<u64>, ..., <u64>, <u64>)\n" < 128 bytes.
	char buf[128];
	char* p = buf;
	BufAppend(&p, "[uba] ", 6);
	switch (trap)
	{
		case SYS_nr_openat:
		{
			handled = 1;
			auto trueOpen = [](const char* p, int fl, int m) { return Real_openat(errno, AT_FDCWD, p, fl, m); };
			if (!a2 || !__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE))
			{
				s64 r = trueOpen((const char*)a2, (int)a3, (int)a4);
				WriteSavedFromKernelRet(saved, r < 0 ? -(s64)errno : r);
				break;
			}
			StringBuffer<> resolvedTemp;
			const char* path = ResolvePath((int)a1, (const char*)a2, resolvedTemp);
			if (!path) { WriteSavedFromKernelRet(saved, RawSyscall6(SYS_nr_openat, a1, a2, a3, a4, 0, 0)); break; } // unknown dirfd → kernel
			int fd = Shared_open("openat", path, (int)a3, (int)a4, trueOpen);
			WriteSavedFromKernelRet(saved, fd < 0 ? -(s64)errno : (s64)fd);
			break;
		}
		case SYS_nr_newfstatat:
		{
			handled = 1;
			int dirfd = (int)a1; const char* path = (const char*)a2; struct stat* sb = (struct stat*)a3; int flags = (int)a4;
			if (!path || !sb || !__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE))
			{
				int r = Real_fstatat(errno, dirfd, path, sb, flags);
				WriteSavedFromKernelRet(saved, r < 0 ? -(s64)errno : 0);
				break;
			}
			StringBuffer<> resolvedTemp;
			const char* resolved = ResolvePath(dirfd, path, resolvedTemp);
			if (!resolved)
			{
				int r = Real_fstatat(errno, dirfd, path, sb, flags);
				WriteSavedFromKernelRet(saved, r < 0 ? -(s64)errno : 0);
				break;
			}
			int r = SharedStatPath("newfstatat", resolved, sb, (flags & AT_SYMLINK_NOFOLLOW) == 0);
			WriteSavedFromKernelRet(saved, r < 0 ? -(s64)errno : 0);
			break;
		}
		case SYS_nr_statx:
		{
			handled = 1;
			int r = __atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE)
				? Shared_statx("statx", (int)a1, (const char*)a2, (int)a3, (unsigned int)a4, (struct ::statx*)a5)
				: Real_statx(errno, (int)a1, (const char*)a2, (int)a3, (unsigned int)a4, (struct ::statx*)a5);
			WriteSavedFromKernelRet(saved, r < 0 ? -(s64)errno : 0);
			break;
		}
		// faccessat answers from the dirtable (existence ⇒ F_OK). We don't model EACCES — build runs as a single user, dirtable carries no perms.
		case SYS_nr_faccessat:
		case SYS_nr_faccessat2:
		{
			handled = 1;
			int r = __atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE)
				? Shared_faccessat("faccessat", (int)a1, (const char*)a2, (int)a3, (int)a4)
				: Real_faccessat(errno, (int)a1, (const char*)a2, (int)a3, (int)a4);
			WriteSavedFromKernelRet(saved, r < 0 ? -(s64)errno : 0);
			break;
		}
		case SYS_nr_readlinkat:
		{
			handled = 1;
			int dfd = (int)a1; const char* path = (const char*)a2; char* outBuf = (char*)a3; u64 bufsiz = a4;
			if (!path || !__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE))
			{
				long r = Real_readlinkat(errno, dfd, path, outBuf, bufsiz);
				WriteSavedFromKernelRet(saved, r < 0 ? -(s64)errno : r);
				break;
			}
			// g_virtualApplication empty (stub) ⇒ Shared_prepareReadlink would synthesize 0-byte /proc/self/exe — confuses Go's os.Executable().
			if (path && g_virtualApplication.count == 0
				&& path[0] == '/' && path[1] == 'p' && path[2] == 'r' && path[3] == 'o' && path[4] == 'c' && path[5] == '/'
				&& path[6] == 's' && path[7] == 'e' && path[8] == 'l' && path[9] == 'f' && path[10] == '/'
				&& path[11] == 'e' && path[12] == 'x' && path[13] == 'e' && path[14] == 0)
			{
				long r = Real_readlinkat(errno, dfd, path, outBuf, bufsiz);
				WriteSavedFromKernelRet(saved, r < 0 ? -(s64)errno : r);
				break;
			}
			u64 outLen = 0; int err = 0;
			switch (Shared_prepareReadlink("readlinkat", path, outBuf, bufsiz, outLen, err))
			{
			case SharedReadlinkAction::Synthesized: saved[15] = (u64)outLen;    saved[14] = 0; saved[13] = 0;          break;
			case SharedReadlinkAction::Error:       saved[15] = (u64)(s64)-1;   saved[14] = 0; saved[13] = (u64)err;   break;
			case SharedReadlinkAction::Passthrough: { long r = Real_readlinkat(errno, dfd, path, outBuf, bufsiz); WriteSavedFromKernelRet(saved, r < 0 ? -(s64)errno : r); break; }
			}
			break;
		}
		case SYS_nr_mkdirat:
		{
			handled = 1;
			int rc = __atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE)
				? Shared_mkdirat("mkdirat", (int)a1, (const char*)a2, (unsigned int)a3)
				: Real_mkdirat(errno, (int)a1, (const char*)a2, (unsigned int)a3);
			WriteSavedFromKernelRet(saved, rc < 0 ? -(s64)errno : 0);
			break;
		}
		case SYS_nr_renameat:
		case SYS_nr_renameat2: // flags ignored, mirroring LD_PRELOAD
		{
			handled = 1;
			const char* fn = trap == SYS_nr_renameat2 ? "renameat2" : "renameat";
			int rc = __atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE)
				? Shared_renameat(fn, (int)a1, (const char*)a2, (int)a3, (const char*)a4)
				: Real_renameat(errno, (int)a1, (const char*)a2, (int)a3, (const char*)a4);
			WriteSavedFromKernelRet(saved, rc < 0 ? -(s64)errno : 0);
			break;
		}
		case SYS_nr_read:    BufAppend(&p, "read(", 5);    BufDec(&p, a1); BufAppend(&p, ", ..., ", 7); BufDec(&p, a3); BufAppend(&p, ")\n", 2); break;
		case SYS_nr_write:   BufAppend(&p, "write(", 6);   BufDec(&p, a1); BufAppend(&p, ", ..., ", 7); BufDec(&p, a3); BufAppend(&p, ")\n", 2); break;
		case SYS_nr_pread64: BufAppend(&p, "pread64(", 8); BufDec(&p, a1); BufAppend(&p, ", ..., ", 7); BufDec(&p, a3); BufAppend(&p, ", ", 2); BufDec(&p, a4); BufAppend(&p, ")\n", 2); break;
		case SYS_nr_lseek:   BufAppend(&p, "lseek(", 6);   BufDec(&p, a1); BufAppend(&p, ", ", 2);     BufDec(&p, a2); BufAppend(&p, ", ", 2);  BufDec(&p, a3); BufAppend(&p, ")\n", 2); break;
		case SYS_nr_close:
		{
			handled = 1;
			int fd = (int)a1;
			if (!__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE)) { WriteSavedFromKernelRet(saved, Real_close(errno, fd) < 0 ? -(s64)errno : 0); break; }
			auto trueClose = [&]() { return Real_close(errno, fd); };
			int rc = Shared_close("close", fd, trueClose, StubFstatSnapshot);
			DEBUG_LOG(TC("close: fd=%d -> %d"), fd, rc);
			WriteSavedFromKernelRet(saved, rc < 0 ? -(s64)errno : 0);
			break;
		}
		case SYS_nr_getdents64: // Serve CreateDirFd fds from dirtable; kernel passthrough otherwise.
		{
			handled = 1;
			int fd = (int)a1; void* buf = (void*)a2; unsigned int cnt = (unsigned int)a3;
			if (!buf || !__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE))
			{
				s64 r = RawSyscall6(SYS_nr_getdents64, (s64)fd, (s64)buf, (s64)cnt, 0, 0, 0);
				DEBUG_LOG(TC("getdents64: fd=%d cnt=%u (pre-init/null) -> %lld"), fd, cnt, (long long)r);
				WriteSavedFromKernelRet(saved, r);
				break;
			}
			int capturedErr = 0;
			long lr = Shared_getdents64("getdents64", fd, buf, cnt, &capturedErr);
			s64 r;
			if (lr < 0 && capturedErr == 0) { r = RawSyscall6(SYS_nr_getdents64, (s64)fd, (s64)buf, (s64)cnt, 0, 0, 0); DEBUG_LOG(TC("getdents64: fd=%d cnt=%u (untracked, kernel) -> %lld"), fd, cnt, (long long)r); }
			else if (lr < 0)                { r = -(s64)capturedErr; DEBUG_LOG(TC("getdents64: fd=%d cnt=%u -> -%d"), fd, cnt, capturedErr); }
			else                            { r = (s64)lr;           DEBUG_LOG(TC("getdents64: fd=%d cnt=%u -> %lld (UBA dir)"), fd, cnt, (long long)r); }
			WriteSavedFromKernelRet(saved, r);
			break;
		}
		// fstat must agree with newfstatat on the same path — GNU rm's fts walk verifies st_dev/st_ino across fdopendir+later-fstat.
		case SYS_nr_fstat:
		{
			handled = 1;
			int fd = (int)a1; struct stat* sbOut = (struct stat*)a2;
			if (!sbOut || !__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE)) { int r = Real_fstat(errno, fd, sbOut); WriteSavedFromKernelRet(saved, r < 0 ? -(s64)errno : 0); break; }
			const char* origName = nullptr;
			{
				ScopedReadLock lock(g_fileHandlesLock);
				auto it = g_fileHandles.find(fd);
				if (it != g_fileHandles.end() && it->second.fileObject && it->second.fileObject->fileInfo)
					origName = it->second.fileObject->fileInfo->originalName;
			}
			if (origName) { int r = SharedStatPath("fstat", origName, sbOut, true); WriteSavedFromKernelRet(saved, r < 0 ? -(s64)errno : 0); break; }
			int r = Real_fstat(errno, fd, sbOut);
			DEBUG_LOG(TC("fstat: fd=%d NODETOUR -> %d"), fd, r);
			WriteSavedFromKernelRet(saved, r < 0 ? -(s64)errno : 0);
			break;
		}
		case SYS_nr_unlinkat:
		{
			handled = 1;
			int rc = __atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE)
				? Shared_unlinkat("unlinkat", (int)a1, (const char*)a2, (int)a3)
				: Real_unlinkat(errno, (int)a1, (const char*)a2, (int)a3);
			WriteSavedFromKernelRet(saved, rc < 0 ? -(s64)errno : 0);
			break;
		}
		case SYS_nr_fchmodat:
		{
			handled = 1;
			int dirfd = (int)a1; const char* path = (const char*)a2; mode_t mode = (mode_t)a3; int flags = (int)a4;
			if (!path || !__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE)) { int r = Real_fchmodat(errno, dirfd, path, mode, flags); WriteSavedFromKernelRet(saved, r < 0 ? -(s64)errno : 0); break; }
			StringBuffer<> resolvedTemp;
			const char* resolved = ResolvePath(dirfd, path, resolvedTemp);
			if (!resolved) { int r = Real_fchmodat(errno, dirfd, path, mode, flags); WriteSavedFromKernelRet(saved, r < 0 ? -(s64)errno : 0); break; }
			int err = 0;
			int rc = Shared_chmod("fchmodat", resolved, mode, err, [flags](const char* p, mode_t m) { return Real_fchmodat(errno, AT_FDCWD, p, m, flags); });
			if (rc != 0 && err == 0) err = errno;
			WriteSavedFromKernelRet(saved, rc < 0 ? -(s64)err : 0);
			break;
		}
		// Dirtable hit → success without applying times (TODO: overlay-track). Mirrors LD_PRELOAD.
		case SYS_nr_utimensat:
		{
			handled = 1;
			int dirfd = (int)a1; const char* path = (const char*)a2;
			if (!path || !__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE)) { s64 r = RawSyscall6(SYS_nr_utimensat, (s64)dirfd, (s64)a2, (s64)a3, (s64)a4, 0, 0); DEBUG_LOG(TC("utimensat: %s (pre-init/null) -> %lld"), path ? path : "(null)", (long long)r); WriteSavedFromKernelRet(saved, r); break; }
			StringBuffer<> resolvedTemp;
			const char* resolved = ResolvePath(dirfd, path, resolvedTemp);
			if (!resolved) { s64 r = RawSyscall6(SYS_nr_utimensat, (s64)dirfd, (s64)path, (s64)a3, (s64)a4, 0, 0); DEBUG_LOG(TC("utimensat: dirfd=%d %s (unresolvable) -> %lld"), dirfd, path, (long long)r); WriteSavedFromKernelRet(saved, r); break; }
			StringBuffer<> fixedFile;
			if (!FixPath(fixedFile, resolved) || !CanDetour2(fixedFile)) { s64 r = RawSyscall6(SYS_nr_utimensat, (s64)dirfd, (s64)path, (s64)a3, (s64)a4, 0, 0); DEBUG_LOG(TC("utimensat: %s NODETOUR -> %lld"), resolved, (long long)r); WriteSavedFromKernelRet(saved, r); break; }
			DevirtualizePath(fixedFile);
			StringKey key = ToFilenameKey(fixedFile);
			DirectoryTableEntryInformation info;
			if (Rpc_GetEntryInformation(info, key, fixedFile, false)) { DEBUG_LOG(TC("utimensat: %s (-> %s) DIRTABLE -> 0 (skipped, NeedsFix)"), path, fixedFile.data); saved[15] = 0; saved[14] = 0; saved[13] = 0; break; }
			DEBUG_LOG(TC("utimensat: %s (-> %s) DIRTABLE -> -1 (ENOENT)"), path, fixedFile.data);
			saved[15] = (u64)(s64)-1; saved[14] = 0; saved[13] = (u64)ENOENT;
			break;
		}
		// dup2/dup3 kernel-implicitly close newFd. Finalize the about-to-be-overwritten write fd FIRST or Deinit corrupts the writtenFile entry.
		case SYS_nr_dup:
		case SYS_nr_dup2:
		case SYS_nr_dup3:
		{
			handled = 1;
			int srcFd = (int)a1;
			int newFd = (int)a2;
			if ((trap == SYS_nr_dup2 || trap == SYS_nr_dup3) && srcFd != newFd && __atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE))
				Shared_PreFinalizeForOverwrite("dup2", newFd, StubFstatSnapshot);
			s64 r = RawSyscall6((s64)trap, (s64)a1, (s64)a2, (s64)a3, 0, 0, 0);
			if (r >= 0 && __atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE))
			{
				bool newCloexec = (trap == SYS_nr_dup3) && (((s64)a3 & O_CLOEXEC) != 0); // dup/dup2 clear CLOEXEC; dup3 honors flags
				DuplicateFd(srcFd, (int)r, newCloexec);
			}
			DEBUG_LOG(TC("dup(%d) -> %lld"), srcFd, (long long)r);
			WriteSavedFromKernelRet(saved, r);
			break;
		}
		// Agent's kernel cwd is the staging dir, not the build root. Return g_virtualWorkingDir so callers see UBA's view.
		case SYS_nr_getcwd:
		{
			handled = 1;
			char* buf = (char*)a1;
			size_t size = (size_t)a2;
			u32 len = g_virtualWorkingDir.count;
			if (len > 1 && g_virtualWorkingDir.data[len - 1] == '/') --len;
			if (!buf || size < (size_t)len + 1) { saved[15] = (u64)(s64)-1; saved[14] = 0; saved[13] = 34; DEBUG_LOG(TC("getcwd: -> -1 (ERANGE, size=%llu need=%u)"), (u64)size, len + 1); break; }
			memcpy(buf, g_virtualWorkingDir.data, len);
			buf[len] = 0;
			saved[15] = (u64)(len + 1); saved[14] = 0; saved[13] = 0;
			DEBUG_LOG(TC("getcwd: -> %s (synthesized)"), buf);
			break;
		}
		case SYS_nr_chdir:
		{
			handled = 1;
			const char* path = (const char*)a1;
			if (!path || !*path) { DEBUG_LOG(TC("chdir: NULL/empty -> -1 (ENOENT)")); saved[15] = (u64)(s64)-1; saved[14] = 0; saved[13] = 2; break; }
			int err = 0;
			int rc = Shared_chdir("chdir", path, err);
			if (rc == 0) { saved[15] = 0; saved[14] = 0; saved[13] = 0; }
			else         { DEBUG_LOG(TC("chdir: %s -> -1 (errno %d)"), path, err); saved[15] = (u64)(s64)-1; saved[14] = 0; saved[13] = (u64)err; }
			break;
		}
		case SYS_nr_fchdir:
		{
			handled = 1;
			int err = 0;
			int rc = Shared_fchdir((int)a1, err);
			if (rc == 0) { saved[15] = 0;             saved[14] = 0; saved[13] = 0; }
			else         { saved[15] = (u64)(s64)-1;  saved[14] = 0; saved[13] = (u64)err; }
			break;
		}
		// Relative-path execve on the agent gets the staging-dir cwd (not the build root). Mirror LD_PRELOAD's Internal_exec: send
		// MessageType_ExecProcess so host stages a patched copy, then execve into the returned realApplication.
		case SYS_nr_execve:
		{
			handled = 1;
			if (!__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE)) { s64 r = RawSyscall6(SYS_nr_execve, (s64)a1, (s64)a2, (s64)a3, 0, 0, 0); DEBUG_LOG(TC("execve: %s (pre-init passthrough) -> %lld"), a1 ? (const char*)a1 : "(null)", (long long)r); WriteSavedFromKernelRet(saved, r); break; }
			// trueExecve returns 0 (unreachable on success — kernel replaces us) or positive errno on failure. RawSyscall6 returns -errno; flip sign.
			auto trueExecve      = [](const char* p, char* const av[], char* const ev[]) -> int { s64 r = RawSyscall6(SYS_nr_execve, (s64)p, (s64)av, (s64)ev, 0, 0, 0); return (r >= 0) ? 0 : (int)(-r); };
			auto trueNanosleepUs = [](u32 us) { struct timespec ts; ts.tv_sec = 0; ts.tv_nsec = (long)us * 1000; RawSyscall3(SYS_nr_nanosleep, (s64)&ts, 0, 0); };
			int r = Shared_Internal_exec("execve", (const char*)a1, (char* const*)a2, (char* const*)a3, /*fixupPath*/ true, trueExecve, trueNanosleepUs, StubFstatSnapshot);
			DEBUG_LOG(TC("execve: %s -> %d"), a1 ? (const char*)a1 : "(null)", r);
			WriteSavedFromKernelRet(saved, (s64)r);
			break;
		}
		// CLONE_VM|CLONE_VFORK: child shares parent's address space until execve. Child's chdir-for-cmd.Dir would poison parent's g_virtualWorkingDir.
		// Snapshot+restore on the parent's stack — kernel preserves these pages across vfork, and child's eventual SYS_chdir+execve re-applies its desired cwd.
		case SYS_nr_clone:
		case SYS_nr_clone3:
		{
			handled = 1;
			constexpr u64 vwdMaxSnap = 1024;
			char savedVwdData[vwdMaxSnap];
			u32 savedVwdCount = 0;
			const bool snap = (g_virtualWorkingDir.count > 0 && g_virtualWorkingDir.count < vwdMaxSnap);
			if (snap) { savedVwdCount = g_virtualWorkingDir.count; for (u32 i = 0; i <= savedVwdCount; ++i) savedVwdData[i] = g_virtualWorkingDir.data[i]; }
			s64 r = RawSyscall6((s64)trap, (s64)a1, (s64)a2, (s64)a3, (s64)a4, (s64)a5, (s64)a6);
			if (snap) { g_virtualWorkingDir.count = savedVwdCount; for (u32 i = 0; i <= savedVwdCount; ++i) g_virtualWorkingDir.data[i] = savedVwdData[i]; }
			DEBUG_LOG(TC("clone%s: -> %lld (vwd-snap=%s)"), trap == SYS_nr_clone3 ? "3" : "", (long long)r, snap ? "yes" : "no");
			WriteSavedFromKernelRet(saved, r);
			break;
		}
		// Fork-child Go syscall.Exit bypasses runtime.exit.abi0 (it goes RawSyscall→Syscall6→our hook). Without firing MessageType_Exit here the
		// host's IsActive sees the pid reaped before we report, returns ECHILD on waitid, and trips UbaProcess.cpp:498's m_messageCount != 0 assert.
		case SYS_nr_exit:
		case SYS_nr_exit_group:
		{
			handled = 1;
			int exitCode = (int)a1;
			StubFlushHandlesAtExit(trap == SYS_nr_exit_group ? "SYS_exit_group" : "SYS_exit");
			if (__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE)) Shared_SendExit((u32)exitCode);
			DEBUG_LOG(TC("syscall.exit_group: code=%d (Exit RPC sent)"), exitCode);
			for (;;) RawSyscall1(SYS_nr_exit_group, (s64)exitCode); // never returns
		}

		// wait4/waitid: handled=2 runs trampoline first (preserves Go's enter/exit-syscall tracking), then SyscallPostHook fires Rpc_UpdateTables
		// — REQUIRED invariant per memory rule. Earlier handled=1 design corrupted Go M state ("syscall frame is no longer valid").
		case SYS_nr_wait4:
		case SYS_nr_waitid:
			handled = 2;
			DEBUG_LOG(TC("syscall.%s: handled=2 (post-hook UpdateTables)"), trap == SYS_nr_wait4 ? TC("wait4") : TC("waitid"));
			break;
		case 102: case 107: handled = 1; WriteSavedFromKernelRet(saved, g_virtualUid); DEBUG_LOG(TC("syscall.getuid/geteuid: -> %u"), g_virtualUid); break;
		case 104: case 108: handled = 1; WriteSavedFromKernelRet(saved, g_virtualGid); DEBUG_LOG(TC("syscall.getgid/getegid: -> %u"), g_virtualGid); break;
		// Passthrough syscalls (mmap, mprotect, munmap, brk, rt_sigaction, rt_sigprocmask, rt_sigreturn, sched_yield, madvise, fcntl, getrlimit,
		// capget, prctl, arch_prctl, setrlimit, gettid, futex, sched_getaffinity, clock_gettime, clock_nanosleep, set_robust_list, prlimit64).
		case 9: case 10: case 11: case 12: case 13: case 14: case 15: case 24: case 28: case 72: case 97: case 125: case 157: case 158: case 160:
		case 186: case 202: case 204: case 228: case 230: case 273: case 302:
			break;
		// Mount-NS syscalls from stub-patched Go targets: fail loud rather than passthrough.
		case SYS_nr_mount:
		case SYS_nr_umount2:
		case SYS_nr_pivot_root:
		case SYS_nr_chroot:
			STUB_NOT_REACHED("mount-NS syscall via bridge (mount/umount2/pivot_root/chroot)");
		default:
			DEBUG_LOG(TC("syscall(0x%llx) (passthrough)"), (u64)trap);
			break;
	}
	(void)a5; (void)a6;

	// Per-syscall trace to debug log. Skip if only the "[uba] " prefix was written (handler logged via DEBUG_LOG instead of BufAppend).
	size_t len = (size_t)(p - buf);
	if (len > 6) StubDbgWrite(buf, len);

	// Second alt-stack peak sample at function-exit. volatile char so the compiler doesn't elide the rsp probe.
	if (g_altStackBase)
	{
		volatile char rspProbe = 0;
		u64 exitUsed = (u64)((u8*)g_altStackTop - (u8*)&rspProbe);
		if (exitUsed > g_altStackPeakBytes)
		{
			g_altStackPeakBytes = exitUsed;
			DEBUG_LOG(TC("alt-stack peak: %llu bytes (trap=%llu) post-handler"), (unsigned long long)exitUsed, (unsigned long long)trap);
		}
	}

	return handled; // signal mask + Go's rsp restored by SyscallBridge
}

// SharedEvent::Set/IsSet. UbaEvent.cpp drags pthread so we don't link it; our session creates EventFutex (tag 3) for static-detoured children.
namespace uba {

void SharedEvent::Set()                  { EventWakeAuto((u8*)m_data); }
bool SharedEvent::IsSet(u32 timeOutMs)   { return EventWaitAuto((u8*)m_data, timeOutMs) != 0; }

// UbaPlatform.cpp's WriteLongString pulls Oodle; we don't link it. ProcessCreator's only caller passes threshold = CommunicationMemSize-4096 ≈ 60 KB,
// far above any realistic compiler-driver cmdline. Implement only the under-threshold path; raw-exit if a pathological caller exceeds it.
void BinaryWriter::WriteLongString(const StringView& str, u32 threshold)
{
	if (str.count >= threshold) RawSyscall1(231 /* SYS_exit_group */, 137);
	WriteByte(0);
	WriteString(str);
}

} // namespace uba

static void InstallSyscall6Hook(void)
{
	u64 sc6 = g_syscall6Addr;

	// RWX trampoline: 14 displaced bytes + 14-byte JMP back to sc6+14.
	s64 mmapRet = RawSyscall6(SYS_mmap, 0, PAGE_SIZE, PROT_READ | PROT_WRITE | PROT_EXEC, MAP_PRIVATE | MAP_ANONYMOUS, -1, 0);
	if ((void*)mmapRet == MAP_FAILED) { StubWrite(2, s_msgTrampFail, sizeof(s_msgTrampFail) - 1); return; }
	u8* tramp = (u8*)mmapRet;

	// Grant RWX to two pages — the 14-byte JMP may span a page boundary.
	u64 page = sc6 & ~((u64)(PAGE_SIZE - 1));
	s64 mp = RawSyscall3(SYS_mprotect, (s64)page, (s64)(PAGE_SIZE * 2), PROT_READ | PROT_WRITE | PROT_EXEC);
	if (mp < 0) { StubWrite(2, s_msgMprotFail, sizeof(s_msgMprotFail) - 1); return; }

	u8* sc6p = (u8*)sc6;

	// Syscall.Syscall6's prologue on Linux x86-64 is pure register MOVs (no RIP-relative operands) so the 14 bytes relocate verbatim.
	for (int i = 0; i < 14; ++i) tramp[i] = sc6p[i];
	WriteJmp14(tramp + 14, sc6 + 14);
	WriteJmp14(sc6p, (u64)(void*)SyscallBridge);
	g_syscall6Trampoline = tramp;

	(void)RawSyscall3(SYS_mprotect, (s64)page, (s64)(PAGE_SIZE * 2), PROT_READ | PROT_EXEC);

	STUB_INIT_LOG(s_msgHooked, sizeof(s_msgHooked) - 1);
}

// Declared in UbaStaticStub.S.
extern "C" void ExitBridge(void);

// Receives exit code, sends MessageType_Exit, then issues SYS_exit_group itself. Noreturn so the asm bridge can tail into it.
extern "C" __attribute__((used, visibility("hidden"), noreturn))
void ExitCore(int exitCode)
{
	// Flush BEFORE Shared_SendExit: host's CloseFile handlers must run before Exit tears process state down. Shared_SendExit is fire-and-forget.
	StubFlushHandlesAtExit("ExitCore");

	if (__atomic_load_n(&g_initDone, __ATOMIC_ACQUIRE))
		Shared_SendExit((u32)exitCode);

	// Infinite loop prevents compiler-emitted fall-through into the next TU function (observed on Go `compile`: exit_group returned, fell into DetourCore SEGV).
	for (;;)
		RawSyscall1(231 /* SYS_exit_group */, (s64)exitCode);
}

// Patch runtime.exit.abi0's prologue with a 14-byte JMP to ExitBridge. No trampoline back — the target terminates; we issue exit_group ourselves.
static void InstallExitHook(void)
{
	u64 ex = g_runtimeExitAddr;
	if (ex == 0xDEADBEEFEEEEE500ULL) return; // sentinel not patched

	u64 page = ex & ~((u64)(PAGE_SIZE - 1));
	s64 mp = RawSyscall3(SYS_mprotect, (s64)page, (s64)(PAGE_SIZE * 2), PROT_READ | PROT_WRITE | PROT_EXEC);
	if (mp < 0)
	{
		static const char m[] __attribute__((section(".text"))) = "[uba_stub] exit-hook mprotect failed\n";
		StubWrite(2, m, sizeof(m) - 1);
		return;
	}

	WriteJmp14((u8*)ex, (u64)(void*)ExitBridge);
	(void)RawSyscall3(SYS_mprotect, (s64)page, (s64)(PAGE_SIZE * 2), PROT_READ | PROT_EXEC);

	static const char m[] __attribute__((section(".text"))) = "[uba_stub] exit hook installed\n";
	STUB_INIT_LOG(m, sizeof(m) - 1);
}

// C entry for the asm vfork bridge; runs on alt-stack. Returns kernel-style pid (0/child-pid/-1).
extern "C" __attribute__((used, visibility("hidden")))
long StaticForkForBridge(void)
{
	return (long)Shared_fork();
}

// Declared in UbaStaticStub.S.
extern "C" void VforkBridge(void);

// Patch syscall.rawVforkSyscall.abi0's prologue with a 14-byte JMP to VforkBridge. Bridge swaps clone3+CLONE_VM|CLONE_VFORK for plain SYS_fork (CoW; child
// chdir doesn't poison parent). Skipped if the patcher didn't bake an address (binary doesn't pull in os/exec).
static void InstallVforkHook(void)
{
	u64 vfa = g_vforkSyscallAddr;
	if (vfa == g_vforkSyscallSentinel) return;

	u64 page = vfa & ~((u64)(PAGE_SIZE - 1));
	s64 mp = RawSyscall3(SYS_mprotect, (s64)page, (s64)(PAGE_SIZE * 2), PROT_READ | PROT_WRITE | PROT_EXEC);
	if (mp < 0)
	{
		static const char m[] __attribute__((section(".text"))) = "[uba_stub] vfork-hook mprotect failed\n";
		StubWrite(2, m, sizeof(m) - 1);
		return;
	}

	WriteJmp14((u8*)vfa, (u64)(void*)VforkBridge);
	(void)RawSyscall3(SYS_mprotect, (s64)page, (s64)(PAGE_SIZE * 2), PROT_READ | PROT_EXEC);

	static const char m[] __attribute__((section(".text"))) = "[uba_stub] vfork hook installed (clone3->fork)\n";
	STUB_INIT_LOG(m, sizeof(m) - 1);
}

// Shared_Init (UbaDetoursSharedPosix.cpp) drives MessageType_Init + table mapping + DirectoryTable adopt + env adoption.

// Noinline because Go `compile`/`link` SEGV'd when this body was inlined into DetourCore (tight initial stack region).
static __attribute__((noinline, unused)) void LogEventTags(void)
{
	const u8* ev = (const u8*)g_shmMem;
	u8 tC = ev[0]; u8 tW = ev[104]; u8 tR = ev[208];
	char line[64]; char* p = line;
	for (size_t i = 0; i < sizeof(s_msgEventTags) - 1; ++i) *p++ = s_msgEventTags[i];
	*p++ = (char)('0' + tC);
	*p++ = ' '; *p++ = 'w'; *p++ = '='; *p++ = (char)('0' + tW);
	*p++ = ' '; *p++ = 'r'; *p++ = '='; *p++ = (char)('0' + tR);
	*p++ = '\n';
	StubWrite(2, line, (size_t)(p - line));
}

// Crash handler for SIGSEGV/SIGBUS/SIGILL/SIGFPE — writes one line to /tmp/uba_crash.<pid>.log then re-raises with SIG_DFL.
// LD_PRELOAD workers get the equivalent via libUbaDetours's chained sigaction; this is the stub-patched counterpart.

extern "C" void CrashRestorer(void); // UbaStaticStub.S — SYS_rt_sigreturn trampoline

// Linux x86_64 kernel sigaction layout (NOT glibc's struct sigaction).
struct uba_k_sigaction
{
	void (*sa_handler)(int, void*, void*);  // sa_sigaction (SA_SIGINFO)
	unsigned long sa_flags;
	void (*sa_restorer)(void);
	unsigned long sa_mask[1];               // 8-byte kernel sigset_t
};

#define UBA_SA_SIGINFO   0x00000004ul
#define UBA_SA_RESTORER  0x04000000ul

// Async-signal-safe. `volatile char*` destinations defeat clang's loop vectorizer: movdqu loads of the const k_* arrays SEGV'd inside the patched blob's
// PT_LOAD region in stub-patched Go binaries (e.g. Android clang++ wrapper). Volatile forces scalar byte-by-byte copies; `optimize("no-tree-vectorize")` is GCC-only.
static void CrashHandler(int sig, void* info, void* uc)
{
	// siginfo_t kernel layout: si_signo@0, si_errno@4, si_code@8, si_addr@16. ucontext_t mcontext starts @40; gregs[REG_RIP=16] → 40+16*8=168.
	int   si_code    = *(int*)((char*)info + 8);
	void* fault_addr = *(void**)((char*)info + 16);
	void* rip        = *(void**)((char*)uc + 168);
	s64 pid = RawSyscall0(39  /* SYS_getpid */);
	s64 tid = RawSyscall0(186 /* SYS_gettid */);

	// Build "/tmp/uba_crash.<pid>.log"
	char path[64]; char* p = path;
	static const char k_pre[] __attribute__((section(".text"), used)) = "/tmp/uba_crash.";
	static const char k_suf[] __attribute__((section(".text"), used)) = ".log";
	for (size_t i = 0; i < sizeof(k_pre) - 1; ++i) *p++ = k_pre[i];
	p += StubDecU64(p, (u64)pid);
	for (size_t i = 0; i < sizeof(k_suf) - 1; ++i) *p++ = k_suf[i];
	*p = 0;

	s64 fd = RawSyscall6(SYS_openat, (s64)AT_FDCWD, (s64)path, (s64)(O_WRONLY | O_CREAT | O_APPEND), (s64)0644, 0, 0);
	if (fd >= 0)
	{
		char buf[512];
		volatile char* b = buf; // see top-of-function comment on `volatile`
		static const char k_h0[]   __attribute__((section(".text"), used)) = "[stub-crash] sig=";
		static const char k_pid[]  __attribute__((section(".text"), used)) = " pid=";
		static const char k_tid[]  __attribute__((section(".text"), used)) = " tid=";
		static const char k_rip[]  __attribute__((section(".text"), used)) = " rip=0x";
		static const char k_flt[]  __attribute__((section(".text"), used)) = " fault=0x";
		static const char k_sic[]  __attribute__((section(".text"), used)) = " si_code=";
		static const char k_rDir[] __attribute__((section(".text"), used)) = " region=DirTable";
		static const char k_rMap[] __attribute__((section(".text"), used)) = " region=FileTable";
		static const char k_rOv[]  __attribute__((section(".text"), used)) = " region=OverlayTable";
		static const char k_rShm[] __attribute__((section(".text"), used)) = " region=ShmEvents";
		static const char k_rOth[] __attribute__((section(".text"), used)) = " region=other";

		for (size_t i = 0; i < sizeof(k_h0)  - 1; ++i) *b++ = k_h0[i];   b += StubDecU64((char*)b, (u64)sig);
		for (size_t i = 0; i < sizeof(k_pid) - 1; ++i) *b++ = k_pid[i];  b += StubDecU64((char*)b, (u64)pid);
		for (size_t i = 0; i < sizeof(k_tid) - 1; ++i) *b++ = k_tid[i];  b += StubDecU64((char*)b, (u64)tid);
		for (size_t i = 0; i < sizeof(k_rip) - 1; ++i) *b++ = k_rip[i];  StubHex16((char*)b, (u64)rip);        b += 16;
		for (size_t i = 0; i < sizeof(k_flt) - 1; ++i) *b++ = k_flt[i];  StubHex16((char*)b, (u64)fault_addr); b += 16;
		for (size_t i = 0; i < sizeof(k_sic) - 1; ++i) *b++ = k_sic[i];
		if (si_code < 0) { *b++ = '-'; b += StubDecU64((char*)b, (u64)(-si_code)); }
		else             {             b += StubDecU64((char*)b, (u64) si_code);   }

		// Classify fault_addr against the known UBA mappings. Memory pointers fetched from the live table objects (populated by Shared_Init).
		u64 addr = (u64)fault_addr;
		const char* tag = k_rOth; size_t tlen = sizeof(k_rOth) - 1;
		u64 dirBeg = g_directoryTable ? (u64)g_directoryTable->m_memory        : 0;
		u64 ovBeg  = g_directoryTable ? (u64)g_directoryTable->m_overlayMemory : 0;
		u64 mapBeg = (u64)g_mappedFileTable.m_mem;
		u64 shmBeg = (u64)g_shmMem;
		if      (dirBeg && addr >= dirBeg && addr < dirBeg + 128ull * 1024 * 1024) { tag = k_rDir; tlen = sizeof(k_rDir) - 1; }
		else if (mapBeg && addr >= mapBeg && addr < mapBeg +  16ull * 1024 * 1024) { tag = k_rMap; tlen = sizeof(k_rMap) - 1; }
		else if (ovBeg  && addr >= ovBeg  && addr < ovBeg  +  16ull * 1024 * 1024) { tag = k_rOv;  tlen = sizeof(k_rOv)  - 1; }
		else if (shmBeg && addr >= shmBeg && addr < shmBeg + 4096)                 { tag = k_rShm; tlen = sizeof(k_rShm) - 1; }
		for (size_t i = 0; i < tlen; ++i) *b++ = tag[i];

		*b++ = '\n';
		RawSyscall3(SYS_write, fd, (s64)buf, (s64)(b - buf));
		RawSyscall1(SYS_close, fd);
		RawSyscall3(SYS_write, 2, (s64)buf, (s64)(b - buf)); // also stderr — /tmp file gets cleaned up; stderr persists in host detour log
	}

	// SIG_DFL + tgkill the faulting thread so the kernel still terminates (and produces a core dump if ulimit allows).
	struct uba_k_sigaction dfl = { 0, 0, 0, { 0 } };
	RawSyscall6(13  /* SYS_rt_sigaction */, sig, (s64)&dfl, 0, 8, 0, 0);
	RawSyscall6(234 /* SYS_tgkill */, pid, tid, sig, 0, 0, 0);
}

// Forward decl — body lives below DetourCore.
static __attribute__((noinline, used)) void EntryMark(const char* msg, u32 len);

// Installs CrashHandler for SIGSEGV/SIGBUS/SIGILL/SIGFPE. `unused` because the DetourCore call site is currently `#if 0`'d (see comment there to re-enable).
// Destinations of const-string copy loops MUST be `volatile char*` — see CrashHandler's comment on movdqu vectorization in stub-patched Go binaries.
__attribute__((unused))
static void InstallCrashHandlers(void)
{
	static const char k_a[] __attribute__((section(".text"), used)) = "[uba-entry] install: enter\n";
	EntryMark(k_a, sizeof(k_a) - 1);

	// Write the install log FIRST (before rt_sigaction) so we know whether this pid even entered InstallCrashHandlers; append rt_sigaction returns once known.
	s64 pid = RawSyscall0(39 /* SYS_getpid */);

	static const char k_b[] __attribute__((section(".text"), used)) = "[uba-entry] install: after getpid\n";
	EntryMark(k_b, sizeof(k_b) - 1);

	char path[64];
	volatile char* p = path; // defeats movdqu-from-rodata vectorization for the byte writes below

	static const char k_b_alloc[] __attribute__((section(".text"), used)) = "[uba-entry] install: stack alloc\n";
	EntryMark(k_b_alloc, sizeof(k_b_alloc) - 1);

	*p++ = '/'; *p++ = 't'; *p++ = 'm'; *p++ = 'p'; *p++ = '/';
	*p++ = 'u'; *p++ = 'b'; *p++ = 'a'; *p++ = '_';
	*p++ = 'i'; *p++ = 'n'; *p++ = 's'; *p++ = 't'; *p++ = 'a'; *p++ = 'l'; *p++ = 'l'; *p++ = '.';

	static const char k_b_pre[] __attribute__((section(".text"), used)) = "[uba-entry] install: pre dec_u64\n";
	EntryMark(k_b_pre, sizeof(k_b_pre) - 1);

	p += StubDecU64((char*)p, (u64)pid);

	static const char k_b_post[] __attribute__((section(".text"), used)) = "[uba-entry] install: post dec_u64\n";
	EntryMark(k_b_post, sizeof(k_b_post) - 1);

	*p++ = '.'; *p++ = 'l'; *p++ = 'o'; *p++ = 'g';
	*p = 0;

	static const char k_c[] __attribute__((section(".text"), used)) = "[uba-entry] install: path built\n";
	EntryMark(k_c, sizeof(k_c) - 1);

	s64 fd = RawSyscall6(SYS_openat, (s64)AT_FDCWD, (s64)path, (s64)(O_WRONLY | O_CREAT | O_APPEND), (s64)0644, 0, 0);

	static const char k_d[] __attribute__((section(".text"), used)) = "[uba-entry] install: openat done\n";
	EntryMark(k_d, sizeof(k_d) - 1);

	if (fd >= 0)
	{
		char ebuf[64];
		volatile char* eb = ebuf;
		*eb++ = '['; *eb++ = 's'; *eb++ = 't'; *eb++ = 'u'; *eb++ = 'b';
		*eb++ = '-'; *eb++ = 'i'; *eb++ = 'n'; *eb++ = 's'; *eb++ = 't';
		*eb++ = 'a'; *eb++ = 'l'; *eb++ = 'l'; *eb++ = '-'; *eb++ = 'e';
		*eb++ = 'n'; *eb++ = 't'; *eb++ = 'e'; *eb++ = 'r'; *eb++ = ']';
		*eb++ = ' '; *eb++ = 'p'; *eb++ = 'i'; *eb++ = 'd'; *eb++ = '=';
		eb += StubDecU64((char*)eb, (u64)pid);
		*eb++ = '\n';
		RawSyscall3(SYS_write, fd, (s64)ebuf, (s64)(eb - ebuf));
	}

	static const char k_e[] __attribute__((section(".text"), used)) = "[uba-entry] install: pre rt_sigaction\n";
	EntryMark(k_e, sizeof(k_e) - 1);

	struct uba_k_sigaction sa;
	sa.sa_handler  = CrashHandler;
	sa.sa_flags    = UBA_SA_SIGINFO | UBA_SA_RESTORER;
	sa.sa_restorer = CrashRestorer;
	sa.sa_mask[0]  = 0;
	static const int sigs[] = { 11 /*SEGV*/, 7 /*BUS*/, 4 /*ILL*/, 8 /*FPE*/ };
	s64 results[4] = { 0, 0, 0, 0 };
	for (size_t i = 0; i < sizeof(sigs) / sizeof(sigs[0]); ++i)
		results[i] = RawSyscall6(13 /* SYS_rt_sigaction */, sigs[i], (s64)&sa, 0, 8, 0, 0);

	static const char k_f[] __attribute__((section(".text"), used)) = "[uba-entry] install: post rt_sigaction\n";
	EntryMark(k_f, sizeof(k_f) - 1);

	if (fd >= 0)
	{
		char buf[256];
		volatile char* b = buf;
		static const char k_h[]   __attribute__((section(".text"), used)) = "[stub-install-done] pid=";
		static const char k_seg[] __attribute__((section(".text"), used)) = " segv=";
		static const char k_bus[] __attribute__((section(".text"), used)) = " bus=";
		static const char k_ill[] __attribute__((section(".text"), used)) = " ill=";
		static const char k_fpe[] __attribute__((section(".text"), used)) = " fpe=";
		for (size_t i = 0; i < sizeof(k_h) - 1; ++i) *b++ = k_h[i];
		b += StubDecU64((char*)b, (u64)pid);
		const char* tags[]  = { k_seg, k_bus, k_ill, k_fpe };
		size_t      tlens[] = { sizeof(k_seg) - 1, sizeof(k_bus) - 1, sizeof(k_ill) - 1, sizeof(k_fpe) - 1 };
		for (int i = 0; i < 4; ++i)
		{
			for (size_t j = 0; j < tlens[i]; ++j) *b++ = tags[i][j];
			if (results[i] < 0) { *b++ = '-'; b += StubDecU64((char*)b, (u64)(-results[i])); }
			else                {             b += StubDecU64((char*)b, (u64) results[i]);   }
		}
		*b++ = '\n';
		RawSyscall3(SYS_write, fd, (s64)buf, (s64)(b - buf));
		RawSyscall1(SYS_close, fd);
	}
}

// Earliest-possible-from-C diagnostic — append `msg` to /tmp/uba_entry.log so we can tell how far init progressed if a stub binary SEGVs before InstallCrashHandlers runs.
static __attribute__((noinline, used)) void EntryMark(const char* msg, u32 len)
{
	s64 fd = RawSyscall6(SYS_openat, (s64)AT_FDCWD, (s64)"/tmp/uba_entry.log", (s64)(O_WRONLY | O_CREAT | O_APPEND), (s64)0644, 0, 0);
	if (fd >= 0)
	{
		RawSyscall3(SYS_write, fd, (s64)msg, (s64)len);
		RawSyscall1(SYS_close, fd);
	}
}

extern "C" __attribute__((used, visibility("default")))
void DetourCore(char** envp)
{
	// InstallCrashHandlers() and its /tmp/uba_entry.log diagnostics are gated off below — see the big block-comment inside `#if 0` for why and how to re-enable.
#if 0
	// What this block does (when enabled):
	//   - Installs CrashHandler for SIGSEGV/SIGBUS/SIGILL/SIGFPE via rt_sigaction(2), with a sa_restorer trampoline in UbaStaticStub.S issuing SYS_rt_sigreturn.
	//   - Handler writes one line to /tmp/uba_crash.<pid>.log on fault (rip, fault_addr, si_code, region classification), then SIG_DFL+re-raise.
	// Why it's off: in stub-PATCHED Go binaries (Android clang++ wrapper) clang at -O2 auto-vectorizes the small const-string copy loops into movdqu stores, which
	// SEGV with fault_addr=NULL inside the patched blob's PT_LOAD region. Things that DID NOT help: `volatile char*` everywhere, optimize("no-tree-vectorize")
	// (clang rejects), -fno-vectorize -fno-slp-vectorize on the TU. To re-enable, drop UbaStaticStubCore.cpp to -O1 in DetoursStatic/Build.sh.
	static const char k_core[] __attribute__((section(".text"), used)) = "[uba-entry] DetourCore entered\n";
	EntryMark(k_core, sizeof(k_core) - 1);
	InstallCrashHandlers();
	static const char k_postinst[] __attribute__((section(".text"), used)) = "[uba-entry] InstallCrashHandlers returned\n";
	EntryMark(k_postinst, sizeof(k_postinst) - 1);
#endif

	// Placement-new the shared g_fileHandles UnorderedMap into its .text/VARIABLE_MEM storage.
	InitFileHandles();

	// PID-NS check gates clone3+CLONE_PIDFD readback; common (non-NS) path stays plain fork.
	g_parentInNonInitNs = CheckParentInNonInitPidNs();

	// Alt-stack: guard-paged layout [pre 4KB PROT_NONE][stack PROT_RW][post 4KB PROT_NONE]. Stack grows DOWN so overflow walks into the pre-guard and faults
	// deterministically. Without guards, overflow silently corrupts adjacent mappings (most damagingly Go's 0xc0_00_00_00_00_00 goroutine arena), producing
	// "PC inside goroutine stack" SEGV_ACCERR crashes that look like wild jumps. mmap rather than .bss — our linker script packs .bss into .text.
	// Safe to do here: DetourCore runs strictly before the Syscall6 hook is installed, and the bridge is the only producer of alt-stack switches.
	{
		constexpr u64 kGuardBytes = 4096;
		u64 totalBytes = kGuardBytes + (u64)kAltStackBytes + kGuardBytes;
		void* m = mmap(nullptr, totalBytes, PROT_READ | PROT_WRITE, MAP_PRIVATE | MAP_ANONYMOUS, -1, 0);
		if (m == MAP_FAILED)
		{
			static const char msg[] __attribute__((section(".text"))) = "[uba_stub] alt-stack mmap failed; aborting\n";
			StubWrite(2, msg, sizeof(msg) - 1);
			RawSyscall1(231 /*exit_group*/, 1);
		}
		// Both mprotect failures are fatal — running without the guard would mask the very class of bug we're trying to catch.
		if (RawSyscall6(10 /*SYS_mprotect*/, (s64)m, (s64)kGuardBytes, (s64)PROT_NONE, 0, 0, 0) != 0)
		{
			static const char msg[] __attribute__((section(".text"))) = "[uba_stub] alt-stack pre-guard mprotect failed; aborting\n";
			StubWrite(2, msg, sizeof(msg) - 1);
			RawSyscall1(231 /*exit_group*/, 1);
		}
		if (RawSyscall6(10 /*SYS_mprotect*/, (s64)((u8*)m + kGuardBytes + kAltStackBytes), (s64)kGuardBytes, (s64)PROT_NONE, 0, 0, 0) != 0)
		{
			static const char msg[] __attribute__((section(".text"))) = "[uba_stub] alt-stack post-guard mprotect failed; aborting\n";
			StubWrite(2, msg, sizeof(msg) - 1);
			RawSyscall1(231 /*exit_group*/, 1);
		}
		g_altStackBase = (u8*)m + kGuardBytes;
		g_altStackTop  = g_altStackBase + kAltStackBytes;
	}

	// PR_SET_PDEATHSIG=SIGHUP (catchable, NOT SIGKILL) so a dying child still runs SYS_exit_group → MessageType_Exit. Under SIGKILL the exit handshake is dropped.
	(void)RawSyscall6(157 /* SYS_prctl */, 1 /* PR_SET_PDEATHSIG */, 1 /* SIGHUP */, 0, 0, 0, 0);

	// Single envp pass: UBA_COMID, UBA_LOGFILE, UBA_CWD, UBA_REMOTE.
	const char* value   = 0;
	const char* logPath = 0;
	const char* cwdPath = 0;
	for (char** p = envp; *p; ++p)
	{
		if      (!value           && StubStartsWith(*p, s_envPrefix))       value   = *p + sizeof(s_envPrefix)       - 1;
		else if (!logPath         && StubStartsWith(*p, s_envLogPrefix))    logPath = *p + sizeof(s_envLogPrefix)    - 1;
		else if (!cwdPath         && StubStartsWith(*p, s_envCwdPrefix))    cwdPath = *p + sizeof(s_envCwdPrefix)    - 1;
		else if (!g_runningRemote && StubStartsWith(*p, s_envRemotePrefix)) g_runningRemote = true;
		else if (!g_ubaFileMappingDir && StubStartsWith(*p, s_envFileMapDirPrefix)) g_ubaFileMappingDir = *p + sizeof(s_envFileMapDirPrefix) - 1;
		else if (StubStartsWith(*p, s_envMemfdPrefix)) g_ubaUseMemfd = (*p + sizeof(s_envMemfdPrefix) - 1)[0] == '1';
	}

	// Mirror LD_PRELOAD (UbaDetoursMainPosix.cpp:170): chdir into UBA_CWD so AT_FDCWD-relative paths resolve against the session's intended dir, not parent's.
	if (cwdPath && *cwdPath)
		(void)RawSyscall1(80 /* SYS_chdir */, (s64)cwdPath);

	// O_APPEND (not O_TRUNC) protects two scenarios from data loss: (1) patched binary execves into another and inherits UBA_LOGFILE; (2) child inherits parent's
	// UBA_LOGFILE via UBA_EXEC. UbaCli arranges fresh per-process paths at session start so this is only relevant on re-entry. fd is shared between g_logFd
	// (stub's primitive per-syscall log) and g_debugFile (shared DEBUG_LOG_* macros). Mirrors UbaDetoursFunctionsPosix.cpp:4201's execve-reached O_APPEND.
	if (logPath && *logPath)
	{
		s64 fd = RawSyscall6(SYS_openat, (s64)AT_FDCWD, (s64)logPath, (s64)(O_WRONLY | O_CREAT | O_APPEND), (s64)0644, 0, 0);
		if (fd >= 0) { g_logFd = (int)fd; g_debugFile = (FileHandle)fd; }
	}

	if (!value) { StubWrite(2, s_msgNoEnv, sizeof(s_msgNoEnv) - 1); return; }

	// Parse "<uid>+<offset>".
	u64 uid = 0, offset = 0;
	const char* cursor = value;
	if (!ParseU64Dec(&cursor, &uid) || *cursor != '+') { StubWrite(2, s_msgBadVal, sizeof(s_msgBadVal) - 1); return; }
	++cursor; // past '+'
	if (!ParseU64Dec(&cursor, &offset))                 { StubWrite(2, s_msgBadVal, sizeof(s_msgBadVal) - 1); return; }

	// Comm shm attach via the shared FileMapping_MapFromHost (memfd path: pidfd_open + pidfd_getfd + mmap via RawSyscallDirect — see UbaFileMapping.inl).
	u8* rptr = FileMapping_MapFromHost(FileMappingHandle::FromU64(uid), UBA_COMM_MEM_SIZE, offset, /*writable*/true, "Communication");
	if (!rptr) { StubWrite(2, s_msgMapFail, sizeof(s_msgMapFail) - 1); return; }

	g_shmMem            = (void*)rptr;
	g_messageMappingMem = (u8*)g_shmMem + 3 * UBA_EVENT_SIZE;
	g_cancelEvent       = (SharedEvent*)((u8*)g_shmMem + 0 * UBA_EVENT_SIZE);
	g_readEvent         = (SharedEvent*)((u8*)g_shmMem + 1 * UBA_EVENT_SIZE);
	g_writeEvent        = (SharedEvent*)((u8*)g_shmMem + 2 * UBA_EVENT_SIZE);

	STUB_INIT_LOG(s_msgAttached, sizeof(s_msgAttached) - 1);
	STUB_INIT_LOG(s_nl, 1);

	// LogEventTags() disabled — SEGV'd on Go `compile`/`link`. Diagnostic only; session creates tag-3 (EventFutex) slots and our SharedEvent dispatch handles it.
	//LogEventTags();

	// Report Syscall6 address. Quiet on glibc-static targets (the errno hook slot is the discriminator).
	if (g_syscall6Addr == 0xDEADBEEFC0FFEE00ULL)
	{
		if (!__atomic_load_n(&g_glibc_errno_hook_fn, __ATOMIC_ACQUIRE))
			StubWrite(2, s_msgNoSyscall, sizeof(s_msgNoSyscall) - 1);
	}
	else
	{
		char addr[16];
		StubHex16(addr, g_syscall6Addr);
		STUB_INIT_LOG(s_msgSyscall, sizeof(s_msgSyscall) - 1);
		STUB_INIT_LOG(addr, 16);
		STUB_INIT_LOG(s_nl, 1);
		InstallSyscall6Hook();
	}

	// runtime.exit.abi0 issues SYS_exit_group inline, bypassing Syscall6. Hook it so we still send MessageType_Exit.
	InstallExitHook();

	InstallVforkHook();

	// BLAKE3 sanity probe disabled — SEGV'd on Go `compile`/`link` inside blake3_hasher_init even though merge_zips / build_license_metadata succeed.
	// Real hashing path runs through Rpc_CreateFileW and is exercised by every CreateFile. Reference (16-byte BLAKE3 of "/tmp/static-detour/hello.txt"):
	//   eb 96 44 ac aa a4 2e fb c9 87 df 96 6e e7 00 c0  →  a=0xfb2ea4aaac4496eb, b=0xc000e76e96df87c9

	// InitSharedVariables placement-news g_communicationLock / g_stats / g_memoryBlock / DirectoryTable / MappedFileTable storage. Must precede the Init RPC because
	// Shared_Init's RPC_MESSAGE acquires g_communicationLock. Previously this lived in the post-init block (after SendInitRpc) but Shared_Init drives both now.
	InitSharedVariables();

	// Shared_Init: MessageType_Init send/parse + table mapping + DirectoryTable adopt + env adoption. One function shared with LD_PRELOAD's Init().
	Shared_Init(envp);
	__atomic_store_n(&g_initDone, 1, __ATOMIC_RELEASE);
}
