// Copyright Epic Games, Inc. All Rights Reserved.
//
// Stub TU that compiles UbaFileMapping.inl's memfd path into the static-stub blob.
// All inl Logger/FileHandle-using functions are gated out via #if !UBA_STUB_BUILD,
// leaving only the stub-needed parts (FileMapping_MapFromHost / MapFromHost2,
// the placeholder helpers, and FileMapping_DuplicateFromHost). Everything uses
// RawSyscallDirect — no glibc, no GOT loads, safe for the stub's freestanding link.

#include "UbaBase.h"
#include "UbaPlatform.h"      // UBA_ASSERT / UBA_ASSERTF (no-op in stub)
#include "UbaFileMapping.h"
#include "UbaLinuxRawSyscall.h"

namespace uba
{
#include "UbaFileMapping.inl"
}
