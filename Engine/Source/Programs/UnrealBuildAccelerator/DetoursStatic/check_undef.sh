#!/bin/bash
# Helper: dump undefined symbols per object from the most recent tmp build dir.
NM=/home/honk/git/android/prebuilts/clang/host/linux-x86/clang-r547379/bin/llvm-nm
TMPDIR=$(ls -td /tmp/tmp.* 2>/dev/null | head -1)
echo "Using $TMPDIR"
for o in "$TMPDIR"/*.o; do
    name=$(basename "$o")
    echo "==== $name ===="
    "$NM" -u "$o"
done
