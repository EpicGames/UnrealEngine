#!/bin/bash
TMPDIR=$(ls -td /tmp/tmp.* 2>/dev/null | head -1)
NM=/home/honk/git/android/prebuilts/clang/host/linux-x86/clang-r547379/bin/llvm-nm
READELF=/home/honk/git/android/prebuilts/clang/host/linux-x86/clang-r547379/bin/llvm-readelf
echo "Using $TMPDIR"
echo "=== g_memoryBlock / g_memoryBlockMem syms ==="
"$NM" "$TMPDIR/stub.elf" | grep -E "g_memoryBlock"
echo
echo "=== init_array / relative relocs ==="
"$READELF" -a "$TMPDIR/stub.elf" | grep -E "\.init_array|init_array|DYNAMIC|RELATIVE|IRELATIVE" | head -30
echo
echo "=== section headers ==="
"$READELF" -S "$TMPDIR/stub.elf" | head -40
