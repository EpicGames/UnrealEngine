#!/bin/bash
CB=/home/honk/git/android/prebuilts/clang/host/linux-x86/clang-r547379/bin
elf=$(ls -td /tmp/tmp.* 2>/dev/null | head -1)/stub.elf
echo "elf=$elf"
echo
echo '=== operator new / delete / malloc symbols ==='
"$CB/llvm-nm" -C "$elf" | grep -iE 'operator new|operator delete|^[0-9a-f]+ [tTwWuU] (malloc|free|calloc|realloc)\b'
echo
echo '=== HashMap/Allocator symbols ==='
"$CB/llvm-nm" -C "$elf" | grep -iE 'HashMap|MemoryBlock|Allocator' | head -15
echo
echo '=== m_lookup / m_memoryBlock ==='
"$CB/llvm-nm" -C "$elf" | grep -iE 'm_lookup|m_memoryBlock' | head -10
