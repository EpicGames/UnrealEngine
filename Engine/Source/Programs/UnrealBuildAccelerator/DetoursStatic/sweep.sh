#!/bin/bash
# Run every candidate target through UbaCli local mode, classify the outcome.
# Expected to pass: merge_zips, build_license_metadata, compile (newly fixed),
#                   link, soong_zip, sbox, zipsync, zip2zip.
# Deferred: clang, clang++ (glibc-static).

set +e

UBA=/mnt/d/devel/fn/Engine/Binaries/Linux/UnrealBuildAccelerator/UbaCli
AOSP=/home/honk/git/android
ANDROID_BIN=$AOSP/out/host/linux-x86/bin

run_case() {
    local name=$1
    local binary=$2
    shift 2
    local args=("$@")
    local log=/tmp/sweep_${name}.log
    local uba_copy="${binary}.uba"
    rm -f "$log" "$uba_copy" /home/honk/UbaCli/debuglog.log
    if [ ! -x "$binary" ]; then
        echo "[$name] MISSING BINARY: $binary"
        return
    fi
    (cd "$AOSP" && timeout 15 "$UBA" -workdir="$AOSP" local "$binary" "${args[@]}" > "$log" 2>&1)
    local rc=$?

    # Classify
    local status="UNKNOWN"
    local detail=""
    if grep -q 'SIGSEGV\|killed by signal' "$log"; then
        status="FAIL-CRASH"
        detail=$(grep -m1 'SIGSEGV\|killed by signal' "$log")
    elif grep -q '\[uba_stub\] Init resp' "$log"; then
        local cfcount
        cfcount=$(grep -c '\[uba_stub\] CreateFile(' "$log")
        status="PASS"
        detail="${cfcount} CreateFile RPCs, rc=$rc"
    else
        status="NO-INIT"
        detail="stub never sent Init resp, rc=$rc"
    fi
    printf "%-30s %-12s  %s\n" "$name" "$status" "$detail"
}

echo "=== UBA static-detour stub sweep ==="
echo

# Known-good (retest).
run_case "merge_zips-help"      "$ANDROID_BIN/merge_zips" --help
run_case "merge_zips-openat"    "$ANDROID_BIN/merge_zips" -j /tmp/mz_sweep.zip /tmp/nonexistent.zip
run_case "build_license_meta"   "$ANDROID_BIN/build_license_metadata" --help
run_case "soong_zip-help"       "$ANDROID_BIN/soong_zip" --help
run_case "sbox-help"            "$ANDROID_BIN/sbox" --help
run_case "zipsync-help"         "$ANDROID_BIN/zipsync" --help
run_case "zip2zip-help"         "$ANDROID_BIN/zip2zip" --help

# Previously broken — now should pass.
GO_TOOL=$AOSP/prebuilts/go/linux-x86/pkg/tool/linux_amd64
run_case "go-compile"           "$GO_TOOL/compile" -o /tmp/sweep_compile.a.tmp -p p build/soong/third_party/zip/reader.go
run_case "go-link"              "$GO_TOOL/link" -V

# Deferred — may not work yet.
CLANG_ROOT=$AOSP/prebuilts/clang/host/linux-x86/clang-r547379/bin
run_case "clang++-version"      "$CLANG_ROOT/clang++" --version
run_case "clang-version"        "$CLANG_ROOT/clang" --version

echo
echo "=== sweep done ==="
