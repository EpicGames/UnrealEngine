// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaHost.h"
#include "UbaAWS.h"
#include "UbaBinaryParser.h"
#include "UbaCacheClient.h"
#include "UbaConfig.h"
#include "UbaCoordinatorWrapper.h"
#include "UbaNetworkBackendQuic.h"
#include "UbaNetworkBackendTcp.h"
#include "UbaNetworkClient.h"
#include "UbaProcess.h"
#include "UbaRootPaths.h"
#include "UbaScheduler.h"
#include "UbaStorageServer.h"
#include "UbaSessionServer.h"

// Several exported entry points cast 2-arg ProcessHandle_ExitCallback* to the 3-arg
// ProcessStartInfo::ExitedCallback*. The trailing ProcessExitedResponse& is ignored at
// the call site for these callers, but Clang's -Wcast-function-type-mismatch (and MSVC
// C4191 under /Wall) flags it. Wrap each cast with the macros below.
#if defined(__clang__)
	#define UBA_PUSH_DISABLE_CAST_FN_WARNING \
		_Pragma("clang diagnostic push") \
		_Pragma("clang diagnostic ignored \"-Wcast-function-type-mismatch\"")
	#define UBA_POP_DISABLE_CAST_FN_WARNING _Pragma("clang diagnostic pop")
#elif defined(_MSC_VER)
	#define UBA_PUSH_DISABLE_CAST_FN_WARNING \
		__pragma(warning(push)) __pragma(warning(disable:4191))
	#define UBA_POP_DISABLE_CAST_FN_WARNING __pragma(warning(pop))
#else
	#define UBA_PUSH_DISABLE_CAST_FN_WARNING
	#define UBA_POP_DISABLE_CAST_FN_WARNING
#endif

namespace uba
{
	CallbackLogWriter::CallbackLogWriter(BeginScopeCallback begin, EndScopeCallback end, LogCallback log) : m_beginScope(begin), m_endScope(end), m_logCallback(log)
	{
	}

	void CallbackLogWriter::BeginScope()
	{
		(*m_beginScope)();
	}
	void CallbackLogWriter::EndScope()
	{
		(*m_endScope)();
	}

	void CallbackLogWriter::Log(LogEntryType type, const uba::tchar* str, u32 strLen, const uba::tchar* prefix, u32 prefixLen)
	{
		StringBuffer<> strBuf;
		if (prefixLen && strLen + prefixLen + 3 < strBuf.capacity) // TODO: Send prefix and prefixLen through callback
		{
			strBuf.Append(prefix, prefixLen);
			strBuf.Append(TCV(" - "));
			strBuf.Append(str, strLen);
			strLen += prefixLen + 3;
			str = strBuf.data;
		}
		(*m_logCallback)(type, str, strLen);
	}

	class NetworkServerWithBackend : public NetworkServer
	{
	public:
		NetworkServerWithBackend(bool& outSuccess, const NetworkServerCreateInfo& info, NetworkBackend* nb)
		: NetworkServer(outSuccess, info), backend(nb)
		{
		}

		NetworkBackend* backend;
	};

	class NetworkClientWithBackend : public NetworkClient
	{
	public:
		NetworkClientWithBackend(bool& outSuccess, const NetworkClientCreateInfo& info, NetworkBackend* nb, const tchar* name)
		: NetworkClient(outSuccess, info, name), backend(nb)
		{
		}

		NetworkBackend* backend;
	};

	class RootPathsWithLogger : public RootPaths
	{
	public:
		RootPathsWithLogger(LogWriter& writer) : logger(writer) {}
		LoggerWithWriter logger;
	};

	class CacheClientWithCounter : public CacheClient
	{
	public:
		CacheClientWithCounter(const CacheClientCreateInfo& info) : CacheClient(info) {}
		Atomic<u32> active = 0;
	};

	struct CacheClientActiveScope
	{
		CacheClientActiveScope(CacheClient* c) : client(*(CacheClientWithCounter*)c) { ++client.active; }
		~CacheClientActiveScope() { --client.active; }
		CacheClientWithCounter& client;
	};

	Config& GetConfig(const tchar* fileName = nullptr)
	{
		LoggerWithWriter logger(g_nullLogWriter);

		static Config defaultConfig;

		if (fileName && *fileName && !defaultConfig.IsLoaded() && Contains(fileName, TC("UbaHost.toml")))
		{
			defaultConfig.LoadFromFile(logger, fileName);
			return defaultConfig;
		}
		else if (!fileName || !*fileName)
		{
			if (defaultConfig.IsLoaded())
				return defaultConfig;
			StringBuffer<> temp;
			GetDirectoryOfCurrentModule(logger, temp);
			temp.EnsureEndsWithSlash().Append(TCV("UbaHost.toml"));
			defaultConfig.LoadFromFile(logger, temp.data);
			return defaultConfig;
		}
		else
		{
			auto config = new Config();
			config->LoadFromFile(logger, fileName);
			return *config;
		}
	}

	struct ExportsDowngradedLogger : public LoggerWithWriter
	{
		ExportsDowngradedLogger(LogWriter& writer, const tchar* prefix) : LoggerWithWriter(writer, prefix) {}
		virtual void Log(LogEntryType type, const tchar* str, u32 strLen) override { LoggerWithWriter::Log(Max(LogEntryType_Info, type), str, strLen); }
	};

	// Not great with global but 
	MutexHandle g_exclusiveMutex = InvalidMutexHandle;

	Trace* g_globalTrace;
	LogWriter* g_globalLogWriter;
}

extern "C"
{
	uba::LogWriter* LogWriter_GetDefault()
	{
		return &uba::g_consoleLogWriter;
	}

	uba::LogWriter* LogWriter_GetNull()
	{
		return &uba::g_nullLogWriter;
	}

	uba::LogWriter* LogWriter_CreateCallback(uba::CallbackLogWriter::BeginScopeCallback begin, uba::CallbackLogWriter::EndScopeCallback end, uba::CallbackLogWriter::LogCallback log)
	{
		using namespace uba;
		#if !PLATFORM_LINUX
		AddExceptionHandler();
		#endif
		InitMemory();

		LogWriter* writer = new CallbackLogWriter(begin, end, log);

		LoggerWithWriter logger(*writer, TC(""));
		SetMemoryWorkingSet(logger, DefaultMemoryWorkingSet);

		#if !PLATFORM_WINDOWS
		SetupOrphanReaper();
		#endif

		return writer;
	}

	void LogWriter_SetGlobal(uba::LogWriter* writer)
	{
		uba::g_globalLogWriter = writer;
	}

	void LogWriter_DestroyCallback(uba::LogWriter* writer)
	{
		if (writer != &uba::g_consoleLogWriter)
			delete writer;
	}

	uba::LogWriter* CreateCallbackLogWriter(uba::CallbackLogWriter::BeginScopeCallback begin, uba::CallbackLogWriter::EndScopeCallback end, uba::CallbackLogWriter::LogCallback log)
	{
		return LogWriter_CreateCallback(begin, end, log);
	}

	void DestroyCallbackLogWriter(uba::LogWriter* writer)
	{
		LogWriter_DestroyCallback(writer);
	}

	uba::Config* Config_Load(const uba::tchar* configFile)
	{
		return &uba::GetConfig(configFile);
	}

	uba::Config* Config_Create()
	{
		return new uba::Config();
	}

	void Config_Destroy(uba::Config* config)
	{
		delete config;
	}

	uba::ConfigTable* Config_RootTable(uba::Config& config)
	{
		return &config;
	}

	uba::ConfigTable* Config_AddTable(uba::Config& config, const uba::tchar* name)
	{
		return &config.AddTable(name);
	}

	void ConfigTable_AddValueInt(uba::ConfigTable& table, const uba::tchar* key, int value)
	{
		table.AddValue(key, value);
	}

	void ConfigTable_AddValueU32(uba::ConfigTable& table, const uba::tchar* key, uba::u32 value)
	{
		table.AddValue(key, value);
	}

	void ConfigTable_AddValueU64(uba::ConfigTable& table, const uba::tchar* key, uba::u64 value)
	{
		table.AddValue(key, value);
	}

	void ConfigTable_AddValueBool(uba::ConfigTable& table, const uba::tchar* key, bool value)
	{
		table.AddValue(key, value);
	}

	void ConfigTable_AddValueString(uba::ConfigTable& table, const uba::tchar* key, const uba::tchar* str)
	{
		table.AddValue(key, str);
	}

	uba::NetworkServer* NetworkServer_Create(uba::LogWriter& writer, uba::u32 workerCount, uba::u32 sendSize, uba::u32 receiveTimeoutSeconds, bool useQuic)
	{
		using namespace uba;

		if constexpr (!IsArmBinary)
			if (IsRunningArm())
				LoggerWithWriter(writer, TC("")).Warning(TC("  Running x64 binary on arm64 system. Use arm binaries instead"));


		NetworkBackend* networkBackend;
		#if UBA_USE_QUIC
		if (useQuic)
			networkBackend = new NetworkBackendQuic(writer);
		else
		#endif
		{
			NetworkBackendTcpCreateInfo info(writer);
			info.Apply(GetConfig());
			networkBackend = new NetworkBackendTcp(info);
		}

		NetworkServerCreateInfo info(writer);

		info.workerCount = workerCount;
		info.sendSize = sendSize;
		info.receiveTimeoutSeconds = receiveTimeoutSeconds;
		info.useThreadedSend = !IsLinux; // Threaded send is turned off for linux since it is not needed (it improves perf/throughput on win and mac)
		info.Apply(GetConfig());

		bool success = true;
		auto server = new NetworkServerWithBackend(success, info, networkBackend);
		if (!success)
		{
			delete server;
			return nullptr;
		}

		server->SetTrace(uba::g_globalTrace);
		return server;
	}

	void NetworkServer_Destroy(uba::NetworkServer* server)
	{
		auto s = (uba::NetworkServerWithBackend*)server;
		auto networkBackend = s->backend;
		delete s;
		delete networkBackend;
	}

	bool NetworkServer_StartListen(uba::NetworkServer* server, int port, const uba::tchar* ip, const uba::tchar* crypto)
	{
		using namespace uba;

		auto s = (NetworkServerWithBackend*)server;

		bool requiresCrypto = false;
		if (crypto && *crypto)
		{
			//server->GetLogger().Error(TC("CRYPTO: %s"), crypto);
			u8 crypto128Data[16];
			if (!CryptoFromString(crypto128Data, 16, crypto))
				return server->GetLogger().Error(TC("Failed to parse crypto key %s"), crypto);
			s->RegisterCryptoKey(crypto128Data);
			requiresCrypto = true;
		}

		return s->StartListen(*s->backend, u16(port), ip, requiresCrypto);
	}

	void NetworkServer_Stop(uba::NetworkServer* server)
	{
		auto s = (uba::NetworkServerWithBackend*)server;
		auto networkBackend = s->backend;
		networkBackend->StopListen();
		server->DisconnectClients();
	}

	void NetworkServer_SetClientsConfig(uba::NetworkServer* server, const uba::Config& config)
	{
		server->SetClientsConfig(config);
	}

	bool NetworkServer_AddClient(uba::NetworkServer* server, const uba::tchar* ip, int port, const uba::tchar* crypto)
	{
		using namespace uba;
		u8 crypto128Data[16];
		u8* crypto128 = nullptr;
		if (CryptoFromString(crypto128Data, 16, crypto))
			crypto128 = crypto128Data;

		auto s = (NetworkServerWithBackend*)server;
		return s->AddClient(*s->backend, ip, u16(port), crypto128);
	}

	uba::StorageServer* StorageServer_InternalCreate(uba::StorageServerCreateInfo& info)
	{
		using namespace uba;
		#if UBA_USE_CLOUD
		auto oldDir = info.rootDir;
		auto oldZone = info.zone;
		StringBuffer<> fixedRootDir;
		fixedRootDir.count = GetFullPathNameW(info.rootDir, fixedRootDir.capacity, fixedRootDir.data, NULL);
		fixedRootDir.Replace('/', PathSeparator).EnsureEndsWithSlash();
		info.rootDir = fixedRootDir.data;
		Cloud cloud;
		if (!info.zone || !*info.zone)
		{
			ExportsDowngradedLogger downgradedLogger(info.writer, TC("Cloud"));
			if (cloud.QueryAvailabilityZone(downgradedLogger, info.rootDir))
				info.zone = cloud.GetAvailabilityZone();
		}

		StringBuffer<256> zoneTemp;
		if (!info.zone || !*info.zone)
			if (GetZone(zoneTemp))
				info.zone = zoneTemp.data;

		info.exclusiveMutex = g_exclusiveMutex;
		g_exclusiveMutex = InvalidMutexHandle;

		#endif

		auto storageServer = new StorageServer(info);

		#if UBA_USE_CLOUD
		info.rootDir = oldDir;
		info.zone = oldZone;
		info.exclusiveMutex = InvalidMutexHandle;
		#endif
		return storageServer;
	}

	uba::StorageServer* StorageServer_Create(uba::NetworkServer& server, const uba::tchar* rootDir, uba::u64 casCapacityBytes, bool storeCompressed, uba::LogWriter& writer, const uba::tchar* zone)
	{
		using namespace uba;
		StorageServerCreateInfo info(server, rootDir, writer);
		info.Apply(GetConfig());
		if (zone && !*zone)
			info.zone = zone;
		info.casCapacityBytes = casCapacityBytes;
		info.storeCompressed = storeCompressed;
		return StorageServer_InternalCreate(info);
	}

	uba::StorageServer* StorageServer_Create2(uba::NetworkServer& server, const uba::Config& config, uba::LogWriter& writer)
	{
		using namespace uba;
		StorageServerCreateInfo info(server, TC(""), writer);
		info.Apply(config);
		return StorageServer_InternalCreate(info);
	}

	void StorageServer_PreloadCasTable(uba::StorageServer* storageServer)
	{
		if (!storageServer->IsCasTableLoaded())
			storageServer->LoadCasTable();
	}

	void StorageServer_Destroy(uba::StorageServer* storageServer)
	{
		delete storageServer;
	}

	void StorageServer_SaveCasTable(uba::StorageServer* storageServer)
	{
		storageServer->SaveCasTable(true);
	}

	void StorageServer_RegisterDisallowedPath(uba::StorageServer* storageServer, const uba::tchar* path)
	{
		storageServer->RegisterDisallowedPath(path);
	}

	void StorageServer_DeleteFile(uba::StorageServer* storageServer, const uba::tchar* file)
	{
		storageServer->DeleteCasForFile(file);
	}

	uba::u32 ProcessHandle_GetExitCode(const uba::ProcessHandle* handle)
	{
		return handle->GetExitCode();
	}

	bool ProcessHandle_IsCancelled(const uba::ProcessHandle* handle)
	{
		return handle->GetExitCode() == uba::ProcessCancelExitCode;
	}

	uba::u8 ProcessHandle_GetExecutionType(const uba::ProcessHandle* handle)
	{
		return handle->GetExecutionType();
	}

	const uba::tchar* ProcessHandle_GetExecutingHost(const uba::ProcessHandle* handle)
	{
		return handle->GetExecutingHost();
	}

	const uba::tchar* ProcessHandle_GetLogLine(const uba::ProcessHandle* handle, uba::u32 index)
	{
		const auto& lines = handle->GetLogLines();
		if (index >= lines.size()) return nullptr;
		return lines[index].text.c_str();
	}

	uba::u64 ProcessHandle_GetHash(const uba::ProcessHandle* handle)
	{
		return handle->GetHash();
	}

	uba::u64 TimeToTick(uba::u64 time)
	{
		return time * 10'000'000 / uba::GetFrequency();
	}

	// 100ns ticks
	uba::u64 ProcessHandle_GetTotalProcessorTime(const uba::ProcessHandle* handle)
	{
		return TimeToTick(handle->GetTotalProcessorTime());
	}

	// 100ns ticks
	uba::u64 ProcessHandle_GetTotalWallTime(const uba::ProcessHandle* handle)
	{
		return TimeToTick(handle->GetTotalWallTime());
	}

	uba::u64 ProcessHandle_GetPeakMemory(const uba::ProcessHandle* handle)
	{
		return handle->GetPeakMemory();
	}

	void ProcessHandle_TraverseOutputFiles(const uba::ProcessHandle* handle, ProcessHandle_OutputFileCallback* callback, void* userData)
	{
		handle->TraverseOutputFiles([&](uba::StringView file) { callback(userData, file.data); });
	}

	bool ProcessHandle_WaitForExit(const uba::ProcessHandle* handle, uba::u32 millisecondsTimeout)
	{
		return handle->WaitForExit(millisecondsTimeout);
	}

	void ProcessHandle_Cancel(const uba::ProcessHandle* handle, bool terminate)
	{
		handle->Cancel();
	}

	void ProcessHandle_Destroy(const uba::ProcessHandle* handle)
	{
		delete handle;
	}

	void DestroyProcessHandle(const uba::ProcessHandle* handle)
	{
		delete handle;
	}

	const uba::ProcessStartInfo* Process_GetStartInfo(const uba::Process& process)
	{
		return &process.GetStartInfo();
	}

	uba::SessionServerCreateInfo* SessionServerCreateInfo_Create(uba::StorageServer& storage, uba::NetworkServer& client, uba::LogWriter& writer, const uba::tchar* rootDir, const uba::tchar* traceOutputFile, bool disableCustomAllocator, bool launchVisualizer, bool resetCas, bool writeToDisk, bool detailedTrace, bool allowWaitOnMem, bool allowKillOnMem, bool storeIntermediateFilesCompressed)
	{
		auto info = new uba::SessionServerCreateInfo(storage, client, writer);
		info->Apply(uba::GetConfig());
		info->rootDir = uba::TStrdup(rootDir);
		info->traceOutputFile = uba::TStrdup(traceOutputFile);
		info->allowCustomAllocator = !disableCustomAllocator;
		info->launchVisualizer = launchVisualizer;
		info->resetCas = resetCas;
		info->shouldWriteToDisk = writeToDisk;
		info->detailedTrace = detailedTrace;
		info->storeIntermediateFilesCompressed = storeIntermediateFilesCompressed;
		//info->remoteTraceEnabled = true;
		info->remoteLogEnabled = true;
		info->trace = uba::g_globalTrace;
		return info;
	}

	void SessionServerCreateInfo_Destroy(uba::SessionServerCreateInfo* info)
	{
		free((void*)info->traceOutputFile);
		free((void*)info->rootDir);
		delete info;
	}

	uba::SessionServer* SessionServer_Create(const uba::SessionServerCreateInfo& info, const uba::u8* environment, uba::u32 environmentSize)
	{
		return new uba::SessionServer(info, environment, environmentSize);
	}
	uba::SessionServer* SessionServer_Create2(uba::StorageServer& s, uba::NetworkServer& ns, const uba::Config& c, uba::LogWriter& lw, const uba::u8* environment, uba::u32 environmentSize)
	{
		uba::SessionServerCreateInfo info(s, ns, lw);
		info.Apply(c);
		return new uba::SessionServer(info, environment, environmentSize);
	}
	void SessionServer_SetRemoteProcessAvailable(uba::SessionServer* server, SessionServer_RemoteProcessAvailableCallback* available, void* userData)
	{
		server->SetRemoteProcessSlotAvailableEvent([available, userData](bool isCrossArchitecture, bool isCrossPlatform) { available(userData, isCrossArchitecture); });
	}
	void SessionServer_SetRemoteProcessReturned(uba::SessionServer* server, SessionServer_RemoteProcessReturnedCallback* returned, void* userData)
	{
		server->SetRemoteProcessReturnedEvent([returned, userData](uba::Process& process) { returned(process, userData); });
	}
	bool SessionServer_RefreshDirectory(uba::SessionServer* server, const uba::tchar* directory)
	{
		return server->RefreshDirectory(directory);
	}
	bool SessionServer_RegisterNewFile(uba::SessionServer* server, const uba::tchar* filePath)
	{
		return server->RegisterNewFile(filePath);
	}
	void SessionServer_RegisterDeleteFile(uba::SessionServer* server, const uba::tchar* filePath)
	{
		server->RegisterDeleteFile(filePath);
	}
	bool SessionServer_RegisterNewDirectory(uba::SessionServer* server, const uba::tchar* directoryPath)
	{
		return server->RegisterNewDirectory(directoryPath, true);
	}
	bool SessionServer_RegisterVirtualFile(uba::SessionServer* server, const uba::tchar* filename, const uba::tchar* sourceFile, uba::u64 sourceOffset, uba::u64 sourceSize)
	{
		return server->RegisterVirtualFile(filename, sourceFile, sourceOffset, sourceSize);
	}
	bool SessionServer_CreateVirtualFile(uba::SessionServer* server, const uba::tchar* filename, const void* memory, uba::u64 memorySize, bool transient, bool registerToDir)
	{
		return server->CreateVirtualFile(filename, memory, memorySize, transient, registerToDir);
	}
	bool SessionServer_DeleteVirtualFile(uba::SessionServer* server, const uba::tchar* filename, bool registerToDir)
	{
		return server->DeleteVirtualFile(filename, registerToDir);
	}
	bool SessionServer_GetOutputFileSize(uba::SessionServer* server, uba::u64& outSize, const uba::tchar* filename)
	{
		return server->GetOutputFileSize(outSize, filename);
	}
	bool SessionServer_GetOutputFileData(uba::SessionServer* server, void* outData, const uba::tchar* filename, bool deleteInternalMapping)
	{
		return server->GetOutputFileData(outData, filename, deleteInternalMapping);
	}
	bool SessionServer_WriteOutputFile(uba::SessionServer* server, const uba::tchar* filename, bool deleteInternalMapping)
	{
		return server->WriteOutputFile(filename, deleteInternalMapping);
	}
	uba::ProcessHandle* SessionServer_RunProcess(uba::SessionServer* server, uba::ProcessStartInfo& info, bool async, bool enableDetour)
	{
		return new uba::ProcessHandle(server->RunProcess(info, async, enableDetour));
	}
	uba::ProcessHandle* SessionServer_RunProcessRemote(uba::SessionServer* server, uba::ProcessStartInfo& info, float weight, const void* knownInputs, uba::u32 knownInputsCount, bool allowCrossArchitecture)
	{
		return new uba::ProcessHandle(server->RunProcessRemote(info, weight, knownInputs, knownInputsCount, allowCrossArchitecture));
	}
	uba::u64 SessionServer_RegisterRoots(uba::SessionServer* server, const void* rootsData, uba::u64 rootsDataSize)
	{
		return server->RegisterRoots(rootsData, rootsDataSize);
	}
	void SessionServer_SetMaxRemoteProcessCount(uba::SessionServer* server, uba::u32 count)
	{
		return server->SetMaxRemoteProcessCount(count);
	}
	void SessionServer_DisableRemoteExecution(uba::SessionServer* server)
	{
		server->GetServer().DisallowNewClients();
		server->DisableRemoteExecution();
	}
	void SessionServer_PrintSummary(uba::SessionServer* server)
	{
		uba::LoggerWithWriter logger(server->GetLogWriter());
		server->PrintSummary(logger);
		server->GetStorage().PrintSummary(logger);
		server->GetServer().PrintSummary(logger);
		uba::KernelStats::GetGlobal().Print(logger, true);
		uba::PrintContentionSummary(logger);
	}
	void SessionServer_CancelAll(uba::SessionServer* server)
	{
		++server->GetServer().GetLogger().isMuted; // Mute forever
		++server->GetLogger().isMuted; // Mute forever
		server->CancelAllProcessesAndWait();
	}
	void SessionServer_SetCustomCasKeyFromTrackedInputs(uba::SessionServer* server, uba::ProcessHandle* handle, const uba::tchar* fileName, const uba::tchar* workingDir)
	{
		const auto& TrackedInputs = handle->GetTrackedInputs();
		server->SetCustomCasKeyFromTrackedInputs(fileName, workingDir, TrackedInputs.data(), (uba::u32)TrackedInputs.size());
	}
	uba::u32 SessionServer_BeginExternalProcess(uba::SessionServer* server, const uba::tchar* description)
	{
		return server->BeginExternalProcess(description);
	}
	void SessionServer_EndExternalProcess(uba::SessionServer* server, uba::u32 id, uba::u32 exitCode)
	{
		server->EndExternalProcess(id, exitCode);
	}

	void SessionServer_UpdateProgress(uba::SessionServer* server, uba::u32 processesTotal, uba::u32 processesDone, uba::u32 errorCount)
	{
		server->UpdateProgress(processesTotal, processesDone, errorCount);
	}

	void SessionServer_UpdateStatus(uba::SessionServer* server, uba::u32 statusRow, uba::u32 statusColumn, const uba::tchar* statusText, uba::LogEntryType statusType, const uba::tchar* statusLink)
	{
		using namespace uba;
		server->GetTrace().StatusUpdate(statusRow, statusColumn, ToView(statusText), statusType, statusLink ? ToView(statusLink) : StringView());
	}

	void SessionServer_AddProcessBreadcrumbs(uba::SessionServer* server, uba::u32 processId, const uba::tchar* breadcrumbs, bool deleteOld)
	{
		server->AddProcessBreadcrumbs(processId, breadcrumbs, deleteOld);
	}
	
	void SessionServer_RegisterCustomService(uba::SessionServer* server, SessionServer_CustomServiceFunction* function, void* userData)
	{
		server->RegisterCustomService([function, userData](uba::Process& process, const void* recv, uba::u32 recvSize, void* send, uba::u32 sendCapacity)
			{
				uba::ProcessHandle h(&process);
				return function(&h, recv, recvSize, send, sendCapacity, userData);
			});
	}

	void SessionServer_RegisterCrossArchitectureMapping(uba::SessionServer* server, const uba::tchar* from, const uba::tchar* to)
	{
		server->RegisterCrossArchitectureMapping(from, to);
	}

	void SessionServer_SaveSnapshotOfTrace(uba::SessionServer* server)
	{
		if (server)
			server->SaveSnapshotOfTrace();
	}

	void SessionServer_AddInfo(uba::SessionServer* server, const uba::tchar* info)
	{
		server->GetTrace().SessionInfo(0, uba::ToView(info));
	}

	uba::Trace* SessionServer_GetTrace(uba::SessionServer* server)
	{
		return &server->GetTrace();
	}

	uba::u32 SessionServer_TaskBegin(uba::SessionServer* server, const uba::tchar* description, const uba::tchar* details, uba::u32 color)
	{
		uba::u32 taskId = server->CreateProcessId();
		server->GetTrace().TaskBegin(taskId, uba::ToView(description), uba::ToView(details), color);
		return taskId;
	}

	void SessionServer_TaskHint(uba::SessionServer* server, uba::u32 taskId, const uba::tchar* hint)
	{
		server->GetTrace().TaskHint(taskId, uba::ToView(hint));
	}

	void SessionServer_TaskEnd(uba::SessionServer* server, uba::u32 taskId, bool success)
	{
		server->GetTrace().TaskEnd(taskId, success);
	}

	bool SessionServer_DisableRemoteExecutionOnAgent(uba::SessionServer* server, const uba::tchar* agentName)
	{
		return server->DisableRemoteExecutionOnAgent(agentName);
	}

	void SessionServer_Destroy(uba::SessionServer* server)
	{
		if (server)
		{
			auto& s = (uba::NetworkServerWithBackend&)server->GetServer();
			s.backend->StopListen();
			s.DisconnectClients();
		}
		delete server;
	}

	uba::RootPaths* RootPaths_Create(uba::LogWriter& writer)
	{
		auto rootPaths = new uba::RootPathsWithLogger(writer);
		#if PLATFORM_WINDOWS
		rootPaths->RegisterIgnoredRoot(rootPaths->logger, TC("z:\\ue")); // TODO: Not hard coded
		#endif
		return rootPaths;
	}

	bool RootPaths_RegisterRoot(uba::RootPaths* rootPaths, const uba::tchar* path, bool includeInKey, uba::u8 id)
	{
		using namespace uba;
		auto rp = (uba::RootPathsWithLogger*)rootPaths;
		return rp->RegisterRoot(rp->logger, path, includeInKey, id);
	}

	bool RootPaths_RegisterSystemRoots(uba::RootPaths* rootPaths, uba::u8 startId)
	{
		using namespace uba;
		auto rp = (uba::RootPathsWithLogger*)rootPaths;
		return rp->RegisterSystemRoots(rp->logger, startId);
	}

	void RootPaths_Destroy(uba::RootPaths* rootPaths)
	{
		delete (uba::RootPathsWithLogger*)rootPaths;
	}


	uba::ProcessStartInfo* ProcessStartInfo_Create(const uba::tchar* application, const uba::tchar* arguments, const uba::tchar* workingDir, const uba::tchar* description, uba::u32 priorityClass, uba::u64 outputStatsThresholdMs, bool trackInputs, const uba::tchar* logFile, ProcessHandle_ExitCallback* exit)
	{
		using namespace uba;
		auto info = new ProcessStartInfoHolder();

		info->applicationStr = application;
		info->argumentsStr = arguments;
		info->workingDirStr = workingDir;
		info->descriptionStr = description;
		info->logFileStr = logFile;

		info->application = info->applicationStr.c_str();
		info->arguments = info->argumentsStr.c_str();
		info->workingDir = info->workingDirStr.c_str();
		info->description = info->descriptionStr.c_str();
		info->logFile = info->logFileStr.c_str();
		info->priorityClass = priorityClass;
		info->trackInputs = trackInputs;
		UBA_PUSH_DISABLE_CAST_FN_WARNING
		info->exitedFunc = (ProcessStartInfo::ExitedCallback*)exit;
		UBA_POP_DISABLE_CAST_FN_WARNING
		return info;
	}

	uba::ProcessStartInfo* ProcessStartInfo_Create2(const uba::tchar* application, const uba::tchar* arguments, const uba::tchar* workingDir, const uba::tchar* description, uba::u32 priorityClass, uba::u64 rootsHandle, bool trackInputs, const uba::tchar* logFile, ProcessHandle_ExitCallback2* exit)
	{
		UBA_PUSH_DISABLE_CAST_FN_WARNING
		auto info = ProcessStartInfo_Create(application, arguments, workingDir, description, priorityClass, 0, trackInputs, logFile, (ProcessHandle_ExitCallback*)exit);
		UBA_POP_DISABLE_CAST_FN_WARNING
		info->rootsHandle = rootsHandle;
		return info;
	}

	uba::ProcessStartInfo* ProcessStartInfo_Create3(const uba::Config& config, const uba::tchar* configTable)
	{
		auto info = new uba::ProcessStartInfoHolder();
		info->Apply(config, configTable);
		return info;
	}

	void ProcessStartInfo_SetExitedCallback(uba::ProcessStartInfo& info, ProcessHandle_ExitCallback* exitedFunc, void* exitedUserData)
	{
		UBA_PUSH_DISABLE_CAST_FN_WARNING
		info.exitedFunc = (uba::ProcessStartInfo::ExitedCallback*)exitedFunc;
		UBA_POP_DISABLE_CAST_FN_WARNING
		info.userData = exitedUserData;
	}
	
	void ProcessStartInfo_Destroy(uba::ProcessStartInfo* info)
	{
		delete (uba::ProcessStartInfoHolder*)info;
	}

	uba::Scheduler* Scheduler_Create(uba::SessionServer* session, uba::u32 maxLocalProcessors, bool enableProcessReuse)
	{
		uba::SchedulerCreateInfo info{*session};
		info.Apply(uba::GetConfig());
		info.maxLocalProcessors = maxLocalProcessors;
		info.enableProcessReuse = enableProcessReuse;
		info.processConfigs = &uba::GetConfig();
		return new uba::Scheduler(info);
	}

	uba::Scheduler* Scheduler_Create2(uba::SessionServer& session, const uba::Config& config)
	{
		uba::SchedulerCreateInfo info{session};
		info.Apply(config);
		info.processConfigs = &uba::GetConfig(); // TODO: Clone provided config?
		return new uba::Scheduler(info);
	}

	uba::Scheduler* Scheduler_Create3(uba::SessionServer& session, uba::CacheClient** cacheClients, uba::u32 cacheClientCount, const uba::Config& config)
	{
		uba::SchedulerCreateInfo info{session};
		info.cacheClients = cacheClients;
		info.cacheClientCount = cacheClientCount;
		info.Apply(config);
		info.processConfigs = &uba::GetConfig(); // TODO: Clone provided config?
		return new uba::Scheduler(info);
	}

	void Scheduler_Start(uba::Scheduler* scheduler)
	{
		scheduler->Start();
	}

	uba::u32 Scheduler_EnqueueProcess(uba::Scheduler* scheduler, const uba::ProcessStartInfo& info, float weight, const void* knownInputs, uba::u32 knownInputsBytes, uba::u32 knownInputsCount)
	{
		return Scheduler_EnqueueProcess2(scheduler, info, weight, true, true, nullptr, 0, knownInputs, knownInputsBytes, knownInputsCount, 0);
	}

	uba::u32 Scheduler_EnqueueProcess2(uba::Scheduler* scheduler, const uba::ProcessStartInfo& info, float weight, bool canDetour, bool canExecuteRemotely, const uba::u32* dependencies, uba::u32 dependencyCount, const void* knownInputs, uba::u32 knownInputsBytes, uba::u32 knownInputsCount, uba::u32 cacheBucket)
	{
		return Scheduler_EnqueueProcess3(scheduler, info, weight, canDetour, canExecuteRemotely, dependencies, dependencyCount, knownInputs, knownInputsBytes, knownInputsCount, cacheBucket, 0, 0);
	}

	uba::u32 Scheduler_EnqueueProcess3(uba::Scheduler* scheduler, const uba::ProcessStartInfo& info, float weight, bool canDetour, bool canExecuteRemotely, const uba::u32* dependencies, uba::u32 dependencyCount, const void* knownInputs, uba::u32 knownInputsBytes, uba::u32 knownInputsCount, uba::u32 cacheBucket, uba::u32 memoryGroup, uba::u64 predictedMemoryUsage)
	{
		return Scheduler_EnqueueProcess4(scheduler, info, weight, canDetour, canExecuteRemotely, false, false, dependencies, dependencyCount, knownInputs, knownInputsBytes, knownInputsCount, cacheBucket, memoryGroup, predictedMemoryUsage);
	}

	uba::u32 Scheduler_EnqueueProcess4(uba::Scheduler* scheduler, const uba::ProcessStartInfo& info, float weight, bool canDetour, bool canExecuteRemotely, bool canCrossArchitecture, bool canCrossPlatform, const uba::u32* dependencies, uba::u32 dependencyCount, const void* knownInputs, uba::u32 knownInputsBytes, uba::u32 knownInputsCount, uba::u32 cacheBucket, uba::u32 memoryGroup, uba::u64 predictedMemoryUsage)
	{
		return Scheduler_EnqueueProcess5(scheduler, info, weight, canDetour, canExecuteRemotely, canCrossArchitecture, canCrossPlatform, dependencies, dependencyCount, knownInputs, knownInputsBytes, knownInputsCount, cacheBucket, memoryGroup, predictedMemoryUsage);
	}

	uba::u32 Scheduler_EnqueueProcess5(uba::Scheduler* scheduler, const uba::ProcessStartInfo& info, float weight, bool canDetour, bool canExecuteRemotely, bool canCrossArchitecture, bool canCrossPlatform, const uba::u32* dependencies, uba::u32 dependencyCount, const void* knownInputs, uba::u32 knownInputsBytes, uba::u32 knownInputsCount, uba::u32 cacheBucket, uba::u64 memoryGroup, uba::u64 predictedMemoryUsage)
	{
		uba::EnqueueProcessInfo epi(info);
		epi.weight = weight;
		epi.dependencies = dependencies;
		epi.dependencyCount = dependencyCount;
		epi.knownInputs = knownInputs;
		epi.knownInputsBytes = knownInputsBytes;
		epi.knownInputsCount = knownInputsCount;
		epi.canDetour = canDetour;
		epi.canExecuteRemotely = canExecuteRemotely;
		epi.canCrossArchitecture = canCrossArchitecture;
		epi.canCrossPlatform = canCrossPlatform;
		epi.cacheBucketId = cacheBucket;
		epi.memoryGroupId = memoryGroup;
		epi.predictedMemoryUsage = predictedMemoryUsage;
		return scheduler->EnqueueProcess(epi);
	}

	void Scheduler_SetMaxLocalProcessors(uba::Scheduler* scheduler, uba::u32 maxLocalProcessors)
	{
		scheduler->SetMaxLocalProcessors(maxLocalProcessors);
	}

	void Scheduler_Stop(uba::Scheduler* scheduler)
	{
		scheduler->Stop();
	}

	void Scheduler_Cancel(uba::Scheduler* scheduler)
	{
		scheduler->Cancel();
	}

	void Scheduler_Destroy(uba::Scheduler* scheduler)
	{
		delete scheduler;
	}

	void Scheduler_GetStats(uba::Scheduler* scheduler, uba::u32& outQueued, uba::u32& outActiveLocal, uba::u32& outActiveRemote, uba::u32& outFinished)
	{
		scheduler->GetStats(outQueued, outActiveLocal, outActiveRemote, outFinished);
	}

	bool Scheduler_IsEmpty(uba::Scheduler* scheduler)
	{
		return scheduler->IsEmpty();
	}

	void Scheduler_SetProcessFinishedCallback(uba::Scheduler* scheduler)
	{
		scheduler->SetProcessFinishedCallback([](const uba::ProcessHandle&)
			{
			});
	}

	void Scheduler_SetPredictedMemoryCallback(uba::Scheduler* scheduler, ProcessHandle_PredictedMemoryCallback* predictedMemory)
	{
		scheduler->SetPredictedMemoryCallback(predictedMemory);
	}

	float Scheduler_GetProcessWeightThatCanRunRemotelyNow(uba::Scheduler* scheduler)
	{
		float total;
		float crossArchitecture;
		float crossPlatform;
		scheduler->GetProcessWeightThatCanRunRemotelyNow(total, crossArchitecture, crossPlatform);
		return total;
	}

	void Scheduler_GetProcessWeightThatCanRunRemotelyNow2(uba::Scheduler* scheduler, float& outTotal, float& outCrossArchitecture, float& outCrossPlatform)
	{
		scheduler->GetProcessWeightThatCanRunRemotelyNow(outTotal, outCrossArchitecture, outCrossPlatform);
	}

	void Scheduler_SetAllowDisableRemoteExecution(uba::Scheduler* scheduler, bool allow)
	{
		scheduler->SetAllowDisableRemoteExecution(true);
	}

	uba::CacheClient* CacheClient_Create(uba::SessionServer* session, bool reportMissReason, const uba::tchar* crypto, const uba::tchar* hint)
	{
		using namespace uba;
		LogWriter& writer = session->GetLogWriter();
		StorageImpl& storage = (StorageImpl&)session->GetStorage();
		auto& server = (NetworkServerWithBackend&)session->GetServer();

		u8 crypto128Data[16];
		u8* crypto128 = nullptr;
		if (crypto && *crypto)
		{
			if (!CryptoFromString(crypto128Data, 16, crypto))
			{
				LoggerWithWriter(writer, TC("UbaCacheClient")).Error(TC("Failed to parse crypto key %s"), crypto);
				return nullptr;
			}
			crypto128 = crypto128Data;
		}

		NetworkClientCreateInfo ncci(writer);
		ncci.receiveTimeoutSeconds = DefaultNetworkReceiveTimeoutSeconds;
		ncci.cryptoKey128 = crypto128;
		ncci.workerCount = 0;
		ncci.useMessagePriority = true;
		ncci.Apply(GetConfig(), TC("CacheNetworkClient"));
		bool ctorSuccess = false;
		auto networkClient = new NetworkClientWithBackend(ctorSuccess, ncci, server.backend, TC("UbaCache"));
		if (!ctorSuccess)
		{
			delete networkClient;
			return nullptr;
		}

		CacheClientCreateInfo info{writer, storage, *networkClient, *session};
		info.Apply(GetConfig());
		if (hint && *hint)
			info.hint = hint;

		info.reportMissReason |= reportMissReason;
		return new CacheClientWithCounter(info);
	}

	bool CacheClient_Connect(uba::CacheClient* cacheClient, const uba::tchar* host, int port)
	{
		using namespace uba;
		auto& networkClient = (NetworkClientWithBackend&)cacheClient->GetClient();
		CacheClientActiveScope ccas(cacheClient);

		bool wasConnected = networkClient.IsConnected();
		if (!networkClient.Connect(*networkClient.backend, host, u16(port)))
			return false;
		if (wasConnected)
			return true;

		((SessionServer&)cacheClient->GetSession()).RegisterNetworkTrafficProvider(u64(cacheClient), [nc = &networkClient](u64& outSent, u64& outReceive, u32& outConnectionCount)
			{
				outSent = nc->GetTotalSentBytes();
				outReceive = nc->GetTotalRecvBytes();
				outConnectionCount = nc->GetConnectionCount();
			});

		auto& storage = cacheClient->GetStorage();
		if (!storage.IsCasTableLoaded())
			storage.LoadCasTable();
		return true;
	}
	
	bool CacheClient_RegisterPathHash(uba::CacheClient* cacheClient, const uba::tchar* path, const uba::tchar* hashString)
	{
		using namespace uba;
		CasKeyHasher hasher;
		hasher.Update(hashString, TStrlen(hashString));
		cacheClient->RegisterPathHash(path, ToCasKey(hasher, true));
		return true;
	}

	void PopulateLogLines(uba::BinaryWriter& out, const uba::ProcessHandle& process)
	{
		using namespace uba;
		auto& logLines = process.GetLogLines();
		for (auto& line : logLines)
		{
			if (out.GetCapacityLeft() < 1 + GetStringWriteSize(line.text.c_str(), line.text.size()))
				break;
			out.WriteString(line.text);
			out.WriteByte(line.type);
		}
	}

	bool CacheClient_WriteToCache(uba::CacheClient* cacheClient, uba::RootPaths* rootPaths, uba::u32 bucket, const uba::ProcessHandle* process, const uba::u8* inputs, uba::u32 inputsSize, const uba::u8* outputs, uba::u32 outputsSize)
	{
		using namespace uba;
		StackBinaryWriter<16*1024> logLinesWriter;
		PopulateLogLines(logLinesWriter, *process);
		CacheClientActiveScope ccas(cacheClient);
		return cacheClient->WriteToCache(*rootPaths, bucket, process->GetStartInfo(), inputs, inputsSize, outputs, outputsSize, logLinesWriter.GetData(), logLinesWriter.GetPosition(), process->GetId());
	}

	bool CacheClient_WriteToCache2(uba::CacheClient* cacheClient, uba::u32 bucket, const uba::ProcessHandle* process, const uba::u8* inputs, uba::u32 inputsSize, const uba::u8* outputs, uba::u32 outputsSize)
	{
		using namespace uba;
		StackBinaryWriter<16*1024> logLinesWriter;
		PopulateLogLines(logLinesWriter, *process);
		CacheClientActiveScope ccas(cacheClient);
		return cacheClient->WriteToCache(bucket, process->GetStartInfo(), inputs, inputsSize, outputs, outputsSize, logLinesWriter.GetData(), logLinesWriter.GetPosition(), process->GetId());
	}

	uba::u32 CacheClient_FetchFromCache(uba::CacheClient* cacheClient, uba::RootPaths* rootPaths, uba::u32 bucket, const uba::ProcessStartInfo& info)
	{
		using namespace uba;
		CacheClientActiveScope ccas(cacheClient);
		CacheResult cacheResult;
		bool res = cacheClient->FetchFromCache(cacheResult, *rootPaths, bucket, info);
		return (res && cacheResult.hit) ? 1 : 0;
	}

	uba::CacheResult* CacheClient_FetchFromCache2(uba::CacheClient* cacheClient, uba::RootPaths* rootPaths, uba::u32 bucket, const uba::ProcessStartInfo& info)
	{
		using namespace uba;
		auto cacheResult = new CacheResult();
		CacheClientActiveScope ccas(cacheClient);
		if (cacheClient->FetchFromCache(*cacheResult, *rootPaths, bucket, info))
			return cacheResult;
		delete cacheResult;
		return nullptr;
	}

	uba::CacheResult* CacheClient_FetchFromCache3(uba::CacheClient* cacheClient, uba::u64 rootsHandle, uba::u32 bucket, const uba::ProcessStartInfo& info)
	{
		using namespace uba;
		auto cacheResult = new CacheResult();
		CacheClientActiveScope ccas(cacheClient);
		if (cacheClient->FetchFromCache(*cacheResult, rootsHandle, bucket, info))
			return cacheResult;
		delete cacheResult;
		return nullptr;
	}

	void CacheClient_RequestServerShutdown(uba::CacheClient* cacheClient, const uba::tchar* reason)
	{
		cacheClient->RequestServerShutdown(reason);
	}

	void CacheClient_Disconnect(uba::CacheClient* cacheClient)
	{
		using namespace uba;
		auto& networkClient = (NetworkClientWithBackend&)cacheClient->GetClient();
		cacheClient->Disable();
		networkClient.Disconnect();
	}

	void CacheClient_Destroy(uba::CacheClient* cacheClient)
	{
		using namespace uba;

		auto& session = (SessionServer&)cacheClient->GetSession();
		session.UnregisterNetworkTrafficProvider(u64(cacheClient));


		auto& networkClient = (NetworkClientWithBackend&)cacheClient->GetClient();
		networkClient.Disconnect();

		while (((CacheClientWithCounter*)cacheClient)->active)
			uba::Sleep(10);

		delete cacheClient;
		delete &networkClient;
	}

	const uba::tchar* CacheResult_GetLogLine(uba::CacheResult* result, uba::u32 index)
	{
		auto& lines = result->logLines;
		if (index >= lines.size())
			return nullptr;
		return lines[index].text.c_str();
	}

	uba::u32 CacheResult_GetLogLineType(uba::CacheResult* result, uba::u32 index)
	{
		auto& lines = result->logLines;
		if (index >= lines.size())
			return 0;
		return uba::u32(lines[index].type);
	}

	void CacheResult_Delete(uba::CacheResult* result)
	{
		delete result;
	}

	uba::Trace* Trace_Create(const uba::tchar* name, const uba::tchar* channel)
	{
		using namespace uba;
		auto trace = new Trace(g_globalLogWriter ? *g_globalLogWriter : g_consoleLogWriter);

		StringBuffer<256> fullName;

		if (name && *name)
			fullName.Append(name).Append('_');

		time_t rawtime;
		time(&rawtime);
		tm ti;
		localtime_s(&ti, &rawtime);
		fullName.Appendf(TC("%02i%02i%02i"), ti.tm_hour, ti.tm_min, ti.tm_sec);

		OwnerInfo ownerInfo = GetOwnerInfo();
		if (ownerInfo.pid)
			fullName.Appendf(TC("_%s%u"), ownerInfo.id, ownerInfo.pid);

		trace->StartWriteAndThread(fullName.data, 256*1024*1024, false, channel);
		return trace;
	}

	uba::u32 Trace_TaskBegin(uba::Trace* trace, const uba::tchar* description, const uba::tchar* details, uba::u32 color)
	{
		using namespace uba;
		static Atomic<u32> taskIdCounter = 2'000'000; // Not to collide with process ids
		u32 taskId = taskIdCounter++;
		trace->TaskBegin(taskId, ToView(description), ToView(details), color);
		return taskId;
	}

	void Trace_TaskHint(uba::Trace* trace, uba::u32 taskId, const uba::tchar* hint)
	{
		trace->TaskHint(taskId, uba::ToView(hint));
	}

	void Trace_TaskEnd(uba::Trace* trace, uba::u32 taskId, bool success)
	{
		trace->TaskEnd(taskId, success);
	}

	void Trace_UpdateStatus(uba::Trace* trace, uba::u32 statusRow, uba::u32 statusColumn, const uba::tchar* statusText, uba::LogEntryType statusType, const uba::tchar* statusLink)
	{
		using namespace uba;
		trace->StatusUpdate(statusRow, statusColumn, ToView(statusText), statusType, statusLink ? ToView(statusLink) : StringView());
	}

	void Trace_Destroy(uba::Trace* trace, const uba::tchar* writeFile)
	{
		if (uba::g_globalTrace == trace)
			uba::g_globalTrace = nullptr;

		trace->StopThread();
		trace->StopWrite(writeFile);
		delete trace;
	}

	void Trace_SetGlobal(uba::Trace* trace)
	{
		uba::g_globalTrace = trace;
	}

	void Uba_SetCustomAssertHandler(Uba_CustomAssertHandler* handler)
	{
		using namespace uba;
		static auto f = [](const tchar* text, u32 textLen, void* userData) { ((Uba_CustomAssertHandler*)userData)(text); };
		SetCustomAssertHandler(f, (void*)handler);
	}

	void Uba_FindImports(const uba::tchar* binary, ImportFunc* func, void* userData)
	{
#if PLATFORM_WINDOWS
		uba::StringBuffer<> errors;
		uba::BinaryInfo info;
		uba::ParseBinary(uba::ToView(binary), {}, info, [&](const uba::tchar* importName, bool isKnown, const uba::tchar* const* importLoaderPaths) { func(importName, userData); }, errors);
#else
		UBA_ASSERT(false);
#endif
	}

	bool Uba_GetExclusiveAccess(const uba::tchar* path)
	{
		using namespace uba;
		ExportsDowngradedLogger downgradedLogger(g_consoleLogWriter, TC("UbaGetExclusiveAccess"));
		g_exclusiveMutex = StorageImpl::GetExclusiveAccess(downgradedLogger, ToView(path), false);
		return g_exclusiveMutex != InvalidMutexHandle;
	}

	struct UbaInstance
	{
		uba::Scheduler* scheduler;
		uba::TString workDir;
		uba::CoordinatorWrapper coordinator;
	};

	void* Uba_Create(const uba::tchar* configFile)
	{
		using namespace uba;
		auto& config = GetConfig(configFile);
		auto networkServer = (uba::NetworkServerWithBackend*)NetworkServer_Create();
		auto storageServer = StorageServer_Create(*networkServer, nullptr, 0, true);

		SessionServerCreateInfo ssci((Storage&)*storageServer, *networkServer);
		ssci.Apply(config);
		auto sessionServer = SessionServer_Create(ssci);

		uba::SchedulerCreateInfo sci{*sessionServer};
		sci.Apply(config);
		sci.processConfigs = &config;
		auto scheduler = new uba::Scheduler(sci);
		scheduler->Start();

		bool networkListen = true;
		if (auto* ubaTable = config.GetTable(TC("Uba")))
			ubaTable->GetValueAsBool(networkListen, TC("NetworkListen"));

		if (networkListen)
			NetworkServer_StartListen(networkServer);

		auto ubaInstance = new UbaInstance();
		ubaInstance->scheduler = scheduler;
		
		StringBuffer<> temp;
		GetCurrentDirectoryW(temp);
		ubaInstance->workDir = temp.data;

		auto createCoordinator = [&]()
			{
				if (auto coordinatorTable = config.GetTable(TC("Coordinator")))
				{
					const tchar* coordinatorName;
					if (coordinatorTable->GetValueAsString(coordinatorName, TC("Name")))
					{
						auto& logger = sessionServer->GetLogger();
						const tchar* rootDir = nullptr;
						coordinatorTable->GetValueAsString(rootDir, TC("RootDir"));
						if (!rootDir)
							rootDir = sessionServer->GetRootDir();
						StringBuffer<512> coordinatorWorkDir(rootDir);
						coordinatorWorkDir.EnsureEndsWithSlash().Append(coordinatorName);
						StringBuffer<512> binariesDir;
						if (!GetDirectoryOfCurrentModule(logger, binariesDir))
							return;

						CoordinatorCreateInfo cinfo;
						cinfo.workDir = coordinatorWorkDir.data;
						cinfo.binariesDir = binariesDir.data;

						u32 maxCoreCount = 0;
						coordinatorTable->GetValueAsU32(maxCoreCount, TC("MaxCoreCount"));

						cinfo.logging = false;
						coordinatorTable->GetValueAsBool(cinfo.logging, TC("Log"));

						const tchar* uri = nullptr;
						if (coordinatorTable->GetValueAsString(uri, TC("Uri")))
							uba::SetEnvironmentVariableW(TC("UE_HORDE_URL"), uri);

						if (!ubaInstance->coordinator.Create(logger, coordinatorName, cinfo, *networkServer->backend, *networkServer, sessionServer->GetTrace(), 0))
							return;
						ubaInstance->coordinator.SetScheduler(scheduler);
						ubaInstance->coordinator.RequestHelpers(maxCoreCount);
					}
				}
			};
		createCoordinator();

		return ubaInstance;
	}

	uba::u32 Uba_RunProcess(void* uba, const uba::tchar* app, const uba::tchar* args, const uba::tchar* workDir, const uba::tchar* desc, void* userData, ProcessHandle_ExitCallback* exit)
	{
		using namespace uba;

		auto& ubaInstance = *(UbaInstance*)uba;

		if (!workDir)
			workDir = ubaInstance.workDir.data();

		auto scheduler = ubaInstance.scheduler;
		ProcessStartInfo info;
		info.application = app;
		info.arguments = args;
		info.workingDir = workDir;
		info.description = desc;
		info.userData = userData;
		UBA_PUSH_DISABLE_CAST_FN_WARNING
		info.exitedFunc = (uba::ProcessStartInfo::ExitedCallback*)exit;
		UBA_POP_DISABLE_CAST_FN_WARNING
		return Scheduler_EnqueueProcess(scheduler, info, 1.0f, nullptr, 0, 0);
	}

	void Uba_RegisterNewFile(void* uba, const uba::tchar* file)
	{
		using namespace uba;
		auto& ubaInstance = *(UbaInstance*)uba;
		ubaInstance.scheduler->GetSession().RegisterNewFile(file);
	}

	void Uba_Destroy(void* uba)
	{
		using namespace uba;
		auto ubaInstance = (UbaInstance*)uba;
		auto scheduler = ubaInstance->scheduler;
		auto sessionServer = &scheduler->GetSession();
		auto storageServer = (StorageServer*)&sessionServer->GetStorage();
		auto networkServer = &sessionServer->GetServer();

		NetworkServer_Stop(networkServer);
		SessionServer_CancelAll(sessionServer);

		delete ubaInstance;

		Scheduler_Destroy(scheduler);
		SessionServer_Destroy(sessionServer);
		StorageServer_Destroy(storageServer);
		NetworkServer_Destroy(networkServer);
	}
}
