// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaLogWriter.h"
#include "UbaDefaultConstants.h"

namespace uba
{
	class CacheClient;
	class Config;
	class ConfigTable;
	class NetworkServer;
	class Process;
	class ProcessHandle;
	class RootPaths;
	class SessionServer;
	class Scheduler;
	class StorageServer;
	class Trace;
	enum ProcessExitedResponse : uba::u8;
	struct CacheResult;
	struct ProcessStartInfo;
	struct SessionServerCreateInfo;

	class CallbackLogWriter : public LogWriter
	{
	public:
		using BeginScopeCallback = void();
		using EndScopeCallback = void();
		using LogCallback = void(LogEntryType type, const uba::tchar* str, u32 strLen);

		UBA_API CallbackLogWriter(BeginScopeCallback* begin, EndScopeCallback* end, LogCallback* log);
		UBA_API virtual void BeginScope() override;
		UBA_API virtual void EndScope() override;
		UBA_API virtual void Log(LogEntryType type, const uba::tchar* str, u32 strLen, const uba::tchar* prefix = nullptr, u32 prefixLen = 0) override;

	private:
		BeginScopeCallback* m_beginScope;
		EndScopeCallback* m_endScope;
		LogCallback* m_logCallback;
	};
}

extern "C"
{
	// LogWriter
	UBA_API uba::LogWriter* LogWriter_GetDefault();
	UBA_API uba::LogWriter* LogWriter_GetNull();
	UBA_API uba::LogWriter* LogWriter_CreateCallback(uba::CallbackLogWriter::BeginScopeCallback begin, uba::CallbackLogWriter::EndScopeCallback end, uba::CallbackLogWriter::LogCallback log);
	UBA_API void LogWriter_SetGlobal(uba::LogWriter* writer);
	UBA_API void LogWriter_DestroyCallback(uba::LogWriter* writer);

	// Config
	UBA_API uba::Config* Config_Load(const uba::tchar* configFile);
	UBA_API uba::Config* Config_Create();
	UBA_API void Config_Destroy(uba::Config* config);
	UBA_API uba::ConfigTable* Config_RootTable(uba::Config& config);
	UBA_API uba::ConfigTable* Config_AddTable(uba::Config& config, const uba::tchar* name);
	UBA_API void ConfigTable_AddValueInt(uba::ConfigTable& table, const uba::tchar* key, int value);
	UBA_API void ConfigTable_AddValueU32(uba::ConfigTable& table, const uba::tchar* key, uba::u32 value);
	UBA_API void ConfigTable_AddValueU64(uba::ConfigTable& table, const uba::tchar* key, uba::u64 value);
	UBA_API void ConfigTable_AddValueBool(uba::ConfigTable& table, const uba::tchar* key, bool value);
	UBA_API void ConfigTable_AddValueString(uba::ConfigTable& table, const uba::tchar* key, const uba::tchar* str);

	// NetworkServer
	UBA_API uba::NetworkServer* NetworkServer_Create(uba::LogWriter& writer = uba::g_consoleLogWriter, uba::u32 workerCount = 64, uba::u32 sendSize = uba::SendDefaultSize, uba::u32 receiveTimeoutSeconds = uba::DefaultNetworkReceiveTimeoutSeconds, bool useQuic = false);
	UBA_API void NetworkServer_Destroy(uba::NetworkServer* server);
	UBA_API bool NetworkServer_StartListen(uba::NetworkServer* server, int port = uba::DefaultPort, const uba::tchar* ip = nullptr, const uba::tchar* crypto = nullptr);
	UBA_API void NetworkServer_Stop(uba::NetworkServer* server);
	UBA_API void NetworkServer_SetClientsConfig(uba::NetworkServer* server, const uba::Config& config);
	UBA_API bool NetworkServer_AddClient(uba::NetworkServer* server, const uba::tchar* ip, int port = uba::DefaultPort, const uba::tchar* crypto = nullptr);

	// StorageServer
	UBA_API uba::StorageServer* StorageServer_Create(uba::NetworkServer& server, const uba::tchar* rootDir, uba::u64 casCapacityBytes, bool storeCompressed, uba::LogWriter& writer = uba::g_consoleLogWriter, const uba::tchar* zone = TC(""));
	UBA_API uba::StorageServer* StorageServer_Create2(uba::NetworkServer& server, const uba::Config& config, uba::LogWriter& writer = uba::g_consoleLogWriter);
	UBA_API void StorageServer_PreloadCasTable(uba::StorageServer* storageServer);
	UBA_API void StorageServer_Destroy(uba::StorageServer* storageServer);
	UBA_API void StorageServer_SaveCasTable(uba::StorageServer* storageServer);
	UBA_API void StorageServer_RegisterDisallowedPath(uba::StorageServer* storageServer, const uba::tchar* path);
	UBA_API void StorageServer_DeleteFile(uba::StorageServer* storage, const uba::tchar* file);

	// ProcessStartInfo
	using ProcessHandle_ExitCallback = void(void* userData, const uba::ProcessHandle&);
	using ProcessHandle_ExitCallback2 = void(void* userData, const uba::ProcessHandle&, uba::ProcessExitedResponse&);
	using ProcessHandle_PredictedMemoryCallback = uba::u64(void* userData);
	UBA_API uba::ProcessStartInfo* ProcessStartInfo_Create(const uba::tchar* application, const uba::tchar* arguments, const uba::tchar* workingDir, const uba::tchar* description, uba::u32 priorityClass, uba::u64 outputStatsThresholdMs, bool trackInputs, const uba::tchar* logFile, ProcessHandle_ExitCallback* exit);
	UBA_API uba::ProcessStartInfo* ProcessStartInfo_Create2(const uba::tchar* application, const uba::tchar* arguments, const uba::tchar* workingDir, const uba::tchar* description, uba::u32 priorityClass, uba::u64 rootsHandle, bool trackInputs, const uba::tchar* logFile, ProcessHandle_ExitCallback2* exit);
	UBA_API uba::ProcessStartInfo* ProcessStartInfo_Create3(const uba::Config& config, const uba::tchar* configTable = TC(""));
	UBA_API void ProcessStartInfo_SetExitedCallback(uba::ProcessStartInfo& info, ProcessHandle_ExitCallback* exitedFunc, void* exitedUserData);
	UBA_API void ProcessStartInfo_Destroy(uba::ProcessStartInfo* info);

	// ProcessHandle
	using ProcessHandle_OutputFileCallback = void(void* userData, const uba::tchar* file);
	UBA_API uba::u32 ProcessHandle_GetExitCode(const uba::ProcessHandle* handle);
	UBA_API bool ProcessHandle_IsCancelled(const uba::ProcessHandle* handle);
	UBA_API uba::u8 ProcessHandle_GetExecutionType(const uba::ProcessHandle* handle);
	UBA_API const uba::tchar* ProcessHandle_GetExecutingHost(const uba::ProcessHandle* handle);
	UBA_API const uba::tchar* ProcessHandle_GetLogLine(const uba::ProcessHandle* handle, uba::u32 index);
	UBA_API uba::u64 ProcessHandle_GetHash(const uba::ProcessHandle* handle);
	UBA_API uba::u64 ProcessHandle_GetTotalProcessorTime(const uba::ProcessHandle* handle);
	UBA_API uba::u64 ProcessHandle_GetTotalWallTime(const uba::ProcessHandle* handle);
	UBA_API uba::u64 ProcessHandle_GetPeakMemory(const uba::ProcessHandle* handle);
	UBA_API void ProcessHandle_TraverseOutputFiles(const uba::ProcessHandle* handle, ProcessHandle_OutputFileCallback* callback, void* userData);
	UBA_API bool ProcessHandle_WaitForExit(const uba::ProcessHandle* handle, uba::u32 millisecondsTimeout);
	UBA_API void ProcessHandle_Cancel(const uba::ProcessHandle* handle, bool terminate);
	UBA_API void ProcessHandle_Destroy(const uba::ProcessHandle* handle);
	UBA_API const uba::ProcessStartInfo* Process_GetStartInfo(const uba::Process& process);

	// SessionServer
	using SessionServer_RemoteProcessAvailableCallback = void(void* userData, bool isCrossArchitecture);
	using SessionServer_RemoteProcessReturnedCallback = void(uba::Process& process, void* userData);
	using SessionServer_CustomServiceFunction = uba::u32(uba::ProcessHandle* handle, const void* recv, uba::u32 recvSize, void* send, uba::u32 sendCapacity, void* userData);

	UBA_API uba::SessionServerCreateInfo* SessionServerCreateInfo_Create(uba::StorageServer& storage, uba::NetworkServer& client, uba::LogWriter& writer, const uba::tchar* rootDir, const uba::tchar* traceOutputFile,
		bool disableCustomAllocator, bool launchVisualizer, bool resetCas, bool writeToDisk, bool detailedTrace, bool allowWaitOnMem = false, bool allowKillOnMem = false, bool storeIntermediateFilesCompressed = false);
	UBA_API void SessionServerCreateInfo_Destroy(uba::SessionServerCreateInfo* info);

	UBA_API uba::SessionServer* SessionServer_Create(const uba::SessionServerCreateInfo& info, const uba::u8* environment = nullptr, uba::u32 environmentSize = 0);
	UBA_API uba::SessionServer* SessionServer_Create2(uba::StorageServer& s, uba::NetworkServer& ns, const uba::Config& c, uba::LogWriter& lw = uba::g_consoleLogWriter, const uba::u8* environment = nullptr, uba::u32 environmentSize = 0);
	UBA_API void SessionServer_SetRemoteProcessAvailable(uba::SessionServer* server, SessionServer_RemoteProcessAvailableCallback* available, void* userData);
	UBA_API void SessionServer_SetRemoteProcessReturned(uba::SessionServer* server, SessionServer_RemoteProcessReturnedCallback* returned, void* userData);

	UBA_API bool SessionServer_RefreshDirectory(uba::SessionServer* server, const uba::tchar* directory);
	UBA_API bool SessionServer_RegisterNewFile(uba::SessionServer* server, const uba::tchar* filePath);
	UBA_API void SessionServer_RegisterDeleteFile(uba::SessionServer* server, const uba::tchar* filePath);
	UBA_API bool SessionServer_RegisterNewDirectory(uba::SessionServer* server, const uba::tchar* directoryPath);
	UBA_API bool SessionServer_RegisterVirtualFile(uba::SessionServer* server, const uba::tchar* filename, const uba::tchar* sourceFile, uba::u64 sourceOffset, uba::u64 sourceSize);
	UBA_API bool SessionServer_CreateVirtualFile(uba::SessionServer* server, const uba::tchar* filename, const void* memory, uba::u64 memorySize, bool transient, bool registerToDir);
	UBA_API bool SessionServer_DeleteVirtualFile(uba::SessionServer* server, const uba::tchar* filename, bool registerToDir);

	UBA_API bool SessionServer_GetOutputFileSize(uba::SessionServer* server, uba::u64& outSize, const uba::tchar* filename);
	UBA_API bool SessionServer_GetOutputFileData(uba::SessionServer* server, void* outData, const uba::tchar* filename, bool deleteInternalMapping);
	UBA_API bool SessionServer_WriteOutputFile(uba::SessionServer* server, const uba::tchar* filename, bool deleteInternalMapping);

	UBA_API uba::ProcessHandle* SessionServer_RunProcess(uba::SessionServer* server, uba::ProcessStartInfo& info, bool async, bool enableDetour);
	UBA_API uba::ProcessHandle* SessionServer_RunProcessRemote(uba::SessionServer* server, uba::ProcessStartInfo& info, float weight, const void* knownInputs = nullptr, uba::u32 knownInputsCount = 0, bool allowCrossArchitecture = false);
	UBA_API uba::u64 SessionServer_RegisterRoots(uba::SessionServer* server, const void* rootsData, uba::u64 rootsDataSize);

	UBA_API void SessionServer_SetMaxRemoteProcessCount(uba::SessionServer* server, uba::u32 count);
	UBA_API void SessionServer_DisableRemoteExecution(uba::SessionServer* server);
	UBA_API void SessionServer_PrintSummary(uba::SessionServer* server);
	UBA_API void SessionServer_CancelAll(uba::SessionServer* server);
	UBA_API void SessionServer_SetCustomCasKeyFromTrackedInputs(uba::SessionServer* server, uba::ProcessHandle* handle, const uba::tchar* fileName, const uba::tchar* workingDir);
	UBA_API uba::u32 SessionServer_BeginExternalProcess(uba::SessionServer* server, const uba::tchar* description);
	UBA_API void SessionServer_EndExternalProcess(uba::SessionServer* server, uba::u32 id, uba::u32 exitCode);
	UBA_API void SessionServer_UpdateProgress(uba::SessionServer* server, uba::u32 processesTotal, uba::u32 processesDone, uba::u32 errorCount);
	UBA_API void SessionServer_UpdateStatus(uba::SessionServer* server, uba::u32 statusRow, uba::u32 statusColumn, const uba::tchar* statusText, uba::LogEntryType statusType, const uba::tchar* statusLink);
	UBA_API void SessionServer_AddProcessBreadcrumbs(uba::SessionServer* server, uba::u32 processId, const uba::tchar* breadcrumbs, bool deleteOld = false);
	UBA_API void SessionServer_RegisterCustomService(uba::SessionServer* server, SessionServer_CustomServiceFunction* function, void* userData = nullptr);
	UBA_API void SessionServer_RegisterCrossArchitectureMapping(uba::SessionServer* server, const uba::tchar* from, const uba::tchar* to);
	UBA_API void SessionServer_SaveSnapshotOfTrace(uba::SessionServer* server);
	UBA_API void SessionServer_AddInfo(uba::SessionServer* server, const uba::tchar* info);
	UBA_API uba::Trace* SessionServer_GetTrace(uba::SessionServer* server);
	UBA_API uba::u32 SessionServer_TaskBegin(uba::SessionServer* server, const uba::tchar* description, const uba::tchar* details, uba::u32 color);
	UBA_API void SessionServer_TaskHint(uba::SessionServer* server, uba::u32 taskId, const uba::tchar* hint);
	UBA_API void SessionServer_TaskEnd(uba::SessionServer* server, uba::u32 taskId, bool success);
	UBA_API bool SessionServer_DisableRemoteExecutionOnAgent(uba::SessionServer* server, const uba::tchar* agentName);
	UBA_API void SessionServer_Destroy(uba::SessionServer* server);

	// Scheduler
	UBA_API uba::Scheduler* Scheduler_Create(uba::SessionServer* session, uba::u32 maxLocalProcessors = ~0u, bool enableProcessReuse = false);
	UBA_API uba::Scheduler* Scheduler_Create2(uba::SessionServer& session, const uba::Config& config);
	UBA_API uba::Scheduler* Scheduler_Create3(uba::SessionServer& session, uba::CacheClient** cacheClients, uba::u32 cacheClientCount, const uba::Config& config);
	UBA_API void Scheduler_Start(uba::Scheduler* scheduler);
	UBA_API uba::u32 Scheduler_EnqueueProcess(uba::Scheduler* scheduler, const uba::ProcessStartInfo& info, float weight = 1.0f, const void* knownInputs = nullptr, uba::u32 knownInputsBytes = 0, uba::u32 knownInputsCount = 0);
	UBA_API uba::u32 Scheduler_EnqueueProcess2(uba::Scheduler* scheduler, const uba::ProcessStartInfo& info, float weight, bool canDetour, bool canExecuteRemotely, const uba::u32* dependencies, uba::u32 dependencyCount, const void* knownInputs, uba::u32 knownInputsBytes, uba::u32 knownInputsCount, uba::u32 cacheBucket);
	UBA_API uba::u32 Scheduler_EnqueueProcess3(uba::Scheduler* scheduler, const uba::ProcessStartInfo& info, float weight, bool canDetour, bool canExecuteRemotely, const uba::u32* dependencies, uba::u32 dependencyCount, const void* knownInputs, uba::u32 knownInputsBytes, uba::u32 knownInputsCount, uba::u32 cacheBucket, uba::u32 memoryGroup, uba::u64 predictedMemoryUsage);
	UBA_API uba::u32 Scheduler_EnqueueProcess4(uba::Scheduler* scheduler, const uba::ProcessStartInfo& info, float weight, bool canDetour, bool canExecuteRemotely, bool canCrossArchitecture, bool canCrossPlatform, const uba::u32* dependencies, uba::u32 dependencyCount, const void* knownInputs, uba::u32 knownInputsBytes, uba::u32 knownInputsCount, uba::u32 cacheBucket, uba::u32 memoryGroup, uba::u64 predictedMemoryUsage);
	UBA_API uba::u32 Scheduler_EnqueueProcess5(uba::Scheduler* scheduler, const uba::ProcessStartInfo& info, float weight, bool canDetour, bool canExecuteRemotely, bool canCrossArchitecture, bool canCrossPlatform, const uba::u32* dependencies, uba::u32 dependencyCount, const void* knownInputs, uba::u32 knownInputsBytes, uba::u32 knownInputsCount, uba::u32 cacheBucket, uba::u64 memoryGroup, uba::u64 predictedMemoryUsage);
	UBA_API void Scheduler_SetMaxLocalProcessors(uba::Scheduler* scheduler, uba::u32 maxLocalProcessors);
	UBA_API void Scheduler_Stop(uba::Scheduler* scheduler);
	UBA_API void Scheduler_Cancel(uba::Scheduler* scheduler);
	UBA_API void Scheduler_Destroy(uba::Scheduler* scheduler);
	UBA_API void Scheduler_GetStats(uba::Scheduler* scheduler, uba::u32& outQueued, uba::u32& outActiveLocal, uba::u32& outActiveRemote, uba::u32& outFinished);
	UBA_API bool Scheduler_IsEmpty(uba::Scheduler* scheduler);
	UBA_API void Scheduler_SetProcessFinishedCallback(uba::Scheduler* scheduler);
	UBA_API void Scheduler_SetPredictedMemoryCallback(uba::Scheduler* scheduler, ProcessHandle_PredictedMemoryCallback* predictedMemory);
	UBA_API float Scheduler_GetProcessWeightThatCanRunRemotelyNow(uba::Scheduler* scheduler);
	UBA_API void Scheduler_GetProcessWeightThatCanRunRemotelyNow2(uba::Scheduler* scheduler, float& outTotal, float& outCrossArchitecture, float& outCrossPlatform);
	UBA_API void Scheduler_SetAllowDisableRemoteExecution(uba::Scheduler* scheduler, bool allow);

	// Cache
	UBA_API uba::CacheClient* CacheClient_Create(uba::SessionServer* session, bool reportMissReason = false, const uba::tchar* crypto = nullptr, const uba::tchar* hint = nullptr);
	UBA_API bool CacheClient_Connect(uba::CacheClient* cacheClient, const uba::tchar* host, int port);
	UBA_API bool CacheClient_RegisterPathHash(uba::CacheClient* cacheClient, const uba::tchar* path, const uba::tchar* hashString);
	UBA_API bool CacheClient_WriteToCache2(uba::CacheClient* cacheClient, uba::u32 bucket, const uba::ProcessHandle* process, const uba::u8* inputs, uba::u32 inputsSize, const uba::u8* outputs, uba::u32 outputsSize);
	UBA_API uba::CacheResult* CacheClient_FetchFromCache3(uba::CacheClient* cacheClient, uba::u64 rootsHandle, uba::u32 bucket, const uba::ProcessStartInfo& info);
	UBA_API void CacheClient_RequestServerShutdown(uba::CacheClient* cacheClient, const uba::tchar* reason);
	UBA_API void CacheClient_Disconnect(uba::CacheClient* cacheClient);
	UBA_API void CacheClient_Destroy(uba::CacheClient* cacheClient);
	UBA_API const uba::tchar* CacheResult_GetLogLine(uba::CacheResult* result, uba::u32 index);
	UBA_API uba::u32 CacheResult_GetLogLineType(uba::CacheResult* result, uba::u32 index);
	UBA_API void CacheResult_Delete(uba::CacheResult* result);

	// Trace (automatically created by session if not provided)
	UBA_API uba::Trace* Trace_Create(const uba::tchar* name, const uba::tchar* channel = TC("Default"));
	UBA_API uba::u32 Trace_TaskBegin(uba::Trace* trace, const uba::tchar* description, const uba::tchar* details, uba::u32 color);
	UBA_API void Trace_TaskHint(uba::Trace* trace, uba::u32 taskId, const uba::tchar* hint);
	UBA_API void Trace_TaskEnd(uba::Trace* trace, uba::u32 taskId, bool success);
	UBA_API void Trace_UpdateStatus(uba::Trace* trace, uba::u32 statusRow, uba::u32 statusColumn, const uba::tchar* statusText, uba::LogEntryType statusType, const uba::tchar* statusLink);
	UBA_API void Trace_Destroy(uba::Trace* trace, const uba::tchar* writeFile);
	UBA_API void Trace_SetGlobal(uba::Trace* trace);

	// Misc
	using Uba_CustomAssertHandler = void(const uba::tchar* text);
	UBA_API void Uba_SetCustomAssertHandler(Uba_CustomAssertHandler* handler);

	using ImportFunc = void(const uba::tchar* importName, void* userData);
	UBA_API void Uba_FindImports(const uba::tchar* binary, ImportFunc* func, void* userData);

	UBA_API bool Uba_GetExclusiveAccess(const uba::tchar* path); // Will take a global mutex if returns true and will hand it over to StorageServer on creation.

	// High level interface using config file instead. Uses scheduler under the hood
	UBA_API void* Uba_Create(const uba::tchar* configFile);
	UBA_API uba::u32 Uba_RunProcess(void* uba, const uba::tchar* app, const uba::tchar* args, const uba::tchar* workDir, const uba::tchar* desc, void* userData, ProcessHandle_ExitCallback* exit);
	UBA_API void Uba_RegisterNewFile(void* uba, const uba::tchar* file);
	UBA_API void Uba_Destroy(void* uba);


	// DEPRECATED, don't use
	UBA_API uba::LogWriter* CreateCallbackLogWriter(uba::CallbackLogWriter::BeginScopeCallback begin, uba::CallbackLogWriter::EndScopeCallback end, uba::CallbackLogWriter::LogCallback log);
	UBA_API void DestroyCallbackLogWriter(uba::LogWriter* writer);
	UBA_API void DestroyProcessHandle(const uba::ProcessHandle* handle);
	UBA_API uba::RootPaths* RootPaths_Create(uba::LogWriter& writer);
	UBA_API bool RootPaths_RegisterRoot(uba::RootPaths* rootPaths, const uba::tchar* path, bool includeInKey, uba::u8 id = 0);
	UBA_API bool RootPaths_RegisterSystemRoots(uba::RootPaths* rootPaths, uba::u8 startId = 0);
	UBA_API void RootPaths_Destroy(uba::RootPaths* rootPaths);
	UBA_API bool CacheClient_WriteToCache(uba::CacheClient* cacheClient, uba::RootPaths* rootPaths, uba::u32 bucket, const uba::ProcessHandle* process, const uba::u8* inputs, uba::u32 inputsSize, const uba::u8* outputs, uba::u32 outputsSize);
	UBA_API uba::u32 CacheClient_FetchFromCache(uba::CacheClient* cacheClient, uba::RootPaths* rootPaths, uba::u32 bucket, const uba::ProcessStartInfo& info);
	UBA_API uba::CacheResult* CacheClient_FetchFromCache2(uba::CacheClient* cacheClient, uba::RootPaths* rootPaths, uba::u32 bucket, const uba::ProcessStartInfo& info);
}
