// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaHost : ModuleRules
{
	public UbaHost(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivatePCHHeaderFile = "../Core/Public/UbaCorePch.h";

		bRequiresPlatformSDK = true;

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		StaticAnalyzerDisabledCheckers.Clear();

		PrivateDependencyModuleNames.AddRange(new string[] {
			"UbaCommon",
		});

		// External modules
		PrivateDependencyModuleNames.AddRange(new string[] {
			"UbaMimalloc",
		});
	}
}
