// Copyright Epic Games, Inc. All Rights Reserved.

#if UBA_USE_MIMALLOC
#include "static.c"
#endif
