// Copyright Epic Games, Inc. All Rights Reserved.

// Compile this file to override new and delete
// To override malloc/free, include mimalloc-override.h

#if UBA_USE_MIMALLOC
// mimalloc-new-delete.h should be included in only one source file!
#include <mimalloc-new-delete.h>
#endif
