// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using Microsoft.Extensions.Logging;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
[SupportedTargetTypes(TargetType.Program)]
public class UbaMimalloc : ModuleRules
{
	public UbaMimalloc(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.NoPCHs;
		CppCompileWarningSettings.UnreachableCodeWarningLevel = WarningLevel.Off;

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) || (!Target.bShouldCompileAsDLL && Target.Platform.IsInGroup(UnrealPlatformGroup.Linux)))
		{
			PublicDependencyModuleNames.AddRange(new string[] {
				"mimalloc212",
			});
		}

		// Enable this to enable allocator tracking... double deletes etc.
		// PublicDefinitions.AddRange(["MI_DEBUG=3", "MI_SECURE=4"]);

		if (Target.LinkType == TargetLinkType.Modular)
		{
			Logger.LogWarning("UbaMimalloc will not override allocation functions when linked Modular");
		}
	}
}
