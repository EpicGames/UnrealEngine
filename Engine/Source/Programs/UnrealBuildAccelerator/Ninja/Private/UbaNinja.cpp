// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaNinjaParser.h"
#include "UbaCacheClient.h"
#include "UbaCacheServer.h"
#include "UbaConfig.h"
#include "UbaCoordinatorWrapper.h"
#include "UbaDefaultConstants.h"
#include "UbaFile.h"
#include "UbaLogger.h"
#include "UbaNetworkBackendMemory.h"
#include "UbaNetworkBackendTcp.h"
#include "UbaNetworkClient.h"
#include "UbaNetworkServer.h"
#include "UbaPathUtils.h"
#include "UbaScheduler.h"
#include "UbaSessionServer.h"
#include "UbaStorageServer.h"
#include "UbaNinjaBuild.h"
#include "UbaNinjaDepsLog.h"
#include "UbaThread.h"
#include <atomic>

#if PLATFORM_LINUX
#include <sys/stat.h>
#include <unistd.h>
#include <stdlib.h>
#include <stdio.h>
#endif

#define UBA_USE_LOCAL_CACHE_SERVER 0
#define UBA_FORCE_REMOTE 0
#define UBA_USE_LOCAL_CLIENT 0
#define UBA_USE_HORDE 1

#if UBA_USE_LOCAL_CLIENT
#include "../../Cli/Private/UbaClient.h"
#endif

namespace uba
{
	Atomic<bool> g_shouldExit = false;
	Atomic<bool> g_ctrlBreakPressed = false;
	Event g_processStateChanged(EventResetType_Auto);

	void CtrlBreakPressed()
	{
		if (g_ctrlBreakPressed)
			FatalError(13, TC("Force terminate"));
		g_shouldExit = true;
		g_ctrlBreakPressed = true;
		g_processStateChanged.Set();
	}

	bool PrintHelp(const tchar* message)
	{
		LoggerWithWriter logger(g_consoleLogWriter, TC(""));
		if (*message)
		{
			logger.Info(TC(""));
			logger.Error(TC("%s"), message);
		}

		logger.Info(TC(""));
		logger.Info(TC("-------------------------------------------"));
		logger.Info(TC("   UbaNinja - Ninja Build Accelerator"));
		logger.Info(TC("-------------------------------------------"));
		logger.Info(TC(""));
		logger.Info(TC("  UbaNinja.exe [options...] [targets...]"));
		logger.Info(TC(""));
		logger.Info(TC("  Options:"));
		logger.Info(TC("   -C <dir>                    Change to directory before reading build.ninja"));
		logger.Info(TC("   -f <file>                   Specify input build file (default: build.ninja)"));
		logger.Info(TC("   -targets=<target1,target2>  Build specific targets (comma separated)"));
		logger.Info(TC("   -clean                      Delete all output files without building"));
		logger.Info(TC("   -dry-run                    Parse and enqueue but don't execute processes"));
		logger.Info(TC("   -rebuild                    Clean all outputs before building"));
		logger.Info(TC("   -no-up-to-date              Skip the up-to-date check (enqueue every needed edge)"));
		logger.Info(TC("   -v                          Verbose: print the fully-expanded command for each edge"));
		logger.Info(TC("   -show-includes              Do not strip /showIncludes from compiler commands"));
		logger.Info(TC("   -write-yaml=<path>          Write all build commands to a YAML file"));
		logger.Info(TC("   -dump-deps=<path>           Write per-process dep graph (TSV) sorted by weight"));
		logger.Info(TC("   -dump-timings=<path>        Write per-process completion timeline (TSV)"));
		logger.Info(TC("   -t <tool>                   Run a subtool (currently: 'commands')"));
		logger.Info(TC("   -stop-on-error              Stop the build on the first edge failure (default: run to completion)"));
		logger.Info(TC("   -skip-edges=<count>         Skip first <count> edges as successful"));
		logger.Info(TC("   -max-edges=<count>          Max number of edges that will be built before exiting"));
		logger.Info(TC(""));
		logger.Info(TC("  Subtools (-t <tool>):"));
		logger.Info(TC("   commands  Print the post-wrap application + argv that would be exec'd"));
		logger.Info(TC("             for each enqueued edge, then exit. Diff vs ninja -t commands"));
		logger.Info(TC("             to spot cmdline truncation introduced by the wrap step."));
		logger.Info(TC(""));
		logger.Info(TC("  Examples:"));
		logger.Info(TC("   UbaNinja.exe -C out/Default chrome"));
		logger.Info(TC(""));

		return false;
	}

	bool WrappedMain(int argc, tchar* argv[])
	{
		u64 startTime = GetTime();

		AddExceptionHandler();
		InitMemory();
		SetMemoryWorkingSet(DefaultMemoryWorkingSet);

		#if PLATFORM_LINUX
		// WSL2 without systemd-logind: $XDG_RUNTIME_DIR points to a missing
		// /run/user/<uid>. UbaNinja and detoured workers (esp. nsjail) need it
		// writable. Fall back to a private /tmp dir; setenv propagates to workers.
		{
			const char* xdg = getenv("XDG_RUNTIME_DIR");
			bool ok = false;
			if (xdg && *xdg)
			{
				struct stat st;
				if (stat(xdg, &st) == 0 && S_ISDIR(st.st_mode) && access(xdg, R_OK|W_OK|X_OK) == 0)
					ok = true;
			}
			if (!ok)
			{
				char fallback[64];
				snprintf(fallback, sizeof(fallback), "/tmp/uba_xdg_%u", (unsigned)getuid());
				mkdir(fallback, 0700);  // EEXIST is fine.
				chmod(fallback, 0700);
				setenv("XDG_RUNTIME_DIR", fallback, 1);
			}
		}
		#endif

		#if !PLATFORM_WINDOWS
		SetupOrphanReaper();
		#endif

		FilteredLogWriter logWriter(g_consoleLogWriter, LogEntryType_Info);
		LoggerWithWriter logger(logWriter, TC(""));

		TString ninjaFile;
		TString buildDir;
		bool cleanOnly = false;
		bool dryRun = false;
		TString yamlFile;
		TString depsDumpFile;
		TString timingsDumpFile;
		bool rebuild = false;
		bool checkUpToDate = false; // TEMPORARY: up-to-date check disabled
		bool verbose = false;
		bool noShowIncludes = true;
		bool toolCommands = false;
		bool stopOnError = false;
		u32 maxEdges = ~0u;
		u32 skipEdges = 0;
		Vector<TString> targets;

		// Parse arguments
		for (int i = 1; i < argc; ++i)
		{
			const tchar* argStr = argv[i];
			StringView arg(ToView(argStr));

			if (TStrcmp(argStr, TC("-help")) == 0 || TStrcmp(argStr, TC("--help")) == 0 ||
			    TStrcmp(argStr, TC("-h")) == 0 || TStrcmp(argStr, TC("/?")) == 0)
			{
				return PrintHelp(TC(""));
			}
			else if (TStrcmp(argStr, TC("-C")) == 0)
			{
				if (i + 1 >= argc)
					return PrintHelp(TC("-C requires a directory argument"));
				buildDir = argv[++i];
			}
			else if (TStrcmp(argStr, TC("-f")) == 0)
			{
				if (i + 1 >= argc)
					return PrintHelp(TC("-f requires a file argument"));
				ninjaFile = argv[++i];
			}
			else if (arg.StartsWith(TCV("-targets=")))
			{
				const tchar* p = arg.data + 9;
				const tchar* end = arg.data + arg.count;
				const tchar* start = p;

				while (p <= end)
				{
					if (p == end || *p == ',')
					{
						if (start < p)
							targets.push_back(TString(start, u32(p - start)));
						if (p < end)
							++p;
						start = p;
					}
					else
						++p;
				}
			}
			else if (TStrcmp(argStr, TC("-clean")) == 0)
			{
				cleanOnly = true;
			}
			else if (TStrcmp(argStr, TC("-dry-run")) == 0)
			{
				dryRun = true;
			}
			else if (TStrcmp(argStr, TC("-rebuild")) == 0)
			{
				rebuild = true;
			}
			else if (TStrcmp(argStr, TC("-no-up-to-date")) == 0)
			{
				checkUpToDate = false;
			}
			else if (TStrcmp(argStr, TC("-v")) == 0)
			{
				verbose = true;
			}
			else if (TStrcmp(argStr, TC("-show-includes")) == 0)
			{
				noShowIncludes = false;
			}
			else if (arg.StartsWith(TCV("-write-yaml=")))
			{
				yamlFile = arg.data + 12;
			}
			else if (arg.StartsWith(TCV("-dump-deps=")))
			{
				depsDumpFile = arg.data + 11;
			}
			else if (arg.StartsWith(TCV("-dump-timings=")))
			{
				timingsDumpFile = arg.data + 14;
			}
			else if (TStrcmp(argStr, TC("-stop-on-error")) == 0)
			{
				stopOnError = true;
			}
			else if (arg.StartsWith(TCV("-max-edges=")))
			{
				arg.Skip(11).Parse(maxEdges);
			}
			else if (arg.StartsWith(TCV("-skip-edges=")))
			{
				arg.Skip(12).Parse(skipEdges);
			}
			else if (TStrcmp(argStr, TC("-t")) == 0)
			{
				if (i + 1 >= argc)
					return PrintHelp(TC("-t requires a tool name (e.g. -t commands)"));
				const tchar* tool = argv[++i];
				if (TStrcmp(tool, TC("commands")) == 0)
				{
					toolCommands = true;
					dryRun = true;
				}
				else
				{
					StringBuffer<> msg;
					msg.Appendf(TC("Unknown -t tool: %s (only 'commands' supported)"), tool);
					return PrintHelp(msg.data);
				}
			}
			else if (argStr[0] != '-')
			{
				if (ninjaFile.empty() && (TStrstr(argStr, TC(".ninja")) || TStrstr(argStr, TC("\\")) || TStrstr(argStr, TC("/"))))
					ninjaFile = argStr;
				else
					targets.push_back(argStr);
			}
			else
			{
				StringBuffer<> msg;
				msg.Appendf(TC("Unknown option: %s"), arg.data);
				return PrintHelp(msg.data);
			}
		}

		if (!buildDir.empty())
		{
			if (!IsAbsolutePath(buildDir.c_str()))
			{
				StringBuffer<> cwd;
				if (!GetCurrentDirectoryW(cwd))
					return logger.Error(TC("Failed to get current directory"));

				StringBuffer<> fullPath;
				fullPath.Append(cwd).EnsureEndsWithSlash().Append(buildDir);
				buildDir = fullPath.data;
			}

			if (ninjaFile.empty())
			{
				StringBuffer<> temp;
				temp.Append(buildDir).EnsureEndsWithSlash().Append(TCV("build.ninja"));

				if (FileExists(logger, temp.data))
					ninjaFile = temp.data;
				else
					return PrintHelp(TC("build.ninja not found in specified directory"));
			}
			else
			{
				if (!IsAbsolutePath(ninjaFile.c_str()))
				{
					StringBuffer<> fullPath;
					fullPath.Append(buildDir).EnsureEndsWithSlash().Append(ninjaFile);
					ninjaFile = fullPath.data;
				}
			}
		}

		if (ninjaFile.empty())
		{
			StringBuffer<> cwd;
			if (!GetCurrentDirectoryW(cwd))
				return PrintHelp(TC("Failed to get current directory"));
			cwd.EnsureEndsWithSlash().Append(TC("build.ninja"));

			if (FileExists(logger, cwd.data))
				ninjaFile = cwd.data;
			else
				return PrintHelp(TC("No ninja file specified and build.ninja not found in current directory"));
		}

		if (!IsAbsolutePath(ninjaFile.c_str()))
		{
			StringBuffer<> cwd;
			if (!GetCurrentDirectoryW(cwd))
				return logger.Error(TC("Failed to get current directory"));

			StringBuffer<> fullPath;
			fullPath.Append(cwd).EnsureEndsWithSlash().Append(ninjaFile);
			ninjaFile = fullPath.data;
		}

		StringBuffer<512> ninjaDir(ninjaFile.c_str());
		{
			const tchar* lastSep = TStrrchr(ninjaDir.data, PathSeparator);
			if (lastSep)
				ninjaDir.Resize(u32(lastSep - ninjaDir.data + 1));
			else
				ninjaDir.EnsureEndsWithSlash();
		}

		NinjaDepsLog depsLog;
		depsLog.Init(StringView(ninjaDir.data, ninjaDir.count), GetTime());

		StringBuffer<128> traceName(TCV("UbaNinja_"));
		Guid guid;
		CreateGuid(guid);
		traceName.Append(GuidToString(guid));
		Trace trace(logWriter);
		trace.StartWriteAndThread(traceName.data, 128ull*1024*1024, true);
		auto tg = MakeGuard([&]()
			{
				trace.StopThread();
				ninjaDir.Append(TCV("UbaNinjaTrace.uba"));
				trace.Write(ninjaDir.data, true);
			});

		Scheduler* schedulerPtr = nullptr;
		u32 cacheBucketId = 0;
		bool forceRemote = UBA_FORCE_REMOTE;
		Event readyToSchedule(EventResetType_Manual);

		Atomic<bool> parserSuccess = true;

		struct TimingEntry
		{
			u64    completionMs;
			u64    durationMs;
			u32    exitCode;
			u8     execType;
			TString description;
			TString primaryOut;
		};
		const u64 buildStartTime = GetTime();
		Vector<TimingEntry> timingEntries;
		Futex timingEntriesLock;
		const bool timingsEnabled = !timingsDumpFile.empty();
		if (timingsEnabled)
			timingEntries.reserve(256 * 1024);

		struct NinjaExitContext
		{
			LoggerWithWriter*    logger;
			NinjaDepsLog*        depsLog;
			Vector<TimingEntry>* timingEntries;
			Futex*               timingEntriesLock;
			bool                 timingsEnabled;
			u64                  buildStartTime;
			WorkManager*         workManager;
			std::atomic<u32>*    pendingRecords;
		};
		std::atomic<u32> pendingRecords{0};
		NinjaExitContext exitCtx;
		exitCtx.logger            = &logger;
		exitCtx.depsLog           = &depsLog;
		exitCtx.timingEntries     = &timingEntries;
		exitCtx.timingEntriesLock = &timingEntriesLock;
		exitCtx.timingsEnabled    = timingsEnabled;
		exitCtx.buildStartTime    = buildStartTime;
		exitCtx.workManager       = nullptr;
		exitCtx.pendingRecords    = &pendingRecords;

		auto onProcessExited = +[](void* userData, const ProcessHandle& handle, ProcessExitedResponse&)
		{
			auto* p = (NinjaExitPayload*)userData;
			if (!p) return;
			auto* ctx = (NinjaExitContext*)p->ctx;
			auto cleanup = MakeGuard([p]() { delete p; });

			if (ctx && ctx->timingsEnabled)
			{
				TimingEntry entry;
				entry.completionMs = TimeToMs(GetTime() - ctx->buildStartTime);
				entry.durationMs   = TimeToMs(handle.GetTotalWallTime());
				entry.exitCode     = handle.GetExitCode();
				entry.execType     = u8(handle.GetExecutionType());
				entry.description  = handle.GetStartInfo().GetDescription();
				entry.primaryOut   = p->primaryOut;
				SCOPED_FUTEX(*ctx->timingEntriesLock, lock);
				ctx->timingEntries->emplace_back(std::move(entry));
			}

			if (ctx && ctx->depsLog && ctx->logger && ctx->workManager && handle.GetExitCode() == 0)
			{
				ctx->depsLog->RegisterOutputs(handle);
				ctx->pendingRecords->fetch_add(1, std::memory_order_acq_rel);
				ctx->workManager->AddWork([
					logger     = ctx->logger,
					depsLog    = ctx->depsLog,
					handle     = handle,
					primaryOut = p->primaryOut,
					restat     = p->restat,
					cmdHash    = p->cmdHash,
					pending    = ctx->pendingRecords
				](const WorkContext&)
				{
					depsLog->RecordResult(*logger, handle,
						primaryOut.empty() ? nullptr : primaryOut.c_str(),
						restat, cmdHash);
					pending->fetch_sub(1, std::memory_order_acq_rel);
				}, 1, TC("DepsRec"));
			}
		};

		NinjaParseState parseState;
		parseState.exitCallback = onProcessExited;
		parseState.exitContext  = &exitCtx;

		Thread parserThread([&]()
			{
				u32 taskId = 1234567;
				trace.TaskBegin(taskId, TCV("Parse"), {}, ColorWork);
				auto fail = [&](bool dummy = false) { parserSuccess = false; g_processStateChanged.Set(); return 0u; };

				// Anchor for relative paths inside the ninja file: cwd-at-invocation
				// (or `-C dir`), NOT the directory of the ninja file itself
				// (Android: `-f out/combined.ninja` from android root).
				StringBuffer<> buildWorkingDir;
				if (!buildDir.empty())
				{
					buildWorkingDir.Append(buildDir);
				}
				else if (!GetCurrentDirectoryW(buildWorkingDir))
				{
					return fail(logger.Error(TC("Failed to get current directory")));
				}
				buildWorkingDir.EnsureEndsWithSlash();

				MemoryBlock ninjaMemory(TC("NinjaParser"));
				ninjaMemory.Init(2ull * 1024 * 1024 * 1024);

				// Load .ninja_log/.ninja_deps in parallel with parsing — on chromium
				// the deps file alone is ~500 MB. Joined before EnqueueCommands runs.
				NinjaLogIndex logIndex;
				std::atomic<bool> logIndexLoaded { false };
				Thread logIndexLoadThread([&]() -> u32
				{
					u32 depsTaskId = 1234569;
					trace.TaskBegin(depsTaskId, TCV("Deps"), {}, ColorWork);
					if (logIndex.Load(logger,
						StringView(ninjaDir.data, ninjaDir.count),
						StringView(buildWorkingDir.data, buildWorkingDir.count)))
					{
						logIndexLoaded.store(true, std::memory_order_release);
					}
					trace.TaskEnd(depsTaskId, true);
					return 0;
				}, TC("LogIdxLoad"));

				NinjaParser parser(ninjaMemory);

				u64 enqueueStartTime = GetTime();
				u64 parseStartTime = GetTime();
				if (!ParseNinjaFile(logger, parser, ninjaFile.c_str(), targets, parseState, buildWorkingDir.data))
				{
					logIndexLoadThread.Wait(); // captures must stay valid
					return fail();
				}
				u64 parseTime = GetTime() - parseStartTime;

				trace.TaskEnd(taskId, true);

				readyToSchedule.IsSet();
				if (!schedulerPtr)
				{
					logIndexLoadThread.Wait();
					return fail();
				}

				// Join logIndex load before EnqueueCommands reads it.
				u64 logIndexWaitStart = GetTime();
				logIndexLoadThread.Wait();
				u64 logIndexWaitTime = GetTime() - logIndexWaitStart;
				bool logIndexLoadedFinal = logIndexLoaded.load(std::memory_order_acquire);
				logger.Detail(TC("LogIdxLoad: %s wait after parse (loaded=%u)"),
					TimeToText(logIndexWaitTime).str, u32(logIndexLoadedFinal ? 1 : 0));

				EnqueueCommandsContext context { logger, parser, *schedulerPtr, parseState, logIndex };
				context.workingDir = buildWorkingDir;
				context.rebuild = rebuild;
				context.cleanOnly = cleanOnly;
				context.noShowIncludes = noShowIncludes;
				context.checkUpToDate = checkUpToDate;
				context.verbose = verbose;
				context.dryRun = dryRun;
				context.yamlFile = yamlFile.empty() ? nullptr : yamlFile.c_str();
				context.cacheBucketId = cacheBucketId;
				context.forceRemote = forceRemote;
				context.toolCommands = toolCommands;
				context.ninjaFile = ninjaFile.c_str();
				context.logIndexLoaded = logIndexLoadedFinal;
				context.depsDumpFile = depsDumpFile.empty() ? nullptr : depsDumpFile.c_str();
				context.maxEdges = maxEdges;
				context.skipEdges = skipEdges;
				context.depsLog = &depsLog;

				u64 enqueueOnlyStart = GetTime();
				if (!EnqueueCommands(context))
					return fail();
				u64 enqueueOnlyTime = GetTime() - enqueueOnlyStart;

				u64 enqueueTime = GetTime() - enqueueStartTime;
				logger.Info(TC("Parse and Enqueue completed in %s (parse=%s, enqueue=%s)"),
					TimeToText(enqueueTime).str, TimeToText(parseTime).str, TimeToText(enqueueOnlyTime).str);
				g_processStateChanged.Set();
				return 0u;

			}, TC("NinjPars"));


		Config config;
		StringBuffer<> configFile;
		if (GetCurrentDirectoryW(configFile))
		{
			configFile.EnsureEndsWithSlash().Append(TC("UbaNinja.toml"));
			if (FileExists(logger, configFile.data))
			{
				logger.Info(TC("Loading config from %s"), configFile.data);
				config.LoadFromFile(logger, configFile.data);
			}
		}

		config.AddTable(TC("Session")).AddValue(TC("DirTableSnapshotEnabled"), true);

		TString coordinatorName;
		
		#if UBA_USE_HORDE
		coordinatorName = TC("Horde");
		#endif

		CoordinatorWrapper coordinator;

		NetworkBackendTcpCreateInfo nbInfo(logWriter);
		nbInfo.Apply(config);
		NetworkBackendTcp networkBackend(nbInfo);

		bool ctorSuccess = true;
		NetworkServerCreateInfo nsci(logWriter);
		nsci.workerCount = 512;
		nsci.Apply(config);
		NetworkServer networkServer(ctorSuccess, nsci);
		if (!ctorSuccess)
			return logger.Error(TC("Failed to create NetworkServer"));

		{
			Config clientConfig;
			clientConfig.AddTable(TC("Storage")).AddValue(TC("SendOneBigAtTheTime"), false);
			clientConfig.AddTable(TC("Session")).AddValue(TC("DirTableSnapshotEnabled"), true);
			networkServer.SetClientsConfig(clientConfig);
		}

		StringBuffer<> rootDir;
		if (!GetCurrentDirectoryW(rootDir))
			return logger.Error(TC("Failed to get current directory"));
		rootDir.EnsureEndsWithSlash().Append(TC(".uba"));

		u64 storageCapacity = 40ull * 1024 * 1024 * 1024;
		StorageServerCreateInfo storageInfo(networkServer, rootDir.data, logWriter);
		storageInfo.casCapacityBytes = storageCapacity;
		storageInfo.storeCompressed = true;
		storageInfo.Apply(config);
		StorageServer storageServer(storageInfo);

		SessionServerCreateInfo ssci(storageServer, networkServer, logWriter);
		ssci.rootDir = rootDir.data;
		ssci.useFillLock = false;
		ssci.traceEnabled = true;
		ssci.writeFilesBottleneck = 24;
		ssci.writeFilesFileMapMaxMb = 1;
		ssci.trace = &trace;
		ssci.allowLazyCopyOnWrite = false;

		//ssci.detailedTrace = true;
		//ssci.remoteTraceEnabled = true;

		#if UBA_DEBUG
		ssci.logToFile = true;
		ssci.remoteLogEnabled = true;
		#endif

		ssci.Apply(config);
		SessionServer sessionServer(ssci);

		#if UBA_DEBUG
		logger.Info(TC("Detour logging enabled. Written to %s%clog"), sessionServer.GetSessionDir(), PathSeparator);
		#endif

		#if PLATFORM_WINDOWS
		SetConsoleCtrlHandler([](DWORD signal) { CtrlBreakPressed(); return TRUE; }, TRUE);
		#else
		signal(SIGINT, [](int) { CtrlBreakPressed(); });
		signal(SIGTERM, [](int) { CtrlBreakPressed(); });
		#endif

		CacheClient* cacheClients[8];
		u32 cacheClientCount = 0;

#if UBA_USE_LOCAL_CACHE_SERVER
		NetworkBackendMemory networkMemoryBackend1(logWriter);
		NetworkBackendMemory networkMemoryBackend2(logWriter);
		NetworkBackendMemory networkMemoryBackend3(logWriter);
		NetworkBackendMemory networkMemoryBackend4(logWriter);

		logger.Info(TC(""));
		logger.Info(TC("Starting cache server..."));

		// Setup cache server/client which is used for out-of-date checking
		NetworkServerCreateInfo cnsci(logWriter);
		NetworkServer cacheNetworkServer(ctorSuccess, cnsci);
		StringBuffer<> cacheRootDir;
		//if (!GetCurrentDirectoryW(cacheRootDir))
		//	return logger.Error(TC("Failed to get current directory"));
		cacheRootDir.EnsureEndsWithSlash().Append(TC(".ubacache"));

		StorageServerCreateInfo cacheStorageInfo(cacheNetworkServer, cacheRootDir.data, logWriter);
		cacheStorageInfo.casCapacityBytes = 40ull * 1024 * 1024 * 1024;
		cacheStorageInfo.allowHintAsFallback = false;
		cacheStorageInfo.writeReceivedCasFilesToDisk = true;
		cacheStorageInfo.allowDeleteVerified = true;
		cacheStorageInfo.allowDeferredDeletes = false;
		StorageServer cacheStorageServer(cacheStorageInfo);
		CacheServerCreateInfo csInfo(cacheStorageServer, cacheRootDir.data, logWriter);
		CacheServer cacheServer(csInfo);


		if (!dryRun && !cleanOnly)
		{
			cacheNetworkServer.StartListen(networkMemoryBackend1);
			cacheNetworkServer.StartListen(networkMemoryBackend2);
			cacheNetworkServer.StartListen(networkMemoryBackend3);
			cacheNetworkServer.StartListen(networkMemoryBackend4);
			cacheServer.Load(false);
		}
#endif

		auto createCoordinator = [&](Scheduler* scheduler)
			{
				if (coordinatorName.empty())
					return true;
				if (scheduler)
					coordinator.SetScheduler(scheduler);
				if (coordinator.IsCreated())
					return true;

				StringBuffer<512> coordinatorWorkDir(rootDir);
				coordinatorWorkDir.EnsureEndsWithSlash().Append(coordinatorName);
				StringBuffer<512> binariesDir;
				if (!GetDirectoryOfCurrentModule(logger, binariesDir))
					return false;

				CoordinatorCreateInfo cinfo;
				cinfo.workDir = coordinatorWorkDir.data;
				cinfo.binariesDir = binariesDir.data;
				cinfo.maxCoreCount = 0;
				cinfo.logging = false;
				if (!coordinator.Create(logger, coordinatorName.c_str(), cinfo, networkBackend, networkServer, trace, 0))
					return false;
				return true;
			};

		NetworkClientCreateInfo nccInfo(logWriter);

#if UBA_USE_LOCAL_CACHE_SERVER
		NetworkClient cacheNetworkClient(ctorSuccess, nccInfo);
		cacheNetworkClient.Connect(networkMemoryBackend1, TC("Memory"));
		cacheNetworkClient.Connect(networkMemoryBackend2, TC("Memory"));
		cacheNetworkClient.Connect(networkMemoryBackend3, TC("Memory"));
		cacheNetworkClient.Connect(networkMemoryBackend4, TC("Memory"));
		CacheClientCreateInfo ccInfo(logWriter, storageServer, cacheNetworkClient, sessionServer);
		ccInfo.useRoots = false;
		CacheClient cacheClient(ccInfo);
		cacheClients[cacheClientCount++] = &cacheClient;
		cacheBucketId = 1;
#else

		StringBuffer<256> cacheServerEndpoint;
		Guid sessionKey;
#if 0
		if (true)
		{
			bool writeAccess;
			createCoordinator(nullptr);
			if (coordinator.RequestCacheServer(cacheServerEndpoint, sessionKey, writeAccess))
			{
				//cacheNetworkClient.Connect(networkBackend, TC("Memory"));
				nccInfo.cryptoKey128 = (u8*)&sessionKey;

				//nccInfo.useThreadedSend = true;
				cacheBucketId = 1;
			}
		}
#endif
		nccInfo.workerCount = 10;
		NetworkClient cacheNetworkClient(ctorSuccess, nccInfo);

		CacheClientCreateInfo ccInfo(logWriter, storageServer, cacheNetworkClient, sessionServer);
		ccInfo.useRoots = false;
		//ccInfo.reportMissReason = true;
		CacheClient cacheClient(ccInfo);

		if (cacheServerEndpoint.count)
		{
			u64 port = DefaultPort;
			bool success = true;
			if (const tchar* colon = cacheServerEndpoint.First(':'))
			{
				success = Parse(port, colon+1, TStrlen(colon)+1);
				cacheServerEndpoint.Resize(colon - cacheServerEndpoint.data);
			}

			if (success)
				if (cacheNetworkClient.Connect(networkBackend, cacheServerEndpoint.data, (u16)port))
				{
					IndexContainer container(31);
					networkServer.ParallelFor(30, container, [&](const WorkContext&, auto& it)
						{
							cacheNetworkClient.Connect(networkBackend, cacheServerEndpoint.data, (u16)port);
						}, TCV("Connect"));

					cacheClients[cacheClientCount++] = &cacheClient;
				}
		}
#endif

		sessionServer.RegisterNetworkTrafficProvider(u64(&cacheNetworkClient), [nc = &cacheNetworkClient](u64& outSent, u64& outReceive, u32& outConnectionCount)
			{
				outSent = nc->GetTotalSentBytes();
				outReceive = nc->GetTotalRecvBytes();
				outConnectionCount = nc->GetConnectionCount();
			});


		u32 maxLocalProcessors = GetLogicalProcessorCount();
		if (maxLocalProcessors > 60)
			maxLocalProcessors -= 8;

		if (maxLocalProcessors > 100)
			maxLocalProcessors = 100;

#if UBA_USE_LOCAL_CLIENT
		StringBuffer<> clientRootDir;
		if (!GetCurrentDirectoryW(clientRootDir))
			return logger.Error(TC("Failed to get current directory"));
		clientRootDir.EnsureEndsWithSlash().Append(TC(".ubaclient"));
		ClientInitInfo ci { logWriter, networkBackend, clientRootDir.data, TC("localhost"), DefaultPort };
		ci.maxProcessorCount = maxLocalProcessors;
		forceRemote = true;
#endif

		// This creates insane amount of I/O.. faster to just disable
		// If we want it enabled, let's put in in a non-temp folder
		sessionServer.AddEnvironmentVariable(TC("NODE_DISABLE_COMPILE_CACHE"), TC("1"));

		//StringBuffer<> nodeDir;
		//if (!GetCurrentDirectoryW(nodeDir))
		//	return logger.Error(TC("Failed to get current directory"));
		//nodeDir.EnsureEndsWithSlash().Append(TC(".node_cache"));
		//sessionServer.AddEnvironmentVariable(TC("NODE_COMPILE_CACHE"), nodeDir.data);

		//sessionServer.AddEnvironmentVariable(TC("_JAVA_OPTIONS"), TC("\"-Djava.io.tmpdir=$TMPDIR\""))


		storageServer.LoadCasTable();

		SchedulerCreateInfo sci(sessionServer);
		sci.maxLocalProcessors = maxLocalProcessors;
		sci.maxRacingPercent = 100;
		sci.cacheClientCount = cacheClientCount;
		sci.cacheClients = cacheClients;
		sci.writeToCache = true;
		sci.maxParallelCacheQueries = 800;//Min(GetLogicalProcessorCount(), 96u);
		sci.enableSpeculativeCacheTest = true;
		sci.forceRemote = forceRemote;
		sci.leakMemoryAtShutdown = true;
		//sci.traceDetails = true;
		//sci.writeTrackedInputs = true;
		sci.Apply(config);

		Scheduler scheduler(sci);

		auto destroyCoordinator = MakeGuard([&]() { coordinator.Destroy(); });

		Atomic<u32> failedProcessCount = 0;
		Atomic<bool> stopOnErrorTriggered = false;

		scheduler.SetProcessFinishedCallback([&](const ProcessHandle& handle)
			{
				auto g = MakeGuard([](){ g_processStateChanged.Set(); });

				u32 exitCode = handle.GetExitCode();
				if (exitCode == 0 || exitCode == ProcessCancelExitCode)
					return;

				++failedProcessCount;
				logger.BeginScope();
				auto lg = MakeGuard([&]() { logger.EndScope(); });

				if (g_shouldExit || (stopOnError && stopOnErrorTriggered.exchange(true)))
					return;

				const ProcessStartInfo& startInfo = handle.GetStartInfo();
				logger.Error(TC("Process failed (Id: %u, exit code %u): %s"), handle.GetId(), exitCode, startInfo.GetDescription());

				if (stopOnError)
				{
					networkServer.GetLogger().isMuted++;
					scheduler.GetSession().GetLogger().isMuted++;
					g_shouldExit = true;
					logger.Error(TC("  Application: %s"), startInfo.application);
					logger.Error(TC("  WorkingDir:  %s"), startInfo.workingDir);
					if (startInfo.logFile && *startInfo.logFile)
						logger.Error(TC("  Logfile:     %s"), startInfo.logFile);
					logger.Error(TC("  Arguments:   %s"), startInfo.arguments);
				}

				const auto& logLines = handle.GetLogLines();
				for (const auto& logLine : logLines)
					logger.Logf(logLine.type, TC("  %s"), logLine.text.c_str());
			});

		if (!dryRun && !cleanOnly)
		{
			createCoordinator(&scheduler);

			coordinator.RequestHelpers(2600);
			scheduler.Start();
		}

		bool listenSuccess = networkServer.StartListen(networkBackend); (void)listenSuccess;


#if UBA_USE_LOCAL_CLIENT
		if (!listenSuccess)
		{
			g_shouldExit = true;
			g_processStateChanged.Set();
			readyToSchedule.Set();
			return logger.Error(TC("Failed to listen to port"));
		}
		Client c;
		c.Init(ci);
#endif

		schedulerPtr = &scheduler;
		exitCtx.workManager = &scheduler.GetSession().GetWorkManager();
		readyToSchedule.Set();

		u32 queued, activeLocal, activeRemote, finished;
		scheduler.GetStats(queued, activeLocal, activeRemote, finished);

		if (cleanOnly || dryRun)
		{
			// Wait out the parser thread; cancelling mid-enqueue trips an assert on EnqueueSuspendedProcess.
			parserThread.Wait();
			scheduler.GetStats(queued, activeLocal, activeRemote, finished);
			scheduler.Cancel();
			if (dryRun)
			{
				logger.Info(TC(""));
				logger.Info(TC("Dry-run mode: Would build %u processes"), queued);
			}
			return true;
		}

		u32 lastFinished = 0;
		while (true)
		{
			scheduler.GetStats(queued, activeLocal, activeRemote, finished);
			if (finished != lastFinished)
			{
				lastFinished = finished;
				StringBuffer<32> logEntry;
				logEntry.Appendf(TC("\r[%u/%u]"), finished, scheduler.GetTotalProcesses());
				if (u32 fc = failedProcessCount.load())
					logEntry.Appendf(TC(" Failed: %u"), fc);
				LogToConsoleNoLineFeed(LogEntryType_Info, logEntry.data, logEntry.count);
			}

			if (!parserThread.Wait(10))
				continue;

			if (finished == scheduler.GetTotalProcesses() || g_shouldExit)
				break;

			g_processStateChanged.IsSet();
		}

		if (g_shouldExit)
		{
			logger.Info(TC("Cancelling..."));
			networkServer.DisallowNewClients();
			scheduler.Cancel();
		}
		else
			while (!scheduler.IsEmpty()) // flush cache writes
				Sleep(10);

		cacheNetworkClient.Disconnect();

#if UBA_USE_LOCAL_CACHE_SERVER
		cacheNetworkServer.DisconnectClients();
#endif

		networkBackend.StopListen();
		networkServer.DisconnectClients(); // also disconnects network visualizers
		destroyCoordinator.Execute();
		scheduler.Stop(); // leak memory; process is exiting
		scheduler.GetStats(queued, activeLocal, activeRemote, finished);

		UBA_ASSERT(pendingRecords.load(std::memory_order_acquire) == 0);

		if (!dryRun && !cleanOnly)
			depsLog.Save(logger, StringView(ninjaDir.data, ninjaDir.count));

		if (!timingsDumpFile.empty() && !timingEntries.empty())
		{
			std::sort(timingEntries.begin(), timingEntries.end(), [](const TimingEntry& a, const TimingEntry& b) { return a.completionMs < b.completionMs; });

			Vector<char> buf;
			buf.reserve(8 * 1024 * 1024);
			auto appendStr = [&](const char* s) { buf.insert(buf.end(), s, s + strlen(s)); };
			auto appendU64 = [&](u64 v) {
				char tmp[32];
				snprintf(tmp, sizeof(tmp), "%llu", (unsigned long long)v);
				appendStr(tmp);
			};
			auto appendU32 = [&](u32 v) {
				char tmp[16];
				snprintf(tmp, sizeof(tmp), "%u", v);
				appendStr(tmp);
			};
			auto appendTStr = [&](const TString& s) {
				for (tchar c : s) buf.push_back((char)(u8)c);
			};
			static const char* execLabels[] = {
				"native", "local", "remote", "cache", "skipped"
			};

			appendStr("# UbaNinja timings dump\n");
			appendStr("# Process timeline sorted by completion time (ms from build start).\n");
			appendStr("# Columns: completionMs\\tdurationMs\\texit\\texecType\\tdesc\\tprimaryOut\n");

			for (const auto& e : timingEntries)
			{
				appendU64(e.completionMs); appendStr("\t");
				appendU64(e.durationMs); appendStr("\t");
				appendU32(e.exitCode); appendStr("\t");
				appendStr((e.execType < sizeof_array(execLabels)) ? execLabels[e.execType] : "?");
				appendStr("\t");
				appendTStr(e.description); appendStr("\t");
				appendTStr(e.primaryOut); appendStr("\n");
			}

			FileHandle fh = CreateFileW(timingsDumpFile.c_str(), GENERIC_WRITE, FILE_SHARE_READ, CREATE_ALWAYS, DefaultAttributes());
			if (fh != InvalidFileHandle)
			{
				WriteFile(logger, timingsDumpFile.c_str(), fh, buf.data(), u32(buf.size()));
				CloseFile(timingsDumpFile.c_str(), fh);
				logger.Info(TC("Wrote timings dump: %s (%u entries)"), timingsDumpFile.c_str(), u32(timingEntries.size()));
			}
			else
				logger.Error(TC("Failed to create timings dump: %s"), timingsDumpFile.c_str());
		}

		u32 failed = failedProcessCount;
		logger.Info(TC(""));
		if (g_ctrlBreakPressed)
			logger.Info(TC("Build cancelled! (Total processed: %u)"), finished);
		else if (failed == 0)
			logger.Info(TC("Build successful! (Total processed: %u in %s)"), finished, TimeToText(GetTime() - startTime, true).str);
		else
			logger.Info(TC("Build failed! (%u failed)"), failed);

#if UBA_USE_LOCAL_CACHE_SERVER
		cacheServer.RunMaintenance(false, true, []() { return false; });
#endif

		return failed == 0;
	}
}

#if PLATFORM_WINDOWS
int wmain(int argc, wchar_t* argv[]) { return uba::WrappedMain(argc, argv) ? 0 : 1; }
#else
int main(int argc, char* argv[]) { return uba::WrappedMain(argc, argv) ? 0 : 1; }
#endif
