// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaNinjaBuild.h"
#include "UbaCacheClient.h"
#include "UbaDirectoryIterator.h"
#include "UbaDirectoryTableHolder.h" // for GetDirKey helper
#include "UbaFile.h"
#include "UbaHash.h"
#include "UbaLogger.h"
#include "UbaNinjaChromiumWorkarounds.h"
#include "UbaNinjaDepsLog.h"
#include "UbaNinjaParser.h"
#include "UbaPathUtils.h"
#include "UbaProcessStartInfoHolder.h"
#include "UbaScheduler.h"
#include "UbaSessionServer.h"
#include "UbaSynchronization.h"
#include "UbaWorkManager.h"
#include <algorithm>
#include <atomic>
#include <cstdio>
#if !PLATFORM_WINDOWS
#include <unistd.h>
#endif

namespace uba
{
	// Up-to-date check: clears edgesToBuildFlags for edges confirmed clean.
	// Clean = all outputs exist, recorded cmd hash matches, no input newer
	// than oldest output, no upstream producer is dirty.
	// Directory scans parallelize via WorkManager.

	// In-place strip of "/showIncludes" (and following space) — matches the
	// transform EnqueueCommands does so cmd hashes align with recorded ones.
	static void StripShowIncludesInPlace(Vector<char>& buf)
	{
		const char* pattern = "/showIncludes";
		u32 patternLen = 13;
		u32 writePos = 0;
		for (u32 readPos = 0; readPos < buf.size(); )
		{
			if (readPos + patternLen <= buf.size() &&
				memcmp(&buf[readPos], pattern, patternLen) == 0)
			{
				while (readPos < buf.size() && buf[readPos] != ' ')
					++readPos;
				if (readPos < buf.size() && buf[readPos] == ' ')
					++readPos;
			}
			else
			{
				buf[writePos++] = buf[readPos++];
			}
		}
		buf.resize(writePos);
	}

	static u32 SuppressUpToDateEdges(
		LoggerWithWriter& logger,
		NinjaParser& parser,
		StringView workingDir,
		WorkManager& workManager,
		const Vector<u32>& topoOrder,
		NinjaParseState& state,
		bool noShowIncludes,
		const NinjaLogIndex& logIndex)
	{
		u64 t0 = GetTime();

		const auto& edges         = parser.GetEdges();
		const u32   edgeCount     = u32(edges.size());
		auto&       stringPool    = const_cast<StringPool&>(parser.GetStringPool());
		auto&       outputToEdge  = state.outputToEdge;
		auto&       edgesToBuildFlags = state.edgesToBuildFlags;
		const u32   phonyId       = state.phonyId;

		if (logIndex.logByOutputKey.empty() && logIndex.depsByOutputKey.empty())
		{
			logger.Info(TC("Up-to-date check: no prior build log, rebuilding everything"));
			return 0;
		}

		u64 tAfterLoad = GetTime();

		// Phase 1: collect unique directories to stat. From every path
		// referenced by needed edges (outputs/implicit-outs/inputs/implicit-deps)
		// + discovered header paths from .ninja_deps. order-only deps excluded
		// (ninja semantics: ordering only, not freshness).
		struct DirEntry
		{
			StringKey key;        // lowercased-dir hash (mtime-cache key scheme)
			u32       pathOffset; // into dirPathBuf (trailing sep included)
			u32       pathLen;
		};
		Vector<tchar>    dirPathBuf;
		Vector<DirEntry> uniqueDirs;
		UnorderedMap<StringKey, u32> dirKeyToIndex; // dedup
		dirPathBuf.reserve(1 * 1024 * 1024);
		uniqueDirs.reserve(16384);

		// Per-edge precomputed StringKeys (one expansion pass each). Phase 3
		// queries mtime cache + logIndex directly off these.
		struct EdgePaths
		{
			u32 outStart   = 0, outEnd   = 0;
			u32 implOStart = 0, implOEnd = 0;
			u32 inStart    = 0, inEnd    = 0;
			u32 implIStart = 0, implIEnd = 0;
			StringKey primaryOutKey = {}; // outputs[0] hash (log lookup)
		};
		Vector<EdgePaths> edgePaths(edgeCount);
		Vector<StringKey> pathKeys;
		pathKeys.reserve(1 << 20);

		Vector<char>       expandBuf;
		StringBuffer<512>  tcharBuf;
		StringBuffer<512>  absBuf;
		expandBuf.reserve(1024);

		// Register directory (deduped). Stored path always ends with a separator.
		auto registerDir = [&](StringView absFilePath)
		{
			StringBuffer<512> dirPath;
			const tchar*      lastSlash;
			StringKey         dirKey;
			if (!GetDirKey(dirKey, dirPath, lastSlash, absFilePath))
				return;
			auto ins = dirKeyToIndex.emplace(dirKey, u32(uniqueDirs.size()));
			if (ins.second)
			{
				dirPath.EnsureEndsWithSlash();
				u32 dirOff = u32(dirPathBuf.size());
				dirPathBuf.resize(dirOff + dirPath.count);
				memcpy(dirPathBuf.data() + dirOff, dirPath.data, dirPath.count * sizeof(tchar));
				DirEntry de;
				de.key        = dirKey;
				de.pathOffset = dirOff;
				de.pathLen    = dirPath.count;
				uniqueDirs.push_back(de);
			}
		};

		// Expand $vars, canonicalize to abs, hash → StringKey, push to pathKeys,
		// register parent dir.
		auto pushPathKey = [&](u32 stringId, u32 scopeId) -> StringKey
		{
			StringKey empty{};
			CharStringView rel = stringPool.GetString(stringId);
			if (rel.length == 0)
			{
				pathKeys.push_back(empty);
				return empty;
			}

			expandBuf.clear();
			parser.ExpandVariables(rel.data, rel.length, expandBuf, nullptr, scopeId);
			if (expandBuf.empty())
			{
				pathKeys.push_back(empty);
				return empty;
			}

			u32 n = u32(expandBuf.size());
			if (n + 1 > tcharBuf.capacity) n = tcharBuf.capacity - 1;
			tcharBuf.count = n;
			for (u32 i = 0; i < n; ++i)
			{
				char c = expandBuf[i];
				tcharBuf.data[i] = (tchar)(u8)((c == '\\' || c == '/') ? PathSeparator : c);
			}
			tcharBuf.data[tcharBuf.count] = 0;

			FixPath(absBuf.Clear(), StringView(tcharBuf.data, tcharBuf.count), workingDir);
			StringKey fileKey = CaseInsensitiveFs
				? ToStringKeyLower(StringView(absBuf.data, absBuf.count))
				: ToStringKeyNoCheck(absBuf.data, absBuf.count);

			registerDir(StringView(absBuf.data, absBuf.count));

			pathKeys.push_back(fileKey);
			return fileKey;
		};

		for (u32 i = 0; i < edgeCount; ++i)
		{
			if (!edgesToBuildFlags[i]) continue;
			const auto& e = edges[i];
			if (e.ruleName == phonyId) continue;

			EdgePaths& ep = edgePaths[i];

			ep.outStart = u32(pathKeys.size());
			for (u32 j = 0; j < e.outputs.size(); ++j)
			{
				StringKey k = pushPathKey(e.outputs[j], e.scopeId);
				if (j == 0)
					ep.primaryOutKey = k;
			}
			ep.outEnd = u32(pathKeys.size());

			ep.implOStart = u32(pathKeys.size());
			for (u32 id : e.implicitOutputs) pushPathKey(id, e.scopeId);
			ep.implOEnd = u32(pathKeys.size());

			ep.inStart = u32(pathKeys.size());
			for (u32 id : e.inputs) pushPathKey(id, e.scopeId);
			ep.inEnd = u32(pathKeys.size());

			ep.implIStart = u32(pathKeys.size());
			for (u32 id : e.implicitDeps) pushPathKey(id, e.scopeId);
			ep.implIEnd = u32(pathKeys.size());

			// Register parent dirs of discovered-deps too.
			if (ep.primaryOutKey.a != 0 || ep.primaryOutKey.b != 0)
			{
				auto it = logIndex.depsByOutputKey.find(ep.primaryOutKey);
				if (it != logIndex.depsByOutputKey.end())
				{
					const auto& de = it->second;
					for (u32 k = 0; k < de.depCount; ++k)
					{
						u32 off = logIndex.flatDepPathOffset[de.depOffset + k];
						u32 len = logIndex.flatDepPathLen[de.depOffset + k];
						registerDir(StringView(logIndex.flatDepPathBuf.data() + off, len));
					}
				}
			}
		}

		u64 tAfterCollect = GetTime();

		// Phase 2: parallel directory stat. Each unique parent dir walked once
		// via TraverseDir; mtimes go into a shared map keyed by lowercased-full-path.
		UnorderedMap<StringKey, u64> mtimeByFileKey;
		mtimeByFileKey.reserve(256 * 1024);
		Futex mtimeLock;

		if (!uniqueDirs.empty())
		{
			IndexContainer container(uniqueDirs.size());
			workManager.ParallelFor(0, container,
				// `local` captured per-worker; reused across that worker's dirs.
				[&, local = Vector<std::pair<StringKey, u64>>()](const WorkContext&, auto& it) mutable
				{
					if (local.capacity() == 0) local.reserve(128);
					local.clear();

					const DirEntry& de = uniqueDirs[it.index];
					StringView dirPath(dirPathBuf.data() + de.pathOffset, de.pathLen);

					TraverseDir(logger, dirPath, [&](const DirectoryEntry& entry)
						{
							if (IsDirectory(entry.attributes))
								return;

							StringBuffer<512> full;
							full.Append(dirPath);
							full.Append(entry.name, entry.nameLen);

							StringKey fileKey = CaseInsensitiveFs
								? ToStringKeyLower(StringView(full.data, full.count))
								: ToStringKeyNoCheck(full.data, full.count);
							local.emplace_back(fileKey, entry.lastWritten);
						});

					if (!local.empty())
					{
						SCOPED_FUTEX(mtimeLock, lock);
						for (auto& kv : local)
							mtimeByFileKey.emplace(kv.first, kv.second);
					}
				}, TCV("UpToDateStatDir"));
		}

		u64 tAfterStat = GetTime();

		// Phase 3: per-edge freshness check (Pass A). Independent O(1) lookups on mtimeByFileKey.
		Vector<u8> clean(edgeCount, 0);

		auto getMtime = [&](StringKey key) -> u64
		{
			auto it = mtimeByFileKey.find(key);
			return it == mtimeByFileKey.end() ? 0 : it->second;
		};

		// DBG: dirty-reason counters (indexed by reason id)
		std::atomic<u32> dbgReason[8]{};
		auto dbgFirstOutName = [&](u32 i) -> CharStringView {
			const auto& e = edges[i];
			return e.outputs.empty() ? CharStringView{"<noout>", 7} : stringPool.GetString(e.outputs[0]);
		};

		IndexContainer edgeContainer(edgeCount);
		workManager.ParallelFor(0, edgeContainer,
			// fullCmd / temp / fullT captured per-worker.
			[&,
				fullCmd = Vector<char>(),
				temp    = Vector<char>(),
				fullT   = TString()
			](const WorkContext&, auto& it) mutable
			{
				u32 i = it.index;
				if (!edgesToBuildFlags[i]) return;
				const auto& edge = edges[i];
				if (edge.ruleName == phonyId) { clean[i] = 1; return; }

				if (edge.outputs.empty() && edge.implicitOutputs.empty())
				{
					++dbgReason[0];
					return;
				}

				const EdgePaths& ep = edgePaths[i];

				// `restat` rules opt into ninja semantics where commands may
				// not rewrite all outputs every run (e.g. lld-link's import
				// .lib). For these, use the log's recorded mtime as reference
				// rather than the (possibly-stale) on-disk output mtime.
				bool restatRule = false;
				{
					auto rmIt = parser.GetRuleMap().find(edge.ruleName);
					if (rmIt != parser.GetRuleMap().end())
						restatRule = parser.GetRules()[rmIt->second].restat;
				}
				// AOSP phony_output=true: virtual outputs not on disk.
				// Same treatment as restat — log mtime is reference, skip stat-existence.
				const bool useLogMtime = restatRule || edge.phonyOutput;

				// 1. Outputs exist on disk (skipped for phony_output).
				bool anyMissing = false;
				u64 minOutStat = ~u64(0);
				if (!edge.phonyOutput)
				{
					for (u32 k = ep.outStart; k < ep.outEnd; ++k)
					{
						u64 t = getMtime(pathKeys[k]);
						if (!t) { anyMissing = true; break; }
						if (t < minOutStat) minOutStat = t;
					}
					if (!anyMissing)
						for (u32 k = ep.implOStart; k < ep.implOEnd; ++k)
						{
							u64 t = getMtime(pathKeys[k]);
							if (!t) { anyMissing = true; break; }
							if (t < minOutStat) minOutStat = t;
						}
					if (anyMissing || minOutStat == ~u64(0))
					{
						++dbgReason[1];
						if (dbgReason[1] <= 3)
						{
							auto n = dbgFirstOutName(i);
							logger.Detail(TC("UTD[missing-out]: %.*hs"), n.length, n.data);
						}
						return;
					}
				}

				// 2. Command hash check against .ninja_log
				auto logIt = logIndex.logByOutputKey.find(ep.primaryOutKey);
				if (logIt == logIndex.logByOutputKey.end())
				{
					++dbgReason[2];
					if (dbgReason[2] <= 3)
					{
						auto n = dbgFirstOutName(i);
						logger.Detail(TC("UTD[no-log-entry]: %.*hs"), n.length, n.data);
					}
					return;
				}

				// Reference mtime: log-recorded for restat/phony, min(outputs) otherwise.
				u64 minOut = useLogMtime ? logIt->second.outMtime : minOutStat;

				// Recompute the cmd hash exactly as dispatch did so it agrees with
				// what the writer recorded.
				fullCmd.clear();
				parser.ExpandCommand(edge, fullCmd, temp);
				if (fullCmd.empty())
					return;
				if (noShowIncludes)
					StripShowIncludesInPlace(fullCmd);

				fullT.resize(fullCmd.size());
				for (u32 k = 0, kn = u32(fullCmd.size()); k < kn; ++k)
					fullT[k] = (tchar)(u8)fullCmd[k];

				StringKey curCmdHash = ToStringKeyNoCheck(fullT.c_str(), u32(fullT.length()));
				if (curCmdHash.a != logIt->second.cmdHash)
				{
					++dbgReason[3];
					if (dbgReason[3] <= 3)
					{
						auto n = dbgFirstOutName(i);
						logger.Detail(TC("UTD[cmd-hash]: %.*hs cur=%llx log=%llx"), n.length, n.data,
							(unsigned long long)curCmdHash.a, (unsigned long long)logIt->second.cmdHash);
					}
					return;
				}

				// 3. Declared input freshness
				bool inputNewer = false;
				u32 whichInput = ~0u;
				u32 whichInputIdKind = 0; // 0=input, 1=implicitDep
				u32 whichInputEdgeIdx = 0;
				u64 whichInputMtime = 0;
				for (u32 k = ep.inStart; k < ep.inEnd; ++k)
				{
					u64 t = getMtime(pathKeys[k]);
					if (t > minOut) { inputNewer = true; whichInput = k - ep.inStart; whichInputIdKind = 0; whichInputEdgeIdx = k; whichInputMtime = t; break; }
				}
				if (!inputNewer)
					for (u32 k = ep.implIStart; k < ep.implIEnd; ++k)
					{
						u64 t = getMtime(pathKeys[k]);
						if (t > minOut) { inputNewer = true; whichInput = k - ep.implIStart; whichInputIdKind = 1; whichInputEdgeIdx = k; whichInputMtime = t; break; }
					}
				if (inputNewer)
				{
					++dbgReason[4];
					if (dbgReason[4] <= 3)
					{
						auto n = dbgFirstOutName(i);
						// Find the input's stringId
						u32 inputStringId = 0;
						if (whichInputIdKind == 0 && whichInput < edge.inputs.size())
							inputStringId = edge.inputs[whichInput];
						else if (whichInputIdKind == 1 && whichInput < edge.implicitDeps.size())
							inputStringId = edge.implicitDeps[whichInput];
						auto in = stringPool.GetString(inputStringId);
						logger.Detail(TC("UTD[input-newer]: %.*hs  in=\"%.*hs\"(%s) inT=%llu outT=%llu delta=%lld"),
							n.length, n.data, in.length, in.data,
							whichInputIdKind == 0 ? TC("declared") : TC("implicit"),
							(unsigned long long)whichInputMtime, (unsigned long long)minOut,
							(long long)(whichInputMtime - minOut));
						(void)whichInputEdgeIdx;
					}
					return;
				}

				// 4. Discovered dep (.ninja_deps) freshness
				auto depIt = logIndex.depsByOutputKey.find(ep.primaryOutKey);
				if (depIt != logIndex.depsByOutputKey.end())
				{
					const auto& de = depIt->second;
					u32 whichDep = ~0u;
					u64 whichDepT = 0;
					for (u32 k = 0; k < de.depCount; ++k)
					{
						u64 t = getMtime(logIndex.flatDepKeys[de.depOffset + k]);
						if (!t) continue;
						if (t > minOut) { inputNewer = true; whichDep = k; whichDepT = t; break; }
					}
					if (inputNewer)
					{
						++dbgReason[5];
						if (dbgReason[5] <= 3 && whichDep != ~0u)
						{
							auto n = dbgFirstOutName(i);
							u32 off = logIndex.flatDepPathOffset[de.depOffset + whichDep];
							u32 len = logIndex.flatDepPathLen[de.depOffset + whichDep];
							logger.Detail(TC("UTD[dep-newer]: %.*hs  dep=\"%.*s\" depT=%llu outT=%llu delta=%lld"),
								n.length, n.data,
								len, logIndex.flatDepPathBuf.data() + off,
								(unsigned long long)whichDepT, (unsigned long long)minOut,
								(long long)(whichDepT - minOut));
						}
						return;
					}
				}

				clean[i] = 1;
			}, TCV("UpToDateEdge"));

		u64 tAfterCheck = GetTime();

		// Phase 4: dirty propagation. Reverse topoOrder visits producers
		// before consumers; any dirty producer dirties its consumer.
		Vector<char> p4Expand;
		for (auto it = topoOrder.rbegin(); it != topoOrder.rend(); ++it)
		{
			u32 i = *it;
			if (!edgesToBuildFlags[i] || !clean[i]) continue;

			const auto& edge = edges[i];
			// Literal lookup first; on miss expand `$vars` against consumer's scope
			// (Soong emits `| ${stdlibZip}` etc.). See BFS-pass resolveDepId.
			auto hasDirtyProducer = [&](u32 inputId) -> bool
			{
				u32 p = ~0u;
				if (inputId < outputToEdge.size() && outputToEdge[inputId] != ~0u)
					p = outputToEdge[inputId];
				else
				{
					CharStringView s = stringPool.GetString(inputId);
					if (s.length == 0 || memchr(s.data, '$', s.length) == nullptr)
						return false;
					p4Expand.clear();
					parser.ExpandVariables(s.data, s.length, p4Expand, &edge.variables, edge.scopeId);
					u32 newLen = u32(p4Expand.size());
					if (newLen == 0)
						return false;
					u32 newId = stringPool.Intern(p4Expand.data(), newLen);
					if (newId < outputToEdge.size() && outputToEdge[newId] != ~0u)
						p = outputToEdge[newId];
					else
						return false;
				}
				if (p >= edgeCount || !edgesToBuildFlags[p]) return false;
				return !clean[p];
			};
			for (u32 id : edge.inputs)
				if (hasDirtyProducer(id)) { clean[i] = 0; break; }
			if (clean[i])
				for (u32 id : edge.implicitDeps)
					if (hasDirtyProducer(id)) { clean[i] = 0; break; }
		}

		// Phase 5: suppress build for confirmed-clean edges.
		u32 suppressed = 0;
		for (u32 i = 0; i < edgeCount; ++i)
		{
			if (edgesToBuildFlags[i] && clean[i])
			{
				edgesToBuildFlags[i] = 0;
				++suppressed;
			}
		}

		u32 dirty = 0;
		for (u32 i = 0; i < edgeCount; ++i)
			if (edgesToBuildFlags[i] && edges[i].ruleName != phonyId)
				++dirty;

		logger.Info(TC("Up-to-date: %u dirty, %u skipped  [%u dirs, load %s, collect %s, stat %s, check %s, total %s]"),
			dirty, suppressed, u32(uniqueDirs.size()),
			TimeToText(tAfterLoad   - t0).str,
			TimeToText(tAfterCollect - tAfterLoad).str,
			TimeToText(tAfterStat    - tAfterCollect).str,
			TimeToText(tAfterCheck   - tAfterStat).str,
			TimeToText(GetTime()     - t0).str);

		return suppressed;
	}

	bool ParseNinjaFile(LoggerWithWriter& logger, NinjaParser& parser,
	                    const tchar* ninjaFile, const Vector<TString>& specifiedTargets,
	                    NinjaParseState& outState, const tchar* buildRoot)
	{
		const auto& stringPool = parser.GetStringPool();

		outState.phonyId = const_cast<StringPool&>(stringPool).Intern("phony", 5);

		// Pre-intern target names so the parser's edge-callback finds them.
		Vector<char> targetUtf8;
		if (!specifiedTargets.empty())
		{
			logger.Detail(TC("Building %u specified targets"), u32(specifiedTargets.size()));
			for (const auto& target : specifiedTargets)
			{
				targetUtf8.clear();
				for (u32 i = 0; i < target.length(); ++i)
					targetUtf8.push_back((char)(u8)target[i]);
				u32 targetId = const_cast<StringPool&>(stringPool).Intern(targetUtf8.data(), u32(targetUtf8.size()));
				if (targetId >= outState.wantedOutputs.size())
					outState.wantedOutputs.resize(targetId + 1, 0);
				outState.wantedOutputs[targetId] = 1;
			}
		}
		// else: no targets — resolved after parsing via file's `default` statement.

		outState.outputToEdge.reserve(131072);
		outState.edgesToBuildFlags.reserve(65536);
		outState.edgeToProcessIndex.reserve(65536);

		parser.SetEdgeParsedCallback([](void* userData, u32 edgeIndex, const NinjaBuildEdge& edge)
		{
			auto& state = *static_cast<NinjaParseState*>(userData);
			if (edgeIndex >= state.edgesToBuildFlags.size())
			{
				state.edgesToBuildFlags.resize(edgeIndex + 1, 0);
				state.edgeToProcessIndex.resize(edgeIndex + 1, ~0u);
			}
			auto registerOutput = [&](u32 outputId)
			{
				if (outputId >= state.outputToEdge.size())
				{
					u32 newSize = outputId + 1;
					state.outputToEdge.resize(newSize, ~0u);
					if (state.wantedOutputs.size() < newSize)
						state.wantedOutputs.resize(newSize, 0);
				}
				state.outputToEdge[outputId] = edgeIndex;
			};
			for (u32 outputId : edge.outputs)         registerOutput(outputId);
			for (u32 outputId : edge.implicitOutputs) registerOutput(outputId);
		}, &outState);

		logger.Detail(TC(""));
		logger.Detail(TC("Parsing ninja file: %s"), ninjaFile);

		u64 parseStartTime = GetTime();
		if (!parser.Parse(logger, ninjaFile, buildRoot))
		{
			logger.Error(TC("Failed to parse ninja file"));
			return false;
		}
		u64 parseTime = GetTime() - parseStartTime;
		logger.Detail(TC("Parse completed in %s"), TimeToText(parseTime).str);
		logger.Detail(TC("  Lines: %u"), parser.GetLineCount());
		logger.Detail(TC("  Edges: %u"), u32(parser.GetEdges().size()));

		// Warn about generator edges (real ninja re-runs them; we skip).
		// Two-pass scan: rules first (small list), then edges only if any generator rule exists.
		{
			const auto& rules = parser.GetRules();
			u32 generatorRuleIds[16];
			u32 generatorRuleCount = 0;
			for (const auto& rule : rules)
				if (rule.generator && generatorRuleCount < 16)
					generatorRuleIds[generatorRuleCount++] = rule.name;

			if (generatorRuleCount > 0)
			{
				for (const auto& edge : parser.GetEdges())
				{
					bool found = false;
					for (u32 i = 0; i < generatorRuleCount; ++i)
						if (edge.ruleName == generatorRuleIds[i]) { found = true; break; }
					if (!found)
						continue;

					const NinjaRule* matchedRule = nullptr;
					for (const auto& rule : rules)
						if (rule.name == edge.ruleName) { matchedRule = &rule; break; }
					if (!matchedRule) continue;

					StringBuffer<256> ruleName;
					ruleName.Append(stringPool.GetCStr(matchedRule->name));
					StringBuffer<256> command;
					command.Append(stringPool.GetCStr(matchedRule->command));
					StringBuffer<512> output;
					if (!edge.outputs.empty())
						output.Append(stringPool.GetCStr(edge.outputs[0]));
					logger.Warning(TC("Skipping generator rule '%s' (output: '%s', command: '%s'). Ninja would re-run this to regenerate the build file."),
						ruleName.data, output.data, command.data);
				}
			}
		}

		// Real ninja uses `default` when no targets given; fall back to "build everything" only if absent.
		if (specifiedTargets.empty())
		{
			const auto& defaults = parser.GetDefaults();
			if (!defaults.empty())
			{
				logger.Detail(TC("No targets specified, using %u default target(s) from build.ninja"), u32(defaults.size()));
				for (u32 defaultId : defaults)
				{
					if (defaultId >= outState.wantedOutputs.size())
						outState.wantedOutputs.resize(defaultId + 1, 0);
					outState.wantedOutputs[defaultId] = 1;
				}
			}
			else
			{
				logger.Detail(TC("No targets specified and no default statement, building everything"));
				outState.wantAll = true;
			}
		}

		return true;
	}

	void ResolveNinjaDir(StringBufferBase& out, const tchar* ninjaFile, StringView workingDir)
	{
		out.Clear();
		// .ninja_log/.ninja_deps live next to the ninja file (AOSP: `out/.ninja_deps`),
		// not in cwd. Derive from ninjaFile; fall back to workingDir.
		if (ninjaFile && *ninjaFile)
		{
			u32 nfLen = u32(TStrlen(ninjaFile));
			u32 lastSep = nfLen;
			for (u32 i = nfLen; i > 0; --i)
				if (ninjaFile[i - 1] == '/' || ninjaFile[i - 1] == '\\')
				{ lastSep = i - 1; break; }
			if (lastSep < nfLen)
				out.Append(ninjaFile, lastSep);
			// Relative ninjaFile: resolve under workingDir so parent dir is absolute.
			if (out.count == 0 || (out.count > 0 &&
				out.data[0] != PathSeparator
				#if PLATFORM_WINDOWS
				&& !(out.count >= 2 && out.data[1] == ':')
				#endif
			))
			{
				StringBuffer<512> abs;
				abs.Append(workingDir).EnsureEndsWithSlash().Append(out.data, out.count);
				out.Clear().Append(abs.data, abs.count);
			}
			out.EnsureEndsWithSlash();
		}
		if (out.count == 0)
			out.Append(workingDir).EnsureEndsWithSlash();
	}

	bool EnqueueCommands(const EnqueueCommandsContext& context)
	{
		LoggerWithWriter& logger = context.logger;
		NinjaParser& parser = context.parser;
		NinjaParseState& state = context.state;

		auto& outputToEdge       = state.outputToEdge;
		auto& edgesToBuildFlags  = state.edgesToBuildFlags;
		auto& edgeToProcessIndex = state.edgeToProcessIndex;
		auto& wantedOutputs      = state.wantedOutputs;
		const u32 phonyId        = state.phonyId;

		const auto& stringPool = parser.GetStringPool();
		const auto& edges      = parser.GetEdges();

		// Post-parse: ExpandCommand/ExpandVariables thread-safe (Intern is Futex-protected).

		bool wantAll = state.wantAll;

		u64 totalStartTime = GetTime();
		u64 queueStartTime = GetTime();

		u64 tBfs = 0, tClean = 0, tTopo = 0, tUpToDate = 0;
		u64 tPass1 = 0, tPass2Dispatch = 0, tYaml = 0;
		u64 tPhase = GetTime();

		if (wantAll)
		{
			for (u32 i = 0; i < u32(edges.size()); ++i)
				edgesToBuildFlags[i] = 1;
		}
		else
		{
			// BFS backward from target outputs.
			Vector<u32> edgesToVisit;
			edgesToVisit.reserve(edges.size() / 4);

			for (u32 outputId = 0; outputId < wantedOutputs.size(); ++outputId)
			{
				if (!wantedOutputs[outputId] || outputId >= outputToEdge.size())
					continue;
				u32 edgeIdx = outputToEdge[outputId];
				if (edgeIdx != ~0u && edgeIdx < edgesToBuildFlags.size() && !edgesToBuildFlags[edgeIdx])
				{
					edgesToBuildFlags[edgeIdx] = 1;
					edgesToVisit.push_back(edgeIdx);
				}
			}

			logger.Detail(TC("Finding needed edges (BFS from %u targets)..."), u32(edgesToVisit.size()));

			for (u32 visitIdx = 0; visitIdx < edgesToVisit.size(); ++visitIdx)
			{
				u32 edgeIdx = edgesToVisit[visitIdx];
				const auto& edge = edges[edgeIdx];
				// Literal lookup first; on miss, expand `$vars` against edge scope.
				// Literal-first matters: parsed-token `$` may come from `$$` escapes
				// (Soong emits Java inner-class names like `$$Gson$$X.java`); naively
				// expanding those mangles real paths.
				Vector<char> bfsExpand;
				auto resolveDepId = [&](u32 depId) -> u32
				{
					if (depId < outputToEdge.size() && outputToEdge[depId] != ~0u)
						return outputToEdge[depId];
					CharStringView s = stringPool.GetString(depId);
					if (s.length == 0 || memchr(s.data, '$', s.length) == nullptr)
						return ~0u;
					bfsExpand.clear();
					const_cast<NinjaParser&>(parser).ExpandVariables(s.data, s.length, bfsExpand, &edge.variables, edge.scopeId);
					u32 newLen = u32(bfsExpand.size());
					if (newLen == 0)
						return ~0u;
					u32 newId = const_cast<StringPool&>(stringPool).Intern(bfsExpand.data(), newLen);
					if (newId < outputToEdge.size() && outputToEdge[newId] != ~0u)
						return outputToEdge[newId];
					return ~0u;
				};
				auto markProducer = [&](u32 inputId)
				{
					u32 producerIdx = resolveDepId(inputId);
					if (producerIdx != ~0u && producerIdx < edgesToBuildFlags.size() && !edgesToBuildFlags[producerIdx])
					{
						edgesToBuildFlags[producerIdx] = 1;
						edgesToVisit.push_back(producerIdx);
					}
				};
				for (u32 id : edge.inputs)       markProducer(id);
				for (u32 id : edge.implicitDeps)  markProducer(id);
				for (u32 id : edge.orderOnlyDeps) markProducer(id);
			}

			logger.Detail(TC("Marked %u needed edges"), u32(edgesToVisit.size()));
		}

		tBfs = GetTime() - tPhase;
		tPhase = GetTime();

		WorkManager& workManager = context.scheduler.GetSession().GetWorkManager();

		if (context.depsLog && !context.cleanOnly)
		{
			u64 regStart = GetTime();
			u64 registered = 0;

			Vector<char>    expandScratch;
			StringBuffer<>  combined;
			expandScratch.reserve(512);

			auto registerOutputPath = [&](u32 outputId, u32 scopeId)
			{
				CharStringView rel = stringPool.GetString(outputId);
				if (rel.length == 0)
					return;

				expandScratch.clear();
				parser.ExpandVariables(rel.data, rel.length, expandScratch, nullptr, scopeId);
				if (expandScratch.empty())
					return;

				combined.Clear();
				combined.Append(context.workingDir).EnsureEndsWithSlash();
				for (u32 j = 0, jn = u32(expandScratch.size()); j < jn; ++j)
				{
					char c = expandScratch[j];
					combined.Append((tchar)(u8)(c == '/' ? PathSeparator : c));
				}

#if PLATFORM_WINDOWS
				tchar fullPath[4096];
				if (!GetFullPathNameW(combined.data, sizeof_array(fullPath), fullPath, nullptr))
					return;
				context.depsLog->RegisterStaticOutput(StringView(fullPath, u32(TStrlen(fullPath))));
#else
				StringBuffer<4096> fullPath;
				if (!FixPath(fullPath, StringView(combined.data, combined.count), context.workingDir))
					return;
				context.depsLog->RegisterStaticOutput(StringView(fullPath.data, fullPath.count));
#endif
				++registered;
			};

			for (u32 i = 0, n = u32(edges.size()); i < n; ++i)
			{
				if (i >= edgesToBuildFlags.size() || !edgesToBuildFlags[i])
					continue;
				if (edges[i].ruleName == phonyId)
					continue; // phony rules produce nothing on disk
				const auto& e = edges[i];
				for (u32 outputId : e.outputs)         registerOutputPath(outputId, e.scopeId);
				for (u32 outputId : e.implicitOutputs) registerOutputPath(outputId, e.scopeId);
			}

			logger.Detail(TC("Registered %llu static outputs into cross-edge set in %s"),
				(unsigned long long)registered, TimeToText(GetTime() - regStart).str);
		}

		if (context.rebuild || context.cleanOnly)
		{
			logger.Detail(TC("Deleting output files"));
			u64 cleanStart = GetTime();

			// edge.outputs may contain unexpanded $vars (e.g. "$builddir/foo.obj"),
			// so ExpandVariables before joining with workingDir.
			auto deleteOutput = [&](u32 outputId, u32 scopeId, Vector<char>& scratch)
			{
				CharStringView rel = stringPool.GetString(outputId);
				if (rel.length == 0)
					return;

				scratch.clear();
				parser.ExpandVariables(rel.data, rel.length, scratch, nullptr, scopeId);
				if (scratch.empty())
					return;

				StringBuffer<> combined;
				combined.Append(context.workingDir).EnsureEndsWithSlash();
				for (u32 j = 0, jn = u32(scratch.size()); j < jn; ++j)
				{
					char c = scratch[j];
					combined.Append((tchar)(u8)(c == '/' ? PathSeparator : c));
				}

				tchar fullPath[4096];
				if (!GetFullPathNameW(combined.data, sizeof_array(fullPath), fullPath, nullptr))
					return;

				u32 attr = GetFileAttributesW(fullPath);
				if (attr == INVALID_FILE_ATTRIBUTES || IsDirectory(attr))
					return;

				DeleteFileW(fullPath);
			};

			Atomic<u32> deletedEdges = 0;

			for (u32 i = 0; i < u32(edges.size()); ++i)
			{
				if (!edgesToBuildFlags[i] || edges[i].ruleName == phonyId)
					continue;
				workManager.AddWork([&, i](const WorkContext&)
					{
						Vector<char> scratch;
						scratch.reserve(512);
						for (u32 id : edges[i].outputs)
							deleteOutput(id, edges[i].scopeId, scratch);
						for (u32 id : edges[i].implicitOutputs)
							deleteOutput(id, edges[i].scopeId, scratch);
						++deletedEdges;
					}, 1, TC("NinjaFlush"));
			}
			workManager.DoWork(1000000);
			workManager.FlushWork();

			logger.Detail(TC("Cleaned outputs for %u edges in %s"), deletedEdges.load(), TimeToText(GetTime() - cleanStart).str);

			tClean = GetTime() - tPhase;
			tPhase = GetTime();

			if (context.cleanOnly)
				return true;
		}

		// Pass 1 priority = critical-path length from edge to sink.
		// weight[e] = 1 + max(weight[consumer]). Phonies pass weight through
		// without contributing. Sum-based and fan-out blends were rejected:
		// chromium's diamond phony graph blows sum-weights to 1e125, and
		// huge-fan-out mojom edges with shallow chains starve the actual
		// deep TS critical path.
		// Computed via Kahn's topo sort; GN doesn't guarantee topo order.
		u32 edgeCount = u32(edges.size());

		// Kahn's traversal: consumer-first. Reversed (producer-first) for
		// dirty propagation in the up-to-date check.
		Vector<u32> topoOrder;
		topoOrder.reserve(edgeCount);

		Vector<double> edgeWeights(edgeCount, 1.0);
		{
			Vector<u32> inDegree(edgeCount, 0);
			// Lazy `$var` expansion on miss; see BFS-pass resolveDepId. Without
			// it `${stdlibZip}`-style edges break topo + dirty propagation.
			Vector<char> kahnExpand;
			auto resolveProducer = [&](const NinjaBuildEdge& consumer, u32 depId) -> u32
			{
				if (depId < outputToEdge.size() && outputToEdge[depId] != ~0u)
					return outputToEdge[depId];
				CharStringView s = stringPool.GetString(depId);
				if (s.length == 0 || memchr(s.data, '$', s.length) == nullptr)
					return ~0u;
				kahnExpand.clear();
				parser.ExpandVariables(s.data, s.length, kahnExpand, &consumer.variables, consumer.scopeId);
				u32 newLen = u32(kahnExpand.size());
				if (newLen == 0)
					return ~0u;
				u32 newId = const_cast<StringPool&>(stringPool).Intern(kahnExpand.data(), newLen);
				if (newId < outputToEdge.size() && outputToEdge[newId] != ~0u)
					return outputToEdge[newId];
				return ~0u;
			};
			for (u32 i = 0; i < edgeCount; ++i)
			{
				if (!edgesToBuildFlags[i]) continue;
				const auto& consumer = edges[i];
				auto addDep = [&](u32 inputId)
				{
					u32 p = resolveProducer(consumer, inputId);
					if (p == ~0u || p >= edgeCount || !edgesToBuildFlags[p]) return;
					inDegree[p]++;
				};
				for (u32 id : consumer.inputs)        addDep(id);
				for (u32 id : consumer.implicitDeps)  addDep(id);
				for (u32 id : consumer.orderOnlyDeps) addDep(id);
			}

			Vector<u32> queue;
			queue.reserve(edgeCount);
			for (u32 i = 0; i < edgeCount; ++i)
				if (edgesToBuildFlags[i] && inDegree[i] == 0)
					queue.push_back(i);

			Vector<double> accumulated(edgeCount, 0.0);
			while (!queue.empty())
			{
				u32 cur = queue.back(); queue.pop_back();
				topoOrder.push_back(cur);
				const auto& consumer = edges[cur];
				bool isPhony = consumer.ruleName == phonyId;
				double w = isPhony ? accumulated[cur] : (1.0 + accumulated[cur]);
				// Bias inside the Kahn loop so max-propagation carries it
				// upstream to every transitive producer.
				if (!isPhony)
				{
					w += GetPriorityWeight(stringPool, consumer);
					edgeWeights[cur] = w;
				}

				auto propagate = [&](u32 inputId)
				{
					u32 p = resolveProducer(consumer, inputId);
					if (p == ~0u || p >= edgeCount || !edgesToBuildFlags[p]) return;
					if (w > accumulated[p]) accumulated[p] = w; // max, not sum
					if (--inDegree[p] == 0)
						queue.push_back(p);
				};
				for (u32 id : consumer.inputs)        propagate(id);
				for (u32 id : consumer.implicitDeps)  propagate(id);
				for (u32 id : consumer.orderOnlyDeps) propagate(id);
			}
		}

		tTopo = GetTime() - tPhase;
		tPhase = GetTime();

		// .ninja_log + .ninja_deps come from the caller (loaded in parallel
		// with parsing). Used for the up-to-date check AND Pass 2's depfile
		// scheduling-dep injection (e.g. clang foo.pb.cc → atoms.pb.h). Stock
		// ninja gets this via dyndep / deps-log; we map abs paths to producer edges.
		if (context.logIndexLoaded)
		{
			logger.Info(TC("NinjaLogIndex: %u log entries, %u deps entries, %u flat dep keys"), u32(context.logIndex.logByOutputKey.size()), u32(context.logIndex.depsByOutputKey.size()), u32(context.logIndex.flatDepKeys.size()));
		}
		else if (!context.rebuild && !context.dryRun && !context.cleanOnly)
		{
			logger.Warning(TC("NinjaLogIndex unavailable; depfile-discovered scheduling dependencies will be unavailable"));
		}

		// Up-to-date check — see SuppressUpToDateEdges. Skipped for -rebuild/-clean/-dry-run.
		if (!context.rebuild && !context.dryRun && !context.cleanOnly && context.checkUpToDate)
		{
			SuppressUpToDateEdges(logger, parser, context.workingDir, workManager, topoOrder, state, context.noShowIncludes, context.logIndex);
		}
		else if (!context.checkUpToDate)
		{
			logger.Info(TC("Up-to-date check skipped (-no-up-to-date)"));
		}

		tUpToDate = GetTime() - tPhase;
		tPhase = GetTime();

		#if 0 // Old path; preserved until new SuppressUpToDateEdges has baked in.
		u32 skippedAsUpToDate = 0;
		if (!context.rebuild && !context.dryRun && !context.cleanOnly)
		{
			u64 dirtyStart = GetTime();

			DirectoryTableHolder dirTable(logger.m_writer);

			// Per-edge clean flag (1 = tentatively up-to-date, 0 = must rebuild)
			Vector<u8> clean(edgeCount, 0);

			// Shared path buffers for the sequential dirty-check loop.
			StringBuffer<512> relBuf, absBuf;
			StringBuffer<512> dirBuf;

			// Returns the last-write time of the file identified by stringId,
			// or 0 if the file does not exist or has no valid mtime.
			// The containing directory is populated into dirTable on first access
			// for each unique directory; subsequent lookups are O(1).
			auto getMtime = [&](u32 stringId) -> u64
			{
				CharStringView rel = stringPool.GetString(stringId);
				relBuf.Clear().Append(rel.data, rel.length);
				FixPath(absBuf.Clear(), StringView(relBuf.data, relBuf.count), context.workingDir);

				const tchar* lastSlash;
				StringKey dirKey;
				if (!GetDirKey(dirKey, dirBuf.Clear(), lastSlash, StringView(absBuf.data, absBuf.count)))
					return 0;

				dirTable.WriteDirectoryEntries(dirKey, StringView(dirBuf.data, dirBuf.count));

				DirTableOffset offset;
				if (dirTable.m_directoryTable.EntryExists(
						StringView(absBuf.data, absBuf.count), false, &offset) != DirectoryTable::Exists_Yes)
					return 0;

				DirectoryTable::EntryInformation info;
				dirTable.m_directoryTable.GetEntryInformation(info, offset);
				return info.lastWrite;
			};

			// Pass A: independent per-edge mtime check.
			for (u32 edgeIdx = 0; edgeIdx < edgeCount; ++edgeIdx)
			{
				if (!edgesToBuildFlags[edgeIdx]) continue;
				const auto& edge = edges[edgeIdx];
				if (edge.ruleName == phonyId) { clean[edgeIdx] = 1; continue; }

				// If no outputs are declared (unusual for non-phony), always rebuild.
				if (edge.outputs.empty() && edge.implicitOutputs.empty()) continue;

				// Find the minimum mtime across all outputs. Set to 0 if any is absent.
				// minOutMtime stays ~u64(0) only until the first output is examined.
				u64 minOutMtime = ~u64(0);
				for (u32 id : edge.outputs)
				{
					u64 t = getMtime(id);
					if (!t) { minOutMtime = 0; break; }
					if (t < minOutMtime) minOutMtime = t;
				}
				if (minOutMtime) // skip if an explicit output was already missing (minOutMtime==0)
					for (u32 id : edge.implicitOutputs)
					{
						u64 t = getMtime(id);
						if (!t) { minOutMtime = 0; break; }
						if (t < minOutMtime) minOutMtime = t;
					}

				// minOutMtime == 0  → at least one output is missing → dirty.
				// minOutMtime == ~u64(0) cannot happen here because we checked hasOutputs above
				// and the loops above would have updated it to a real timestamp.
				if (!minOutMtime) continue; // Missing output → dirty

				// Dirty if any declared input is newer than the oldest output.
				bool inputNewer = false;
				for (u32 id : edge.inputs)
				{
					u64 t = getMtime(id);
					if (t > minOutMtime) { inputNewer = true; break; }
				}
				if (!inputNewer)
					for (u32 id : edge.implicitDeps)
					{
						u64 t = getMtime(id);
						if (t > minOutMtime) { inputNewer = true; break; }
					}

				if (!inputNewer) clean[edgeIdx] = 1;
			}

			// Pass B: propagate dirty status from producers to consumers.
			// topoOrder is consumer-first (Kahn's traversal). Iterating in reverse
			// gives producer-first order, guaranteeing each edge's producers are
			// evaluated before the edge itself.
			for (auto it = topoOrder.rbegin(); it != topoOrder.rend(); ++it)
			{
				u32 edgeIdx = *it;
				if (!edgesToBuildFlags[edgeIdx] || !clean[edgeIdx]) continue;

				const auto& edge = edges[edgeIdx];
				auto hasDirtyProducer = [&](u32 inputId) -> bool
				{
					if (inputId >= outputToEdge.size()) return false;
					u32 p = outputToEdge[inputId];
					if (p == ~0u || p >= edgeCount || !edgesToBuildFlags[p]) return false;
					return !clean[p];
				};

				for (u32 id : edge.inputs)
					if (hasDirtyProducer(id)) { clean[edgeIdx] = 0; break; }
				if (clean[edgeIdx])
					for (u32 id : edge.implicitDeps)
						if (hasDirtyProducer(id)) { clean[edgeIdx] = 0; break; }
			}

			// Suppress build for edges confirmed up-to-date.
			for (u32 i = 0; i < edgeCount; ++i)
			{
				if (edgesToBuildFlags[i] && clean[i])
				{
					edgesToBuildFlags[i] = 0;
					++skippedAsUpToDate;
				}
			}

			u32 dirtyCount = 0;
			for (u32 i = 0; i < edgeCount; ++i)
				if (edgesToBuildFlags[i] && edges[i].ruleName != phonyId)
					++dirtyCount;

			logger.Detail(TC("Up-to-date check: %u dirty, %u skipped (%s)"),
				dirtyCount, skippedAsUpToDate, TimeToText(GetTime() - dirtyStart).str);
		}
		#endif

		// YAML export
		struct YamlEntry
		{
			// Full command line lives in `arguments`; no separate `executable`.
			TString arguments;
			TString workingDir;
			TString description;
			double  weight             = 1.0;
			bool    canExecuteRemotely = true;
			bool    canDetour          = true;
			Vector<u32> dependencies; // processIdx values
			TString primaryOut;       // unique key (joinable with timings dump)
		};
		Vector<YamlEntry> yamlEntries;
		u32 baseProcessIdx = ~0u;

		// Pass 1: reserve a scheduler slot for each needed non-phony edge,
		// in descending critical-path-weight order. sortedEdges is hoisted to
		// function scope so Pass 2 dispatches over it directly.
		Vector<u32> sortedEdges;
		u32         pass1PhonyCount = 0;
		{
			sortedEdges.reserve(u32(edges.size()));
			for (u32 i = 0; i < u32(edges.size()); ++i)
			{
				if (!edgesToBuildFlags[i]) continue;
				const auto& e = edges[i];
				if (e.ruleName == phonyId) { ++pass1PhonyCount; continue; }
				sortedEdges.push_back(i);
			}
			std::sort(sortedEdges.begin(), sortedEdges.end(), [&](u32 a, u32 b) {
				return edgeWeights[a] > edgeWeights[b];
			});
			const bool captureEntries = (context.yamlFile != nullptr) || (context.depsDumpFile != nullptr);
			if (captureEntries && !sortedEdges.empty())
				yamlEntries.resize(sortedEdges.size());
			for (u32 i : sortedEdges)
			{
				u32 processIdx = context.scheduler.EnqueueSuspendedProcess();
				if (captureEntries && baseProcessIdx == ~0u)
					baseProcessIdx = processIdx;
				edgeToProcessIndex[i] = processIdx;
				if (captureEntries)
				{
					u32 slot = processIdx - baseProcessIdx;
					if (slot < yamlEntries.size())
						yamlEntries[slot].weight = edgeWeights[i];
				}
			}
			logger.Detail(TC("Reserved %u process slots"), u32(sortedEdges.size()));
		}

		tPass1 = GetTime() - tPhase;
		tPhase = GetTime();

		u32 verboseTotal = u32(sortedEdges.size());

		Futex verboseLock;
		std::atomic<u32> verboseCounter{0};

		// Not static: a second parser would have a different StringPool.
		u32 inIdLocal         = const_cast<StringPool&>(stringPool).Intern("in", 2);
		u32 in_newlineIdLocal = const_cast<StringPool&>(stringPool).Intern("in_newline", 10);
		u32 outIdLocal        = const_cast<StringPool&>(stringPool).Intern("out", 3);

		// pathKeyToEdge: abs-path key → edgeIdx for outputs + implicit outputs.
		// Pass 2 uses this to resolve depfile-discovered abs paths back to producer
		// edges, so we can inject scheduling deps for headers learned from the prior
		// build (without it, clang -c foo.pb.cc races with protoc's atoms.pb.h
		// rewrite → SIGBUS BUS_ADRERR on mmap).
		// Only built when we have a deps log.
		// Empty in current build — Pass 2's depfile-discovered scheduling-dep
		// injection is #if 0'd (see comment there). Declarations stay because
		// Pass2Context captures them by reference; population is skipped to
		// avoid the ~5M-edge parallel pass on AOSP for nothing.
		UnorderedMap<StringKey, u32> pathKeyToEdge;
		// Per-edge primary-output key for logIndex.depsByOutputKey lookup in Pass 2.
		Vector<StringKey> primaryOutKeyByEdge;
#if 0
		if (logIndexLoaded && !logIndex.depsByOutputKey.empty())
		{
			u64 mapStart = GetTime();
			// AOSP combined.ninja: several million output keys; sized to match
			// StringPool's 8M reserve so the map never rehashes mid-build.
			pathKeyToEdge.reserve(8 * 1024 * 1024);
			primaryOutKeyByEdge.resize(u32(edges.size()));

			// Per-worker batched flush (1024 pairs per Futex acquire).
			Futex mapLock;
			struct KvFlush
			{
				Futex& lock;
				UnorderedMap<StringKey, u32>& dst;
				Vector<std::pair<StringKey, u32>> kvs;

				KvFlush(Futex& l, UnorderedMap<StringKey, u32>& d) : lock(l), dst(d) { kvs.reserve(2048); }
				// Copy ctor: empty kvs per worker (AddWork copies before any work runs).
				KvFlush(const KvFlush& o) : lock(o.lock), dst(o.dst), kvs() { kvs.reserve(2048); }
				~KvFlush() { flushNow(); }

				void push(StringKey k, u32 v)
				{
					kvs.emplace_back(k, v);
					if (kvs.size() >= 2048) flushNow();
				}
				void flushNow()
				{
					if (kvs.empty()) return;
					SCOPED_FUTEX(lock, l);
					for (auto& kv : kvs) dst.emplace(kv.first, kv.second);
					kvs.clear();
				}
			};

			IndexContainer container(u32(edges.size()));
			workManager.ParallelFor<128>(workManager.GetWorkerCount(), container,
				// Captures per-worker; kvFlush dtor flushes residual on lambda destroy.
				[&,
					expandBuf = Vector<char>(),
					kvFlush   = KvFlush{mapLock, pathKeyToEdge}
				](const WorkContext&, auto& it) mutable
				{
					if (expandBuf.capacity() == 0) expandBuf.reserve(1024);

					u32 i = it.index;
					if (!edgesToBuildFlags[i]) return;
					const auto& e = edges[i];

					StringBuffer<512> tcharBuf;
					StringBuffer<512> absBuf;

					auto absKeyFromStringId = [&](u32 stringId, u32 scopeId, StringKey& outKey) -> bool
					{
						CharStringView rel = stringPool.GetString(stringId);
						if (rel.length == 0) return false;
						expandBuf.clear();
						parser.ExpandVariables(rel.data, rel.length, expandBuf, nullptr, scopeId);
						if (expandBuf.empty()) return false;
						u32 n = u32(expandBuf.size());
						if (n + 1 > tcharBuf.capacity) n = tcharBuf.capacity - 1;
						tcharBuf.count = n;
						for (u32 j = 0; j < n; ++j)
						{
							char c = expandBuf[j];
							tcharBuf.data[j] = (tchar)(u8)((c == '\\' || c == '/') ? PathSeparator : c);
						}
						tcharBuf.data[tcharBuf.count] = 0;
						FixPath(absBuf.Clear(), StringView(tcharBuf.data, tcharBuf.count), workingDir);
						outKey = CaseInsensitiveFs
							? ToStringKeyLower(StringView(absBuf.data, absBuf.count))
							: ToStringKeyNoCheck(absBuf.data, absBuf.count);
						return true;
					};

					// Phonies must appear here too — addDependencyByEdge in Pass 2
					// recurses into phony.inputs only after the phony hits this map.
					bool first = true;
					for (u32 outId : e.outputs)
					{
						StringKey k{};
						if (absKeyFromStringId(outId, e.scopeId, k))
						{
							kvFlush.push(k, i);
							if (first) { primaryOutKeyByEdge[i] = k; first = false; }
						}
					}
					for (u32 outId : e.implicitOutputs)
					{
						StringKey k{};
						if (absKeyFromStringId(outId, e.scopeId, k))
							kvFlush.push(k, i);
					}
				},
				AsView(TC("PathKeyMap"))
			);

			logger.Info(TC("Depfile dep map: %u path keys, %u edges with primary key  [%s]"),
				u32(pathKeyToEdge.size()),
				u32(primaryOutKeyByEdge.size()),
				TimeToText(GetTime() - mapStart).str);
		}
#endif

		// Pass 2: dispatch one work item per needed non-phony edge. All per-edge work
		// runs inside the work item on WorkManager threads. Intern is Futex-protected;
		// ExpandVariables is read-only.
		std::atomic<u32> processedCount{0};
		std::atomic<u32> skippedCount{0};

		DirectoryCache dirCache;

		// Pass 2 captures bundled here; lambda reads via `pass2.<field>`.
		struct Pass2Context
		{
			u32          verboseTotal;
			u32          phonyId;
			u32          inId;
			u32          in_newlineId;
			u32          outId;
			u32          baseProcessIdx;

			// Read-only post-parse:
			const Vector<NinjaBuildEdge>& edges;
			const Vector<u32>&            outputToEdge;
			const Vector<u8>&             edgesToBuildFlags;
			const Vector<u32>&            edgeToProcessIndex;

			// All empty on first build (no .ninja_deps); dep walk falls through.
			const UnorderedMap<StringKey, u32>& pathKeyToEdge;
			const Vector<StringKey>&            primaryOutKeyByEdge;

			const StringPool& stringPool;
			DirectoryCache&   dirCache;
			NinjaParseState&  state;

			std::atomic<u32>& processedCount;
			std::atomic<u32>& skippedCount;

			Futex&            verboseLock;
			std::atomic<u32>& verboseCounter;

			Vector<YamlEntry>& yamlEntries;
		};

		Pass2Context pass2 {
			verboseTotal,
			phonyId, inIdLocal, in_newlineIdLocal, outIdLocal,
			baseProcessIdx,
			edges, outputToEdge, edgesToBuildFlags, edgeToProcessIndex,
			pathKeyToEdge, primaryOutKeyByEdge,
			stringPool, dirCache, state,
			processedCount, skippedCount,
			verboseLock, verboseCounter,
			yamlEntries,
		};

		// Pass 2: ParallelFor over the weight-sorted sortedEdges (Pass 1).
		// Pass 1's phony-skip count folds into skippedCount.
		skippedCount.fetch_add(pass1PhonyCount);
		IndexContainer pass2Container(u32(sortedEdges.size()));
		workManager.ParallelFor<32>(workManager.GetWorkerCount(), pass2Container,
			// All Vectors/TStrings captured per-worker; clear()-then-fill keeps
			// buffer capacity across edges (~14 allocations/edge × millions otherwise).
			[&, 
				visitedEdgesFlags     = Vector<u8>(),
				addedProcessesFlags   = Vector<u8>(),
				processDependencies   = Vector<u32>(),
				fullCommandBuffer     = Vector<char>(),
				tempBuffer            = Vector<char>(),
				fullCommand           = TString(),
				rspFullPath           = TString(),
				rspContent            = Vector<char>(),
				inputsStr             = Vector<char>(),
				inputsNewlineStr      = Vector<char>(),
				outputsStr            = Vector<char>(),
				rspfilePathBuffer     = Vector<char>(),
				oldRelRspT            = TString(),
				newRelRspT            = TString(),
				description           = TString(),
				createDirExpandBuf    = Vector<char>(),
				knownInputsBuffer     = Vector<tchar>(),
				knownOutputsBuffer    = Vector<tchar>(),
				primaryOutExpandBuf  = Vector<char>()
			](const WorkContext&, auto& containerIt) mutable
			{
				u32 edgeIdx    = sortedEdges[containerIt.index];
				u32 processIdx = pass2.edgeToProcessIndex[edgeIdx];

				auto&       edges             = pass2.edges;
				auto&       outputToEdge      = pass2.outputToEdge;
				auto&       edgesToBuildFlags = pass2.edgesToBuildFlags;
				auto&       edgeToProcessIndex= pass2.edgeToProcessIndex;
				auto&       parser            = context.parser;
				auto&       stringPool        = pass2.stringPool;
				auto&       scheduler         = context.scheduler;
				auto&       logger            = context.logger;
				auto&       dirCache          = pass2.dirCache;
				auto&       state             = pass2.state;
				auto&       processedCount    = pass2.processedCount;
				auto&       skippedCount      = pass2.skippedCount;
				auto&       verboseLock       = pass2.verboseLock;
				auto&       verboseCounter    = pass2.verboseCounter;
				auto&       yamlEntries       = pass2.yamlEntries;
				const auto  workingDir        = context.workingDir;
				const u32   verboseTotal      = pass2.verboseTotal;
				const u32   phonyId           = pass2.phonyId;
				const u32   inId              = pass2.inId;
				const u32   in_newlineId      = pass2.in_newlineId;
				const u32   outId             = pass2.outId;
				const u32   baseProcessIdx    = pass2.baseProcessIdx;

				const auto& edge = edges[edgeIdx];

				// fullCommandBuffer must be cleared (ExpandVariables appends).
				fullCommandBuffer.clear();
				parser.ExpandCommand(edge, fullCommandBuffer, tempBuffer);

				if (fullCommandBuffer.empty())
				{
					skippedCount++;
					return;
				}

				if (context.noShowIncludes)
				{
					const char* pattern = "/showIncludes";
					u32 patternLen = 13;
					u32 writePos = 0;
					for (u32 readPos = 0; readPos < fullCommandBuffer.size(); )
					{
						if (readPos + patternLen <= fullCommandBuffer.size() &&
							memcmp(&fullCommandBuffer[readPos], pattern, patternLen) == 0)
						{
							while (readPos < fullCommandBuffer.size() && fullCommandBuffer[readPos] != ' ')
								++readPos;
							if (readPos < fullCommandBuffer.size() && fullCommandBuffer[readPos] == ' ')
								++readPos;
						}
						else
						{
							fullCommandBuffer[writePos++] = fullCommandBuffer[readPos++];
						}
					}
					fullCommandBuffer.resize(writePos);
				}

				// Widen to tchar; resize keeps capacity across edges.
				fullCommand.resize(fullCommandBuffer.size());
				for (u32 ci = 0, cn = u32(fullCommandBuffer.size()); ci < cn; ++ci)
					fullCommand[ci] = (tchar)(u8)fullCommandBuffer[ci];

				// -v: emit `[N/total] cmd` atomically per line.
				if (context.verbose)
				{
					u32 n = verboseCounter.fetch_add(1) + 1;
					SCOPED_FUTEX(verboseLock, vl);
					logger.Info(TC("[%u/%u] %s"), n, verboseTotal, fullCommand.c_str());
				}

				// -t commands: run the same shell-wrap pass dispatch does, then
				// print post-wrap argv. Diff vs `ninja -t commands` to spot truncation.
				if (context.toolCommands && !fullCommand.empty())
				{
					ProcessStartInfo siProbe;
					siProbe.application = TC("");
					siProbe.arguments = fullCommand.c_str();
					siProbe.workingDir = workingDir.data;
					ProcessStartInfoHolder probeHolder(siProbe);
					probeHolder.Expand();
					SCOPED_FUTEX(verboseLock, vl);
					logger.Info(TC("[ut-cmd] app=%s args=%s"),
						probeHolder.application ? probeHolder.application : TC(""),
						probeHolder.arguments ? probeHolder.arguments : TC(""));
					skippedCount++;
					return;
				}

				if (fullCommand.empty())
				{
					skippedCount++;
					return;
				}

				// We hand `fullCommand` to the scheduler with empty `application`;
				// ProcessStartInfoHolder::Expand handles shell-wrap detection.

				// Hash BEFORE rsp substitution. Used for:
				//   1. rsp filename (must stay stable across runs)
				//   2. NinjaExitPayload::cmdHash (must match what SuppressUpToDateEdges
				//      hashes — which never runs rsp logic). Post-substitution
				//      hashing would make every rsp-using edge look dirty.
				StringKey cmdKey = ToStringKeyNoCheck(fullCommand.c_str(), u32(fullCommand.length()));
				tchar cmdHashHex[17];
				for (u32 k = 0; k < 16; ++k)
				{
					u32 nibble = u32((cmdKey.a >> ((15 - k) * 4)) & 0xF);
					cmdHashHex[k] = (tchar)(nibble < 10 ? (TC('0') + nibble) : (TC('a') + (nibble - 10)));
				}
				cmdHashHex[16] = 0;

				// Clear so a previous edge's value doesn't trigger a stale rsp write
				// (downstream gates on !rspFullPath.empty()).
				rspFullPath.clear();
				rspContent.clear();

				if (!context.dryRun)
				{
					auto ruleIt = parser.GetRuleMap().find(edge.ruleName);
					if (ruleIt != parser.GetRuleMap().end())
					{
						const auto& rule = parser.GetRules()[ruleIt->second];
						if (rule.rspfile != StringId_Empty && rule.rspfile_content != StringId_Empty)
						{
							// rspVars layers $in / $in_newline / $out on top of
							// edge.variables for the rspfile templates. Map copy
							// is unavoidable: rspfile_content recursion may need
							// $root etc. resolved against the edge's own scope.
							UnorderedMap<u32, u32> rspVars = edge.variables;

							inputsStr.clear();
							inputsNewlineStr.clear();
							outputsStr.clear();
							rspfilePathBuffer.clear();

							// String-pool paths are unescaped. Two requirements when
							// feeding them back to ExpandVariables and into Soong's
							// NinjaRspFileReader:
							//   1. Escape `$` → `$$` so the recursive expand pass
							//      restores literal (`internal/$Gson$Preconditions.java`).
							//   2. Quote entries containing space/tab/newline/`'` so
							//      NinjaRspFileReader doesn't split on whitespace.
							//      `'\''` escapes embedded quotes (Soong contract).
							auto appendEscaped = [](Vector<char>& dst, const char* data, u32 length)
								{
									bool needQuote = false;
									for (u32 j = 0; j < length; ++j)
									{
										char c = data[j];
										if (c == ' ' || c == '\t' || c == '\n' || c == '\'')
										{
											needQuote = true;
											break;
										}
									}
									if (needQuote) dst.push_back('\'');
									for (u32 j = 0; j < length; ++j)
									{
										char c = data[j];
										if (c == '\'' && needQuote)
										{
											// '\'' = close, backslash, literal, reopen.
											dst.push_back('\'');
											dst.push_back('\\');
											dst.push_back('\'');
											dst.push_back('\'');
										}
										else if (c == '$')
										{
											dst.push_back('$');
											dst.push_back('$');
										}
										else
										{
											dst.push_back(c);
										}
									}
									if (needQuote) dst.push_back('\'');
								};

							for (u32 i = 0; i < edge.inputs.size(); ++i)
							{
								CharStringView input = stringPool.GetString(edge.inputs[i]);
								if (i > 0) { inputsStr.push_back(' '); inputsNewlineStr.push_back('\n'); }
								appendEscaped(inputsStr, input.data, input.length);
								appendEscaped(inputsNewlineStr, input.data, input.length);
							}
							if (!inputsStr.empty())
								rspVars[inId] = const_cast<StringPool&>(stringPool).InternThreadSafe(inputsStr.data(), u32(inputsStr.size()));
							if (!inputsNewlineStr.empty())
								rspVars[in_newlineId] = const_cast<StringPool&>(stringPool).InternThreadSafe(inputsNewlineStr.data(), u32(inputsNewlineStr.size()));

							for (u32 i = 0; i < edge.outputs.size(); ++i)
							{
								if (i > 0) outputsStr.push_back(' ');
								CharStringView output = stringPool.GetString(edge.outputs[i]);
								appendEscaped(outputsStr, output.data, output.length);
							}
							if (!outputsStr.empty())
								rspVars[outId] = const_cast<StringPool&>(stringPool).InternThreadSafe(outputsStr.data(), u32(outputsStr.size()));

							CharStringView rspfileStr = stringPool.GetString(rule.rspfile);
							parser.ExpandVariables(rspfileStr.data, rspfileStr.length, rspfilePathBuffer, &rspVars, edge.scopeId);

							CharStringView rspfileContentStr = stringPool.GetString(rule.rspfile_content);
							parser.ExpandVariables(rspfileContentStr.data, rspfileContentStr.length, rspContent, &rspVars, edge.scopeId);

							StringBuffer<> fullRspPath;
							fullRspPath.Append(workingDir);
							fullRspPath.EnsureEndsWithSlash();
							u32 charIdx = 0;
							if (rspfilePathBuffer.size() >= 2 && rspfilePathBuffer[0] == '.' &&
								(rspfilePathBuffer[1] == '/' || rspfilePathBuffer[1] == '\\'))
								charIdx = 2;
							for (; charIdx < rspfilePathBuffer.size(); ++charIdx)
								fullRspPath.Append((tchar)(u8)rspfilePathBuffer[charIdx]);

							rspFullPath = fullRspPath.data;

							// Relocate rsp files to `out/uba_rsp/<cmdhash>.rsp` so
							// sibling edges don't `rm` them. Soong's metalava-sbox edge
							// wipes its output dir, taking the consuming javac edge's
							// rspfile with it. uba_rsp/ lives under workingDir so UBA's
							// DIRTABLE picks up host writes naturally.
							oldRelRspT.clear();
							oldRelRspT.reserve(rspfilePathBuffer.size());
							for (char c : rspfilePathBuffer) oldRelRspT.push_back((tchar)(u8)c);

							u32 oldStrip = 0;
							if (oldRelRspT.size() >= 2 && oldRelRspT[0] == TC('.') &&
								(oldRelRspT[1] == TC('/') || oldRelRspT[1] == TC('\\')))
								oldStrip = 2;

							newRelRspT.clear();
							newRelRspT.reserve(32);
							newRelRspT.append(TC("out/uba_rsp/"));
							newRelRspT.append(cmdHashHex, 16);
							newRelRspT.append(TC(".rsp"));

							StringBuffer<512> newAbsRsp;
							newAbsRsp.Append(workingDir);
							newAbsRsp.EnsureEndsWithSlash();
							newAbsRsp.Append(newRelRspT.c_str());

							// Only relocate when the rsp path actually appears in
							// the cmd. Some rules (sbox manifest `rsp_files:`) read
							// the rsp at the original absolute path; relocation
							// would orphan it.
							u32 substitutionCount = 0;
							if (oldRelRspT.size() > oldStrip)
							{
								const tchar* oldKeyData = oldRelRspT.c_str() + oldStrip;
								u32          oldKeyLen  = u32(oldRelRspT.size() - oldStrip);
								u32          newRelLen  = u32(newRelRspT.size());
								size_t       findPos    = 0;
								while ((findPos = fullCommand.find(oldKeyData, findPos)) != TString::npos)
								{
									fullCommand.replace(findPos, oldKeyLen, newRelRspT.c_str(), newRelLen);
									findPos += newRelLen;
									++substitutionCount;
								}
							}

							if (substitutionCount > 0)
								rspFullPath = TString(newAbsRsp.data, newAbsRsp.count);
							// else: keep original (sbox manifest case).
						}
					}
				}

				// Description = friendly rule label + short output path.
				CharStringView ruleNameStr = stringPool.GetString(edge.ruleName);

				// Chromium rules: `<TYPE>__<target>`; take the prefix.
				u32 prefixLen = 0;
				while (prefixLen + 1 < ruleNameStr.length &&
				       !(ruleNameStr.data[prefixLen] == '_' && ruleNameStr.data[prefixLen + 1] == '_'))
					++prefixLen;
				if (prefixLen == ruleNameStr.length)
					prefixLen = ruleNameStr.length < 20 ? ruleNameStr.length : 20;

				const tchar* label = nullptr;
				auto prefixIs = [&](const char* s, u32 n) {
					return prefixLen == n && memcmp(ruleNameStr.data, s, n) == 0;
				};
				if      (prefixIs("CXX_COMPILER",  12)) label = TC("C++");
				else if (prefixIs("C_COMPILER",    10)) label = TC("C");
				else if (prefixIs("LINK",           4)) label = TC("Link");
				else if (prefixIs("SOLINK_MODULE", 13)) label = TC("Link.so");
				else if (prefixIs("SOLINK",         6)) label = TC("Link.so");
				else if (prefixIs("ALINK",          5)) label = TC("Lib");
				else if (prefixIs("ACTION",         6)) label = TC("Action");
				else if (prefixIs("STAMP",          5)) label = TC("Stamp");
				else if (prefixIs("COPY",           4)) label = TC("Copy");

				description.clear();
				if (label)
					description = label;
				else
					for (u32 i = 0; i < prefixLen; ++i)
						description += (tchar)(u8)ruleNameStr.data[i];

				if (!edge.outputs.empty())
				{
					CharStringView out = stringPool.GetString(edge.outputs[0]);

					const char* lastSep = nullptr;
					const char* prevSep = nullptr;
					for (u32 i = 0; i < out.length; ++i)
						if (out.data[i] == '/' || out.data[i] == '\\')
						{ prevSep = lastSep; lastSep = out.data + i; }

					const char* ext = nullptr;
					for (u32 i = out.length; i > 0; --i)
					{
						if (out.data[i - 1] == '.' ) { ext = out.data + i - 1; break; }
						if (out.data[i - 1] == '/' || out.data[i - 1] == '\\') break;
					}

					// Object files: show parent/file.o (many share the same basename).
					bool isObj = ext && (strcmp(ext, ".o") == 0 || strcmp(ext, ".obj") == 0);
					const char* nameStart = (isObj && prevSep) ? prevSep + 1
					                      : lastSep            ? lastSep + 1
					                      :                      out.data;
					u32 nameLen = u32(out.data + out.length - nameStart);

					if (!description.empty())
						description += TC(" ");
					for (u32 i = 0; i < nameLen; ++i)
						description += (tchar)(u8)nameStart[i];
				}

				if (description.size() > 60) { description.resize(57); description += TC("..."); }

				// rsp file I/O. dirCache is thread-safe (Futex).
				if (!rspFullPath.empty())
				{
					const tchar* lastSep = nullptr;
					for (const tchar* p = rspFullPath.c_str(); *p; ++p)
						if (*p == '/' || *p == '\\') lastSep = p;
					if (lastSep && lastSep > rspFullPath.c_str())
					{
						TString dirPath(rspFullPath.c_str(), u32(lastSep - rspFullPath.c_str()));
						dirCache.CreateDirectory(logger, dirPath.c_str());
					}
					FileHandle rspHandle = CreateFileW(rspFullPath.c_str(), GENERIC_WRITE, FILE_SHARE_READ, CREATE_ALWAYS, DefaultAttributes());
					if (rspHandle == InvalidFileHandle)
					{
						logger.Error(TC("Failed to create rsp file %s (%s)"), rspFullPath.c_str(), LastErrorToText().data);
					}
					else
					{
						WriteFile(logger, rspFullPath.c_str(), rspHandle, rspContent.data(), u32(rspContent.size()));
						CloseFile(rspFullPath.c_str(), rspHandle);

						// Register in DIRTABLE so detoured consumers see the rsp.
						// Without this, an "empty dir" entry can race the rsp write
						// → consumer's open() returns ENOENT.
						scheduler.GetSession().RegisterNewFile(rspFullPath.c_str());
					}
				}

				// Create output dirs (matches ninja's pre-edge mkdir).
				{
					StringBuffer<512> tmp1, tmp2;
					auto createOutputDir = [&](u32 id)
					{
						CharStringView rel = stringPool.GetString(id);
						createDirExpandBuf.clear();
						parser.ExpandVariables(rel.data, rel.length, createDirExpandBuf, nullptr, edge.scopeId);
						tmp1.Clear().Append(createDirExpandBuf.data(), u32(createDirExpandBuf.size()));
						FixPath(tmp2.Clear(), tmp1, workingDir);
						while (tmp2.count > 0 && tmp2.data[tmp2.count - 1] != PathSeparator)
							--tmp2.count;
						if (tmp2.count > 0)
						{
							tmp2.data[tmp2.count] = 0;
							dirCache.CreateDirectory(logger, tmp2.data);
						}
					};
					for (u32 id : edge.outputs)        createOutputDir(id);
					for (u32 id : edge.implicitOutputs) createOutputDir(id);
				}

				// Dep walk
				visitedEdgesFlags.clear();
				addedProcessesFlags.clear();
				processDependencies.clear();

				// Literal lookup first; expand `$vars` only on miss. See BFS-pass resolveDepId.
				Vector<char> dispExpand;
				auto resolveDepEdgeIdx = [&](u32 depTargetId) -> u32
				{
					if (depTargetId < outputToEdge.size() && outputToEdge[depTargetId] != ~0u)
						return outputToEdge[depTargetId];
					CharStringView s = stringPool.GetString(depTargetId);
					if (s.length == 0 || memchr(s.data, '$', s.length) == nullptr)
						return ~0u;
					dispExpand.clear();
					const_cast<NinjaParser&>(parser).ExpandVariables(s.data, s.length, dispExpand, &edge.variables, edge.scopeId);
					u32 newLen = u32(dispExpand.size());
					if (newLen == 0)
						return ~0u;
					u32 newId = const_cast<StringPool&>(stringPool).Intern(dispExpand.data(), newLen);
					if (newId < outputToEdge.size() && outputToEdge[newId] != ~0u)
						return outputToEdge[newId];
					return ~0u;
				};

				auto addDependency = [&](u32 depTargetId, auto& self) -> void
				{
					u32 depEdgeIdx = resolveDepEdgeIdx(depTargetId);
					if (depEdgeIdx == ~0u)
						return;
					if (!edgesToBuildFlags[depEdgeIdx])
						return;
					if (depEdgeIdx >= visitedEdgesFlags.size())
						visitedEdgesFlags.resize(depEdgeIdx + 1, 0);
					if (visitedEdgesFlags[depEdgeIdx] || depEdgeIdx == edgeIdx)
						return;
					visitedEdgesFlags[depEdgeIdx] = 1;

					const auto& depEdge = edges[depEdgeIdx];
					if (depEdge.ruleName == phonyId)
					{
						for (u32 id : depEdge.inputs) self(id, self);
						for (u32 id : depEdge.implicitDeps) self(id, self);
						for (u32 id : depEdge.orderOnlyDeps) self(id, self);
					}
					else
					{
						u32 depProcessIdx = edgeToProcessIndex[depEdgeIdx];
						if (depProcessIdx != ~0u)
						{
							if (depProcessIdx >= addedProcessesFlags.size())
								addedProcessesFlags.resize(depProcessIdx + 1, 0);
							if (!addedProcessesFlags[depProcessIdx])
							{
								processDependencies.push_back(depProcessIdx);
								addedProcessesFlags[depProcessIdx] = 1;
							}
						}
					}
				};

				// Add producer by edgeIdx (depfile lookup gives us the edge directly).
				// phony.inputs are stringIds → delegate back to addDependency.
				// Only consumer is the depfile-discovered injection block below,
				// currently #if 0'd — see comment there for re-enable plan.
#if 0
				auto addDependencyByEdge = [&](u32 depEdgeIdx) -> void
				{
					if (!edgesToBuildFlags[depEdgeIdx])
						return;
					if (depEdgeIdx >= visitedEdgesFlags.size())
						visitedEdgesFlags.resize(depEdgeIdx + 1, 0);
					if (visitedEdgesFlags[depEdgeIdx] || depEdgeIdx == edgeIdx)
						return;
					visitedEdgesFlags[depEdgeIdx] = 1;

					const auto& depEdge = edges[depEdgeIdx];
					if (depEdge.ruleName == phonyId)
					{
						for (u32 id : depEdge.inputs) addDependency(id, addDependency);
						for (u32 id : depEdge.implicitDeps) addDependency(id, addDependency);
						for (u32 id : depEdge.orderOnlyDeps) addDependency(id, addDependency);
					}
					else
					{
						u32 depProcessIdx = edgeToProcessIndex[depEdgeIdx];
						if (depProcessIdx != ~0u)
						{
							if (depProcessIdx >= addedProcessesFlags.size())
								addedProcessesFlags.resize(depProcessIdx + 1, 0);
							if (!addedProcessesFlags[depProcessIdx])
							{
								processDependencies.push_back(depProcessIdx);
								addedProcessesFlags[depProcessIdx] = 1;
							}
						}
					}
				};
#endif

				for (u32 id : edge.inputs) addDependency(id, addDependency);
				for (u32 id : edge.implicitDeps) addDependency(id, addDependency);
				for (u32 id : edge.orderOnlyDeps) addDependency(id, addDependency);
				// AOSP `|@` validations don't block consumers but must run as
				// side effects. Treating them as deps of THIS edge is conservative —
				// possibly earlier than strict semantics, never out-of-order.
				for (u32 id : edge.validations) addDependency(id, addDependency);

				// Inject depfile-discovered scheduling deps so generated headers
				// (e.g. atoms.pb.h) get an edge to their producer.
				//
				// DISABLED — over-serializes chromium-style builds. When a `restat`
				// producer (e.g. `devtools_resources_grit`) is dirty, this pins every
				// compile that previously read one of its headers behind it. Stock
				// ninja serializes the same way but recovers via restat-skip: if the
				// producer reruns and its output content is unchanged (the common
				// case), all dirty dependents are skipped. UBA does not implement
				// restat-skip yet, so the injection is a pure throughput loss with
				// no offsetting win. Pre-shelf behavior (no injection) was racy on
				// the first incremental build after a producer's content changed,
				// but self-corrected on the next build via the depfile freshness
				// check — net faster.
				//
				// To re-enable correctly:
				//   1. Hash producer outputs before the edge runs (restat=1 or
				//      phony_output=1 only).
				//   2. After the edge succeeds, hash again. If unchanged, mark the
				//      edge a "content no-op."
				//   3. For each dependent of a content-no-op producer, re-run the
				//      SuppressUpToDateEdges check using the producer's pre-run
				//      mtime; if the dependent's output is newer, call
				//      Scheduler::SkipProcess on it before it dispatches.
				//   4. Only after (1)–(3) ship, flip this back on.
#if 0
				if (!pass2.logIndex.depsByOutputKey.empty() &&
				    edgeIdx < pass2.primaryOutKeyByEdge.size())
				{
					StringKey primaryKey = pass2.primaryOutKeyByEdge[edgeIdx];
					if (primaryKey.a != 0 || primaryKey.b != 0)
					{
						auto it = pass2.logIndex.depsByOutputKey.find(primaryKey);
						if (it != pass2.logIndex.depsByOutputKey.end())
						{
							const auto& de = it->second;
							for (u32 k = 0; k < de.depCount; ++k)
							{
								StringKey depKey = pass2.logIndex.flatDepKeys[de.depOffset + k];
								auto eIt = pass2.pathKeyToEdge.find(depKey);
								if (eIt != pass2.pathKeyToEdge.end())
									addDependencyByEdge(eIt->second);
							}
						}
					}
				}
#endif


				struct CacheInfo
				{
					Scheduler* scheduler;
					const StringPool* stringPool;
					StringView workingDir;
					Vector<u32> inputs;
					Vector<u32> outputs;
				};

				// Diagnostic dump for the d8 "Unrecognized option: -" bug
				// (CL #118). Captures fullCommand byte-for-byte to /tmp/uba_ninja_d8_cmd.<pid>.log.
#if !PLATFORM_WINDOWS
				if (fullCommand.find(TC("/d8 -J")) != TString::npos
					|| fullCommand.find(TC("/d8 ")) != TString::npos)
				{
					char path[64];
					int n = 0; const char* p1 = "/tmp/uba_ninja_d8_cmd.";
					while (*p1) path[n++] = *p1++;
					unsigned int pid = (unsigned int)getpid();
					char digs[16]; int dl = 0; unsigned int v = pid;
					if (!v) digs[dl++] = '0';
					while (v) { digs[dl++] = '0' + (char)(v % 10); v /= 10; }
					while (dl--) path[n++] = digs[dl];
					const char* sfx = ".log";
					while (*sfx) path[n++] = *sfx++;
					path[n] = 0;
					FILE* f = fopen(path, "a");
					if (f)
					{
						fprintf(f, "=== description=%s ===\n", description.c_str());
						fprintf(f, "len=%llu\n", (unsigned long long)fullCommand.size());
						fprintf(f, "fullCommand=%s\n", fullCommand.c_str());
						fputs("hex around OnError (if present): ", f);
						auto pos = fullCommand.find(TC("OnError"));
						if (pos != TString::npos)
						{
							size_t lo = pos > 8 ? pos - 8 : 0;
							size_t hi = pos + 64 < fullCommand.size() ? pos + 64 : fullCommand.size();
							for (size_t i = lo; i < hi; ++i)
							{
								unsigned char c = (unsigned char)fullCommand[i];
								if (c >= 0x20 && c < 0x7f && c != '"' && c != '\\') fputc(c, f);
								else fprintf(f, "<%02x>", c);
							}
							fputc('\n', f);
						}
						else
						{
							fputs("(OnError not found)\n", f);
						}
						fputs("\n", f);
						fclose(f);
					}
				}
#endif

				// Empty application → Expand() tokenizes / sh-c-wraps; PrepareProcess resolves bare names.
				ProcessStartInfo pinfo;
				pinfo.application = TC("");
				pinfo.arguments   = fullCommand.c_str();
				pinfo.workingDir  = workingDir.data;
				pinfo.description = description.c_str();
				pinfo.trackInputs = true;

				EnqueueProcessInfo epi(pinfo);
				epi.weight             = 1.0f;
				epi.canDetour          = true;
				epi.canExecuteRemotely = true;
				epi.dependencies       = processDependencies.data();
				epi.dependencyCount    = u32(processDependencies.size());
				epi.cacheBucketId      = context.cacheBucketId;

				// Hack-version of building/skip parts of the build
				u32 pc = processedCount;
				if (pc < context.skipEdges || pc >= context.skipEdges + context.maxEdges)
					epi.skipAsSuccessWhenScheduled = true;


				knownInputsBuffer.clear();
				u32 knownInputsCount = 0;
				{
					knownInputsBuffer.reserve((edge.inputs.size() + edge.implicitDeps.size()) * (workingDir.count + 10));
					StringBuffer<512> tmp1;
					StringBuffer<512> tmp2;
					auto addInput = [&](u32 id)
						{
							auto view = stringPool.GetString(id);
							tmp1.Clear().Append(view.data, view.length);
							FixPath(tmp2.Clear(), tmp1, workingDir);
							u32 oldSize = u32(knownInputsBuffer.size());
							knownInputsBuffer.resize(oldSize + tmp2.count + 1);
							memcpy(knownInputsBuffer.data() + oldSize, tmp2.data, tmp2.count * sizeof(tchar));
							knownInputsBuffer[oldSize + tmp2.count] = 0;
							++knownInputsCount;
						};
					for (u32 id : edge.inputs) addInput(id);
					for (u32 id : edge.implicitDeps) addInput(id);
					if (knownInputsCount)
					{
						knownInputsBuffer.push_back(0); // final empty terminator
						epi.knownInputs      = knownInputsBuffer.data();
						epi.knownInputsBytes = u32(knownInputsBuffer.size() * sizeof(tchar));
						epi.knownInputsCount = knownInputsCount;
					}
				}

				knownOutputsBuffer.clear();
				u32 knownOutputsCount = 0;
				{
					knownOutputsBuffer.reserve((edge.outputs.size() + edge.implicitOutputs.size()) * (workingDir.count + 10));
					StringBuffer<512> tmp1;
					StringBuffer<512> tmp2;
					auto addOutput = [&](u32 id)
					{
						auto view = stringPool.GetString(id);
						tmp1.Clear().Append(view.data, view.length);
						FixPath(tmp2.Clear(), tmp1, workingDir);
						u32 oldSize = u32(knownOutputsBuffer.size());
						knownOutputsBuffer.resize(oldSize + tmp2.count + 1);
						memcpy(knownOutputsBuffer.data() + oldSize, tmp2.data, tmp2.count * sizeof(tchar));
						knownOutputsBuffer[oldSize + tmp2.count] = 0;
						++knownOutputsCount;
					};
					for (u32 id : edge.outputs)
						addOutput(id);
					for (u32 id : edge.implicitOutputs)
						addOutput(id);
					if (knownOutputsCount)
					{
						knownOutputsBuffer.push_back(0); // final empty terminator
						epi.knownOutputs      = knownOutputsBuffer.data();
						epi.knownOutputsBytes = u32(knownOutputsBuffer.size() * sizeof(tchar));
						epi.knownOutputsCount = knownOutputsCount;
					}
				}

				//if (!context.forceRemote)
				{
					if (ShouldForceRunLocally(description))
						epi.canExecuteRemotely = false;
				}

				// Used by exit payload and yamlEntries[slot].primaryOut.
				TString primaryOutAbs;
				if (!edge.outputs.empty())
				{
					primaryOutExpandBuf.clear();
					CharStringView primaryRel = stringPool.GetString(edge.outputs[0]);
					parser.ExpandVariables(primaryRel.data, primaryRel.length, primaryOutExpandBuf, nullptr, edge.scopeId);
					if (!primaryOutExpandBuf.empty())
					{
						StringBuffer<512> wide;
						wide.count = u32(primaryOutExpandBuf.size());
						if (wide.count + 1 > wide.capacity) wide.count = wide.capacity - 1;
						for (u32 k = 0; k < wide.count; ++k)
						{
							char c = primaryOutExpandBuf[k];
							wide.data[k] = (tchar)(u8)((c == '\\' || c == '/') ? PathSeparator : c);
						}
						wide.data[wide.count] = 0;
						StringBuffer<512> abs;
						FixPath(abs, StringView(wide.data, wide.count), workingDir);
						primaryOutAbs.assign(abs.data, abs.count);
					}
				}

				// Heap payload owned/freed by exitedFunc.
				if (state.exitCallback && !primaryOutAbs.empty())
				{
					bool restatRule = false;
					{
						auto rmIt = parser.GetRuleMap().find(edge.ruleName);
						if (rmIt != parser.GetRuleMap().end())
							restatRule = parser.GetRules()[rmIt->second].restat;
					}
					const bool useWallClockMtime = restatRule || edge.phonyOutput;

					auto* payload = new NinjaExitPayload;
					payload->ctx        = state.exitContext;
					payload->cmdHash    = cmdKey.a;
					payload->restat     = useWallClockMtime;
					payload->primaryOut = primaryOutAbs;

					pinfo.userData   = payload;
					pinfo.exitedFunc = state.exitCallback;
				}

				scheduler.ResumeQueuedProcess(processIdx, epi);

				if ((context.yamlFile || context.depsDumpFile) && baseProcessIdx != ~0u)
				{
					u32 slot = processIdx - baseProcessIdx;
					auto& entry = yamlEntries[slot];
					entry.arguments          = fullCommand;
					entry.workingDir         = TString(workingDir.data, workingDir.count);
					entry.description        = description;
					// entry.weight set in Pass 1.
					entry.canExecuteRemotely = epi.canExecuteRemotely;
					entry.canDetour          = epi.canDetour;
					entry.dependencies       = processDependencies;
					entry.primaryOut         = primaryOutAbs;
				}

				processedCount++;
			},
			AsView(TC("NinjaResume"))
		);

		// ParallelFor is synchronous; whole parallel-run time → tPass2Dispatch.
		tPass2Dispatch = GetTime() - tPhase;
		tPhase = GetTime();

		if (context.yamlFile && !yamlEntries.empty())
		{
			Vector<char> buf;
			buf.reserve(4 * 1024 * 1024);

			auto appendStr = [&](const char* s) {
				buf.insert(buf.end(), s, s + strlen(s));
			};
			auto appendTStr = [&](const TString& s) {
				for (tchar c : s) buf.push_back((char)(u8)c);
			};
			auto appendU32 = [&](u32 v) {
				char tmp[16];
				#if PLATFORM_WINDOWS
				_itoa_s(v, tmp, sizeof(tmp), 10);
				#else
				snprintf(tmp, 16, "%d", v);
				#endif
				appendStr(tmp);
			};

			appendStr("processes:\n");

			for (u32 slot = 0; slot < u32(yamlEntries.size()); ++slot)
			{
				const auto& e = yamlEntries[slot];
				if (e.arguments.empty())
					continue;
				u32 id = baseProcessIdx + slot;

				appendStr("  - id: "); appendU32(id); appendStr("\n");
				appendStr("    arg: "); appendTStr(e.arguments);  appendStr("\n");
				appendStr("    dir: "); appendTStr(e.workingDir); appendStr("\n");
				appendStr("    desc: "); appendTStr(e.description); appendStr("\n");

				if (e.weight != 1.0)
				{
					char tmp[64];
					snprintf(tmp, sizeof(tmp), "    weight: %.2f\n", e.weight);
					appendStr(tmp);
				}
				if (!e.canExecuteRemotely)
					appendStr("    remote: false\n");
				if (!e.canDetour)
					appendStr("    detour: false\n");

				if (!e.dependencies.empty())
				{
					appendStr("    dep: [");
					for (u32 di = 0; di < u32(e.dependencies.size()); ++di)
					{
						if (di > 0) appendStr(", ");
						appendU32(e.dependencies[di]);
					}
					appendStr("]\n");
				}
			}

			FileHandle fh = CreateFileW(context.yamlFile, GENERIC_WRITE, FILE_SHARE_READ, CREATE_ALWAYS, DefaultAttributes());
			if (fh != InvalidFileHandle)
			{
				WriteFile(logger, context.yamlFile, fh, buf.data(), u32(buf.size()));
				CloseFile(context.yamlFile, fh);
				logger.Info(TC("Wrote YAML file: %s (%u entries)"), context.yamlFile, u32(yamlEntries.size()));
			}
			else
				logger.Error(TC("Failed to create YAML file: %s"), context.yamlFile);
		}

		tYaml = GetTime() - tPhase;

		// Deps analysis dump: one TSV row per process, sorted by weight desc.
		// Columns: id weight depCount remote detour desc primaryOut deps
		if (context.depsDumpFile && !yamlEntries.empty())
		{
			Vector<char> buf;
			buf.reserve(8 * 1024 * 1024);

			auto appendStr = [&](const char* s) {
				buf.insert(buf.end(), s, s + strlen(s));
			};
			auto appendTStr = [&](const TString& s) {
				for (tchar c : s) buf.push_back((char)(u8)c);
			};
			auto appendU32 = [&](u32 v) {
				char tmp[16];
				#if PLATFORM_WINDOWS
				_itoa_s(v, tmp, sizeof(tmp), 10);
				#else
				snprintf(tmp, 16, "%u", v);
				#endif
				appendStr(tmp);
			};
			auto appendDouble = [&](double v) {
				char tmp[64];
				snprintf(tmp, sizeof(tmp), "%.1f", v);
				appendStr(tmp);
			};

			u64 totalDeps = 0;
			u32 maxDeps = 0, maxDepsSlot = 0;
			double maxWeight = 0.0;
			u32 maxWeightSlot = 0;
			Vector<u32> byWeight;
			byWeight.reserve(yamlEntries.size());
			for (u32 slot = 0; slot < u32(yamlEntries.size()); ++slot)
			{
				const auto& e = yamlEntries[slot];
				if (e.arguments.empty()) continue;
				byWeight.push_back(slot);
				totalDeps += e.dependencies.size();
				if (u32(e.dependencies.size()) > maxDeps) { maxDeps = u32(e.dependencies.size()); maxDepsSlot = slot; }
				if (e.weight > maxWeight) { maxWeight = e.weight; maxWeightSlot = slot; }
			}
			u32 nonEmpty = u32(byWeight.size());
			std::sort(byWeight.begin(), byWeight.end(), [&](u32 a, u32 b) {
				return yamlEntries[a].weight > yamlEntries[b].weight;
			});

			char tmp[256];
			snprintf(tmp, sizeof(tmp), "# UbaNinja deps dump\n# total processes: %u\n# total deps: %llu\n# avg deps/process: %.2f\n# max deps: %u (slot %u)\n# max weight: %.1f (slot %u)\n",
				nonEmpty, (unsigned long long)totalDeps,
				nonEmpty ? double(totalDeps) / double(nonEmpty) : 0.0,
				maxDeps, maxDepsSlot, maxWeight, maxWeightSlot);
			appendStr(tmp);

			appendStr("# top-30 highest weight (likely bottlenecks — many transitive consumers):\n");
			u32 topN = byWeight.size() < 30 ? u32(byWeight.size()) : 30;
			for (u32 i = 0; i < topN; ++i)
			{
				u32 slot = byWeight[i];
				const auto& e = yamlEntries[slot];
				snprintf(tmp, sizeof(tmp), "#   id=%u weight=%.1f deps=%u  ",
					baseProcessIdx + slot, e.weight, u32(e.dependencies.size()));
				appendStr(tmp);
				appendTStr(e.description);
				appendStr("\n");
			}

			appendStr("#\n# Columns: id\\tweight\\tdepCount\\tremote\\tdetour\\tdesc\\tprimaryOut\\tdeps\n");

			// Body: all entries, sorted by weight descending.
			for (u32 slot : byWeight)
			{
				const auto& e = yamlEntries[slot];
				appendU32(baseProcessIdx + slot); appendStr("\t");
				appendDouble(e.weight); appendStr("\t");
				appendU32(u32(e.dependencies.size())); appendStr("\t");
				appendStr(e.canExecuteRemotely ? "1" : "0"); appendStr("\t");
				appendStr(e.canDetour ? "1" : "0"); appendStr("\t");
				appendTStr(e.description); appendStr("\t");
				appendTStr(e.primaryOut); appendStr("\t");
				for (u32 di = 0; di < u32(e.dependencies.size()); ++di)
				{
					if (di > 0) appendStr(",");
					appendU32(e.dependencies[di]);
				}
				appendStr("\n");
			}

			FileHandle fh = CreateFileW(context.depsDumpFile, GENERIC_WRITE, FILE_SHARE_READ, CREATE_ALWAYS, DefaultAttributes());
			if (fh != InvalidFileHandle)
			{
				WriteFile(logger, context.depsDumpFile, fh, buf.data(), u32(buf.size()));
				CloseFile(context.depsDumpFile, fh);
				logger.Info(TC("Wrote deps dump: %s (%u processes, %llu total deps)"), context.depsDumpFile, nonEmpty, (unsigned long long)totalDeps);
			}
			else
				logger.Error(TC("Failed to create deps dump: %s"), context.depsDumpFile);
		}

		u64 totalTime = GetTime() - totalStartTime;
		u64 queueTime = GetTime() - queueStartTime;

		u32 totalBuilt = 0;
		for (u32 i = 0; i < u32(edges.size()); ++i)
			if (edgesToBuildFlags[i]) ++totalBuilt;

		logger.Detail(TC("Enqueued %u commands, skipped %u (of %u needed edges)"), processedCount.load(), skippedCount.load(), totalBuilt);
		logger.Detail(TC("  Queue time: %s"), TimeToText(queueTime).str);
		logger.Detail(TC("  Total time: %s"), TimeToText(totalTime).str);

		// Stable line — always print all phases (zero for skipped) so it's grep-friendly.
		logger.Info(TC("Enqueue phases: bfs=%s clean=%s topo=%s uptodate=%s pass1=%s dispatch=%s yaml=%s"),
			TimeToText(tBfs).str,
			TimeToText(tClean).str,
			TimeToText(tTopo).str,
			TimeToText(tUpToDate).str,
			TimeToText(tPass1).str,
			TimeToText(tPass2Dispatch).str,
			TimeToText(tYaml).str);
		return true;
	}

}
