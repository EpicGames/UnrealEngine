// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaNinjaParser.h"
#include "UbaNinjaStringPool.h"
#include "UbaStringBuffer.h"

namespace uba
{
	// Per-edge workarounds for known issues when building Chromium or AOSP through UbaNinja.
	//
	// Some edges must be forced to run locally:
	//   - A handful of files hit bugs in Wine when compiled on Windows remote helpers.
	//   - A few steps are heavy / transfer huge numbers of files and are simply faster locally.
	//   - v8_context_snapshot.bin needs glibc 2.38 but Epic's Horde farm runs AL2023.
	//
	// TODO: The list is matched against the edge description (e.g. "C++ color_data.cc").
	//       Longer term this should move to either:
	//         (a) a well-known edge variable in the .ninja file (e.g. `uba_local = 1`)
	//             that GN / the generator can set, so fixes live at source, or
	//         (b) a sidecar file next to build.ninja listing output paths to force-local.
	//       Until then, entries should be added / removed here as new Wine bugs surface
	//       and are fixed.
	inline bool ShouldForceRunLocally(const TString& description)
	{
		static const StringView forceRunLocally[] =
		{
			// Requires three bug fixes in Wine
			TCV("color_data.cc"),
			TCV("tracing_service_idl.h"),
			TCV("elevation_service_idl.h"),
			TCV("dxcetw.h"),
			TCV("updater_idl.h"),
			TCV("updater_internal_idl.h"),
			TCV("updater_legacy_idl.h"),
			TCV("at_rule_descriptors.cc"),
			TCV("css_value_keywords.cc"),
			TCV("css_property_names.cc"),
			TCV("acls.stamp"),

			// Heavy steps, don't schedule on slow machine
			TCV("eslint.config.mjs"),

			// Crazy much traffic, lots of files.. runs much faster locally
			TCV("generated_resources__strings_grit.d.stamp"),
			TCV("components_strings__strings_grit.d.stamp"),

			#if PLATFORM_LINUX
			TCV("v8_context_snapshot.bin"), // Requires glibc 2.38 and Epic's horde farm uses AWS AL2023
			#endif

			// TMP FOR AOSP
			TCV("m.trusty_security_vm_x86_64.elf_.generato trusty_security..."), // Takes a very long time
			TCV("m.i18n.module.intra.core.api.stubs.source_android_common...."), // Takes a very long time 
			TCV("m.art.module.intra.core.api.stubs.source_android_common.m..."), // Takes a very long time
			TCV("m.UserDictionaryProvider_android_common.dexpreopt.UserDic..."), // Ran remotely at end and hung
			TCV("m.EasterEgg_android_common.dexpreopt.EasterEg package.inv..."), // Ran remotely at end and hung
			TCV("g.java.java platformprotos.jar"), 
		};

		for (StringView str : forceRunLocally)
			if (description == str.data)
				return true;

		StringView sv(description);
		if (sv.EndsWith(TCV(".css.js")))
			return true;
		if (sv.EndsWith(TCV("_ts.json")))
			return true;

		// Link steps, just lots of file transfers
		if (sv.StartsWith(TCV("solin ")) || sv.StartsWith(TCV("lin ")) ||  sv.StartsWith(TCV("alin ")))
			return true;

		return false;
	}

	// Bias added to an edge's Kahn critical-path weight to sort it earlier in
	// Pass 1. Critical/Expensive sit far above max real Kahn weight (~hundreds
	// on AOSP) so tiering dominates; within a tier Kahn weight still orders.
	// Keyed on raw rule name (no description-rewriting via CXX_COMPILER→C++);
	// chromium-style rules with `__` need full pre-`__` prefix.
	inline double GetPriorityWeight(const StringPool& stringPool, const NinjaBuildEdge& edge)
	{
		if (edge.outputs.empty())
			return 0.0;

		CharStringView ruleName    = stringPool.GetString(edge.ruleName);
		CharStringView firstOutput = stringPool.GetString(edge.outputs[0]);

		const char* baseStart = firstOutput.data;
		for (u32 i = 0; i < firstOutput.length; ++i)
			if (firstOutput.data[i] == '/' || firstOutput.data[i] == '\\')
				baseStart = firstOutput.data + i + 1;
		u32 baseLen = u32(firstOutput.data + firstOutput.length - baseStart);

		auto ruleIs = [&](const char* r, u32 rLen)
		{
			return ruleName.length == rLen && memcmp(ruleName.data, r, rLen) == 0;
		};
		auto baseIs = [&](const char* b, u32 bLen)
		{
			return baseLen == bLen && memcmp(baseStart, b, bLen) == 0;
		};

		#define UBA_RULE_IS(r)   ruleIs(r, u32(sizeof(r) - 1))
		#define UBA_BASE_IS(b)   baseIs(b, u32(sizeof(b) - 1))

		constexpr double Critical  = 1e8;
		constexpr double Expensive = 1e6;

		if (UBA_RULE_IS("g.java.java"))
		{
			if (UBA_BASE_IS("BluetoothLib.jar"))   return Critical;
			if (UBA_BASE_IS("platformprotos.jar")) return Critical;
		}

		// kotlin compiles are slow per edge; front-load them.
		if (UBA_RULE_IS("g.java.kotlin"))
			return Expensive;

		#undef UBA_RULE_IS
		#undef UBA_BASE_IS
		return 0.0;
	}
}
