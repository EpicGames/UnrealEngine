// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaNinjaDepsLog.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaFile.h"
#include "UbaFileAccessor.h"
#include "UbaHash.h"
#include "UbaLogger.h"
#include "UbaMap.h"
#include "UbaPathUtils.h"
#include "UbaProcessHandle.h"
#include "UbaProcessStartInfo.h"
#include "UbaTimer.h"

namespace uba
{
	// High bit of the leading u32: SET = deps record, CLEAR = path record.
	// Must match stock ninja's deps_log.cc.
	static constexpr u32 kDepsMask = 0x80000000u;

	static void NormalizePathForKey(StringBufferBase& path)
	{
		if (CaseInsensitiveFs)
			ToLower(path.data);
	}

	// Transient/derivative files that must NOT become deps. Recording them
	// either invalidates the consumer every run (kernel pseudo-files) or
	// marks it dirty because the file is gone next run (sandbox/temp).
	// .pyc/.pyo/.pyd: depend on the .py source, not the bytecode.
	static bool IsTransientDep(const tchar* path, u32 len)
	{
		if (len == 0) return false;

		auto startsWith = [&](const tchar* prefix, u32 plen)
		{
			return len >= plen && memcmp(path, prefix, plen * sizeof(tchar)) == 0;
		};
		if (startsWith(TC("/dev/"),  5)) return true;
		if (startsWith(TC("/sys/"),  5)) return true;
		if (startsWith(TC("/proc/"), 6)) return true;
		if (startsWith(TC("/tmp/"),  5)) return true;

		auto containsSepAnchored = [&](const tchar* needle, u32 nlen, bool needLeftSep)
		{
			if (len < nlen) return false;
			for (u32 i = 0; i + nlen <= len; ++i)
			{
				if (memcmp(path + i, needle, nlen * sizeof(tchar)) != 0)
					continue;
				if (needLeftSep && i != 0 && path[i - 1] != TC('/') && path[i - 1] != TC('\\'))
					continue;
				return true;
			}
			return false;
		};
		if (containsSepAnchored(TC(".uba/sessions/"),  14, true)) return true;
		if (containsSepAnchored(TC(".uba\\sessions\\"), 14, true)) return true;
		if (containsSepAnchored(TC("uba_rsp/"),  8, true)) return true;
		if (containsSepAnchored(TC("uba_rsp\\"), 8, true)) return true;
		if (containsSepAnchored(TC("/soong/.temp/sbox/"), 18, false)) return true;
		if (containsSepAnchored(TC("\\Temp\\"),                 6, false)) return true;
		if (containsSepAnchored(TC("\\AppData\\Local\\Temp\\"), 20, false)) return true;

		// __pycache__ — separator-anchored on both sides
		for (u32 i = 0; i + 11 <= len; ++i)
		{
			if (memcmp(path + i, TC("__pycache__"), 11 * sizeof(tchar)) != 0)
				continue;
			bool leftOk  = (i == 0)        || path[i - 1]  == TC('/') || path[i - 1]  == TC('\\');
			bool rightOk = (i + 11 == len) || path[i + 11] == TC('/') || path[i + 11] == TC('\\');
			if (leftOk && rightOk) return true;
		}

		auto endsWith = [&](const tchar* suffix, u32 slen)
		{
			return len >= slen && memcmp(path + (len - slen), suffix, slen * sizeof(tchar)) == 0;
		};
		if (endsWith(TC(".tmp"),  4)) return true;
		if (endsWith(TC(".pyc"),  4)) return true;
		if (endsWith(TC(".pyo"),  4)) return true;
		if (endsWith(TC(".pyd"),  4)) return true;
		if (endsWith(TC(".swp"),  4)) return true;
		if (endsWith(TC(".swo"),  4)) return true;
		if (endsWith(TC(".lock"), 5)) return true;

		return false;
	}

	NinjaDepsLog::NinjaDepsLog() = default;
	NinjaDepsLog::~NinjaDepsLog() = default;

	void NinjaDepsLog::Init(StringView workingDir, u64 buildStartTime)
	{
		m_workingDir.assign(workingDir.data, workingDir.count);
		if (!m_workingDir.empty() && m_workingDir.back() != PathSeparator)
			m_workingDir += PathSeparator;
		m_buildStart = buildStartTime;

		m_pathBuf.reserve(64  * 1024);
		m_depsBuf.reserve(128 * 1024);
		m_logBuf.reserve(64  * 1024);

		// Load prior run; up-to-date edges never call RecordResult and would
		// otherwise vanish from the log on next run.
		LoggerWithWriter quiet(g_nullLogWriter, TC(""));

		{
			StringBuffer<512> logPath;
			logPath.Append(m_workingDir.c_str()).Append(TC(".ninja_log"));
			FileAccessor fa(quiet, logPath.data);
			if (fa.OpenMemoryRead(0, /*errorOnFail=*/false))
			{
				static const char header[] = "# ninja log v5\n";
				constexpr u64 headerLen = sizeof(header) - 1;
				if (fa.GetSize() >= headerLen && memcmp(fa.GetData(), header, headerLen) == 0)
				{
					const u8* p    = (const u8*)fa.GetData() + headerLen;
					const u8* pEnd = (const u8*)fa.GetData() + fa.GetSize();
					while (p < pEnd && *p == 0) ++p; // skip stray nulls from legacy off-by-one headers
					u64 tail = u64(pEnd - p);
					if (tail > 0)
					{
						m_logBuf.resize(tail);
						memcpy(m_logBuf.data(), p, tail);
					}
				}
			}
		}

		// .ninja_deps v4: compact loaded deps to latest-per-outId. Without it,
		// repeated runs grow file unboundedly (5+ GB on AOSP).
		{
			StringBuffer<512> depsPath;
			depsPath.Append(m_workingDir.c_str()).Append(TC(".ninja_deps"));
			FileAccessor fa(quiet, depsPath.data);
			if (!fa.OpenMemoryRead(0, /*errorOnFail=*/false))
				return;

			static const char header[] = "# ninjadeps\n";
			constexpr u64 headerLen = sizeof(header) - 1;
			if (fa.GetSize() < headerLen + 4 || memcmp(fa.GetData(), header, headerLen) != 0)
				return;
			u32 version;
			memcpy(&version, (const u8*)fa.GetData() + headerLen, 4);
			if (version != 4)
				return;

			const u8* fileBase = (const u8*)fa.GetData();
			const u8* p   = fileBase + headerLen + 4;
			const u8* end = fileBase + fa.GetSize();

			UnorderedMap<u32, u64> latestByOutId;
			latestByOutId.reserve(64 * 1024);

			while (p + 4 <= end)
			{
				u32 sizeWithFlag;
				memcpy(&sizeWithFlag, p, 4);
				bool isDeps = (sizeWithFlag & kDepsMask) != 0;
				u32 recordSize = sizeWithFlag & ~kDepsMask;
				if (p + 4 + recordSize > end)
					break;

				if (!isDeps)
				{
					if (recordSize < 4) break;
					u32 paddedLen = recordSize - 4;
					u32 pathLen = paddedLen;
					while (pathLen > 0 && p[4 + pathLen - 1] == 0)
						--pathLen;

					// m_pathIds keys by canonical absolute path (same shape
					// GetOrAddPath uses). Without FixPath here, run 2 can't find
					// run 1's paths and re-interns every one, doubling m_pathBuf.
					// On-disk record bytes stay relative-as-written (line below).
					#if PLATFORM_WINDOWS
					tchar wide[4096];
					int wLen = MultiByteToWideChar(CP_UTF8, 0, (const char*)p + 4, (int)pathLen,
						wide, (int)(sizeof_array(wide) - 1));
					if (wLen < 0) wLen = 0;
					StringView relView(wide, u32(wLen));
					#else
					StringView relView((const tchar*)(p + 4), pathLen);
					#endif

					StringBuffer<4096> absKey;
					if (relView.count != 0)
						FixPath(absKey, relView, StringView(m_workingDir.c_str(), u32(m_workingDir.size())));

					u32 checksum;
					memcpy(&checksum, p + 4 + paddedLen, 4);
					u32 pathId = ~checksum;
					m_pathIds.emplace(TString(absKey.data, absKey.count), pathId);
					if (pathId + 1 > m_nextPathId)
						m_nextPathId = pathId + 1;

					u64 before = m_pathBuf.size();
					m_pathBuf.resize(before + 4 + recordSize);
					memcpy(m_pathBuf.data() + before, p, 4 + recordSize);
				}
				else if (recordSize >= 12)
				{
					u32 outId;
					memcpy(&outId, p + 4, 4);
					latestByOutId[outId] = u64(p - fileBase); // last write wins
				}
				p += 4 + recordSize;
			}

			Vector<u64> keptOffsets;
			keptOffsets.reserve(latestByOutId.size());
			for (auto& kv : latestByOutId)
				keptOffsets.push_back(kv.second);
			std::sort(keptOffsets.begin(), keptOffsets.end());

			u64 totalDepsBytes = 0;
			for (u64 off : keptOffsets)
			{
				u32 sizeWithFlag;
				memcpy(&sizeWithFlag, fileBase + off, 4);
				totalDepsBytes += 4 + (sizeWithFlag & ~kDepsMask);
			}
			m_depsBuf.reserve(totalDepsBytes);
			for (u64 off : keptOffsets)
			{
				u32 sizeWithFlag;
				memcpy(&sizeWithFlag, fileBase + off, 4);
				u32 totalSize = 4 + (sizeWithFlag & ~kDepsMask);
				u64 before = m_depsBuf.size();
				m_depsBuf.resize(before + totalSize);
				memcpy(m_depsBuf.data() + before, fileBase + off, totalSize);
			}
		}
	}

	void NinjaDepsLog::MakeRelative(StringBufferBase& out, StringView absPath) const
	{
		u32 wdLen = u32(m_workingDir.size());
		if (wdLen > 0 &&
			absPath.count > wdLen &&
			memcmp(absPath.data, m_workingDir.c_str(), wdLen * sizeof(tchar)) == 0)
		{
			out.Append(absPath.data + wdLen, absPath.count - wdLen);
		}
		else
		{
			out.Append(absPath.data, absPath.count);
		}
	}

	u32 NinjaDepsLog::GetOrAddPath(StringView absPath)
	{
		{
			SCOPED_READ_LOCK(m_pathLock, rl);
			auto it = m_pathIds.find(TString(absPath.data, absPath.count));
			if (it != m_pathIds.end())
				return it->second;
		}

		SCOPED_WRITE_LOCK(m_pathLock, wl);

		TString key(absPath.data, absPath.count);
		auto it = m_pathIds.find(key);
		if (it != m_pathIds.end())
			return it->second;

		u32 id = m_nextPathId++;

		// Path record v4: u32 padded_len+4 (high bit clear) | path null-padded
		// to 4-byte boundary | u32 ~id checksum.
		// Write to m_pathBuf BEFORE m_pathIds emplace so any reader observing
		// the id is guaranteed to see the record.

		StringBuffer<512> relBuf;
		MakeRelative(relBuf, absPath);

#if PLATFORM_WINDOWS
		char narrow[4096];
		int narrowLen = WideCharToMultiByte(CP_UTF8, 0, relBuf.data, (int)relBuf.count,
		                                    narrow, (int)(sizeof(narrow) - 1), nullptr, nullptr);
		if (narrowLen < 0) narrowLen = 0;
		const char* pathStr = narrow;
		u32 pathLen = u32(narrowLen);
#else
		const char* pathStr = relBuf.data;
		u32 pathLen = relBuf.count;
#endif

		u32 paddedLen  = (pathLen + 3u) & ~3u;
		u32 recordSize = paddedLen + 4u;  // padded path + checksum

		u64 offset = m_pathBuf.size();
		m_pathBuf.resize(offset + 4u + paddedLen + 4u, u8(0));
		u8* p = m_pathBuf.data() + offset;

		u32 sizeWithFlag = recordSize;
		memcpy(p, &sizeWithFlag, 4);  p += 4;

		if (pathLen) memcpy(p, pathStr, pathLen);
		p += paddedLen;

		u32 checksum = ~id;
		memcpy(p, &checksum, 4);

		m_pathIds.emplace(std::move(key), id);
		return id;
	}

	// Returns wall-clock time in the same units as stored file mtime:
	// Windows FILETIME (100ns since 1601) / POSIX 100ns since 1970.
	static u64 NowAsFileMtime()
	{
#if PLATFORM_WINDOWS
		FILETIME ft;
		GetSystemTimeAsFileTime(&ft);
		return (u64(ft.dwHighDateTime) << 32) | u64(ft.dwLowDateTime);
#else
		timespec ts;
		clock_gettime(CLOCK_REALTIME, &ts);
		return u64(ts.tv_sec) * 10'000'000ull + u64(ts.tv_nsec / 100);
#endif
	}

	void NinjaDepsLog::RegisterStaticOutput(StringView absPath)
	{
		if (absPath.count == 0)
			return;

		StringBuffer<512> path;
		path.Append(absPath.data, absPath.count);
		NormalizePathForKey(path);

		SCOPED_WRITE_LOCK(m_outputKeysLock, wl);
		m_outputKeys.emplace(path.data, path.count);
	}

	void NinjaDepsLog::FilterDepsForOutputs(Logger& logger)
	{
		if (m_outputKeys.empty() || m_depsBuf.empty())
			return;

		u32 maxId = m_nextPathId;
		Vector<u8> isOutput;
		isOutput.resize(maxId, 0);
		u64 outputPathCount = 0;
		{
			SCOPED_READ_LOCK(m_outputKeysLock, rl);
			SCOPED_READ_LOCK(m_pathLock, pl);
			for (auto& kv : m_pathIds)
			{
				if (m_outputKeys.find(kv.first) != m_outputKeys.end())
				{
					if (kv.second < maxId)
						isOutput[kv.second] = 1;
					++outputPathCount;
				}
			}
		}

		if (outputPathCount == 0)
			return;

		Vector<u8> newDepsBuf;
		newDepsBuf.reserve(m_depsBuf.size());

		u64 totalDropped = 0;
		Vector<u32> kept;
		const u8* p   = m_depsBuf.data();
		const u8* end = p + m_depsBuf.size();
		while (p + 4 <= end)
		{
			u32 sizeWithFlag;
			memcpy(&sizeWithFlag, p, 4);
			u32 recordSize = sizeWithFlag & ~kDepsMask;
			if (p + 4 + recordSize > end)
				break;

			if ((sizeWithFlag & kDepsMask) == 0 || recordSize < 12 || (recordSize & 3) != 0)
			{
				u64 off = newDepsBuf.size();
				newDepsBuf.resize(off + 4 + recordSize);
				memcpy(newDepsBuf.data() + off, p, 4 + recordSize);
				p += 4 + recordSize;
				continue;
			}

			u32 depCount = (recordSize - 12) / 4;
			kept.clear();
			kept.reserve(depCount);
			for (u32 i = 0; i < depCount; ++i)
			{
				u32 depId;
				memcpy(&depId, p + 4 + 12 + i*4, 4);
				if (depId < maxId && isOutput[depId])
				{
					++totalDropped;
					continue;
				}
				kept.push_back(depId);
			}

			u32 keptCount = u32(kept.size());
			u32 newRecordSize = 12 + 4 * keptCount;
			u32 newSizeWithFlag = newRecordSize | kDepsMask;
			u64 off = newDepsBuf.size();
			newDepsBuf.resize(off + 4 + newRecordSize);
			u8* w = newDepsBuf.data() + off;
			memcpy(w, &newSizeWithFlag, 4);  w += 4;
			memcpy(w, p + 4,  12);           w += 12; // outId + mtimeLo + mtimeHi
			if (keptCount)
				memcpy(w, kept.data(), 4 * keptCount);

			p += 4 + recordSize;
		}

		logger.Detail(TC("NinjaDepsLog: Save-time filter dropped %llu output-as-dep refs (%llu paths flagged)"),
			(unsigned long long)totalDropped, (unsigned long long)outputPathCount);

		m_depsBuf = std::move(newDepsBuf);
	}

	void NinjaDepsLog::RegisterOutputs(const ProcessHandle& handle)
	{
		const auto& trackedOutputs = handle.GetTrackedOutputs();
		if (trackedOutputs.empty())
			return;

		// Stage lowercased keys outside the lock; only the merge runs contended.
		Vector<TString> localKeys;
		{
			BinaryReader r(trackedOutputs.data(), 0, trackedOutputs.size());
			StringBuffer<512> outPath;
			while (r.GetLeft())
			{
				outPath.Clear();
				r.ReadString(outPath);
				if (!outPath.count)
					continue;
				NormalizePathForKey(outPath);
				localKeys.emplace_back(outPath.data, outPath.count);
			}
		}

		if (localKeys.empty())
			return;

		SCOPED_WRITE_LOCK(m_outputKeysLock, wl);
		for (auto& k : localKeys)
			m_outputKeys.emplace(std::move(k));
	}

	void NinjaDepsLog::RecordResult(Logger& logger, const ProcessHandle& handle,
		const tchar* primaryOutputAbsPath, bool restat, u64 cmdHashOverride)
	{
		const auto& trackedOutputs = handle.GetTrackedOutputs();
		const auto& trackedInputs  = handle.GetTrackedInputs();

		StringBuffer<512> firstOutput;
		if (primaryOutputAbsPath && *primaryOutputAbsPath)
		{
			firstOutput.Append(primaryOutputAbsPath);
		}
		else
		{
			if (trackedOutputs.empty())
				return;
			BinaryReader r(trackedOutputs.data(), 0, trackedOutputs.size());
			if (!r.GetLeft())
				return;
			r.ReadString(firstOutput);
		}
		if (!firstOutput.count)
			return;

		u64 wallTime = handle.GetTotalWallTime();
		u64 nowTime  = GetTime();
		u64 endMs    = TimeToMs(nowTime > m_buildStart ? nowTime - m_buildStart : 0);
		u64 startMs  = endMs > TimeToMs(wallTime) ? endMs - TimeToMs(wallTime) : 0;

		// restat edges: record "now" — command may not have rewritten output,
		// stat'd mtime would falsely register older than inputs.
		u64 mtime64 = 0;
		if (restat)
		{
			mtime64 = NowAsFileMtime();
		}
		else
		{
			FileBasicInformation fbi;
			if (GetFileBasicInformation(fbi, logger, firstOutput.data, /*errorOnFail=*/false))
				mtime64 = fbi.lastWriteTime;
		}

		// Filter inputs against cross-edge written-set. Producer-before-consumer
		// is guaranteed: RegisterOutputs runs synchronously in the exit callback
		// before the consumer can be dispatched.
		Vector<u32> depIds;
		if (!trackedInputs.empty())
		{
			SCOPED_READ_LOCK(m_outputKeysLock, rl);
			BinaryReader r(trackedInputs.data(), 0, trackedInputs.size());
			StringBuffer<512> depPath;
			depIds.reserve(128);
			while (r.GetLeft())
			{
				depPath.Clear();
				r.ReadString(depPath);
				if (!depPath.count)
					continue;
				NormalizePathForKey(depPath);
				if (m_outputKeys.find(TString(depPath.data, depPath.count)) != m_outputKeys.end())
					continue;
				if (IsTransientDep(depPath.data, depPath.count))
					continue;
				depIds.push_back(GetOrAddPath(StringView(depPath.data, depPath.count)));
			}
		}

		NormalizePathForKey(firstOutput);
		u32 outId = GetOrAddPath(StringView(firstOutput.data, firstOutput.count));

		// Deps record v4: u32 size|kDepsMask | u32 outId | u32 mtimeLo | u32 mtimeHi | u32 depIds[]
		u32 depCount   = u32(depIds.size());
		u32 recordSize = 4u * (3u + depCount);
		u32 totalBytes = 4u + recordSize;

		u32 mtimeLo = u32(mtime64 & 0xFFFFFFFFu);
		u32 mtimeHi = u32(mtime64 >> 32);

		{
			Vector<u8> rec(totalBytes);
			u8* p = rec.data();
			u32 sizeWithFlag = recordSize | kDepsMask;
			memcpy(p, &sizeWithFlag, 4); p += 4;
			memcpy(p, &outId,      4); p += 4;
			memcpy(p, &mtimeLo,    4); p += 4;
			memcpy(p, &mtimeHi,    4); p += 4;
			if (depCount)
				memcpy(p, depIds.data(), 4u * depCount);

			SCOPED_WRITE_LOCK(m_depsLock, wl);
			u64 off = m_depsBuf.size();
			m_depsBuf.resize(off + totalBytes);
			memcpy(m_depsBuf.data() + off, rec.data(), totalBytes);
		}

		// Log record v5: start_ms\tend_ms\tmtime\toutput_path\tcmd_hash_hex\n
		{
			const ProcessStartInfo& si = handle.GetStartInfo();
			// cmdHashOverride is the caller's pre-Expand hash; without it we hash
			// the post-Expand arguments (identity for non-shell-wrapped callers).
			u64 cmdHashVal = cmdHashOverride;
			if (!cmdHashVal)
			{
				StringKey k = ToStringKeyNoCheck(si.arguments, TStrlen(si.arguments));
				cmdHashVal = k.a;
			}

			StringBuffer<512> relOut;
			MakeRelative(relOut, StringView(firstOutput.data, firstOutput.count));

#if PLATFORM_WINDOWS
			char narrowOut[4096];
			int narrowOutLen = WideCharToMultiByte(CP_UTF8, 0, relOut.data, (int)relOut.count,
			                                       narrowOut, (int)(sizeof(narrowOut) - 1), nullptr, nullptr);
			if (narrowOutLen < 0) narrowOutLen = 0;
			narrowOut[narrowOutLen] = '\0';
			const char* outPathStr = narrowOut;
#else
			const char* outPathStr = relOut.data;
#endif

			char logLine[4096];
			int logLineLen = snprintf(logLine, sizeof(logLine),
				"%llu\t%llu\t%llu\t%s\t%llx\n",
				(unsigned long long)startMs,
				(unsigned long long)endMs,
				(unsigned long long)mtime64,
				outPathStr,
				(unsigned long long)cmdHashVal);

			if (logLineLen > 0 && logLineLen < (int)sizeof(logLine))
			{
				SCOPED_WRITE_LOCK(m_logLock, wl);
				u64 off = m_logBuf.size();
				m_logBuf.resize(off + u64(logLineLen));
				memcpy(m_logBuf.data() + off, logLine, u64(logLineLen));
			}
		}
	}

	// Drop every-but-latest deps record per outId. Mirror of Init's load-time
	// compaction; needed at save because RecordResult appends without rewriting
	// loaded records, doubling on-disk file every run otherwise.
	static void CompactDepsBuf(Vector<u8>& depsBuf)
	{
		if (depsBuf.empty())
			return;

		UnorderedMap<u32, u64> latestByOutId;
		latestByOutId.reserve(64 * 1024);

		const u8* base = depsBuf.data();
		const u8* end  = base + depsBuf.size();
		const u8* p    = base;
		while (p + 4 <= end)
		{
			u32 sizeWithFlag;
			memcpy(&sizeWithFlag, p, 4);
			u32 recordSize = sizeWithFlag & ~kDepsMask;
			if (p + 4 + recordSize > end)
				break;
			if ((sizeWithFlag & kDepsMask) != 0 && recordSize >= 12 && (recordSize & 3) == 0)
			{
				u32 outId;
				memcpy(&outId, p + 4, 4);
				latestByOutId[outId] = u64(p - base);
			}
			p += 4 + recordSize;
		}

		Vector<u64> keptOffsets;
		keptOffsets.reserve(latestByOutId.size());
		for (auto& kv : latestByOutId)
			keptOffsets.push_back(kv.second);
		std::sort(keptOffsets.begin(), keptOffsets.end()); // preserve file order

		u64 totalBytes = 0;
		for (u64 off : keptOffsets)
		{
			u32 sizeWithFlag;
			memcpy(&sizeWithFlag, base + off, 4);
			totalBytes += 4 + (sizeWithFlag & ~kDepsMask);
		}

		Vector<u8> newBuf;
		newBuf.reserve(totalBytes);
		for (u64 off : keptOffsets)
		{
			u32 sizeWithFlag;
			memcpy(&sizeWithFlag, base + off, 4);
			u32 total = 4 + (sizeWithFlag & ~kDepsMask);
			u64 before = newBuf.size();
			newBuf.resize(before + total);
			memcpy(newBuf.data() + before, base + off, total);
		}

		depsBuf = std::move(newBuf);
	}

	// Drop every-but-latest line per output path. .ninja_log is append-only
	// by design (stock ninja periodically calls RecompactLog); we do it every
	// save — cheap text scan, ~100K lines.
	static void CompactLogBuf(Vector<u8>& logBuf)
	{
		if (logBuf.empty())
			return;

		struct LineRange { u64 start; u64 endIncludingNewline; };
		UnorderedMap<std::string, LineRange> latestByPath;
		latestByPath.reserve(64 * 1024);

		const u8* base = logBuf.data();
		const u8* end  = base + logBuf.size();
		const u8* p    = base;
		while (p < end)
		{
			const u8* lineEnd = (const u8*)memchr(p, '\n', size_t(end - p));
			const u8* contentEnd = lineEnd ? lineEnd : end;

			// Output path field sits between 3rd and 4th tabs.
			const u8* tab3 = nullptr;
			const u8* tab4 = nullptr;
			u32 tabsSeen = 0;
			for (const u8* q = p; q < contentEnd; ++q)
			{
				if (*q == '\t')
				{
					++tabsSeen;
					if (tabsSeen == 3) tab3 = q;
					else if (tabsSeen == 4) { tab4 = q; break; }
				}
			}

			if (tab3 && tab4 && tab4 > tab3 + 1)
			{
				std::string key((const char*)(tab3 + 1), size_t(tab4 - tab3 - 1));
				LineRange& lr = latestByPath[std::move(key)];
				lr.start = u64(p - base);
				lr.endIncludingNewline = u64((lineEnd ? lineEnd + 1 : end) - base);
			}

			p = lineEnd ? lineEnd + 1 : end;
		}

		Vector<LineRange> kept;
		kept.reserve(latestByPath.size());
		for (auto& kv : latestByPath)
			kept.push_back(kv.second);
		std::sort(kept.begin(), kept.end(),
			[](const LineRange& a, const LineRange& b){ return a.start < b.start; });

		u64 totalBytes = 0;
		for (auto& r : kept)
			totalBytes += (r.endIncludingNewline - r.start);

		Vector<u8> newBuf;
		newBuf.reserve(totalBytes);
		for (auto& r : kept)
		{
			u64 before = newBuf.size();
			u64 len = r.endIncludingNewline - r.start;
			newBuf.resize(before + len);
			memcpy(newBuf.data() + before, base + r.start, len);
		}

		logBuf = std::move(newBuf);
	}

	bool NinjaDepsLog::Save(Logger& logger, StringView ninjaDir)
	{
		// Drop output-as-dep refs that slipped past RecordResult's per-call
		// filter (path-form mismatch between producer's trackedOutputs and
		// consumer's trackedInputs, etc.). m_outputKeys is fully populated by
		// save time via parse-time RegisterStaticOutput + runtime RegisterOutputs.
		FilterDepsForOutputs(logger);

		u64 depsBefore = m_depsBuf.size();
		u64 logBefore  = m_logBuf.size();
		CompactDepsBuf(m_depsBuf);
		CompactLogBuf(m_logBuf);
		if (m_depsBuf.size() != depsBefore || m_logBuf.size() != logBefore)
			logger.Detail(TC("NinjaDepsLog: save-time compaction (deps %llu→%llu MB, log %llu→%llu KB)"),
				(unsigned long long)(depsBefore/(1024*1024)), (unsigned long long)(m_depsBuf.size()/(1024*1024)),
				(unsigned long long)(logBefore/1024), (unsigned long long)(m_logBuf.size()/1024));

		StringBuffer<512> dirBuf;
		dirBuf.Append(ninjaDir).EnsureEndsWithSlash();

		StringBuffer<512> depsPathBuf(dirBuf);
		depsPathBuf.Append(TC(".ninja_deps"));

		StringBuffer<512> logPathBuf(dirBuf);
		logPathBuf.Append(TC(".ninja_log"));

		{
			FileHandle fh = CreateFileW(depsPathBuf.data, GENERIC_WRITE, 0, CREATE_ALWAYS, DefaultAttributes());
			if (fh == InvalidFileHandle)
				return logger.Error(TC("NinjaDepsLog: Failed to create %s"), depsPathBuf.data);
			auto fg = MakeGuard([&]() { CloseFile(depsPathBuf.data, fh); });

			static const char depsHeader[] = "# ninjadeps\n";
			constexpr u32 depsHeaderLen = u32(sizeof(depsHeader) - 1);
			const u32 depsVersion = 4u;

			if (!WriteFile(logger, depsPathBuf.data, fh, depsHeader, depsHeaderLen))
				return false;
			if (!WriteFile(logger, depsPathBuf.data, fh, &depsVersion, 4))
				return false;

			// Path records first — deps records reference path ids so order matters.
			if (!m_pathBuf.empty())
				if (!WriteFile(logger, depsPathBuf.data, fh, m_pathBuf.data(), m_pathBuf.size()))
					return false;

			if (!m_depsBuf.empty())
				if (!WriteFile(logger, depsPathBuf.data, fh, m_depsBuf.data(), m_depsBuf.size()))
					return false;
		}

		{
			FileHandle fh = CreateFileW(logPathBuf.data, GENERIC_WRITE, 0, CREATE_ALWAYS, DefaultAttributes());
			if (fh == InvalidFileHandle)
				return logger.Error(TC("NinjaDepsLog: Failed to create %s"), logPathBuf.data);
			auto fg = MakeGuard([&]() { CloseFile(logPathBuf.data, fh); });

			static const char logHeader[] = "# ninja log v5\n";
			constexpr u32 logHeaderLen = u32(sizeof(logHeader) - 1);

			if (!WriteFile(logger, logPathBuf.data, fh, logHeader, logHeaderLen))
				return false;
			if (!m_logBuf.empty())
				if (!WriteFile(logger, logPathBuf.data, fh, m_logBuf.data(), m_logBuf.size()))
					return false;
		}

		logger.Detail(TC("Wrote %s (%u paths) and %s"),
			depsPathBuf.data, m_nextPathId, logPathBuf.data);

		return true;
	}

	// NinjaLogIndex — reader for .ninja_log (v5) and .ninja_deps (v4)

	namespace
	{
		// Char relative path → absolute-path StringKey (lowercase on Windows).
		// absScratch returns the absolute tchar path for the caller.
		StringKey RelativePathToAbsKey(const char* data, u32 len, StringView workingDir,
			StringBufferBase& tcharScratch, StringBufferBase& absScratch)
		{
			tcharScratch.count = 0;
			if (len > tcharScratch.capacity - 1)
				len = u32(tcharScratch.capacity - 1);
			tcharScratch.count = len;
			for (u32 i = 0; i < len; ++i)
			{
				char c = data[i];
				tcharScratch.data[i] = (tchar)(u8)((c == '\\' || c == '/') ? PathSeparator : c);
			}
			tcharScratch.data[tcharScratch.count] = 0;

			FixPath(absScratch.Clear(), StringView(tcharScratch.data, tcharScratch.count), workingDir);
			return CaseInsensitiveFs
				? ToStringKeyLower(StringView(absScratch.data, absScratch.count))
				: ToStringKeyNoCheck(absScratch.data, absScratch.count);
		}

		UBA_NOINLINE bool LoadLogFileImpl(Logger& logger, const char* fileData, u64 fileSize,
			const tchar* fileName, StringView workingDir, NinjaLogIndex& idx)
		{
			static const char header[] = "# ninja log v5\n";
			constexpr u64 headerLen = sizeof(header) - 1;
			if (fileSize < headerLen || memcmp(fileData, header, headerLen) != 0)
				return logger.Error(TC("%s: unexpected header (not a ninja log v5)"), fileName);

			StringBuffer<512> tcharScratch, absScratch;
			const char* p   = fileData + headerLen;
			const char* end = fileData + fileSize;

			// Each record: start_ms\tend_ms\tmtime\toutput_path\tcommand_hash_hex\n
			while (p < end)
			{
				const char* lineEnd = (const char*)memchr(p, '\n', u64(end - p));
				if (!lineEnd)
					lineEnd = end;

				const char* cursor = p;
				const char* tabs[5] = {};
				u32 tabCount = 0;
				while (cursor < lineEnd && tabCount < 5)
				{
					const char* t = (const char*)memchr(cursor, '\t', u64(lineEnd - cursor));
					if (!t)
						break;
					tabs[tabCount++] = t;
					cursor = t + 1;
				}
				if (tabCount == 4)
				{
					u64 mtime = strtoull(tabs[1] + 1, nullptr, 10);
					const char* pathStart = tabs[2] + 1;
					u32 pathLen = u32(tabs[3] - pathStart);
					const char* hashStart = tabs[3] + 1;
					u32 hashLen = u32(lineEnd - hashStart);

					StringKey outKey = RelativePathToAbsKey(pathStart, pathLen, workingDir,
						tcharScratch, absScratch);

					u64 cmdHash = 0;
					for (u32 i = 0; i < hashLen; ++i)
					{
						char c = hashStart[i];
						u8 nibble;
						if      (c >= '0' && c <= '9') nibble = u8(c - '0');
						else if (c >= 'a' && c <= 'f') nibble = u8(c - 'a' + 10);
						else if (c >= 'A' && c <= 'F') nibble = u8(c - 'A' + 10);
						else break;
						cmdHash = (cmdHash << 4) | nibble;
					}

					// Last writer wins (ninja appends; tail entry is freshest).
					NinjaLogIndex::LogEntry& e = idx.logByOutputKey[outKey];
					e.outMtime = mtime;
					e.cmdHash  = cmdHash;
				}
				// Malformed lines silently skipped — matches ninja's tolerant reader.

				p = (lineEnd < end) ? lineEnd + 1 : end;
			}
			return true;
		}

		bool LoadDepsFileImpl(Logger& logger, const u8* fileData, u64 fileSize,
			const tchar* fileName, StringView workingDir, NinjaLogIndex& idx)
		{
			static const char header[] = "# ninjadeps\n";
			constexpr u64 headerLen = sizeof(header) - 1;
			if (fileSize < headerLen + 4 || memcmp(fileData, header, headerLen) != 0)
				return logger.Error(TC("%s: unexpected header (not a ninjadeps file)"), fileName);

			u32 version;
			memcpy(&version, fileData + headerLen, 4);
			if (version != 4)
				return logger.Error(TC("%s: unsupported deps version %u (expected 4)"), fileName, version);

			// Per-file local path id (record order) → abs-path StringKey + bytes
			// (latter for stat-pass parent registration).
			struct LocalPath
			{
				StringKey key;
				u32       offset; // into idx.flatDepPathBuf
				u32       len;
			};
			Vector<LocalPath> localPaths;

			// Pre-pass: walk record headers only to size the output Vectors exactly.
			// One bandwidth-bound sweep avoids ~7 doubling reallocs on chromium's 511 MB file.
			u64 pathRecordCount = 0;
			u64 pathByteSum     = 0;  // sum of padded path bytes
			u64 depsRecordCount = 0;
			u64 depCountSum     = 0;  // sum of depCount across all deps records
			{
				const u8* hp  = fileData + headerLen + 4;
				const u8* hpe = fileData + fileSize;
				while (hp + 4 <= hpe)
				{
					u32 sizeWithFlag;
					memcpy(&sizeWithFlag, hp, 4);
					bool isPath = (sizeWithFlag & kDepsMask) == 0;
					u32 recordSize = sizeWithFlag & ~kDepsMask;
					if (hp + 4 + recordSize > hpe)
						break; // truncated tail
					if (isPath)
					{
						++pathRecordCount;
						if (recordSize >= 4)
							pathByteSum += (recordSize - 4);  // exclude 4-byte checksum
					}
					else
					{
						++depsRecordCount;
						if (recordSize >= 12 && (recordSize & 3) == 0)
							depCountSum += (recordSize - 12) / 4;
					}
					hp += 4 + recordSize;
				}
			}

			localPaths.reserve(u32(pathRecordCount));
			idx.depsByOutputKey.reserve(u32(depsRecordCount));
			idx.flatDepKeys.reserve(u32(depCountSum));
			idx.flatDepPathOffset.reserve(u32(depCountSum));
			idx.flatDepPathLen.reserve(u32(depCountSum));
			// flatDepPathBuf holds tchar-widened abs paths; ~workingDir.count+relLen
			// tchars per entry (upper bound — FixPath may trim `./`).
			idx.flatDepPathBuf.reserve(pathByteSum + pathRecordCount * u64(workingDir.count));

			StringBuffer<512> tcharScratch, absScratch;
			const u8* p   = fileData + headerLen + 4;
			const u8* end = fileData + fileSize;

			while (p + 4 <= end)
			{
				u32 sizeWithFlag;
				memcpy(&sizeWithFlag, p, 4);
				p += 4;

				bool isPath = (sizeWithFlag & kDepsMask) == 0;
				u32 recordSize = sizeWithFlag & ~kDepsMask;

				// Tolerate partial trailing record (cancelled mid-Save).
				if (p + recordSize > end)
					break;

				if (isPath)
				{
					if (recordSize < 4)
						break;
					u32 paddedLen = recordSize - 4;

					u32 pathLen = paddedLen;
					while (pathLen > 0 && p[pathLen - 1] == 0)
						--pathLen;

					u32 checksum;
					memcpy(&checksum, p + paddedLen, 4);
					u32 expectedId = ~checksum;
					if (expectedId != localPaths.size())
					{
						return logger.Error(TC("%s: path record id mismatch (got %u, expected %u)"),
							fileName, expectedId, u32(localPaths.size()));
					}

					StringKey key = RelativePathToAbsKey((const char*)p, pathLen, workingDir,
						tcharScratch, absScratch);

					u32 bufOff = u32(idx.flatDepPathBuf.size());
					idx.flatDepPathBuf.resize(bufOff + absScratch.count);
					memcpy(idx.flatDepPathBuf.data() + bufOff, absScratch.data, absScratch.count * sizeof(tchar));

					LocalPath lp;
					lp.key    = key;
					lp.offset = bufOff;
					lp.len    = absScratch.count;
					localPaths.push_back(lp);
				}
				else
				{
					if (recordSize < 12 || (recordSize & 3) != 0)
						return logger.Error(TC("%s: malformed deps record (size %u)"), fileName, recordSize);

					u32 outLocalId;
					u32 mtimeLo;
					u32 mtimeHi;
					memcpy(&outLocalId, p +  0, 4);
					memcpy(&mtimeLo,    p +  4, 4);
					memcpy(&mtimeHi,    p +  8, 4);

					u32 depCount = (recordSize - 12) / 4;
					if (outLocalId >= localPaths.size())
						return logger.Error(TC("%s: deps record references unknown path id %u"),
							fileName, outLocalId);

					StringKey outKey = localPaths[outLocalId].key;
					u64 mtime = (u64(mtimeHi) << 32) | u64(mtimeLo);

					u32 depOffset = u32(idx.flatDepKeys.size());
					idx.flatDepKeys.resize(depOffset + depCount);
					idx.flatDepPathOffset.resize(depOffset + depCount);
					idx.flatDepPathLen.resize(depOffset + depCount);
					for (u32 i = 0; i < depCount; ++i)
					{
						u32 depLocalId;
						memcpy(&depLocalId, p + 12 + i * 4, 4);
						if (depLocalId >= localPaths.size())
							return logger.Error(TC("%s: deps record references unknown dep id %u"),
								fileName, depLocalId);
						const LocalPath& lp = localPaths[depLocalId];
						idx.flatDepKeys[depOffset + i]       = lp.key;
						idx.flatDepPathOffset[depOffset + i] = lp.offset;
						idx.flatDepPathLen[depOffset + i]    = lp.len;
					}

					NinjaLogIndex::DepsEntry& e = idx.depsByOutputKey[outKey];
					e.outMtime  = mtime;
					e.depOffset = depOffset;
					e.depCount  = depCount;
				}

				p += recordSize;
			}
			return true;
		}
	}

	bool NinjaLogIndex::Load(Logger& logger, StringView ninjaDir, StringView workingDir)
	{
		{
			StringBuffer<512> logPath(ninjaDir);
			logPath.EnsureEndsWithSlash().Append(TC(".ninja_log"));

			FileAccessor fa(logger, logPath.data);
			if (fa.OpenMemoryRead(0, /*errorOnFail=*/false))
			{
				// Exact line count via memchr to pre-size the map and avoid rehashing.
				u64 lineCount = 0;
				{
					const char* lp  = (const char*)fa.GetData();
					const char* lpe = lp + fa.GetSize();
					while (lp < lpe)
					{
						const char* nl = (const char*)memchr(lp, '\n', size_t(lpe - lp));
						if (!nl) break;
						++lineCount;
						lp = nl + 1;
					}
				}
				logByOutputKey.reserve(u32(lineCount));
				if (!LoadLogFileImpl(logger, (const char*)fa.GetData(), fa.GetSize(),
					logPath.data, workingDir, *this))
					return false;
			}
		}

		{
			StringBuffer<512> depsPath(ninjaDir);
			depsPath.EnsureEndsWithSlash().Append(TC(".ninja_deps"));

			FileAccessor fa(logger, depsPath.data);
			if (fa.OpenMemoryRead(0, /*errorOnFail=*/false))
			{
				if (!LoadDepsFileImpl(logger, fa.GetData(), fa.GetSize(),
					depsPath.data, workingDir, *this))
					return false;
			}
		}

		return true;
	}
}
