// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaHash.h"
#include "UbaSynchronization.h"
#include "UbaStringBuffer.h"
#include "UbaUnorderedMap.h"
#include "UbaVector.h"

namespace uba
{
	class Logger;
	class ProcessHandle;

	// Parsed view of a previous run's .ninja_log (v5) and .ninja_deps (v4) files.
	//
	// All lookups are keyed by a StringKey = hash of the *absolute* path (lowercased
	// on case-insensitive filesystems). That matches the keying used for the mtime
	// cache, so the up-to-date check is a single hash-table find per query.
	//
	// Both ninja log files use char-based paths and so does this reader; we widen
	// to tchar only when calling FixPath / ToStringKey.
	//
	// Load once on a single thread during startup; read freely after.
	struct NinjaLogIndex
	{
		struct LogEntry
		{
			u64 outMtime = 0; // mtime recorded in .ninja_log at end of last successful build
			u64 cmdHash  = 0; // hash of the command used last time (low 64 bits of StringKey)
		};
		struct DepsEntry
		{
			u64 outMtime  = 0; // mtime of the output when these deps were captured
			u32 depOffset = 0; // index into flatDepKeys
			u32 depCount  = 0;
		};

		UnorderedMap<StringKey, LogEntry>  logByOutputKey;
		UnorderedMap<StringKey, DepsEntry> depsByOutputKey;
		// referenced by DepsEntry.depOffset/depCount
		Vector<StringKey>                  flatDepKeys;
		// tchar buffer holding absolute paths for each entry in flatDepKeys.
		// flatDepPathOffsets[i]..[i]+len covers bytes in flatDepPathBuf for dep i.
		// We keep the paths because phase 1 of the up-to-date check needs them
		// to register parent directories for the stat pass.
		Vector<tchar>                      flatDepPathBuf;
		Vector<u32>                        flatDepPathOffset;
		Vector<u32>                        flatDepPathLen;

		// Parse .ninja_log and .ninja_deps from ninjaDir. Missing files are treated
		// as "no prior build" and return true with empty maps — not an error.
		// Paths in the log are stored relative to workingDir; we resolve them
		// against workingDir to get absolute paths before hashing.
		bool Load(Logger& logger, StringView ninjaDir, StringView workingDir);
	};

	// Writes .ninja_deps (v4) and .ninja_log (v5) files during a UbaNinja build.
	// All public methods are thread-safe.
	//
	// Usage:
	//   NinjaDepsLog log;
	//   log.Init(workingDir, buildStartTime);
	//   // In process-finished callback (for each successful process):
	//   log.RecordResult(logger, handle);
	//   // After build completes:
	//   log.Save(logger, ninjaDir);
	class NinjaDepsLog
	{
	public:
		NinjaDepsLog();
		~NinjaDepsLog();

		// Call before any RecordResult calls.
		// workingDir must end with a path separator.
		void Init(StringView workingDir, u64 buildStartTime);

		// Add handle.GetTrackedOutputs() to cross-edge written-set. MUST be
		// called synchronously from the exit callback before any async
		// RecordResult dispatch — producer-before-consumer ordering relies on it.
		void RegisterOutputs(const ProcessHandle& handle);

		// Register one statically-declared edge output (absolute path) into the
		// cross-edge set. Called from EnqueueCommands BEFORE any edge runs, so
		// the set is complete the moment RecordResult executes — independent of
		// run order, cache state, or partial-build failures.
		void RegisterStaticOutput(StringView absPath);

		// Walk kept deps records and drop dep_ids whose path is in m_outputKeys.
		// Belt-and-braces for anything that slipped past the per-call filter.
		void FilterDepsForOutputs(Logger& logger);

		// Record deps + timing for a successful process (exit code == 0).
		// primaryOutputAbsPath, when non-null, is the edge's declared primary
		// output (use it to avoid keying off a sidecar like .pdb / .rc.res.d).
		// restat=true (rule has restat=1 / phony_output): mtime stored as Now()
		// so a no-op rerun doesn't dirty-mark the edge next run.
		// cmdHashOverride, non-zero: stored verbatim; otherwise we hash
		// si.arguments. Callers know the pre-Expand command; arguments here may
		// be post-Expand (sh -c "..."), so the override avoids coupling the log
		// format to Expand()'s transformations.
		void RecordResult(Logger& logger, const ProcessHandle& handle,
			const tchar* primaryOutputAbsPath = nullptr,
			bool restat = false,
			u64 cmdHashOverride = 0);

		// ninjaDir must end with a path separator.
		bool Save(Logger& logger, StringView ninjaDir);

	private:
		// Returns the path ID; writes a path record to m_pathBuf if new.
		u32 GetOrAddPath(StringView absPath);

		// Strip m_workingDir prefix from absPath if present; otherwise copy as-is.
		void MakeRelative(StringBufferBase& out, StringView absPath) const;

		TString           m_workingDir;      // trailing separator

		// Path records appended to m_pathBuf under write lock BEFORE id is
		// inserted into m_pathIds — any reader observing the id sees the record.
		ReaderWriterLock  m_pathLock;
		UnorderedMap<TString, u32> m_pathIds;
		Vector<u8>        m_pathBuf;
		u32               m_nextPathId = 0;

		ReaderWriterLock  m_depsLock;
		Vector<u8>        m_depsBuf;

		ReaderWriterLock  m_logLock;
		Vector<u8>        m_logBuf;

		// Absolute paths (lowercased on case-insensitive FS) of every file any
		// edge in this build produces. RecordResult consults this to drop
		// intermediate-artifact inputs (*.mojom-shared.h etc.).
		ReaderWriterLock        m_outputKeysLock;
		UnorderedSet<TString>   m_outputKeys;

		u64               m_buildStart = 0;
	};
}
