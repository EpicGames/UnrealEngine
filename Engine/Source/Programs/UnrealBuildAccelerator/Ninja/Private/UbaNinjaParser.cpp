// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaNinjaParser.h"
#include "UbaEnvironment.h"
#include "UbaFile.h"
#include "UbaFileAccessor.h"
#include "UbaLogger.h"
#include "UbaPlatform.h"
#include "UbaStringBuffer.h"
#include "UbaThread.h"
#include "UbaTimer.h"
#include <atomic>

namespace uba
{
	// Delimiter lookup for ParseBuild hot path (1 array lookup vs 7 compares).
	alignas(64) static const bool g_isDelimiter[256] = {
		0,0,0,0,0,0,0,0,0,1,1,0,0,1,0,0,  // 0-15:  \t=9, \n=10, \r=13
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 16-31
		1,0,0,0,1,0,0,0,0,0,0,0,0,0,0,0,  // 32-47: space=32, $=36
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 48-63
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 64-79
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 80-95
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 96-111
		0,0,0,0,0,0,0,0,0,0,0,0,1,0,0,0,  // 112-127: |=124
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 128-143
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 144-159
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 160-175
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 176-191
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 192-207
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 208-223
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 224-239
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0   // 240-255
	};

	// Separator lookup (Windows only - POSIX needs just '/').
#if PLATFORM_WINDOWS
	alignas(64) static const bool g_isSep[256] = {
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 0-15
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 16-31
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,1,  // 32-47: '/' = 47
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 48-63
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 64-79
		0,0,0,0,0,0,0,0,0,0,0,0,1,0,0,0,  // 80-95: '\\' = 92
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 96-111
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,  // 112-127
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,
		0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,
	};
#define UBA_IS_SEP(c) g_isSep[(u8)(c)]
#else
#define UBA_IS_SEP(c) ((c) == '/')
#endif

	// Strip leading "./" and collapse mid-path "<sep>.<sep>".
	// Matches ninja's CanonicalizePath for these specific patterns.
	static u32 StripDotSlash(char* data, u32 len)
	{
		if (len < 2) return len;

		u32 src = 0;
		while (src + 2 <= len && data[src] == '.' && UBA_IS_SEP(data[src + 1]))
			src += 2;

		// Use memchr to jump '.' to '.' then verify neighbors; most tokens have
		// no '.' or '.' only in filenames so the scan returns cheaply.
		u32 midSrc = len; // sentinel: not found
		{
			const char* scan = data + src;
			const char* end  = data + len;
			while (scan + 1 < end)
			{
				const char* dot = (const char*)memchr(scan, '.', size_t(end - scan) - 1);
				if (!dot) break;
				u32 pos = u32(dot - data);
				// data[pos+1] valid (memchr length-1); data[pos-1] guarded by pos>0.
				if (UBA_IS_SEP(data[pos + 1]) && pos > 0 && UBA_IS_SEP(data[pos - 1]))
				{
					midSrc = pos;
					break;
				}
				scan = dot + 1;
			}
		}

		if (midSrc == len && src == 0)
			return len;

		if (midSrc == len)
		{
			len -= src;
			memmove(data, data + src, len);
			return len;
		}

		u32 dst = midSrc - src;
		if (src > 0)
			memmove(data, data + src, dst);

		u32 s = midSrc + 2;
		while (s + 1 < len && data[s] == '.' && UBA_IS_SEP(data[s + 1]))
			s += 2;

		// Bulk-copy remainder, collapsing further "<sep>.<sep>".
#if !PLATFORM_WINDOWS
		while (s < len)
		{
			const char* nextSep = (const char*)memchr(data + s, '/', len - s);
			if (!nextSep)
			{
				memmove(data + dst, data + s, len - s);
				dst += len - s;
				break;
			}
			u32 sepPos = u32(nextSep - data);
			u32 span = sepPos - s + 1; // include separator
			memmove(data + dst, data + s, span);
			dst += span;
			s = sepPos + 1;
			while (s + 1 < len && data[s] == '.' && data[s + 1] == '/')
				s += 2;
		}
#else
		// Windows: two-pointer byte loop (dual-sep set makes memchr less clean).
		while (s < len)
		{
			char c = data[s++];
			data[dst++] = c;
			if (UBA_IS_SEP(c) && s + 1 < len && data[s] == '.' && UBA_IS_SEP(data[s + 1]))
				s += 2;
		}
#endif

		return dst;
	}

	NinjaParser::NinjaParser(MemoryBlock& memoryBlock)
		: m_memoryBlock(memoryBlock)
		, m_stringPool(memoryBlock)
		, m_tokenArena(TC("NinjaTokens"))
		, m_commandId(StringId_Empty)
		, m_descriptionId(StringId_Empty)
		, m_depfileId(StringId_Empty)
		, m_depsId(StringId_Empty)
		, m_rspfileId(StringId_Empty)
		, m_rspfile_contentId(StringId_Empty)
		, m_poolId(StringId_Empty)
		, m_restatId(StringId_Empty)
		, m_generatorId(StringId_Empty)
		, m_depthId(StringId_Empty)
		, m_inId(StringId_Empty)
		, m_outId(StringId_Empty)
		, m_phonyOutputId(StringId_Empty)
	{
		// 1 GB virtual reserve, lazy commit. Android: ~5M u32 tokens = 20 MB.
		m_tokenArena.Init(1ull * 1024 * 1024 * 1024);

		// Sized for largest realistic builds (chromium / AOSP).
		m_edges.reserve(200000);
		m_rules.reserve(12000);
		m_pools.reserve(100);
		m_globalVars.reserve(120000);
		m_ruleMap.reserve(12000);
		m_poolMap.reserve(100);
		// StringPool capacity must hold full build — pool cannot grow under
		// concurrent Intern/GetString. AOSP interns ~3M+ strings.
		m_stringPool.Reserve(8 * 1024 * 1024);

		// Pre-reserve so phase 2 nested-subninja push_backs don't reallocate
		// while other workers hold references into m_scopeVars[i].
		m_scopeVars.reserve(65536);
		m_scopeParents.reserve(65536);

		m_commandId          = m_stringPool.Intern("command", 7);
		m_descriptionId      = m_stringPool.Intern("description", 11);
		m_depfileId          = m_stringPool.Intern("depfile", 7);
		m_depsId             = m_stringPool.Intern("deps", 4);
		m_rspfileId          = m_stringPool.Intern("rspfile", 7);
		m_rspfile_contentId  = m_stringPool.Intern("rspfile_content", 15);
		m_poolId             = m_stringPool.Intern("pool", 4);
		m_restatId           = m_stringPool.Intern("restat", 6);
		m_generatorId        = m_stringPool.Intern("generator", 9);
		m_depthId            = m_stringPool.Intern("depth", 5);
		m_inId               = m_stringPool.Intern("in", 2);
		m_outId              = m_stringPool.Intern("out", 3);
		m_phonyOutputId      = m_stringPool.Intern("phony_output", 12);
	}

	NinjaParser::~NinjaParser()
	{
		for (MemoryBlock* a : m_workerArenas)
			delete a;
	}

	bool NinjaParser::Parse(Logger& logger, const tchar* filename, const tchar* buildRoot)
	{
		u64 startTime = GetTime();

		// Set build root BEFORE ParseFile so its first-call default is skipped.
		if (buildRoot && *buildRoot)
		{
			StringBuffer<> root;
			root.Append(buildRoot);
			root.EnsureEndsWithSlash();
			Vector<char> rootUtf8;
			rootUtf8.reserve(root.count);
			for (u32 i = 0; i < root.count; ++i)
				rootUtf8.push_back((char)(u8)root.data[i]);
			m_buildRootDirId = m_stringPool.Intern(rootUtf8.data(), u32(rootUtf8.size()));
		}

		// Phase 1: top-level + include'd files write directly into parser members.
		// Subninjas are queued for phase 2 parallel parsing.
		Vector<SubninjaWork> subninjaQueue;
		subninjaQueue.reserve(1024);

		ParseOutputs topOutputs { &m_edges, &m_rules, &m_pools, &m_defaults };
		SubninjaQueueRef phase1Sub { &subninjaQueue, nullptr, nullptr };
		bool success = ParseFile(logger, filename, 0, topOutputs, phase1Sub);
		if (!success)
		{
			m_parseTimeMs = TimeToMs(GetTime() - startTime);
			return false;
		}

		// Phase 2: work-stealing parallel parse. Nested subninjas are pushed
		// back to the shared queue so idle workers can claim them — important
		// for chromium-style "one top-level includes hundreds" graphs.
		const u32 initialSubninjaCount = u32(subninjaQueue.size());
		if (initialSubninjaCount > 0)
		{
			u32 logicalCpus = GetLogicalProcessorCount();
			u32 workerCount = logicalCpus;
			if (workerCount > 32) workerCount = 32;
			if (workerCount < 1)  workerCount = 1;
			// Pre-reserve: phase 2 push_back must not reallocate while another
			// worker reads queue[idx]. Reads + writes are both under queueLock.
			subninjaQueue.reserve(initialSubninjaCount + 65536);

			struct WorkerState
			{
				Vector<NinjaBuildEdge> edges;
				Vector<NinjaRule>      rules;
				Vector<NinjaPool>      pools;
				Vector<u32>            defaults;
				bool                   ok = true;
			};
			Vector<WorkerState> states(workerCount);

			// Per-worker arenas: 1 GB virtual reserve each. Parser owns lifetime
			// because TokenSpan pointers reference into them.
			m_workerArenas.reserve(m_workerArenas.size() + workerCount);
			u32 firstWorkerArenaIdx = u32(m_workerArenas.size());
			for (u32 w = 0; w < workerCount; ++w)
			{
				MemoryBlock* arena = new MemoryBlock(TC("NinjaTokensWorker"));
				arena->Init(1ull * 1024 * 1024 * 1024);
				m_workerArenas.push_back(arena);
			}

			Futex            queueLock;
			u32              readIdx     = 0;
			std::atomic<u32> outstanding { initialSubninjaCount };
			std::atomic<u32> failedFlag  { 0 };

			SubninjaQueueRef phase2Sub { &subninjaQueue, &queueLock, &outstanding };

			Vector<Thread*> workers;
			workers.reserve(workerCount);
			for (u32 w = 0; w < workerCount; ++w)
			{
				workers.push_back(new Thread([&, w]() -> u32
				{
					WorkerState& st = states[w];
					ParseOutputs out { &st.edges, &st.rules, &st.pools, &st.defaults };
					MemoryBlock* myArena = m_workerArenas[firstWorkerArenaIdx + w];
					while (true)
					{
						SubninjaWork sw;
						bool gotWork = false;
						{
							SCOPED_FUTEX(queueLock, qlock);
							if (readIdx < subninjaQueue.size())
							{
								sw = std::move(subninjaQueue[readIdx]);
								++readIdx;
								gotWork = true;
							}
						}

						if (!gotWork)
						{
							// outstanding == 0 means everything done; otherwise yield + retry.
							if (outstanding.load(std::memory_order_acquire) == 0)
								break;
							Sleep(0);
							continue;
						}

						bool ok = ParseFile(logger, sw.filename.c_str(), sw.scopeId, out, phase2Sub, myArena);
						outstanding.fetch_sub(1, std::memory_order_release);
						if (!ok)
						{
							st.ok = false;
							failedFlag.fetch_or(1u, std::memory_order_relaxed);
							break;
						}
					}
					return 0;
				}, TC("NinjaSubparse")));
			}
			for (Thread* t : workers) t->Wait();
			for (Thread* t : workers) delete t;

			if (failedFlag.load(std::memory_order_relaxed))
			{
				m_parseTimeMs = TimeToMs(GetTime() - startTime);
				return false;
			}

			// Merge worker-local outputs into parser. Fixed worker order gives
			// deterministic edgeIndex assignment (diagnostic stability only).
			u32 extraEdges    = 0;
			u32 extraRules    = 0;
			u32 extraPools    = 0;
			u32 extraDefaults = 0;
			for (u32 w = 0; w < workerCount; ++w)
			{
				extraEdges    += u32(states[w].edges.size());
				extraRules    += u32(states[w].rules.size());
				extraPools    += u32(states[w].pools.size());
				extraDefaults += u32(states[w].defaults.size());
			}

			m_edges.reserve(m_edges.size() + extraEdges);
			m_rules.reserve(m_rules.size() + extraRules);
			m_pools.reserve(m_pools.size() + extraPools);
			m_defaults.reserve(m_defaults.size() + extraDefaults);
			m_ruleMap.reserve(m_rules.size() + extraRules);
			m_poolMap.reserve(m_pools.size() + extraPools);

			for (u32 w = 0; w < workerCount; ++w)
			{
				WorkerState& st = states[w];

				for (NinjaRule& r : st.rules)
				{
					u32 idx = u32(m_rules.size());
					m_ruleMap[r.name] = idx;
					m_rules.emplace_back(std::move(r));
				}
				for (NinjaPool& p : st.pools)
				{
					u32 idx = u32(m_pools.size());
					m_poolMap[p.name] = idx;
					m_pools.push_back(p);
				}
				for (u32 d : st.defaults)
					m_defaults.push_back(d);

				for (NinjaBuildEdge& e : st.edges)
				{
					u32 edgeIndex = u32(m_edges.size());
					e.edgeIndex = edgeIndex;
					m_edges.emplace_back(std::move(e));
					if (m_edgeParsedCallback)
						m_edgeParsedCallback(m_edgeCallbackUserData, edgeIndex, m_edges[edgeIndex]);
				}
			}
		}

		m_parseTimeMs = TimeToMs(GetTime() - startTime);
		return true;
	}

	u32 NinjaParser::AllocateScopeId(u32 parentScopeId)
	{
		SCOPED_FUTEX(m_scopeAllocLock, lock);
		u32 newScopeId = u32(m_scopeVars.size()) + 1;
		m_scopeVars.emplace_back();
		m_scopeParents.push_back(parentScopeId);
		return newScopeId;
	}

	bool NinjaParser::ParseFile(Logger& logger, const tchar* filename, u32 scopeId,
	                             ParseOutputs outputs, SubninjaQueueRef subninja,
	                             MemoryBlock* tokenArena)
	{
		FileAccessor fa(logger, filename);
		if (!fa.OpenMemoryRead())
			return false;

		// Ninja files are UTF-8/ASCII; work with char* directly.
		const char* fileData = (const char*)fa.GetData();
		u64 fileSize = fa.GetSize();

		PrefetchVirtualMemory(fileData, fileSize);

		ParseContext ctx { logger };
		ctx.data            = fileData;
		ctx.size            = fileSize;
		ctx.pos             = 0;
		ctx.line            = 1;
		ctx.scopeId         = scopeId;
		ctx.outputs         = outputs;
		ctx.subninja        = subninja;
		ctx.tokenArena      = tokenArena ? tokenArena : &m_tokenArena;
		ctx.tokenBuffer.resize(2048); // pre-size so common paths don't allocate

		StringBuffer<> tempDir;
		tempDir.Append(filename);
		const tchar* lastSep = TStrrchr(tempDir.data, PathSeparator);
		if (lastSep)
		{
			tempDir.count = u32(lastSep - tempDir.data + 1);
			tempDir.data[tempDir.count] = 0;
		}
		else
		{
			if (!GetCurrentDirectoryW(tempDir))
				return false;
			tempDir.EnsureEndsWithSlash();
		}

		// Stack buffer; ninja dir paths are well under 1 KB.
		char dirUtf8[1024];
		u32 dirLen = tempDir.count < (u32)sizeof(dirUtf8) ? tempDir.count : u32(sizeof(dirUtf8) - 1);
		for (u32 i = 0; i < dirLen; ++i)
			dirUtf8[i] = (char)(u8)tempDir.data[i];

		ctx.workingDirId = m_stringPool.Intern(dirUtf8, dirLen);

		// First (top-level) call sets build root; safe pre-worker write.
		if (m_buildRootDirId == StringId_Empty)
			m_buildRootDirId = ctx.workingDirId;

		bool success = ParseImpl(ctx);
		m_lineCount.fetch_add(ctx.line, std::memory_order_relaxed);
		return success;
	}

	bool NinjaParser::ParseImpl(ParseContext& ctx)
	{
		const char* line;
		u32 lineLen;

		while (ReadLine(ctx, line, lineLen))
		{
			if (lineLen == 0)
				continue;

			// Dispatch on first char; 'build' is by far the most common.
			bool ok = true;
			switch (line[0])
			{
			case 'b':
				if (lineLen >= 6 && memcmp(line, "build ", 6) == 0)
					ok = ParseBuild(ctx, line, lineLen);
				else
					ok = ParseVariable(ctx, line, lineLen);
				break;
			case 'r':
				if (lineLen >= 5 && memcmp(line, "rule ", 5) == 0)
					ok = ParseRule(ctx, line, lineLen);
				else
					ok = ParseVariable(ctx, line, lineLen);
				break;
			case 'i':
				if (lineLen >= 8 && memcmp(line, "include ", 8) == 0)
					ok = ParseInclude(ctx, line, lineLen);
				else
					ok = ParseVariable(ctx, line, lineLen);
				break;
			case 'd':
				if (lineLen >= 8 && memcmp(line, "default ", 8) == 0)
					ok = ParseDefault(ctx, line, lineLen);
				else
					ok = ParseVariable(ctx, line, lineLen);
				break;
			case 'p':
				if (lineLen >= 5 && memcmp(line, "pool ", 5) == 0)
					ok = ParsePool(ctx, line, lineLen);
				else
					ok = ParseVariable(ctx, line, lineLen);
				break;
			case 's':
				if (lineLen >= 9 && memcmp(line, "subninja ", 9) == 0)
					ok = ParseSubninja(ctx, line, lineLen);
				else
					ok = ParseVariable(ctx, line, lineLen);
				break;
			default:
				ok = ParseVariable(ctx, line, lineLen);
				break;
			}
			if (!ok)
				return false;
		}

		return true;
	}

	bool NinjaParser::ParseRule(ParseContext& ctx, const char* line, u32 lineLen)
	{
		const char* nameStart = line + 5; // skip "rule "
		const char* nameEnd = nameStart;
		while (nameEnd < line + lineLen && *nameEnd != ' ' && *nameEnd != '\t')
			++nameEnd;

		u32 nameId = m_stringPool.Intern(nameStart, u32(nameEnd - nameStart));

		NinjaRule rule;
		rule.name = nameId;

		UnorderedMap<u32, u32> props;
		if (!ReadIndentedLines(ctx, props))
			return false;

		for (auto& kv : props)
		{
			if (kv.first == m_commandId)
				rule.command = kv.second;
			else if (kv.first == m_descriptionId)
				rule.description = kv.second;
			else if (kv.first == m_depfileId)
				rule.depfile = kv.second;
			else if (kv.first == m_depsId)
				rule.deps = kv.second;
			else if (kv.first == m_rspfileId)
				rule.rspfile = kv.second;
			else if (kv.first == m_rspfile_contentId)
				rule.rspfile_content = kv.second;
			else if (kv.first == m_poolId)
				rule.pool = kv.second;
			else if (kv.first == m_restatId)
			{
				const char* valueStr = m_stringPool.GetCStr(kv.second);
				rule.restat = (strcmp(valueStr, "1") == 0 || strcmp(valueStr, "true") == 0);
			}
			else if (kv.first == m_generatorId)
			{
				const char* valueStr = m_stringPool.GetCStr(kv.second);
				rule.generator = (strcmp(valueStr, "1") == 0 || strcmp(valueStr, "true") == 0);
			}
		}

		// Phase 1 updates m_ruleMap directly; phase 2 does so during merge.
		u32 idx = u32(ctx.outputs.rules->size());
		if (ctx.outputs.rules == &m_rules)
			m_ruleMap[rule.name] = idx;
		ctx.outputs.rules->emplace_back(std::move(rule));

		return true;
	}

	bool NinjaParser::ParseBuild(ParseContext& ctx, const char* line, u32 lineLen)
	{
		// "build <outputs>: <rule> <inputs> [| <implicit>] [|| <order-only>] [|@ <validations>]"
		NinjaBuildEdge edge;

		// Token arena helpers. Per-edge sections fill sequentially → contiguous slices.
		MemoryBlock* arena = ctx.tokenArena;
		auto arenaTop = [&]() -> u32* {
			return reinterpret_cast<u32*>(arena->memory + arena->writtenSize);
		};
		auto pushTokenToArena = [&](u32 tokenId) {
			u32* slot = (u32*)arena->AllocateNoLock(sizeof(u32), alignof(u32), TC(""));
			*slot = tokenId;
		};
		auto makeSpan = [](u32* start, u32* stop) -> TokenSpan {
			return { start, u32(stop - start) };
		};

		const char* p = line + 6; // skip "build "
		const char* end = line + lineLen;

		// Skip $: escapes when looking for the outputs:rule separator.
		const char* colonPos = p;
		while (colonPos < end)
		{
			if (*colonPos == ':')
				break;
			if (*colonPos == '$' && colonPos + 1 < end)
				colonPos++;  // skip char after `$` (`$:`, `$ `, `$$`, `$\n`, ...)
			colonPos++;
		}
		if (colonPos >= end)
			return false;

		// `|` (not `||`) before `:` separates regular from implicit outputs.
		// `$|` is not valid ninja syntax so no escape-scan needed here.
		const char* pipePosInOutputs = nullptr;
		for (const char* q = p; q < colonPos - 1; ++q)
		{
			if (q[0] == '|' && q[1] != '|')
			{
				pipePosInOutputs = q;
				break;
			}
		}

		// SplitPaths writes tokens to a scratch Vector<u32>, then we copy each
		// list into the arena as one contiguous span. Per-edge counts are small.
		Vector<u32> stage;
		if (pipePosInOutputs)
		{
			stage.clear();
			SplitPaths(ctx, p, u32(pipePosInOutputs - p), stage);
			u32* s = arenaTop();
			for (u32 id : stage) pushTokenToArena(id);
			edge.outputs = makeSpan(s, arenaTop());

			const char* implStart = pipePosInOutputs + 1;
			stage.clear();
			SplitPaths(ctx, implStart, u32(colonPos - implStart), stage);
			s = arenaTop();
			for (u32 id : stage) pushTokenToArena(id);
			edge.implicitOutputs = makeSpan(s, arenaTop());
		}
		else
		{
			stage.clear();
			SplitPaths(ctx, p, u32(colonPos - p), stage);
			u32* s = arenaTop();
			for (u32 id : stage) pushTokenToArena(id);
			edge.outputs = makeSpan(s, arenaTop());
		}
		p = colonPos + 1;

		while (p < end && (*p == ' ' || *p == '\t'))
			++p;

		const char* ruleStart = p;
		while (p < end && *p != ' ' && *p != '\t')
			++p;

		edge.ruleName = m_stringPool.Intern(ruleStart, u32(p - ruleStart));

		// Parse inputs / dependencies. Tokens go directly into the arena; per-section
		// (start, top) gets recorded so each TokenSpan slice is contiguous.
		// Delimiters: `|` implicit, `||` order-only, `|@` validations (AOSP).
		enum { INPUTS, IMPLICIT, ORDER_ONLY, VALIDATIONS, NUM_SECTIONS } state = INPUTS;
		u32* sectionStart[NUM_SECTIONS] = { nullptr, nullptr, nullptr, nullptr };
		u32* sectionEnd  [NUM_SECTIONS] = { nullptr, nullptr, nullptr, nullptr };
		sectionStart[INPUTS] = arenaTop();

		u32 tokenSize = 0;
		// Dot-free tokens skip StripDotSlash entirely.
		bool tokenHasDot = false;

		while (p < end)
		{
			if (*p == '$' && p + 1 < end)
			{
				// $<space|:|$|newline> escapes; otherwise keep the `$`.
				++p;
				if (*p == ' ' || *p == ':' || *p == '$')
				{
					if (tokenSize >= ctx.tokenBuffer.size())
						ctx.tokenBuffer.resize(tokenSize * 2);
					ctx.tokenBuffer[tokenSize++] = *p;
					++p;
				}
				else if (*p == '\n' || *p == '\r')
				{
					++p;
					if (p < end && p[-1] == '\r' && *p == '\n')
						++p;
				}
				else
				{
					if (tokenSize >= ctx.tokenBuffer.size())
						ctx.tokenBuffer.resize(tokenSize * 2);
					ctx.tokenBuffer[tokenSize++] = '$';
				}
			}
			else if (*p == ' ' || *p == '\t')
			{
				if (tokenSize > 0)
				{
					if (tokenHasDot)
						tokenSize = StripDotSlash(ctx.tokenBuffer.data(), tokenSize);
					if (tokenSize > 0)
					{
						u32 tokenId = m_stringPool.Intern(ctx.tokenBuffer.data(), tokenSize);
						pushTokenToArena(tokenId);
					}
					tokenSize = 0;
					tokenHasDot = false;
				}
				++p;
			}
			else if (*p == '|')
			{
				if (tokenSize > 0)
				{
					if (tokenHasDot)
						tokenSize = StripDotSlash(ctx.tokenBuffer.data(), tokenSize);
					if (tokenSize > 0)
					{
						u32 tokenId = m_stringPool.Intern(ctx.tokenBuffer.data(), tokenSize);
						pushTokenToArena(tokenId);
					}
					tokenSize = 0;
					tokenHasDot = false;
				}

				// Close current section and open the next at arena top (contiguous).
				sectionEnd[state] = arenaTop();

				if (p + 1 < end && p[1] == '|')
				{
					state = ORDER_ONLY;
					p += 2;
				}
				else if (p + 1 < end && p[1] == '@')
				{
					state = VALIDATIONS;
					p += 2;
				}
				else
				{
					state = IMPLICIT;
					++p;
				}
				sectionStart[state] = arenaTop();

				while (p < end && (*p == ' ' || *p == '\t'))
					++p;
			}
			else
			{
				// Bulk scan via delimiter lookup. Branchless OR flags any '.'
				// so the flush path can skip StripDotSlash on dot-free tokens.
				const char* tokenStart = p;
				while (p < end && !g_isDelimiter[(u8)*p])
				{
					tokenHasDot |= (*p == '.');
					++p;
				}

				u32 len = u32(p - tokenStart);
				if (len > 0)
				{
					if (tokenSize + len > ctx.tokenBuffer.size())
						ctx.tokenBuffer.resize((tokenSize + len) * 2);
					memcpy(ctx.tokenBuffer.data() + tokenSize, tokenStart, len);
					tokenSize += len;
				}
				else if (p < end)
				{
					// Zero-length scan: advance to avoid infinite loop.
					++p;
				}
			}
		}

		if (tokenSize > 0)
		{
			if (tokenHasDot)
				tokenSize = StripDotSlash(ctx.tokenBuffer.data(), tokenSize);
			if (tokenSize > 0)
			{
				u32 tokenId = m_stringPool.Intern(ctx.tokenBuffer.data(), tokenSize);
				pushTokenToArena(tokenId);
			}
		}

		sectionEnd[state] = arenaTop();
		edge.inputs        = makeSpan(sectionStart[INPUTS],      sectionEnd[INPUTS]);
		edge.implicitDeps  = makeSpan(sectionStart[IMPLICIT],    sectionEnd[IMPLICIT]);
		edge.orderOnlyDeps = makeSpan(sectionStart[ORDER_ONLY],  sectionEnd[ORDER_ONLY]);
		edge.validations   = makeSpan(sectionStart[VALIDATIONS], sectionEnd[VALIDATIONS]);

		if (!ReadIndentedLines(ctx, edge.variables))
			return false;

		edge.scopeId = ctx.scopeId;

		// AOSP extension: `phony_output = true` marks virtual outputs.
		{
			auto it = edge.variables.find(m_phonyOutputId);
			if (it != edge.variables.end())
			{
				const char* v = m_stringPool.GetCStr(it->second);
				edge.phonyOutput = (strcmp(v, "1") == 0 || strcmp(v, "true") == 0);
			}
		}

		// Phase 1 fires callback inline; phase 2 fires it during merge.
		if (ctx.outputs.edges == &m_edges)
		{
			u32 edgeIndex = u32(m_edges.size());
			edge.edgeIndex = edgeIndex;
			m_edges.emplace_back(std::move(edge));
			if (m_edgeParsedCallback)
				m_edgeParsedCallback(m_edgeCallbackUserData, edgeIndex, m_edges[edgeIndex]);
		}
		else
		{
			ctx.outputs.edges->emplace_back(std::move(edge));
		}

		return true;
	}

	bool NinjaParser::ParseVariable(ParseContext& ctx, const char* line, u32 lineLen)
	{
		const char* eq = line;
		const char* end = line + lineLen;

		while (eq < end && *eq != '=')
			++eq;

		if (eq >= end)
			return false;

		const char* nameStart = line;
		const char* nameEnd = eq;

		while (nameEnd > nameStart && (nameEnd[-1] == ' ' || nameEnd[-1] == '\t'))
			--nameEnd;

		const char* valueStart = eq + 1;

		while (valueStart < end && (*valueStart == ' ' || *valueStart == '\t'))
			++valueStart;

		while (end > valueStart && (end[-1] == ' ' || end[-1] == '\t'))
			--end;

		u32 nameId = m_stringPool.Intern(nameStart, u32(nameEnd - nameStart));
		u32 valueId = m_stringPool.Intern(valueStart, u32(end - valueStart));

		if (ctx.scopeId == 0)
			m_globalVars[nameId] = valueId;
		else
			m_scopeVars[ctx.scopeId - 1][nameId] = valueId;
		return true;
	}

	bool NinjaParser::ParseDefault(ParseContext& ctx, const char* line, u32 lineLen)
	{
		const char* p = line + 8; // skip "default "
		SplitPaths(ctx, p, u32(line + lineLen - p), *ctx.outputs.defaults);
		return true;
	}

	bool NinjaParser::ParsePool(ParseContext& ctx, const char* line, u32 lineLen)
	{
		const char* nameStart = line + 5; // skip "pool "
		const char* nameEnd = nameStart;
		while (nameEnd < line + lineLen && *nameEnd != ' ' && *nameEnd != '\t')
			++nameEnd;

		u32 nameId = m_stringPool.Intern(nameStart, u32(nameEnd - nameStart));

		NinjaPool pool;
		pool.name = nameId;

		UnorderedMap<u32, u32> props;
		if (!ReadIndentedLines(ctx, props))
			return false;

		auto it = props.find(m_depthId);
		if (it != props.end())
		{
			const char* depthStr = m_stringPool.GetCStr(it->second);
			pool.depth = (u32)atoi(depthStr);
		}

		// Phase 1 updates m_poolMap directly; phase 2 does so during merge.
		u32 idx = u32(ctx.outputs.pools->size());
		if (ctx.outputs.pools == &m_pools)
			m_poolMap[pool.name] = idx;
		ctx.outputs.pools->push_back(pool);

		return true;
	}

	bool NinjaParser::ParseInclude(ParseContext& ctx, const char* line, u32 lineLen)
	{
		const char* p = line + 8; // skip "include "
		const char* end = line + lineLen;
		while (p < end && (*p == ' ' || *p == '\t'))
			++p;

		StringBuffer<512> fullPath;
		bool isAbsolute = (p < end && (p[0] == '/' || (p[1] == ':')));

		if (!isAbsolute)
		{
			CharStringView workingDir = m_stringPool.GetString(ctx.workingDirId);
			fullPath.Append(workingDir.data, workingDir.length);
		}
		fullPath.Append(p, u32(end - p));

		StringBuffer<512> fullPathW;
		for (u32 i = 0; i < fullPath.count; ++i)
			fullPathW.Append((tchar)(u8)fullPath.data[i]);

		// `include` shares the parent's scope (unlike `subninja`).
		return ParseFile(ctx.logger, fullPathW.data, ctx.scopeId, ctx.outputs, ctx.subninja, ctx.tokenArena);
	}

	bool NinjaParser::ParseSubninja(ParseContext& ctx, const char* line, u32 lineLen)
	{
		const char* p = line + 9; // skip "subninja "
		const char* end = line + lineLen;
		while (p < end && (*p == ' ' || *p == '\t'))
			++p;

		StringBuffer<512> fullPath;
		bool isAbsolute = (p < end && (p[0] == '/' || (p[1] == ':')));

		if (!isAbsolute)
		{
			// Subninja paths resolve against the build root (top-level dir), not the current file.
			CharStringView buildRootDir = m_stringPool.GetString(m_buildRootDirId);
			fullPath.Append(buildRootDir.data, buildRootDir.length);
		}
		fullPath.Append(p, u32(end - p));

		StringBuffer<512> fullPathW;
		for (u32 i = 0; i < fullPath.count; ++i)
			fullPathW.Append((tchar)(u8)fullPath.data[i]);

		// New scope: variables don't leak to parent but can read it via parent walk.
		u32 newScopeId = AllocateScopeId(ctx.scopeId);

		// Queue for the work-stealing pool so nested subninjas parallelize.
		if (ctx.subninja.queue)
		{
			SubninjaWork sw;
			sw.filename      = fullPathW.data;
			sw.scopeId       = newScopeId;
			sw.parentScopeId = ctx.scopeId;

			if (ctx.subninja.lock)
			{
				SCOPED_FUTEX(*ctx.subninja.lock, qlock);
				ctx.subninja.queue->emplace_back(std::move(sw));
				ctx.subninja.outstanding->fetch_add(1, std::memory_order_release);
			}
			else
			{
				ctx.subninja.queue->emplace_back(std::move(sw));
			}
			return true;
		}

		// Fallback: no queue (would be a Parse() bug). Recurse defensively.
		return ParseFile(ctx.logger, fullPathW.data, newScopeId, ctx.outputs, ctx.subninja, ctx.tokenArena);
	}

	bool NinjaParser::ReadLine(ParseContext& ctx, const char*& outLine, u32& outLen)
	{
		while (ctx.pos < ctx.size)
		{
			const char* lineStart = ctx.data + ctx.pos;
			const char* fileEnd = ctx.data + ctx.size;
			u64 remaining = ctx.size - ctx.pos;

			const char* lf = (const char*)memchr(lineStart, '\n', remaining);
			const char* lineEnd;
			if (lf)
			{
				lineEnd = lf;
				ctx.pos = u64(lf + 1 - ctx.data);
			}
			else
			{
				// No \n: try standalone \r (legacy Mac).
				const char* cr = (const char*)memchr(lineStart, '\r', remaining);
				lineEnd = cr ? cr : fileEnd;
				ctx.pos = u64(lineEnd - ctx.data);
				if (ctx.pos < ctx.size) ++ctx.pos;
			}

			if (lineEnd > lineStart && lineEnd[-1] == '\r')
				--lineEnd;

			++ctx.line;

			while (lineStart < lineEnd && (*lineStart == ' ' || *lineStart == '\t'))
				++lineStart;

			while (lineEnd > lineStart && (lineEnd[-1] == ' ' || lineEnd[-1] == '\t'))
				--lineEnd;

			if (lineStart >= lineEnd || *lineStart == '#')
				continue;

			if (lineEnd[-1] != '$')
			{
				outLine = lineStart;
				outLen = u32(lineEnd - lineStart);
				return true;
			}

			// `$` at EOL = continuation. Merge into ctx.continuationBuf.
			ctx.continuationBuf.clear();

			const char* segStart = lineStart;
			const char* segEnd   = lineEnd - 1; // drop trailing '$'

			for (;;)
			{
				u32 oldSize = u32(ctx.continuationBuf.size());
				u32 segLen  = u32(segEnd - segStart);
				ctx.continuationBuf.resize(oldSize + segLen);
				memcpy(ctx.continuationBuf.data() + oldSize, segStart, segLen);

				if (ctx.pos >= ctx.size)
					break;

				const char* nextStart = ctx.data + ctx.pos;
				u64 nextRemaining = ctx.size - ctx.pos;

				const char* nextLf = (const char*)memchr(nextStart, '\n', nextRemaining);
				const char* nextEnd;
				if (nextLf)
				{
					nextEnd = nextLf;
					ctx.pos = u64(nextLf + 1 - ctx.data);
				}
				else
				{
					const char* cr = (const char*)memchr(nextStart, '\r', nextRemaining);
					nextEnd = cr ? cr : fileEnd;
					ctx.pos = u64(nextEnd - ctx.data);
					if (ctx.pos < ctx.size) ++ctx.pos;
				}

				if (nextEnd > nextStart && nextEnd[-1] == '\r')
					--nextEnd;

				++ctx.line;

				// Strip leading SPACES only — NOT tabs. Ninja's lexer matches
				// `[ ]*"$\n"[ ]*`; tabs indenting continuation lines (Soong's
				// `\t\t` indents) are PART of the value. Stripping tabs here
				// collapses `&&$\n\t\tunzip` to `&&unzip` and breaks bash rules.
				while (nextStart < nextEnd && *nextStart == ' ')
					++nextStart;

				// Trim trailing whitespace.
				while (nextEnd > nextStart && (nextEnd[-1] == ' ' || nextEnd[-1] == '\t'))
					--nextEnd;

				if (nextStart >= nextEnd)
					break; // blank continuation line

				if (nextEnd[-1] == '$')
				{
					segStart = nextStart;
					segEnd   = nextEnd - 1; // drop trailing '$', keep looping
				}
				else
				{
					segStart = nextStart;
					segEnd   = nextEnd;
					oldSize = u32(ctx.continuationBuf.size());
					segLen  = u32(segEnd - segStart);
					ctx.continuationBuf.resize(oldSize + segLen);
					memcpy(ctx.continuationBuf.data() + oldSize, segStart, segLen);
					break;
				}
			}

			outLine = ctx.continuationBuf.data();
			outLen  = u32(ctx.continuationBuf.size());
			return true;
		}

		return false;
	}

	bool NinjaParser::ReadIndentedLines(ParseContext& ctx, UnorderedMap<u32, u32>& vars)
	{
		while (ctx.pos < ctx.size)
		{
			if (ctx.data[ctx.pos] != ' ' && ctx.data[ctx.pos] != '\t')
				break;

			const char* line;
			u32 lineLen;
			if (!ReadLine(ctx, line, lineLen))
				break;

			const char* eq = line;
			const char* end = line + lineLen;

			while (eq < end && *eq != '=')
				++eq;

			if (eq >= end)
				continue;

			const char* nameStart = line;
			const char* nameEnd = eq;
			while (nameEnd > nameStart && (nameEnd[-1] == ' ' || nameEnd[-1] == '\t'))
				--nameEnd;

			const char* valueStart = eq + 1;
			while (valueStart < end && (*valueStart == ' ' || *valueStart == '\t'))
				++valueStart;

			while (end > valueStart && (end[-1] == ' ' || end[-1] == '\t'))
				--end;

			u32 nameId = m_stringPool.Intern(nameStart, u32(nameEnd - nameStart));
			u32 valueId = m_stringPool.Intern(valueStart, u32(end - valueStart));
			vars[nameId] = valueId;
		}

		return true;
	}

	void NinjaParser::SkipWhitespace(const char*& p, const char* end)
	{
		while (p < end && (*p == ' ' || *p == '\t'))
			++p;
	}

	void NinjaParser::SplitPaths(ParseContext& ctx, const char* str, u32 len, Vector<u32>& out)
	{
		const char* p = str;
		const char* end = str + len;
		u32 tokenSize = 0;
		bool tokenHasDot = false;

		while (p < end)
		{
			if (*p == '$' && p + 1 < end)
			{
				++p;
				if (*p == ' ' || *p == ':' || *p == '$')
				{
					if (tokenSize >= ctx.tokenBuffer.size())
						ctx.tokenBuffer.resize(tokenSize * 2 + 1);
					ctx.tokenBuffer[tokenSize++] = *p++;
				}
				else if (*p == '\n' || *p == '\r')
				{
					++p;
					if (p < end && p[-1] == '\r' && *p == '\n')
						++p;
				}
				else
				{
					if (tokenSize >= ctx.tokenBuffer.size())
						ctx.tokenBuffer.resize(tokenSize * 2 + 1);
					ctx.tokenBuffer[tokenSize++] = '$';
				}
			}
			else if (*p == ' ' || *p == '\t')
			{
				if (tokenSize > 0)
				{
					if (tokenHasDot)
						tokenSize = StripDotSlash(ctx.tokenBuffer.data(), tokenSize);
					if (tokenSize > 0)
						out.push_back(m_stringPool.Intern(ctx.tokenBuffer.data(), tokenSize));
					tokenSize = 0;
					tokenHasDot = false;
				}
				++p;
			}
			else
			{
				// Bulk scan using delimiter lookup table + memcpy (same as ParseBuild hot path)
				const char* tokenStart = p;
				while (p < end && !g_isDelimiter[(u8)*p])
				{
					tokenHasDot |= (*p == '.');
					++p;
				}
				u32 scanLen = u32(p - tokenStart);
				if (scanLen > 0)
				{
					if (tokenSize + scanLen > ctx.tokenBuffer.size())
						ctx.tokenBuffer.resize((tokenSize + scanLen) * 2);
					memcpy(ctx.tokenBuffer.data() + tokenSize, tokenStart, scanLen);
					tokenSize += scanLen;
				}
				else if (p < end)
				{
					++p;  // Unknown delimiter - advance to avoid infinite loop
				}
			}
		}

		if (tokenSize > 0)
		{
			if (tokenHasDot)
				tokenSize = StripDotSlash(ctx.tokenBuffer.data(), tokenSize);
			if (tokenSize > 0)
				out.push_back(m_stringPool.Intern(ctx.tokenBuffer.data(), tokenSize));
		}
	}

	void NinjaParser::ExpandCommand(const NinjaBuildEdge& edge, Vector<char>& out, Vector<char>& tempBuffer)
	{
		// Find rule
		auto it = m_ruleMap.find(edge.ruleName);
		if (it == m_ruleMap.end())
			return;

		const NinjaRule& rule = m_rules[it->second];

		// Pre-expand $in / $out / $rspfile into local buffers and pass them to
		// ExpandVariables as an EdgeSpecialVars view. This avoids:
		//   1. Copying edge.variables (allocates buckets + nodes per edge).
		//   2. Three InternThreadSafe calls per edge (one each for the
		//      pre-expanded $in / $out / $rspfile strings) — those went through
		//      the StringPool's per-shard mutex and were the dominant
		//      contention source in EnqueueCommands Pass 2.
		// We pass &edge.variables directly as edgeVars so edge-specific overrides
		// still resolve normally; only $in / $out / $rspfile are intercepted.
		Vector<char> inStr, outStr, rspfilePath;
		// Modest reserve so most edges hit the fast Vector path; will grow on
		// large edges. (The previous 256 KB reserve per edge was wasteful for
		// the common 1–10 input case.)
		inStr.reserve(2048);
		outStr.reserve(512);

		for (u32 i = 0; i < edge.inputs.size(); ++i)
		{
			if (i > 0)
				inStr.push_back(' ');

			CharStringView input = m_stringPool.GetString(edge.inputs[i]);

			tempBuffer.clear();
			ExpandVariables(input.data, input.length, tempBuffer, nullptr, edge.scopeId);
			// Canonicalize: strip leading "./" produced by variable expansion (e.g. $root = ".")
			u32 expandedLen = u32(tempBuffer.size());
			expandedLen = StripDotSlash(tempBuffer.data(), expandedLen);
			tempBuffer.resize(expandedLen);
			bool hasSpace = memchr(tempBuffer.data(), ' ', tempBuffer.size()) != nullptr;
			if (hasSpace) inStr.push_back('"');
			inStr.insert(inStr.end(), tempBuffer.begin(), tempBuffer.end());
			if (hasSpace) inStr.push_back('"');
		}

		for (u32 i = 0; i < edge.outputs.size(); ++i)
		{
			if (i > 0)
				outStr.push_back(' ');

			CharStringView output = m_stringPool.GetString(edge.outputs[i]);

			tempBuffer.clear();
			ExpandVariables(output.data, output.length, tempBuffer, nullptr, edge.scopeId);
			u32 expandedLen = u32(tempBuffer.size());
			expandedLen = StripDotSlash(tempBuffer.data(), expandedLen);
			tempBuffer.resize(expandedLen);
			bool hasSpace = memchr(tempBuffer.data(), ' ', tempBuffer.size()) != nullptr;
			if (hasSpace) outStr.push_back('"');
			outStr.insert(outStr.end(), tempBuffer.begin(), tempBuffer.end());
			if (hasSpace) outStr.push_back('"');
		}

		// Build the specials view. Use a non-null sentinel even if the buffer
		// is empty (Vector::data() can be null when capacity is zero, which
		// would make our "data == nullptr means not set" check misfire). For
		// $in / $out we always set a view; for $rspfile we leave .data == null
		// when the rule has no rspfile so $rspfile references in the command
		// fall through to the normal lookup chain.
		EdgeSpecialVars specials;
		const char* emptyMarker = "";
		specials.in.data    = inStr.empty()  ? emptyMarker : inStr.data();
		specials.in.length  = u32(inStr.size());
		specials.out.data   = outStr.empty() ? emptyMarker : outStr.data();
		specials.out.length = u32(outStr.size());

		if (rule.rspfile != StringId_Empty)
		{
			CharStringView rspfileStr = m_stringPool.GetString(rule.rspfile);
			rspfilePath.reserve(512);
			// Expand the rspfile template against edge.variables and the $in/$out
			// specials we just built. Pass &specials so the template can reference
			// $in or $out (common — e.g. "rspfile = $out.rsp").
			ExpandVariables(rspfileStr.data, rspfileStr.length, rspfilePath,
				&edge.variables, edge.scopeId, StringId_Empty, false, &specials);
			specials.rspfile.data   = rspfilePath.empty() ? emptyMarker : rspfilePath.data();
			specials.rspfile.length = u32(rspfilePath.size());
		}

		// Expand command, with edge.variables passed BY POINTER (no copy) and
		// specials providing $in / $out / $rspfile views.
		CharStringView commandStr = m_stringPool.GetString(rule.command);
		ExpandVariables(commandStr.data, commandStr.length, out,
			&edge.variables, edge.scopeId, /*skipVarId=*/StringId_Empty,
			/*shellEscapeDollarSpace=*/false, &specials);
	}

	void NinjaParser::ExpandVariables(const char* str, u32 len, Vector<char>& out, const UnorderedMap<u32, u32>* edgeVars, u32 scopeId, u32 skipVarId, bool shellEscapeDollarSpace, const EdgeSpecialVars* specials)
	{
		// Fast check for variables using vectorized memchr
		if (!memchr(str, '$', len))
		{
			// Fast path: no variables, bulk copy
			u32 oldSize = u32(out.size());
			out.resize(oldSize + len);
			memcpy(&out[oldSize], str, len);
			return;
		}

		const char* p = str;
		const char* end = str + len;
		const char* literalStart = p;  // Track start of literal sections

		while (p < end)
		{
			if (*p == '$')
			{
				// Copy any literal section before this variable
				if (p > literalStart)
				{
					u32 literalLen = u32(p - literalStart);
					u32 oldSize = u32(out.size());
					out.resize(oldSize + literalLen);
					memcpy(&out[oldSize], literalStart, literalLen);
				}

				++p;
				if (p >= end)
					break;

				// Handle special cases
				if (*p == '$')
				{
					out.push_back('$');
					++p;
					literalStart = p;  // Reset literal start
					continue;
				}
				else if (*p == ' ')
				{
					// Ninja's dollar-space is an escaped space inside a single token.
					// When building a shell command, emit backslash-space so that
					// ParseArguments treats it as an embedded space (not a separator).
					if (shellEscapeDollarSpace)
						out.push_back('\\');
					out.push_back(' ');
					++p;
					literalStart = p;
					continue;
				}
				else if (*p == ':')
				{
					out.push_back(':');
					++p;
					literalStart = p;
					continue;
				}

				// Variable reference
				const char* varStart = p;
				const char* varEnd = p;

				if (*p == '{')
				{
					++varStart;
					++p;
					while (p < end && *p != '}')
						++p;
					varEnd = p;
					if (p < end)
						++p; // Skip closing brace
				}
				else
				{
					while (p < end && ((*p >= 'a' && *p <= 'z') || (*p >= 'A' && *p <= 'Z') ||
					       (*p >= '0' && *p <= '9') || *p == '_'))
					{
						++p;
					}
					varEnd = p;
				}

				u32 varLen = u32(varEnd - varStart);
				if (varLen == 0)
					continue;

				// Look up variable (4-level: specials → edge vars → scope vars → global vars)
				u32 varNameId = m_stringPool.Intern(varStart, varLen);
				u32 valueId = StringId_Empty;

				// 0. Edge specials ($in / $out / $rspfile). These are pre-expanded
				//    by ExpandCommand, so we emit their bytes directly — no
				//    recursion, no further intern. Skipping the normal lookup
				//    chain also avoids the per-edge UnorderedMap copy that the
				//    old code used to stash interned $in/$out/$rspfile values.
				if (specials)
				{
					const CharStringView* hit = nullptr;
					if      (varNameId == m_inId      && specials->in.data)      hit = &specials->in;
					else if (varNameId == m_outId     && specials->out.data)     hit = &specials->out;
					else if (varNameId == m_rspfileId && specials->rspfile.data) hit = &specials->rspfile;
					if (hit)
					{
						u32 oldSize = u32(out.size());
						out.resize(oldSize + hit->length);
						if (hit->length)
							memcpy(&out[oldSize], hit->data, hit->length);
						literalStart = p;
						continue;
					}
				}

				// 1. Check edge-specific variables first (skip if this is the variable we're expanding to avoid infinite loop)
				if (edgeVars && varNameId != skipVarId)
				{
					auto it = edgeVars->find(varNameId);
					if (it != edgeVars->end())
						valueId = it->second;
				}

				// 2. Walk up the scope chain: current subninja -> parent -> ...
				//    Ninja semantics: a subninja inherits its parent's variables.
				//    Android's Soong generates multi-level subninjas where common
				//    variables (e.g. `g.bpf.relPwd = PWD=/proc/self/cwd`) live in
				//    a middle scope and are consumed by edges deeper down the tree.
				{
					u32 s = scopeId;
					while (valueId == StringId_Empty && s > 0)
					{
						auto it = m_scopeVars[s - 1].find(varNameId);
						if (it != m_scopeVars[s - 1].end())
						{
							valueId = it->second;
							break;
						}
						s = (s - 1) < m_scopeParents.size() ? m_scopeParents[s - 1] : 0;
					}
				}

				// 3. Check global variables
				if (valueId == StringId_Empty)
				{
					auto it = m_globalVars.find(varNameId);
					if (it != m_globalVars.end())
						valueId = it->second;
				}

				// Expand value recursively, passing varNameId to prevent circular edge var references
				if (valueId != StringId_Empty)
				{
					CharStringView value = m_stringPool.GetString(valueId);
					ExpandVariables(value.data, value.length, out, edgeVars, scopeId, varNameId, shellEscapeDollarSpace, specials);
				}

				// Reset literal start after variable expansion
				literalStart = p;
			}
			else
			{
				// Regular character - will be copied in bulk with next literal section
				++p;
			}
		}

		// Copy any remaining literal section
		if (p > literalStart)
		{
			u32 literalLen = u32(p - literalStart);
			u32 oldSize = u32(out.size());
			out.resize(oldSize + literalLen);
			memcpy(&out[oldSize], literalStart, literalLen);
		}
	}
}
