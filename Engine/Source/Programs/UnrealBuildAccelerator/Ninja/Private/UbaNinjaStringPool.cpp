// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaNinjaStringPool.h"
#include "UbaPlatform.h"
#include "UbaStringBuffer.h"

// SIMD for hash table probing (cross-platform)
#if defined(__SSE2__) || (defined(_MSC_VER) && (defined(_M_X64) || defined(_M_IX86)))
	#include <emmintrin.h>  // SSE2
	#define UBA_USE_SIMD 1
	#if defined(_MSC_VER)
		#include <intrin.h>
		inline int CountTrailingZeros(int mask) {
			unsigned long index;
			_BitScanForward(&index, mask);
			return (int)index;
		}
	#else
		inline int CountTrailingZeros(int mask) {
			return __builtin_ctz(mask);
		}
	#endif
#else
	#define UBA_USE_SIMD 0
#endif

namespace uba
{
	static constexpr u32 kInitialShardSlots      = 256;
	static constexpr u32 kInitialStringsCapacity = 16 * 1024;

	// Per-thread direct-mapped intern cache. Hot loops hammer the same
	// handful of strings (paths prefixes, $root / $in / $out, rule names);
	// a hit skips the shard mutex and the SIMD probe.
	//
	// e.dataPtr MUST reference the interned StringPool entry, NOT the caller's
	// source span — sources may be mmap'd ninja files unmapped after ParseFile,
	// while cache entries persist across later worker calls.
	//
	// Single-StringPool-safe: UbaNinja constructs one pool per process. Multiple
	// coexisting pools would require tracking the pool pointer per entry.
	struct InternCacheEntry
	{
		const char* dataPtr  = nullptr;
		u32         hashFull = 0;
		u32         length   = 0;
		u32         id       = StringId_Empty;
	};
	static constexpr u32 kInternCacheSize = 256;
	static constexpr u32 kInternCacheMask = kInternCacheSize - 1;
	static thread_local InternCacheEntry tls_internCache[kInternCacheSize] = {};

	StringPool::StringPool(MemoryBlock& memoryBlock)
		: m_memoryBlock(memoryBlock)
	{
		for (u32 s = 0; s < ShardCount; ++s)
		{
			Shard& shard = m_shards[s];
			shard.ctrl.resize(kInitialShardSlots, kEmpty);
			shard.ids.resize(kInitialShardSlots, 0);
			shard.hashMask = kInitialShardSlots - 1;
		}

		// m_strings indexed by dense id; ID 0 = empty placeholder, real ids start at 1.
		m_strings.resize(kInitialStringsCapacity);
		m_strings[0] = StringEntry{ "", 0, 0 };
		m_nextId.store(1, std::memory_order_relaxed);
	}

	void StringPool::Reserve(u32 capacity)
	{
		// CALLER MUST GUARANTEE no concurrent Intern/GetString — m_strings.resize may relocate.
		if (capacity > u32(m_strings.size()))
			m_strings.resize(capacity);

		u32 perShard = (capacity + ShardCount - 1) / ShardCount;
		u32 desired  = (perShard * 4) / 3;
		u32 tableSize = kInitialShardSlots;
		while (tableSize < desired)
			tableSize <<= 1;

		for (u32 s = 0; s < ShardCount; ++s)
		{
			Shard& shard = m_shards[s];
			if (tableSize > shard.ctrl.size())
				RehashShard(shard, tableSize);
		}
	}

	u32 StringPool::HashString(const char* data, u32 length)
	{
		const u32 seed = 0x9747b28c;

		if (length <= 8)
		{
			u32 hash = seed ^ length;
			const u8* bytes = (const u8*)data;

			if (length >= 4)
			{
				hash ^= *(const u32*)bytes;
				hash = (hash << 13) | (hash >> 19);
				bytes += 4;
				length -= 4;
			}
			if (length >= 2)
			{
				hash ^= *(const u16*)bytes;
				bytes += 2;
				length -= 2;
			}
			if (length >= 1)
			{
				hash ^= *bytes;
			}

			hash *= 0x85ebca6b;
			hash ^= hash >> 16;
			return hash ? hash : 1;
		}

		u32 hash = seed;
		const u32* blocks = (const u32*)data;
		u32 numBlocks = length / 4;

		for (u32 i = 0; i < numBlocks; ++i)
		{
			u32 k = blocks[i];
			k *= 0xcc9e2d51;
			k = (k << 15) | (k >> 17);
			k *= 0x1b873593;
			hash ^= k;
			hash = (hash << 13) | (hash >> 19);
			hash = hash * 5 + 0xe6546b64;
		}

		const u8* tail = (const u8*)(data + numBlocks * 4);
		u32 k = 0;
		switch (length & 3)
		{
		case 3: k ^= tail[2] << 16; [[fallthrough]];
		case 2: k ^= tail[1] << 8;  [[fallthrough]];
		case 1: k ^= tail[0];
			k *= 0xcc9e2d51;
			k = (k << 15) | (k >> 17);
			k *= 0x1b873593;
			hash ^= k;
		}

		hash ^= length;
		hash ^= hash >> 16;
		hash *= 0x85ebca6b;
		hash ^= hash >> 13;
		hash *= 0xc2b2ae35;
		hash ^= hash >> 16;

		return hash ? hash : 1;
	}

	u32 StringPool::FindSlot(const Shard& shard, u32 hash, const char* str, u32 length) const
	{
		u32 slot = hash & shard.hashMask;
		u8 h2 = hash & 0x7F;

#if UBA_USE_SIMD
		__m128i needle = _mm_set1_epi8(h2);
		const u32 tableSize = u32(shard.ctrl.size());

		while (true)
		{
			if (slot + 16 <= tableSize)
			{
				__m128i group = _mm_loadu_si128((const __m128i*)&shard.ctrl[slot]);
				int mask = _mm_movemask_epi8(_mm_cmpeq_epi8(needle, group));
				int emptyMask = _mm_movemask_epi8(group);

				while (mask != 0)
				{
					int bitPos = CountTrailingZeros(mask);
					u32 candidateSlot = slot + bitPos;
					u32 candidateId = shard.ids[candidateSlot];

					const StringEntry& entry = m_strings[candidateId];
					if (entry.length == length && memcmp(entry.data, str, length) == 0)
						return candidateSlot;

					mask &= mask - 1;
				}

				if (emptyMask != 0)
					return slot + CountTrailingZeros(emptyMask);

				slot = (slot + 16) & shard.hashMask;
			}
			else
			{
				u8 ctrl = shard.ctrl[slot];

				if (ctrl == kEmpty)
					return slot;

				if (ctrl == h2)
				{
					u32 candidateId = shard.ids[slot];
					const StringEntry& entry = m_strings[candidateId];
					if (entry.length == length && memcmp(entry.data, str, length) == 0)
						return slot;
				}

				slot = (slot + 1) & shard.hashMask;
			}
		}
#else
		while (true)
		{
			u8 ctrl = shard.ctrl[slot];

			if (ctrl == kEmpty)
				return slot;

			if (ctrl == h2)
			{
				u32 candidateId = shard.ids[slot];
				const StringEntry& entry = m_strings[candidateId];
				if (entry.length == length && memcmp(entry.data, str, length) == 0)
					return slot;
			}

			slot = (slot + 1) & shard.hashMask;
		}
#endif
	}

	u32 StringPool::Intern(const char* str, u32 length)
	{
		if (length == 0)
			return StringId_Empty;

		u32 hash = HashString(str, length);

		// Per-thread cache: hits skip the shard mutex. Pointer-eq short-circuits
		// re-passes of the same span; otherwise memcmp guards against reused scratch.
		InternCacheEntry& e = tls_internCache[hash & kInternCacheMask];
		if (e.id != StringId_Empty && e.hashFull == hash && e.length == length)
		{
			if (e.dataPtr == str || memcmp(e.dataPtr, str, length) == 0)
				return e.id;
		}

		Shard& shard = m_shards[hash & ShardMask];
		u32 id;
		{
			SCOPED_FUTEX(shard.mutex, lock);
			id = InternImplLocked(shard, str, length, hash);
		}

		// Cache the STABLE interned pointer (m_strings[id].data lives for the
		// pool's lifetime) — the caller's source may be unmapped after ParseFile.
		// Don't cache StringId_Empty: caller is bailing, would pollute the slot.
		if (id != StringId_Empty)
		{
			e.dataPtr  = m_strings[id].data;
			e.hashFull = hash;
			e.length   = length;
			e.id       = id;
		}
		return id;
	}

	u32 StringPool::InternImplLocked(Shard& shard, const char* str, u32 length, u32 hash)
	{
		u32 slot = FindSlot(shard, hash, str, length);

		if (shard.ctrl[slot] != kEmpty)
			return shard.ids[slot];

		// Allocate dense id, write entry, then publish via shard ctrl/ids.
		// Mutex release at scope exit is the release barrier for readers.
		u32 newId = m_nextId.fetch_add(1, std::memory_order_relaxed);
		if (newId >= u32(m_strings.size()))
		{
			// Capacity exhausted. Roll back id; caller must propagate StringId_Empty.
			m_nextId.fetch_sub(1, std::memory_order_relaxed);

			// Assert loudly — silent empty-return corrupts rspfile_content expansion.
			UBA_SHIPPING_ASSERTF(false, TC("FATAL: StringPool capacity exhausted: %u entries used, requested length=%u. Increase NinjaParser ctor's Reserve()."), u32(m_strings.size()), length);
			return StringId_Empty;
		}

		char* internedData = (char*)m_memoryBlock.Allocate(length + 1, 1, TC("NinjaString"));
		memcpy(internedData, str, length);
		internedData[length] = 0;

		m_strings[newId] = StringEntry{ internedData, length, hash };

		shard.ctrl[slot] = hash & 0x7F;
		shard.ids[slot]  = newId;
		shard.hashCount++;

		if (shard.hashCount * 4 > shard.ctrl.size() * 3)
			RehashShard(shard, u32(shard.ctrl.size()) * 2);

		return newId;
	}

	void StringPool::RehashShard(Shard& shard, u32 newSize)
	{
		Vector<u8>  newCtrl(newSize, kEmpty);
		Vector<u32> newIds(newSize, 0);
		u32 newMask = newSize - 1;

		for (u32 i = 0; i < shard.ctrl.size(); ++i)
		{
			if (shard.ctrl[i] == kEmpty || shard.ctrl[i] == kDeleted)
				continue;

			u32 id = shard.ids[i];
			// m_strings[id] is stable: written under this shard's mutex, never relocated.
			u32 hash = m_strings[id].hash;

			u32 slot = hash & newMask;
			while (newCtrl[slot] != kEmpty)
				slot = (slot + 1) & newMask;

			newCtrl[slot] = hash & 0x7F;
			newIds[slot]  = id;
		}

		shard.ctrl     = std::move(newCtrl);
		shard.ids      = std::move(newIds);
		shard.hashMask = newMask;
	}
}
