// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaLogger.h"
#include "UbaNinjaDepsLog.h"
#include "UbaNinjaParser.h"
#include "UbaProcessHandle.h"
#include "UbaProcessStartInfo.h"
#include "UbaStringBuffer.h"

namespace uba
{
	class Scheduler;

	// Absolute directory of the ninja file (trailing sep). Falls back to workingDir.
	void ResolveNinjaDir(StringBufferBase& out, const tchar* ninjaFile, StringView workingDir);

	// Per-process payload attached via ProcessStartInfo::userData. Pass 2
	// heap-allocates; the caller-supplied exitedFunc owns and deletes it.
	struct NinjaExitPayload
	{
		void*    ctx = nullptr;   // NinjaParseState::exitContext
		u64      cmdHash = 0;     // pre-rsp-substitution
		bool     restat = false;  // restat=1 or phony_output=1
		TString  primaryOut;      // abs path of edge.outputs[0]
	};

	struct NinjaParseState
	{
		Vector<u32>     outputToEdge;
		Vector<u8>      edgesToBuildFlags;
		Vector<u32>     edgeToProcessIndex;
		Vector<u8>      wantedOutputs;
		u32             phonyId = 0;
		bool            wantAll = false;

		// Pass 2 wires exitCallback into ProcessStartInfo::exitedFunc and
		// stores exitContext in each allocated NinjaExitPayload::ctx.
		ProcessStartInfo::ExitedCallback* exitCallback = nullptr;
		void*                             exitContext  = nullptr;
	};

	// Phase 1: intern target IDs, register edge-parsed callback, parse the ninja file.
	// Safe to run on a background thread.
	// `buildRoot` is the anchor for relative paths inside the file (ninja's
	// cwd-at-invocation / `-C dir`). Null defaults to the ninja file's own dir,
	// only correct when those coincide (`-C out/Default` for chromium/llvm).
	bool ParseNinjaFile(LoggerWithWriter& logger, NinjaParser& parser,
	                    const tchar* ninjaFile, const Vector<TString>& specifiedTargets,
	                    NinjaParseState& outState, const tchar* buildRoot = nullptr);

	struct EnqueueCommandsContext
	{
		LoggerWithWriter& logger;
		NinjaParser& parser;
		Scheduler& scheduler;
		NinjaParseState& state;
		NinjaLogIndex& logIndex;

		StringView workingDir;

		bool rebuild;
		bool cleanOnly;
		bool noShowIncludes;
		bool checkUpToDate;

		bool verbose;
		bool dryRun;
		const tchar* yamlFile;

		u32 cacheBucketId;
		bool forceRemote;
		bool toolCommands;

		const tchar* ninjaFile;

		bool logIndexLoaded;

		u32 maxEdges = ~0u;
		u32 skipEdges = 0;

		const tchar* depsDumpFile = nullptr;

		// When set, every selected edge's outputs + implicitOutputs get
		// registered into the cross-edge written-set at parse time.
		class NinjaDepsLog* depsLog = nullptr;
	};

	// Phase 2: BFS target selection, output cleaning, weight computation, scheduler enqueue.
	// checkUpToDate=false: skip mtime check, enqueue every needed edge.
	// verbose: print `[N/total] <command>` per edge (matches `ninja -v`).
	// toolCommands: mimics `ninja -t commands` then returns without scheduling.
	bool EnqueueCommands(const EnqueueCommandsContext& context);
}
