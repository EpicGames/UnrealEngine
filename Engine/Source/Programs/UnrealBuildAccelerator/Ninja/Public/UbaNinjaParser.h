// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaNinjaStringPool.h"
#include "UbaString.h"
#include "UbaUnorderedMap.h"
#include <atomic>

namespace uba
{
	class Logger;

	struct NinjaPool
	{
		u32 name = StringId_Empty;
		u32 depth = 0;
	};

	struct NinjaRule
	{
		u32 name = StringId_Empty;
		u32 command = StringId_Empty;
		u32 description = StringId_Empty;
		u32 depfile = StringId_Empty;
		u32 deps = StringId_Empty; // "gcc", "msvc", or empty
		u32 rspfile = StringId_Empty;
		u32 rspfile_content = StringId_Empty;
		u32 pool = StringId_Empty;
		bool restat = false;
		bool generator = false;
	};

	// Span into NinjaParser's bump-allocated u32 token arena. Arena never
	// reallocates, so pointers are stable for the parser's lifetime.
	struct TokenSpan
	{
		const u32* tokens = nullptr;
		u32        count  = 0;

		bool          empty() const { return count == 0; }
		u32           size()  const { return count; }
		const u32&    operator[](u32 i) const { return tokens[i]; }
		const u32*    data()  const { return tokens; }
		const u32*    begin() const { return tokens; }
		const u32*    end()   const { return tokens + count; }
	};

	struct NinjaBuildEdge
	{
		NinjaBuildEdge() = default;
		NinjaBuildEdge(NinjaBuildEdge&&) = default;
		// Token lists back into NinjaParser's token arena and are filled
		// sequentially per edge — each list is a contiguous arena range.
		TokenSpan outputs;
		TokenSpan implicitOutputs; // after `|` on output side
		TokenSpan inputs;
		TokenSpan implicitDeps;    // after `|` on input side
		TokenSpan orderOnlyDeps;   // after `||` on input side
		// AOSP extension `|@ <paths>`: validations run as a side effect
		// but don't block consumers. Stock ninja files never set this.
		TokenSpan validations;
		u32 ruleName = StringId_Empty;
		UnorderedMap<u32, u32> variables; // edge-specific overrides (name id -> value id)
		u32 edgeIndex = ~0u;
		u32 scopeId = 0; // 0 = main/global, >0 = subninja
		// AOSP extension: `phony_output = true` marks virtual outputs not
		// on disk. Treated like restat for freshness; stat-existence skipped.
		bool phonyOutput = false;
	};

	// Pre-expanded $in / $out / $rspfile views for ExpandVariables. O(1)
	// substitution, skips interning the expanded paths.
	// data == nullptr means "not set; fall through to normal var lookup".
	struct EdgeSpecialVars
	{
		CharStringView in;
		CharStringView out;
		CharStringView rspfile;
	};

	// Edge-parsed callback. In parallel-parse mode the callback fires during
	// the post-parse merge, not during parse — but every edge is delivered
	// before Parse() returns.
	using EdgeParsedCallback = void(*)(void* userData, u32 edgeIndex, const NinjaBuildEdge& edge);

	// Output destinations for ParseRule/Build/Pool/Default.
	// Phase 1: parser member vectors. Phase 2: thread-local, merged later.
	struct ParseOutputs
	{
		Vector<NinjaBuildEdge>* edges    = nullptr;
		Vector<NinjaRule>*      rules    = nullptr;
		Vector<NinjaPool>*      pools    = nullptr;
		Vector<u32>*            defaults = nullptr;
	};

	struct SubninjaWork
	{
		TString filename;
		u32     scopeId       = 0;
		u32     parentScopeId = 0;
	};

	// Shared work-stealing queue for parallel parse. Workers pop under `lock`;
	// nested subninjas re-push so idle workers can claim them.
	// Phase 1 (single-threaded): lock + outstanding null; just push_back.
	struct SubninjaQueueRef
	{
		Vector<SubninjaWork>* queue       = nullptr;
		Futex*                lock        = nullptr;
		std::atomic<u32>*     outstanding = nullptr;
	};

	class NinjaParser
	{
	public:
		NinjaParser(MemoryBlock& memoryBlock);
		~NinjaParser();

		// `buildRoot` anchors subninja/include resolution (ninja's cwd-at-invocation
		// or `-C dir`). Null defaults to the file's own dir, only correct when
		// they coincide (`-C out/Default` for chromium/llvm; wrong for Android
		// `-f out/combined.ninja` from android root).
		bool Parse(Logger& logger, const tchar* filename, const tchar* buildRoot = nullptr);

		void SetEdgeParsedCallback(EdgeParsedCallback callback, void* userData)
		{
			m_edgeParsedCallback = callback;
			m_edgeCallbackUserData = userData;
		}

		const Vector<NinjaRule>& GetRules() const { return m_rules; }
		const Vector<NinjaBuildEdge>& GetEdges() const { return m_edges; }
		const UnorderedMap<u32, u32>& GetVariables() const { return m_globalVars; }
		const Vector<u32>& GetDefaults() const { return m_defaults; }
		const UnorderedMap<u32, u32>& GetRuleMap() const { return m_ruleMap; }
		const Vector<NinjaPool>& GetPools() const { return m_pools; }
		const StringPool& GetStringPool() const { return m_stringPool; }

		// Expands directly into `out` (no per-call allocations).
		void ExpandCommand(const NinjaBuildEdge& edge, Vector<char>& out, Vector<char>& tempBuffer);
		void ExpandVariables(const char* str, u32 len, Vector<char>& out, const UnorderedMap<u32, u32>* edgeVars = nullptr, u32 scopeId = 0, u32 skipVarId = StringId_Empty, bool shellEscapeDollarSpace = false, const EdgeSpecialVars* specials = nullptr);

		u64 GetParseTimeMs() const { return m_parseTimeMs; }
		u32 GetLineCount() const { return m_lineCount.load(std::memory_order_relaxed); }

	private:
		struct ParseContext
		{
			Logger& logger;
			const char* data         = nullptr;
			u64 size                 = 0;
			u64 pos                  = 0;
			u32 line                 = 1;
			u32 workingDirId         = 0;
			u32 scopeId              = 0;
			Vector<char> continuationBuf;
			// Per-thread token assembly scratch for ParseBuild / SplitPaths.
			Vector<char> tokenBuffer;
			// ParseRule/Build/Pool/Default outputs; merged from worker-local in phase 2.
			ParseOutputs outputs;
			// Shared subninja queue. queue==nullptr means inline recurse (unused).
			SubninjaQueueRef subninja;
			// Per-thread token arena. Phase 1: parser's m_tokenArena. Phase 2:
			// per-worker arena from m_workerArenas, kept alive by parser so
			// TokenSpan pointers in m_edges remain valid.
			MemoryBlock* tokenArena = nullptr;
		};

		bool ParseFile(Logger& logger, const tchar* filename, u32 scopeId,
		               ParseOutputs outputs, SubninjaQueueRef subninja,
		               MemoryBlock* tokenArena = nullptr);
		bool ParseImpl(ParseContext& ctx);
		bool ParseRule(ParseContext& ctx, const char* line, u32 lineLen);
		bool ParseBuild(ParseContext& ctx, const char* line, u32 lineLen);
		bool ParseVariable(ParseContext& ctx, const char* line, u32 lineLen);
		bool ParseDefault(ParseContext& ctx, const char* line, u32 lineLen);
		bool ParsePool(ParseContext& ctx, const char* line, u32 lineLen);
		bool ParseInclude(ParseContext& ctx, const char* line, u32 lineLen);
		bool ParseSubninja(ParseContext& ctx, const char* line, u32 lineLen);

		// Thread-safe (m_scopeAllocLock); returns 1-based scope id.
		u32 AllocateScopeId(u32 parentScopeId);

		bool ReadLine(ParseContext& ctx, const char*& outLine, u32& outLen);
		bool ReadIndentedLines(ParseContext& ctx, UnorderedMap<u32, u32>& vars);
		void SkipWhitespace(const char*& p, const char* end);
		void SplitPaths(ParseContext& ctx, const char* str, u32 len, Vector<u32>& out);

		// Caller-owned shared MemoryBlock.
		MemoryBlock& m_memoryBlock;

		StringPool m_stringPool;

		// Bump arena for edge-token spans. 1 GB virtual reserve, pages
		// commit lazily; never reallocates so TokenSpan pointers are stable.
		MemoryBlock m_tokenArena;

		Vector<NinjaRule> m_rules;
		Vector<NinjaBuildEdge> m_edges;
		Vector<NinjaPool> m_pools;
		UnorderedMap<u32, u32> m_globalVars;  // scope 0
		// File-scoped vars per subninja (indexed by scopeId-1). Subninja scope
		// reads parent vars but writes only its own. m_scopeParents[i] is the
		// parent of scope i+1; ExpandVariables walks up the chain.
		Vector<UnorderedMap<u32, u32>> m_scopeVars;
		Vector<u32>            m_scopeParents;
		Vector<u32> m_defaults;
		UnorderedMap<u32, u32> m_ruleMap;
		UnorderedMap<u32, u32> m_poolMap;

		// Guards push_back to m_scopeVars / m_scopeParents in phase 2.
		Futex m_scopeAllocLock;

		// Per-worker token arenas; owned because TokenSpan pointers in m_edges
		// reference them and must outlive worker threads.
		Vector<MemoryBlock*> m_workerArenas;

		// Pre-interned ids for common variable names — skips per-call hash lookup.
		u32 m_commandId;
		u32 m_descriptionId;
		u32 m_depfileId;
		u32 m_depsId;
		u32 m_rspfileId;
		u32 m_rspfile_contentId;
		u32 m_poolId;
		u32 m_restatId;
		u32 m_generatorId;
		u32 m_depthId;
		u32 m_inId;
		u32 m_outId;
		u32 m_phonyOutputId; // AOSP edge variable

		EdgeParsedCallback m_edgeParsedCallback = nullptr;
		void* m_edgeCallbackUserData = nullptr;

		u64 m_parseTimeMs = 0;
		// Atomic — phase 2 workers add concurrently.
		std::atomic<u32> m_lineCount { 0 };
		u32 m_buildRootDirId = StringId_Empty; // anchor for subninja relative paths
	};
}
