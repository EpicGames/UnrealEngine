// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaSynchronization.h"
#include "UbaVector.h"
#include <atomic>

namespace uba
{
	struct CharStringView
	{
		const char* data = nullptr;
		u32 length = 0;

		CharStringView() = default;
		CharStringView(const char* str, u32 len) : data(str), length(len) {}

		bool operator==(const CharStringView& other) const
		{
			if (length != other.length)
				return false;
			return memcmp(data, other.data, length) == 0;
		}
	};

	// 64 hash shards keyed by `hash & 63` (Swiss-table style + Futex per shard).
	// IDs are dense (1..N) from a global counter; ID 0 = empty.
	// m_strings is pre-sized once (Reserve / ctor) and never reallocated, so
	// concurrent Intern + GetString are safe — Intern publishes the id only
	// after the slot is written; per-shard mutex is the release barrier.
	class StringPool
	{
	public:
		StringPool(MemoryBlock& memoryBlock);

		// Thread-safe; locks at most one shard mutex.
		u32 Intern(const char* str, u32 length);
		u32 Intern(const CharStringView& view) { return Intern(view.data, view.length); }
		u32 InternThreadSafe(const char* str, u32 length) { return Intern(str, length); }

		inline CharStringView GetString(u32 id) const
		{
			if (id == 0) return CharStringView{ "", 0 };
			if (id >= m_nextId.load(std::memory_order_acquire))
				return CharStringView{ "", 0 };
			const StringEntry& entry = m_strings[id];
			return CharStringView{ entry.data, entry.length };
		}

		inline const char* GetCStr(u32 id) const
		{
			if (id == 0) return "";
			if (id >= m_nextId.load(std::memory_order_acquire)) return "";
			return m_strings[id].data;
		}

		// MUST be called before any concurrent Intern — resizes m_strings.
		void Reserve(u32 capacity);

		// Number of interned strings (including ID 0).
		inline u32 Size() const { return m_nextId.load(std::memory_order_acquire); }

		static constexpr u32 ShardBits  = 6;
		static constexpr u32 ShardCount = 1u << ShardBits; // 64
		static constexpr u32 ShardMask  = ShardCount - 1;

	private:
		struct StringEntry
		{
			const char* data;
			u32 length;
			u32 hash;
		};

		// Swiss Tables style: ctrl bytes + ids in parallel arrays.
		static constexpr u8 kEmpty   = 0x80;
		static constexpr u8 kDeleted = 0xFF;

		struct Shard
		{
			Vector<u8>  ctrl;
			Vector<u32> ids;
			u32         hashMask  = 0;
			u32         hashCount = 0;
			Futex       mutex;
		};

		MemoryBlock& m_memoryBlock;
		Shard        m_shards[ShardCount];

		// Pre-sized; never grown under concurrent Intern. Index 0 = empty placeholder.
		Vector<StringEntry> m_strings;
		std::atomic<u32>    m_nextId { 1 };

		static u32 HashString(const char* data, u32 length);
		// Non-static — read m_strings on grow. Caller holds the shard mutex.
		void RehashShard(Shard& shard, u32 newSize);
		u32  FindSlot(const Shard& shard, u32 hash, const char* str, u32 length) const;
		u32 InternImplLocked(Shard& shard, const char* str, u32 length, u32 hash);
	};

	static constexpr u32 StringId_Empty = 0;
}
