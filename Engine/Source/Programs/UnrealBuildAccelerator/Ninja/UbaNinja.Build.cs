// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UbaNinja : ModuleRules
{
	public UbaNinja(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivatePCHHeaderFile = "../Core/Public/UbaCorePch.h";

		bRequiresPlatformSDK = true;

		PrivateDependencyModuleNames.AddRange([
			"UbaCommon",
		]);
	}
}
