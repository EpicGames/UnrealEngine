// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaImportLibWriter.h"
#include "UbaFileAccessor.h"
#include "UbaMap.h"
#include "UbaObjectFile.h"

#if PLATFORM_WINDOWS
#define WINDOWS_LEAN_AND_MEAN
#include <windows.h>
#include <algorithm>
#endif

namespace uba
{
#if PLATFORM_WINDOWS

	class ImportLib
	{
		#pragma pack(push, 1)
		struct DATA_DESCRIPTOR
		{
			u16 machine;
			u16 a;
			u32 date;
			u32 sizeOfData;
			u32 b;
			u32 flags;
		};
		struct SECTION_DESCRIPTOR_LONG
		{
			char name[16];
			u32 sizeOfData;
			u32 offset;
			u32 aOffset;
			u32 a;
			u32 aCount;
			u32 b;
		};
		struct SECTION_DESCRIPTOR_SHORT
		{
			union { char name[8]; struct { u32 unknown; u32 offset; } a; };
			u32 b;
			u32 c;
			u16 type;
		};
		#pragma pack(pop)

		u8 m_unknownData0[12] = { 0x02, 0x00, 0x00, 0x00, 0x20, 0x00, 0x09, 0x00, 0x00, 0x00, 0x00, 0x00 };
		u8 m_unknownData1[41] = { 0x27, 0x00, 0x13, 0x10, 0x07, 0x00, 0x00, 0x00, 0xD0, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x0E, 0x00, 0x26, 0x00, 0x6A, 0x81, 0x12, 0x4D, 0x69, 0x63, 0x72, 0x6F, 0x73, 0x6F, 0x66, 0x74, 0x20, 0x28, 0x52, 0x29, 0x20, 0x4C, 0x49, 0x4E, 0x4B };
		u8 m_unknownData2[20] = { 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00 };
		u8 m_unknownData3[30] = { 0xC, 0x00, 0x00, 0x00, 0x03, 0x00, 0x00, 0x00, 0x03, 0x00, 0x00, 0x00, 0x00, 0x00, 0x04, 0x00, 0x00, 0x00, 0x03, 0x00, 0x10, 0x00, 0x00, 0x00, 0x05, 0x00, 0x00, 0x00, 0x03, 0x00 };

		u32 AfterCompId = 0x0101816A;
		u32 PredefinedSymbolsCount = 3;

		struct SymbolInfo
		{
			std::string name;
			u32 offsetIndex;
			union { u32 hint; u32 ordinal; u32 value; };
			u32 type;
			u32 tempOrder;
			u8 extra;
			u8 isData;
		};

		MemoryBlock m_memory;
		u32 m_memoryOffset;

		Vector<SymbolInfo*> m_symbols;
		u32 m_extraSymbolCount = 0;
		u32 m_dataCount;
		u16 m_machine = IMAGE_FILE_MACHINE_AMD64;
		u64 m_date;
		std::string m_moduleName;
		std::string m_headerName;
		bool m_noHeaderName;

		template<typename Func>
		void TraverseSortedSymbols(const Func& func)
		{
			auto sortedVec(m_symbols);
			std::sort(sortedVec.begin(), sortedVec.end(), [](SymbolInfo* a, SymbolInfo* b) { return a->name < b->name; });
			for (auto& s : sortedVec)
				func(*s);
		}

		void AddSymbol(const char* name, bool isData, u32 value)
		{
			u32 offsetIndex = u32(m_symbols.size()) - m_extraSymbolCount;
			SymbolInfo& s0 = *m_symbols.emplace_back(new SymbolInfo());

			s0.isData = isData;
			s0.tempOrder = u32(m_symbols.size()-1) - m_extraSymbolCount - 3;
			s0.offsetIndex = offsetIndex;

			if (s0.isData)
			{
				s0.name = std::string("__imp_") + name;
				s0.value = 0;
				s0.type = 5;
				return;
			}

			s0.name = name;
			s0.hint = value;
			s0.type = 4;

			auto& s1 = *m_symbols.emplace_back(new SymbolInfo());
			s1.name = "__imp_" + s0.name;
			s1.offsetIndex = offsetIndex;
			s1.value = 0;
			s1.type = 0;
			s1.extra = 1;

			++m_extraSymbolCount;
		}

		void InitSymbolList(const char* name)
		{
			char buf[200];

			sprintf_s(buf, sizeof(buf), "__IMPORT_DESCRIPTOR_%s", name);
			auto& s0 = *m_symbols.emplace_back(new SymbolInfo());
			s0.name = buf;
			s0.offsetIndex = 0;
			s0.value = 0;
			s0.type = 0;
			auto& s1 = *m_symbols.emplace_back(new SymbolInfo());
			s1.name = "__NULL_IMPORT_DESCRIPTOR";
			s1.offsetIndex = 1;
			s1.value = 0;
			s1.type = 0;
			sprintf_s(buf, sizeof(buf), "\x7F%s_NULL_THUNK_DATA", name);
			auto& s2 = *m_symbols.emplace_back(new SymbolInfo());
			s2.name = buf;
			s2.offsetIndex = 2;
			s2.value = 0;
			s2.type = 0;
		}

		void Write(const void* data, u64 bytes)
		{
			u32 newOffset = m_memoryOffset + u32(bytes);
			if (m_memory.writtenSize < newOffset)
				m_memory.AllocateNoLock(newOffset - m_memory.writtenSize, 1, TC("ImportLibWriter"));
			memcpy(m_memory.memory + m_memoryOffset, data, bytes);
			m_memoryOffset = newOffset;
		}

		u32 ByteSwap(u32 Value)
		{
			return _byteswap_ulong(Value);
		}
		void WriteU8(u8 value) { Write(&value, sizeof(value)); }
		void WriteU16(u16 value) { Write(&value, sizeof(value)); }
		void WriteU32(u32 value) { Write(&value, sizeof(value)); }
		void SkipWrite(u32 distance) { m_memoryOffset += distance; }
		u32 GetWritePosition() { return m_memoryOffset; }
		void SetWritePosition(u32 offset) { m_memoryOffset = offset; }

		void WriteFileHeader(const char* str, bool prefixWithSlash, u32 a, u32 size)
		{
			IMAGE_ARCHIVE_MEMBER_HEADER FileHeader;
			memset(&FileHeader, ' ', IMAGE_SIZEOF_ARCHIVE_MEMBER_HDR);
		
			u32 stringLength = u32(strlen(str));
			if (prefixWithSlash)
			{
				FileHeader.Name[0] = '/';
				memcpy(FileHeader.Name + 1, str, stringLength);
			}
			else
			{
				memcpy(FileHeader.Name, str, stringLength);
				FileHeader.Name[stringLength] = '/';
			}

			#pragma warning(push)
			#pragma warning(disable : 6386)
			sprintf_s((char*)FileHeader.Date, 13, "%-12I64d", s64(m_date));
			FileHeader.UserID[0] = ' '; // Remove 0 termination from Date
			sprintf_s((char*)FileHeader.Mode, 9, "%-8ho", a);
			sprintf_s((char*)FileHeader.Size, 11, "%-10d", int(size));
			#pragma warning(pop)

			FileHeader.EndHeader[0] = 0x60;
			FileHeader.EndHeader[1] = 0xA;
			Write(&FileHeader, IMAGE_SIZEOF_ARCHIVE_MEMBER_HDR);
		}

		void WriteSectionDescriptionLong(const char* sectionName, u32 sizeOfData, u32 offset, u32 aOffset, u32 a, u32 aCount, u32 b)
		{
			SECTION_DESCRIPTOR_LONG desc;
			memset(&desc, 0, sizeof(desc));
			strcpy_s(desc.name, sizeof(desc.name), sectionName);
			desc.sizeOfData = sizeOfData;
			desc.offset = offset;
			desc.aOffset = aOffset;
			desc.a = a;
			desc.aCount = aCount;
			desc.b = b;
			Write(&desc, sizeof(desc));
		}

		void WriteSectionDescriptionShort(const char* sectionName, u32 b, u32 c, u16 type)
		{
			SECTION_DESCRIPTOR_SHORT desc;
			memset(&desc, 0, sizeof(desc));
			memcpy(desc.name, sectionName, strlen(sectionName));
			desc.b = b;
			desc.c = c;
			desc.type = type;
			Write(&desc, sizeof(desc));
		}

		void WriteSectionDescriptionShort(u32 offset, u32 b, u32 c, u16 type)
		{
			SECTION_DESCRIPTOR_SHORT desc;
			memset(&desc, 0, sizeof(desc));
			desc.a.offset = offset;
			desc.b = b;
			desc.c = c;
			desc.type = type;
			Write(&desc, sizeof(desc));
		}

		void WriteDataDescriptor(u16 a, u32 sizeOfData, u32 b)
		{
			DATA_DESCRIPTOR desc;
			desc.machine = m_machine;
			desc.a = a;
			desc.date = u32(m_date);
			desc.sizeOfData = sizeOfData;
			desc.b = b;
			desc.flags = 0;
			Write(&desc, sizeof(desc));
		}

		void WriteImportLibrary()
		{
			Write(IMAGE_ARCHIVE_START, IMAGE_ARCHIVE_START_SIZE);

			Vector<u32> symbolOffsets;
			symbolOffsets.resize(2 + m_symbols.size() - m_extraSymbolCount);

			{
				// File 1
				u32 offset = GetWritePosition();
				symbolOffsets[0] = offset;

				SkipWrite(IMAGE_SIZEOF_ARCHIVE_MEMBER_HDR);
				WriteU32(ByteSwap(u32(m_symbols.size())));
				SkipWrite(u32(m_symbols.size()) * sizeof(u32));
				for (auto symbol : m_symbols)
					Write(symbol->name.data(), symbol->name.size() + 1);

				u32 offset2 = GetWritePosition();

				SetWritePosition(offset);
				WriteFileHeader("", false, 0, offset2 - (offset + IMAGE_SIZEOF_ARCHIVE_MEMBER_HDR));
				SetWritePosition(offset2);
				if (offset2 & 1)
					Write("\n", 1);

				// File 2
				offset = GetWritePosition();
				symbolOffsets[1] = offset;

				SkipWrite(IMAGE_SIZEOF_ARCHIVE_MEMBER_HDR);
				u32 symbolCount = u32(m_symbols.size()) - m_extraSymbolCount;
				WriteU32(symbolCount);
				SkipWrite(symbolCount * sizeof(u32));
				symbolCount = u32(m_symbols.size());
				WriteU32(symbolCount);
				SkipWrite(symbolCount * sizeof(u16));
				TraverseSortedSymbols([&](SymbolInfo& symbol) { Write(symbol.name.data(), symbol.name.size() + 1); });

				offset2 = GetWritePosition();

				SetWritePosition(offset);
				WriteFileHeader("", false, 0, offset2 - (offset + IMAGE_SIZEOF_ARCHIVE_MEMBER_HDR));
				SetWritePosition(offset2);
				if (offset2 & 1)
					Write("\n", 1);

				u32 len = u32(m_moduleName.size()) + 1;
				if (len > 0x10)
				{
					WriteFileHeader("/", false, 0, len);
					Write(m_moduleName.data(), len);
					if (len & 1)
						Write("\n", 1);
				}
			}

			{
				u32 offset = GetWritePosition();
				symbolOffsets[2] = offset;

				SkipWrite(IMAGE_SIZEOF_ARCHIVE_MEMBER_HDR);

				u8 moduleNameLen = u8(m_moduleName.size());
				u32 stringPad = (moduleNameLen + 1) & 1;

				u32 sizeOfData = (sizeof(DATA_DESCRIPTOR) + (sizeof(SECTION_DESCRIPTOR_LONG) * 3)) +
					(sizeof(m_unknownData0) + sizeof(moduleNameLen) + moduleNameLen +
					sizeof(m_unknownData1)) + (sizeof(m_unknownData2) + sizeof(m_unknownData3)) +
					(moduleNameLen + 1) + stringPad;

				WriteDataDescriptor(3, sizeOfData, 8);

				// header0
				sizeOfData = sizeof(m_unknownData0) + sizeof(moduleNameLen) + moduleNameLen + sizeof(m_unknownData1);
				u32 offset3 = sizeof(DATA_DESCRIPTOR) + (sizeof(SECTION_DESCRIPTOR_LONG) * 3);
				WriteSectionDescriptionLong(".debug$S", sizeOfData, offset3, 0, 0, 0, 0x42100040);

				// header1
				offset3 += sizeOfData;
				sizeOfData = sizeof(m_unknownData2);
				u32 AOffset = offset3 + sizeOfData;
				WriteSectionDescriptionLong(".idata$2", sizeOfData, offset3, AOffset, 0, 3, 0xC0300040);

				// header2
				offset3 += sizeOfData + sizeof(m_unknownData3);
				sizeOfData = moduleNameLen + 1 + stringPad;
				WriteSectionDescriptionLong(".idata$6", sizeOfData, offset3, AOffset, 0, 0, 0xC0200040);

				// data0
				Write(m_unknownData0, sizeof(m_unknownData0));
				Write(&moduleNameLen, sizeof(moduleNameLen));
				Write(m_moduleName.data(), moduleNameLen);
				Write(m_unknownData1, sizeof(m_unknownData1));

				// data1
				Write(m_unknownData2, sizeof(m_unknownData2));

				// data2
				Write(m_unknownData3, sizeof(m_unknownData3));

				// 
				Write(m_moduleName.data(), moduleNameLen + 1);
				if (stringPad)
					WriteU8(0);

				WriteSectionDescriptionShort("@comp.id", AfterCompId, 0xFFFF, 3);

				offset3 = sizeof(sizeOfData);
				WriteSectionDescriptionShort(offset3, 0, 2, 2);

				WriteSectionDescriptionShort(".idata$2", 0xC0000040, 2, 0x68);
				WriteSectionDescriptionShort(".idata$6", 0, 3, 3);
				WriteSectionDescriptionShort(".idata$4", 0xC0000040, 0, 0x68);
				WriteSectionDescriptionShort(".idata$5", 0xC0000040, 0, 0x68);

				offset3 += u32(m_symbols[0]->name.size()) + 1;
				WriteSectionDescriptionShort(offset3, 0, 0, 2);

				offset3 += u32(m_symbols[1]->name.size()) + 1;
				WriteSectionDescriptionShort(offset3, 0, 0, 2);

				offset3 += u32(m_symbols[2]->name.size()) + 1;
				WriteU32(offset3);

				for (u32 i = 0; i < PredefinedSymbolsCount; ++i)
					Write(m_symbols[i]->name.data(), u32(m_symbols[i]->name.size()) + 1);

				u32 offset2 = GetWritePosition();
				SetWritePosition(offset);
				WriteFileHeader(m_headerName.data(), m_noHeaderName, 0, offset2 - (offset + IMAGE_SIZEOF_ARCHIVE_MEMBER_HDR));
				SetWritePosition(offset2);
				if (offset2 & 1)
					Write("\n", 1);
			}

			{
				u32 offset = GetWritePosition();
				symbolOffsets[3] = offset;

				SkipWrite(IMAGE_SIZEOF_ARCHIVE_MEMBER_HDR);

				// DATA DESCRIPTOR
				u8 moduleNameLen = u8(m_moduleName.size());
				u32 sizeOfData = (sizeof(DATA_DESCRIPTOR) + (sizeof(SECTION_DESCRIPTOR_LONG) * 2)) +
					(sizeof(m_unknownData0) + sizeof(moduleNameLen) + moduleNameLen + sizeof(m_unknownData1)) +
					(sizeof(m_unknownData2));

				WriteDataDescriptor(2, sizeOfData, 2);

				// DATA HEADER 1
				sizeOfData = sizeof(m_unknownData0) + sizeof(moduleNameLen) + moduleNameLen + sizeof(m_unknownData1);
				u32 offset3 = sizeof(DATA_DESCRIPTOR) + (sizeof(SECTION_DESCRIPTOR_LONG) * 2);
				WriteSectionDescriptionLong(".debug$S", sizeOfData, offset3, 0, 0, 0, 0x42100040);

				// DATA HEADER 2
				offset3 += sizeOfData;
				sizeOfData = sizeof(m_unknownData2);
				WriteSectionDescriptionLong(".idata$3", sizeOfData, offset3, 0, 0, 0, 0xC0300040);

				// DATA 1
				Write(m_unknownData0, sizeof(m_unknownData0));
				Write(&moduleNameLen, sizeof(moduleNameLen));
				Write(m_moduleName.data(), moduleNameLen);
				Write(m_unknownData1, sizeof(m_unknownData1));

				// DATA 2
				Write(m_unknownData2, sizeof(m_unknownData2));

				WriteSectionDescriptionShort("@comp.id", AfterCompId, 0xFFFF, 3);

				offset3 = sizeof(sizeOfData);
				WriteSectionDescriptionShort(offset3, 0, 2, 2);

				SymbolInfo& nullImportSymbol = *m_symbols[1];
				WriteU32(offset3 + u32(nullImportSymbol.name.size() + 1));
				Write(nullImportSymbol.name.data(), nullImportSymbol.name.size() + 1);

				u32 offset2 = GetWritePosition();
				SetWritePosition(offset);
				WriteFileHeader(m_headerName.data(), m_noHeaderName, 0, offset2 - (offset + IMAGE_SIZEOF_ARCHIVE_MEMBER_HDR));
				SetWritePosition(offset2);
				if (offset2 & 1)
					Write("\n", 1);
			}

			{
				u32 offset = GetWritePosition();
				symbolOffsets[4] = offset;

				SkipWrite(IMAGE_SIZEOF_ARCHIVE_MEMBER_HDR);

				u8 moduleNameLen = u8(m_moduleName.size());

				u32 pad[2] = { 0 };

				u32 sizeOfData = (sizeof(DATA_DESCRIPTOR) + (sizeof(SECTION_DESCRIPTOR_LONG) * 3)) +
					(sizeof(m_unknownData0) + sizeof(moduleNameLen) + moduleNameLen + sizeof(m_unknownData1)) +
					sizeof(pad) +
					sizeof(pad);

				WriteDataDescriptor(3, sizeOfData, 2);

				// header 0
				sizeOfData = sizeof(m_unknownData0) + sizeof(moduleNameLen) + moduleNameLen + sizeof(m_unknownData1);
				u32 offset3 = sizeof(DATA_DESCRIPTOR) + (sizeof(SECTION_DESCRIPTOR_LONG) * 3);
				WriteSectionDescriptionLong(".debug$S", sizeOfData, offset3, 0, 0, 0, 0x42100040);

				// header 1
				offset3 += sizeOfData;
				sizeOfData = sizeof(pad);
				WriteSectionDescriptionLong(".idata$5", sizeOfData, offset3, 0, 0, 0, 0xC0400040);

				// header 2
				offset3 += sizeOfData;
				sizeOfData = sizeof(pad);
				WriteSectionDescriptionLong(".idata$4", sizeOfData, offset3, 0, 0, 0, 0xC0400040);

				// data 0
				Write(m_unknownData0, sizeof(m_unknownData0));
				Write(&moduleNameLen, sizeof(moduleNameLen));
				Write(m_moduleName.data(), moduleNameLen);
				Write(m_unknownData1, sizeof(m_unknownData1));

				// data 1 & 2
				Write(pad, sizeof(pad));
				Write(pad, sizeof(pad));

				WriteSectionDescriptionShort("@comp.id", AfterCompId, 0xFFFF, 3);

				offset3 = sizeof(sizeOfData);
				WriteSectionDescriptionShort(offset3, 0, 2, 2);

				SymbolInfo& thunkDataSymbol = *m_symbols[2];
				WriteU32(offset3 + u32(thunkDataSymbol.name.size() + 1));
				Write(thunkDataSymbol.name.data(), thunkDataSymbol.name.size() + 1);

				u32 offset2 = GetWritePosition();
				SetWritePosition(offset);
				WriteFileHeader(m_headerName.data(), m_noHeaderName, 0, offset2 - (offset + IMAGE_SIZEOF_ARCHIVE_MEMBER_HDR));
				SetWritePosition(offset2);
				if (offset2 & 1)
					Write("\n", 1);
			}

			{
				Vector<SymbolInfo*> sortedFunctions;
				for (u32 i = PredefinedSymbolsCount; i < m_symbols.size(); ++i)
					if (!m_symbols[i]->extra)
						sortedFunctions.push_back(m_symbols[i]);

				std::sort(sortedFunctions.begin(), sortedFunctions.end(), [](SymbolInfo* a, SymbolInfo* b)
					{
						const char* aName = a->name.data();
						if (a->isData)
							aName += 6;
						const char* bName = b->name.data();
						if (b->isData)
							bName += 6;
						return strcmp(aName, bName) < 0;
					});

				for (u32 i = 0, e = u32(sortedFunctions.size()); i != e; ++i)
				{
					auto& symbol = *sortedFunctions[i];
					symbolOffsets[2 + PredefinedSymbolsCount + symbol.tempOrder] = GetWritePosition();

					const char* symbolName = symbol.name.data();
					u32 symbolNameLen = u32(symbol.name.size());
					if (symbol.isData)
					{
						symbolName += 6;
						symbolNameLen -= 6;
					}

					u32 sizeOfData = u32(m_moduleName.size()) + symbolNameLen + 2;
					WriteFileHeader(m_headerName.data(), m_noHeaderName, 0, sizeOfData + sizeof(IMPORT_OBJECT_HEADER));

					IMPORT_OBJECT_HEADER header;
					memset(&header, 0, sizeof(IMPORT_OBJECT_HEADER));
					header.Sig1 = 0;
					header.Sig2 = 0xFFFF;
					header.Version = 0;
					header.Machine = m_machine;
					header.TimeDateStamp = u32(m_date);
					header.SizeOfData = sizeOfData;
					header.Ordinal = u16(i);
					header.Type = symbol.type & 3;
					header.NameType = WORD(symbol.type >> 2);
					header.Reserved = 0;

					Write(&header, sizeof(header));
					Write(symbolName, symbolNameLen + 1);
					Write(m_moduleName.data(), m_moduleName.size() + 1);
					if (GetWritePosition() & 1)
						Write("\n", 1);
				}

				u32 Offset = GetWritePosition();

				// Offsets
				SetWritePosition(symbolOffsets[0] + IMAGE_SIZEOF_ARCHIVE_MEMBER_HDR + sizeof(u32));
				for (auto& symbol : m_symbols)
					WriteU32(ByteSwap(symbolOffsets[2 + symbol->offsetIndex]));

				// Offset table
				SetWritePosition(symbolOffsets[1] + IMAGE_SIZEOF_ARCHIVE_MEMBER_HDR + sizeof(u32));
				for (u32 i = 0; i < (m_symbols.size() - m_extraSymbolCount); ++i)
					Write((symbolOffsets.data() + (2 + i)), sizeof(u32));

				// Offset indices
				SkipWrite(sizeof(u32));
				TraverseSortedSymbols([&](SymbolInfo& symbol) { WriteU16(u16(symbol.offsetIndex + 1)); });
				SetWritePosition(Offset);
			}
		}

	public:

		ImportLib() : m_memory(TC("ImportLib"), 128*1024*1024) // 512mb should be more than enough reserved memory for the biggest import libs
		{
		}

		~ImportLib()
		{
			for (auto ptr : m_symbols)
				delete ptr;
		}

		// Strip the extension off libName, seed the three predefined symbols, and record the
		// full module name (base + ext) for later. Shared by both WriteFile overloads.
		void Begin(const char* libName, std::string& outModuleName)
		{
			char temp[256];
			const char* libExt = strrchr(libName, '.');
			if (libExt)
			{
				u32 len = u32(libExt - libName);
				memcpy(temp, libName, len);
				temp[len] = 0;
				libName = temp;
			}
			else
				libExt = ".dll";

			InitSymbolList(libName);
			outModuleName = std::string(libName) + libExt;
		}

		bool WriteFile(Logger& logger, const Vector<ObjectFile*>& objFiles, const char* libName, const tchar* libFile)
		{
			std::string moduleName;
			Begin(libName, moduleName);

			UnorderedSet<StringKey> handled;
			for (auto objFile : objFiles)
			{
				if (!objFile)
					continue;

				Map<u32, const UnorderedExports::value_type*> sortedSymbols;
				for (auto& kv : objFile->GetExports())
					if (handled.insert(kv.first).second)
						sortedSymbols.try_emplace(kv.second.index, &kv);
				for (auto& outerKv : sortedSymbols)
					AddSymbol(outerKv.second->second.symbol.c_str(), outerKv.second->second.isData, 0);
			}

			return Finalize(logger, moduleName, libFile);
		}

		bool WriteFile(Logger& logger, const UnorderedExports& exports, const char* libName, const tchar* libFile)
		{
			std::string moduleName;
			Begin(libName, moduleName);

			Map<u32, const UnorderedExports::value_type*> sortedSymbols;
			for (auto& kv : exports)
				sortedSymbols.try_emplace(kv.second.index, &kv);
			for (auto& outerKv : sortedSymbols)
				AddSymbol(outerKv.second->second.symbol.c_str(), outerKv.second->second.isData, 0);

			return Finalize(logger, moduleName, libFile);
		}

		bool Finalize(Logger& logger, const std::string& moduleName, const tchar* libFile)
		{
			m_moduleName = moduleName;

			if ((m_moduleName.size() + 1) > 0x10)
			{
				m_headerName = "0";
				m_noHeaderName = TRUE;
			}
			else
			{
				m_headerName = m_moduleName;
				m_noHeaderName = FALSE;
			}

			m_date = ~0ull;//_time64(NULL);

			m_unknownData0[4] = u8(m_moduleName.size()) + 7; // Have no idea why this matches and what it does.
			// BlankProgram-Projects.lib - 0x20
			// BlankProgram-Core.lib - 0x1C
			// BlankProgram-Json.lib - 0x1C
			// BlankProgram-BuildSettings.lib - 0x25

			m_memoryOffset = 0;

			WriteImportLibrary();

			FileAccessor fa(logger, libFile);
			if (!fa.CreateMemoryWrite(false, DefaultAttributes(), m_memory.writtenSize))
				return false;

			memcpy(fa.GetData(), m_memory.memory, m_memory.writtenSize);

			if (!fa.Close())
				return false;

			// Create exp file
			StringBuffer<> expFile;
			expFile.Append(libFile);
			if (const tchar* dot = expFile.Last('.'))
				expFile.Resize(dot - expFile.data).Append(TCV(".exp"));
			FileAccessor faExp(logger, expFile.data);
			if (!faExp.CreateWrite())
				return false;
			return faExp.Close();
		}
	};
#endif // PLATFORM_WINDOWS

	bool ImportLibWriter::Write(Logger& logger, const Vector<ObjectFile*>& objFiles, const char* libName, const tchar* libFile)
	{
#if PLATFORM_WINDOWS
		ImportLib lib;
		return lib.WriteFile(logger, objFiles, libName, libFile);
#else
		return false;
#endif
	}

	bool ImportLibWriter::Write(Logger& logger, const UnorderedExports& exports, const char* libName, const tchar* libFile)
	{
#if PLATFORM_WINDOWS
		ImportLib lib;
		return lib.WriteFile(logger, exports, libName, libFile);
#else
		return false;
#endif
	}
}
