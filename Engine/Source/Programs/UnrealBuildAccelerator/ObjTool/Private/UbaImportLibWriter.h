// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaLogger.h"
#include "UbaObjectFile.h"
#include "UbaStringBuffer.h"

namespace uba
{
	class ObjectFile;
	
	class ImportLibWriter
	{
	public:
		bool Write(Logger& logger, const Vector<ObjectFile*>& objFiles, const char* libName, const tchar* libFile);
		bool Write(Logger& logger, const UnorderedExports& exports, const char* libName, const tchar* libFile);
	};
}
