// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaImportStubWriterElf.h"
#include "UbaElf.h"
#include "UbaFileAccessor.h"
#include "UbaMemory.h"
#include <algorithm>
#include <string.h>

namespace uba
{
	struct Elf64Sym { u32 st_name; u8 st_info; u8 st_other; u16 st_shndx; u64 st_value; u64 st_size; };

	static constexpr u16 SHN_ABS = 0xfff1;
	static constexpr u32 SHT_HASH = 5;
	static constexpr u8 STB_GLOBAL = 1;
	static constexpr u8 STT_FUNC = 2;
	static constexpr u8 STT_OBJECT = 1;
	static constexpr s64 DT_HASH = 4, DT_STRTAB = 5, DT_SYMTAB = 6, DT_STRSZ = 10, DT_SYMENT = 11, DT_SONAME = 14, DT_NULL = 0;

	static u32 ElfHash(const char* s)
	{
		u32 h = 0, g;
		for (; *s; ++s) { h = (h << 4) + (u8)*s; if ((g = h & 0xf0000000)) h ^= g >> 24; h &= ~g; }
		return h;
	}

	bool ImportStubWriterElf::Write(Logger& logger, MemoryBlock& out, const UnorderedExports& exports, const char* soName, u16 machine)
	{
		// Gather exports in a stable order (sorted by name) so output bytes are deterministic.
		Vector<const ExportInfo*> syms;
		syms.reserve(exports.size());
		for (auto& kv : exports)
			if (!kv.second.symbol.empty())
				syms.push_back(&kv.second);
		std::sort(syms.begin(), syms.end(), [](const ExportInfo* a, const ExportInfo* b) { return a->symbol < b->symbol; });

		// Give each data export its own slot in .data so copy relocations stay independent, placed at its real
		// alignment so the dependent's copy of an over-aligned global (e.g. alignas(128)) is aligned too.
		// Functions can share one address (resolved via PLT, never copy-relocated).
		Vector<u64> dataRelAddr(syms.size(), 0);
		u64 dataSpan = 0;
		u64 maxDataAlign = 16;
		for (u64 i = 0; i != syms.size(); ++i)
		{
			if (!syms[i]->isData)
				continue;
			u64 align = syms[i]->alignment < 16 ? 16 : syms[i]->alignment;
			if (align > maxDataAlign)
				maxDataAlign = align;
			dataSpan = (dataSpan + align - 1) & ~(align - 1);
			dataRelAddr[i] = dataSpan;
			dataSpan += syms[i]->size ? syms[i]->size : 1;
		}
		dataSpan = (dataSpan + maxDataAlign - 1) & ~(maxDataAlign - 1);
		if (!dataSpan)
			dataSpan = maxDataAlign;

		u64 nsym = syms.size() + 1; // +1 for the null symbol at index 0

		// .dynstr: [0][soName\0][sym0\0][sym1\0]...
		Vector<char> dstr; dstr.push_back(0);
		u32 sonameOff = u32(dstr.size());
		for (const char* p = soName; ; ++p) { dstr.push_back(*p); if (!*p) break; }
		Vector<u32> nameOff(nsym, 0);
		for (u64 i = 0; i != syms.size(); ++i)
		{
			nameOff[i + 1] = u32(dstr.size());
			for (const char* p = syms[i]->symbol.c_str(); ; ++p) { dstr.push_back(*p); if (!*p) break; }
		}

		// .hash (sysv): nbucket, nchain(==nsym), bucket[nbucket], chain[nsym]
		u32 nbucket = u32(nsym);
		Vector<u32> bucket(nbucket, 0), chain(nsym, 0);
		for (u32 i = 1; i != nsym; ++i)
		{
			u32 h = ElfHash(syms[i - 1]->symbol.c_str()) % nbucket;
			chain[i] = bucket[h]; bucket[h] = i;
		}

		// Layout: [Ehdr][2 Phdr][.hash][.dynsym][.dynstr][.dynamic][.shstrtab][6 Shdr]. file offset == vaddr.
		u64 off = sizeof(Elf64FileHeader);
		u64 phOff = off; off += 2 * sizeof(Elf64ProgramHeader);
		u64 hashOff = off; u64 hashSz = (2 + u64(nbucket) + nsym) * 4; off += hashSz;
		u64 symOff = off; u64 symSz = nsym * sizeof(Elf64Sym); off += symSz;
		u64 strOff = off; u64 strSz = dstr.size(); off += strSz;
		off = (off + 7) & ~7ull;
		u64 dynOff = off;
		Elf64Dyn dyn[] = {
			{ DT_HASH, { hashOff } }, { DT_SYMTAB, { symOff } }, { DT_STRTAB, { strOff } },
			{ DT_STRSZ, { strSz } }, { DT_SYMENT, { sizeof(Elf64Sym) } }, { DT_SONAME, { sonameOff } }, { DT_NULL, { 0 } },
		};
		u64 dynSz = sizeof(dyn); off += dynSz;

		// .shstrtab. Exports point at real ALLOC sections (.text for code, .data for data) — NOT SHN_ABS, which ld
		// would resolve as an absolute definition (baked value, no DT_NEEDED) instead of a dynamic import.
		const char shstr[] = "\0.hash\0.dynsym\0.dynstr\0.dynamic\0.text\0.data\0.shstrtab";
		u32 n_hash = 1, n_dsym = 7, n_dstr = 15, n_dyn = 23, n_text = 32, n_data = 38, n_shstr = 44; // offsets into shstr
		u64 shstrOff = off; off += sizeof(shstr);
		u64 shdrsOff = (off + 7) & ~7ull; u64 total = shdrsOff + 8 * sizeof(Elf64SectionHeader);

		// .text/.data live past the file image in the load segment (stub carries no real content; the real lib
		// replaces it at runtime via matching DT_SONAME). They only need to be ALLOC so ld treats exports as defined.
		u64 textAddr = (total + 15) & ~15ull;
		u64 dataAddr = (textAddr + 16 + maxDataAlign - 1) & ~(maxDataAlign - 1);
		u64 dataEnd = dataAddr + dataSpan;
		u16 textNdx = 5, dataNdx = 6;

		Vector<u8> buf(total, 0);
		auto put = [&](u64 o, const void* p, u64 sz) { memcpy(buf.data() + o, p, sz); };

		Elf64FileHeader eh = {};
		memcpy(eh.e_ident, "\177ELF", 4); eh.e_ident[4] = 2; eh.e_ident[5] = 1; eh.e_ident[6] = 1;
		eh.e_type = 3 /*ET_DYN*/; eh.e_machine = machine; eh.e_version = 1;
		eh.e_phoff = phOff; eh.e_shoff = shdrsOff; eh.e_ehsize = sizeof(Elf64FileHeader);
		eh.e_phentsize = sizeof(Elf64ProgramHeader); eh.e_phnum = 2;
		eh.e_shentsize = sizeof(Elf64SectionHeader); eh.e_shnum = 8; eh.e_shstrndx = 7;
		put(0, &eh, sizeof(eh));

		// RWX: .data must sit in a writable segment or ld refuses copy relocations for exported data symbols
		// ("cannot create a copy relocation for symbol ..."). The stub is never loaded at runtime, so RWX is harmless.
		Elf64ProgramHeader pl = {}; pl.p_type = 1 /*PT_LOAD*/; pl.p_flags = 7 /*R+W+X*/; pl.p_filesz = total; pl.p_memsz = dataEnd; pl.p_align = 0x1000;
		Elf64ProgramHeader pd = {}; pd.p_type = 2 /*PT_DYNAMIC*/; pd.p_flags = 6 /*R+W*/; pd.p_offset = dynOff; pd.p_vaddr = dynOff; pd.p_filesz = dynSz; pd.p_memsz = dynSz; pd.p_align = 8;
		put(phOff, &pl, sizeof(pl)); put(phOff + sizeof(pl), &pd, sizeof(pd));

		Vector<u32> hash; hash.push_back(nbucket); hash.push_back(u32(nsym));
		hash.insert(hash.end(), bucket.begin(), bucket.end()); hash.insert(hash.end(), chain.begin(), chain.end());
		put(hashOff, hash.data(), hash.size() * 4);

		Vector<Elf64Sym> os(nsym); os[0] = Elf64Sym{};
		for (u64 i = 0; i != syms.size(); ++i)
		{
			Elf64Sym& s = os[i + 1];
			s.st_name = nameOff[i + 1];
			s.st_info = u8((STB_GLOBAL << 4) | (syms[i]->isData ? STT_OBJECT : STT_FUNC));
			s.st_shndx = syms[i]->isData ? dataNdx : textNdx;
			s.st_value = syms[i]->isData ? (dataAddr + dataRelAddr[i]) : textAddr;
			s.st_size = syms[i]->size;     // preserved for data copy-relocs
		}
		put(symOff, os.data(), symSz);
		put(strOff, dstr.data(), strSz);
		put(dynOff, dyn, dynSz);
		put(shstrOff, shstr, sizeof(shstr));

		Elf64SectionHeader shs[8] = {};
		shs[1] = { n_hash, SHT_HASH, 2 /*ALLOC*/, hashOff, hashOff, hashSz, 2, 0, 8, 4 };
		shs[2] = { n_dsym, SHT_DYNSYM, 2, symOff, symOff, symSz, 3 /*link .dynstr*/, 1 /*1st global*/, 8, sizeof(Elf64Sym) };
		shs[3] = { n_dstr, SHT_STRTAB, 2, strOff, strOff, strSz, 0, 0, 1, 0 };
		shs[4] = { n_dyn, SHT_DYNAMIC, 3 /*ALLOC+WRITE*/, dynOff, dynOff, dynSz, 3 /*link .dynstr*/, 0, 8, sizeof(Elf64Dyn) };
		shs[5] = { n_text, SHT_PROGBITS, 2 | 4 /*ALLOC+EXECINSTR*/, textAddr, total, 0, 0, 0, 16, 0 };
		shs[6] = { n_data, SHT_PROGBITS, 2 | 1 /*ALLOC+WRITE*/, dataAddr, total, dataSpan, 0, 0, maxDataAlign, 0 };
		shs[7] = { n_shstr, SHT_STRTAB, 0, 0, shstrOff, sizeof(shstr), 0, 0, 1, 0 };
		put(shdrsOff, shs, sizeof(shs));

		memcpy(out.Allocate(total, 1, TC("ImportStubElf")), buf.data(), total);
		return true;
	}

	bool ImportStubWriterElf::Write(Logger& logger, const UnorderedExports& exports, const char* soName, u16 machine, const tchar* outFile)
	{
		MemoryBlock memoryBlock(TC("ImportStubElf"), 256*1024*1024);
		if (!Write(logger, memoryBlock, exports, soName, machine))
			return false;
		FileAccessor file(logger, outFile);
		if (!file.CreateWrite())
			return false;
		if (!file.Write(memoryBlock.memory, memoryBlock.writtenSize))
			return false;
		return file.Close();
	}
}
