// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaObjectFile.h"

namespace uba
{
	struct MemoryBlock;

	// Writes a Linux "import library": a stub ET_DYN .so carrying only a dynamic symbol table
	// (the defined exports) plus DT_SONAME = the real library's name. Linking a dependent against
	// this stub resolves the exports single-pass and emits a DT_NEEDED for soName, exactly like a
	// Windows COFF import lib. No code/relocations are emitted. Cross-platform (host win/linux/mac).
	class ImportStubWriterElf
	{
	public:
		// machine: EM_X86_64 (62) or EM_AARCH64 (183). exports: name + isData + size per symbol.
		static bool Write(Logger& logger, MemoryBlock& out, const UnorderedExports& exports, const char* soName, u16 machine);
		static bool Write(Logger& logger, const UnorderedExports& exports, const char* soName, u16 machine, const tchar* outFile);
	};

	static constexpr u16 ElfMachine_X86_64 = 62;
	static constexpr u16 ElfMachine_Aarch64 = 183;
}
