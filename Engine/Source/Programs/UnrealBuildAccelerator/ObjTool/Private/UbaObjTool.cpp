// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaObjectFile.h"
#include "UbaEnvironment.h"
#include "UbaFile.h"
#include "UbaImportLibWriter.h"
#include "UbaImportStubWriterElf.h"
#include "UbaMap.h"
#include "UbaPathUtils.h"
#include "UbaVersion.h"
#include "UbaWorkManager.h"
#include <algorithm>

namespace uba
{
	const tchar* Version = GetVersionString();
	u32	DefaultProcessorCount = []() { return GetLogicalProcessorCount(); }();

	int PrintHelp(const tchar* message)
	{
		LoggerWithWriter logger(g_consoleLogWriter, TC(""));
		if (*message)
		{
			logger.Info(TC(""));
			logger.Error(TC("%s"), message);
		}
		const tchar* dbgStr = TC("");
		#if UBA_DEBUG
		dbgStr = TC(" (DEBUG)");
		#endif

		logger.Info(TC(""));
		logger.Info(TC("-------------------------------------------"));
		logger.Info(TC("   UbaObjTool v%s%s"), Version, dbgStr);
		logger.Info(TC("-------------------------------------------"));
		logger.Info(TC(""));
		logger.Info(TC("  UbaObjTool.exe [options...] <objfile/libfile>"));
		logger.Info(TC(""));
		logger.Info(TC("   Options:"));
		logger.Info(TC("    -printsymbols            Print the symbols found in obj file"));
		logger.Info(TC("    -stripexports            Will strip exports and write them out in a .exp file"));
		logger.Info(TC("    -writeimplib=<file>      Will create a import library from symbols collected from obj/lib files"));
		logger.Info(TC(""));
		logger.Info(TC("  --- OR ---"));
		logger.Info(TC(""));
		logger.Info(TC("  UbaObjTool.exe @<rspfile>"));
		logger.Info(TC(""));
		logger.Info(TC("   Response file options:"));
		logger.Info(TC("    /S:<objfile>             Obj file to export from. Multiple allowed"));
		logger.Info(TC("    /D:<objfile>             Obj file depending on obj files to strip. Multiple allowed"));
		logger.Info(TC("    /O:<objfile>             Obj file to output containing exports and loopbacks"));
		logger.Info(TC("    /L:<libfile>             Import lib to write directly from the computed export set (Windows)"));
		logger.Info(TC("    /T:<platform>            Target platform"));
		logger.Info(TC("    /M:<module>              Name of module. Needed in emd files"));
		logger.Info(TC("    /E:<symbol>              Additional symbol to be exported. To solve combination of dynlist files"));
		logger.Info(TC("    /COMPRESS                Write '/O' file compressed"));
		logger.Info(TC("    /ELF[:<machine>]         Force an ELF import stub (.so). Optional machine: x86_64 (default) or aarch64/arm64"));
		logger.Info(TC(""));
		return -1;
	}

	int WrappedMain(int argc, tchar* argv[])
	{
		using namespace uba;

		//#if !PLATFORM_LINUX
		//AddExceptionHandler();
		//#endif
		InitMemory();

		TString objFile;
		bool printSymbols = false;
		bool writeImpLib = false;
		bool allowLibInputs = false;
		bool isImpLibRsp = false;

		Vector<TString> objFilesToExport;
		Vector<TString> objFilesDependencies;
		TString extraObjFile;
		Vector<TString> objFilesForImpLib;
		ExtraExports extraExports;
		std::string impLibName;
		TString impLibFile;
		TString exportsImpLibFile; // /L: optional import lib written by the exports (/S+/D) pass
		StringBuffer<32> platform;
		TString moduleName;
		bool forceElf = false; // /ELF[:machine]: force an ELF import stub (.so) instead of a COFF import lib, regardless of target platform string
		u16 elfMachineOverride = 0; // e_machine from /ELF:<machine> (0 = derive from platform string)

		auto elfMachine = [&]() -> u16 { return elfMachineOverride ? elfMachineOverride : (platform.Contains(TC("arm64")) || platform.Contains(TC("aarch64"))) ? ElfMachine_Aarch64 : ElfMachine_X86_64; };

		auto parseArg = [&](const tchar* arg, bool isRsp)
			{
				StringBuffer<> name;
				StringBuffer<> value;

				if (const tchar* equals = TStrchr(arg,'='))
				{
					name.Append(arg, equals - arg);
					value.Append(equals+1);
				}
				else
				{
					const tchar* colon = TStrchr(arg,':');
					if (colon && (!IsWindows || (colon[1] != '\\' && colon[1] != '/')))
					{
						name.Append(arg, colon - arg);
						const tchar* valueStart = colon+1;
						if (*valueStart == '\"')
							++valueStart;
						value.Append(valueStart);
						if (value.data[value.count-1] == '\"')
							value.Resize(value.count-1);

					}
					else
					{
						name.Append(arg);
					}
				}

				auto checkHasValue = [&]()
					{
						if (value.count)
							return true;
						LoggerWithWriter(g_consoleLogWriter, TC("")).Error(TC("   ERROR: %s needs a filename"), name.data);
						return false;
					};

				if (isImpLibRsp)
				{
					if (name.Equals(TCV("/NOLOGO")))
					{
					}
					else if (name.Equals(TCV("/WX")))
					{
					}
					else if (name.Equals(TCV("/errorReport")))
					{
					}
					else if (name.Equals(TCV("/MACHINE")))
					{
						//if (!value.Equals(TCV("x64")))
						//{
						//	logger.Error(TC("only x64 supported"));
						//	return 0;
						//}
					}
					else if (name.Equals(TCV("/SUBSYSTEM")))
					{
					}
					else if (name.Equals(TCV("/DEF")))
					{
						writeImpLib = true;
					}
					else if (name.Equals(TCV("/NAME")))
					{
						char buffer[256];
						value.Parse(buffer, sizeof(buffer));
						impLibName = buffer;
					}
					else if (name.Equals(TCV("/OUT")))
					{
						if (!checkHasValue())
							return -1;
						impLibFile = value.data;
					}
					else if (name.Equals(TCV("/IGNORE")))
					{
					}
					else if (name.Equals(TCV("/NODEFAULTLIB")))
					{
					}
					else if (name.StartsWith(TCV("/ALTERNATENAME")))
					{
					}
					else if (name.Equals(TCV("/LTCG")))
					{
					}
					else if (name.StartsWith(TC("/OPT:")))
					{
					}
					else
					{
						objFilesForImpLib.push_back(name.data);
					}
				}
				else if (name.StartsWith(TC("/D")))
				{
					if (!checkHasValue())
						return -1;
					objFilesDependencies.push_back(value.data);
				}
				else if (name.StartsWith(TC("/S")))
				{
					if (!checkHasValue())
						return -1;
					objFilesToExport.push_back(value.data);
				}
				else if (name.StartsWith(TC("/O")))
				{
					if (!checkHasValue())
						return -1;
					extraObjFile = value.data;
				}
				else if (name.Equals(TCV("/L")))
				{
					if (!checkHasValue())
						return -1;
					exportsImpLibFile = value.data;
				}
				else if (name.StartsWith(TC("/ELF")))
				{
					forceElf = true;
					if (value.count) // optional /ELF:<machine>
						elfMachineOverride = (value.Contains(TC("aarch64")) || value.Contains(TC("arm64")) ) ? ElfMachine_Aarch64 : ElfMachine_X86_64;
				}
				else if (name.StartsWith(TC("/T")))
				{
					if (!checkHasValue())
						return -1;
					platform = value;
				}
				else if (name.StartsWith(TC("/M")))
				{
					if (!checkHasValue())
						return -1;
					moduleName = value.data;
				}
				else if (name.StartsWith(TC("/E")))
				{
					char buffer[512];
					if (!value.Parse(buffer, sizeof(buffer)))
						return PrintHelp(TC("Bad symbol name"));
					extraExports.push_back(buffer);
				}
				else if (name.Equals(TCV("-printsymbols")))
				{
					printSymbols = true;
				}
				else if (name.Equals(TCV("-writeimplib")))
				{
					if (!checkHasValue())
						return -1;
					impLibFile = value.data;
					writeImpLib = true;
					allowLibInputs = true;
				}
				else if (name.Equals(TCV("/LIB")))
				{
					isImpLibRsp = true;
					writeImpLib = true;
				}
				else if (name.Equals(TCV("-?")))
				{
					return PrintHelp(TC(""));
				}
				else if (name[0] != '-' && (!IsWindows || name[0] != '/')) // positional input (POSIX abs paths begin with '/')
				{
					if (writeImpLib)
						objFilesForImpLib.push_back(name.data);
					else if (objFile.empty())
						objFile = name.data;
					else
						objFilesForImpLib.push_back(name.data);
					return 0;
				}
				else
				{
					StringBuffer<> msg;
					msg.Appendf(TC("Unknown argument '%s'"), name.data);
					return PrintHelp(msg.data);
				}
				return 0;
			};

		for (int i=1; i!=argc; ++i)
		{
			const tchar* arg = argv[i];
			if (*arg == '@')
			{
				++arg;
				StringBuffer<> temp;
				if (*arg == '\"')
				{
					temp.Append(arg + 1);
					temp.Resize(temp.count - 1);
					arg = temp.data;
				}
				int res = 0;
				LoggerWithWriter logger(g_consoleLogWriter, TC(""));
				if (!ReadLines(logger, arg, [&](const TString& line)
					{
						res = parseArg(line.c_str(), true);
						return res == 0;
					}))
					return -1;
				if (res != 0)
					return res;
				continue;
			}
			int res = parseArg(arg, false);
			if (res != 0)
				return res;
		}

		FilteredLogWriter logWriter(g_consoleLogWriter, LogEntryType_Info);
		LoggerWithWriter logger(logWriter, TC("UbaObjTool"));

		if (!objFilesToExport.empty())
		{
			CriticalSection cs;
			Atomic<bool> success = true;

			AllExternalImports& allExternalImports = *new AllExternalImports(256*1024); // Imports needed from the outside of the stripped obj files.

			#if defined(__clang_analyzer__)
			auto sg = MakeGuard([&]() { delete &allExternalImports; }); // Let it leak to speedup shutdown
			#endif

			u32 workerCount = DefaultProcessorCount-1;
			WorkManagerImpl workManager(workerCount, TC("UbaWrk/StrpObj"));

			struct ExtAcc
			{
				AllExternalImports& g;
				CriticalSection& cs;
				AllExternalImports l;
				~ExtAcc()
				{
					SCOPED_CRITICAL_SECTION(cs, _);
					g.merge_move(l);
				}
			};

			workManager.AddWork([&](const WorkContext&)
				{
					workManager.ParallelFor(workerCount, objFilesDependencies, [&, acc = ExtAcc{ allExternalImports, cs }](const WorkContext&, auto& it) mutable
						{
							SymbolFile symbolFile;
							if (!symbolFile.ParseFile(logger, (*it).c_str()))
							{
								success = false;
								return;
							}
							acc.l.merge_move(symbolFile.imports);
						}, TCV("ObjFilesDeps"));
				}, 1, TC("ObjFilesDepsAll"));

			AllInternalImports& allInternalImports = *new AllInternalImports(512*1024); // Imports that the obj files has. Note these could be existing in the obj files and then we might need to create loopbacks
			AllExports& allExports = *new AllExports(12*1024*1024); // Exports from all the obj files.
			AllExports& allExplicitExports = *new AllExports(128*1024); // Symbols the obj files explicitly exported (/EXPORT:). Verbatim names; must be exported regardless of dependents.

			#if defined(__clang_analyzer__)
			auto sg2 = MakeGuard([&]() { delete &allInternalImports; delete &allExports; delete &allExplicitExports; }); // Let then leak to speedup shutdown
			#endif

			Map<TString, ObjectFile*> objectFiles;
			auto g = MakeGuard([&]() { for (auto& kv : objectFiles) delete kv.second; });
			CriticalSection cs2;

			struct StripAcc
			{
				AllInternalImports& gi;
				AllExports& ge;
				AllExports& gx;
				CriticalSection& cs;
				CriticalSection& cs2;
				AllInternalImports li;
				AllExports le;
				AllExports lx;
				~StripAcc()
				{
					{
						SCOPED_CRITICAL_SECTION(cs, _);
						ge.merge_move(le);
					}
					{
						SCOPED_CRITICAL_SECTION(cs2, _);
						gi.merge_move(li);
						gx.merge_move(lx);
					}
				}
			};

			workManager.ParallelFor(workerCount, objFilesToExport, [&, acc = StripAcc{ allInternalImports, allExports, allExplicitExports, cs, cs2 }](const WorkContext&, auto& it) mutable
				{
					SymbolFile symbolFile;
					if (!symbolFile.ParseFile(logger, (*it).c_str()))
					{
						success = false;
						return;
					}
					acc.li.merge_move(symbolFile.imports);
					acc.le.merge_move(symbolFile.exports);
					acc.lx.merge_move(symbolFile.explicitExports);
				}, TCV("ObjFilesToStrip"));

			// THIS IS REALLY IMPORTANT. We rely on work state dtors to have all maps being properly populated.
			// .. and not waiting here could cause maps to be populated post being used
			workManager.FlushWork();

			if (!success)
				return -1;

			if (!extraObjFile.empty())
				if (!ObjectFile::CreateExtraFile(logger, extraObjFile, moduleName, platform, allExternalImports, allInternalImports, allExports, allExplicitExports, extraExports, true))
					return -1;
			//logger.Info(TC("Reduced export count from %llu to %llu"), totalExportCount.load(), totalKeptExportCount.size());

			// If /L was given, write the import lib straight from the computed export set in this same pass —
			// no intermediate obj round-trip. Export set matches exports.obj's .drectve:
			// (allExternalImports ∩ allExports) ∪ explicit /EXPORT: ∪ ThisIsAnUnrealEngineModule. Names for the
			// intersection come from the import side (exports may carry only a hash); explicit names are verbatim.
			// Windows target -> COFF import lib (Windows host only). ELF target (Linux/Android) -> stub .so.
			if (!exportsImpLibFile.empty())
			{
				UnorderedExports libExports;
				auto addExport = [&](const char* name, u32 len, u8 isData, u64 size)
					{
						StringKey key = ToStringKeyRaw(name, len);
						libExports.try_emplace(key, ExportInfo{ std::string(name, len), isData, 0, size });
					};

				for (auto& imp : allExternalImports)
				{
					const char* name = imp.c_str();
					u32 len = u32(imp.size());
					if (len > 6 && memcmp(name, "__imp_", 6) == 0) { name += 6; len -= 6; }
					auto findIt = allExports.find(ToStringKeyRaw(name, len));
					if (findIt == allExports.end())
						continue;
					addExport(name, len, findIt->second.isData, findIt->second.size);
				}

				const char* unrealMarker = "ThisIsAnUnrealEngineModule";
				if (allExports.find(ToStringKeyRaw(unrealMarker, u32(strlen(unrealMarker)))) != allExports.end())
					addExport(unrealMarker, u32(strlen(unrealMarker)), 0, 0);

				// Explicit exports — exported regardless of whether any dependent imports them (addExport dedups).
				for (auto& kv : allExplicitExports)
					addExport(kv.second.symbol.c_str(), u32(kv.second.symbol.size()), kv.second.isData, kv.second.size);

				for (auto& sym : extraExports)
					if (!strchr(sym.c_str(), '*'))
						addExport(sym.c_str(), u32(sym.size()), 0, 0);

				// Lib name drives the DLL name / SONAME baked into the import lib. Use /M; fall back to the output filename.
				StringBuffer<> libNameBuf;
				if (!moduleName.empty())
					libNameBuf.Append(moduleName);
				else
					libNameBuf.AppendFileName(exportsImpLibFile.c_str());
				char libNameNarrow[256];
				libNameBuf.Parse(libNameNarrow, sizeof(libNameNarrow));

				bool targetIsElf = forceElf || platform.Contains(TC("Linux")) || platform.Contains(TC("Android"));
				if (targetIsElf)
				{
					// soname must equal the real .so's filename (the /L output basename), so dependents' DT_NEEDED
					// names the real library — not /M (which is the bare module name, no lib prefix / .so suffix).
					StringBuffer<> sonameBuf;
					sonameBuf.AppendFileName(exportsImpLibFile.c_str());
					char sonameNarrow[256];
					sonameBuf.Parse(sonameNarrow, sizeof(sonameNarrow));
					if (!ImportStubWriterElf::Write(logger, libExports, sonameNarrow, elfMachine(), exportsImpLibFile.c_str()))
						return -1;
				}
				else
				{
					#if PLATFORM_WINDOWS
					// Assign index by sorted name. The set above is gathered from unordered containers (and under
					// ParallelFor), so insertion order varies run-to-run; ImportLibWriter orders its offset table by
					// index, so without this the lib would have an identical symbol set but non-deterministic bytes.
					{
						Vector<UnorderedExports::value_type*> ordered;
						ordered.reserve(libExports.size());
						for (auto& kv : libExports)
							ordered.push_back(&kv);
						std::sort(ordered.begin(), ordered.end(), [](const UnorderedExports::value_type* a, const UnorderedExports::value_type* b) { return a->second.symbol < b->second.symbol; });
						u32 i = 0;
						for (auto* kv : ordered)
							kv->second.index = i++;
					}

					ImportLibWriter writer;
					if (!writer.Write(logger, libExports, libNameNarrow, exportsImpLibFile.c_str()))
						return -1;
					logger.Debug(TC("Wrote import lib %s (%u exports)"), exportsImpLibFile.c_str(), (u32)libExports.size());
					#else
					return PrintHelp(TC("COFF import lib (/L for a Windows target) requires a Windows host"));
					#endif
				}
			}
		}
		else if (writeImpLib)
		{
			StringBuffer<> currentDir;
			GetCurrentDirectoryW(currentDir);
			currentDir.EnsureEndsWithSlash();

			if (objFilesForImpLib.empty() && !objFile.empty())
				objFilesForImpLib.push_back(objFile);

			bool targetIsElf = forceElf || platform.Contains(TC("Linux")) || platform.Contains(TC("Android"));

			// ELF has no dllexport: exports are selected by linkage/visibility, not DllStorage. Parse with _All so
			// LLVM IR (LTO bitcode) objects export their externally-linked symbols; the COFF/Windows path keeps _Exports.
			ObjectFileParseMode parseMode = targetIsElf ? ObjectFileParseMode_All : ObjectFileParseMode_Exports;

			Atomic<bool> success = true;
			Vector<ObjectFile*> objFiles;
			objFiles.resize(objFilesForImpLib.size());
			u32 workerCount = DefaultProcessorCount - 1;
			WorkManagerImpl workManager(workerCount, TC("UbaWrk/Load"));
			workManager.ParallelFor(workerCount, objFilesForImpLib, [&](const WorkContext&, auto& it)
			{
				auto& o = *it;
				StringBuffer<> fixedPath;
				FixPath(o.c_str(), currentDir.data, currentDir.count, fixedPath);
				if (fixedPath.EndsWith(TCV(".res")) || (fixedPath.EndsWith(TCV(".lib")) && !allowLibInputs))
					return;
				if (ObjectFile* objectFile = ObjectFile::OpenAndParse(logger, parseMode, fixedPath.data))
				{
					//objectFile->RemoveExportedSymbol("DllMain"); // This is used to patch xinput.lib
					objFiles[it - objFilesForImpLib.begin()] = objectFile;
				}
				else
					success = false;
			}, TCV("OpenAndParse"));
			if (!success)
				return -1;

			if (impLibName.empty() && objFiles.size() == 1)
				impLibName = objFiles[0]->GetLibName();

			if (targetIsElf)
			{
				// Stub must match the real .so's dynamic symbol table: only default-visible symbols are auto-exported,
				// so skip hidden ones (they aren't in the real lib's dynsym unless promoted, which doesn't apply here).
				UnorderedExports merged;
				for (ObjectFile* o : objFiles)
					if (o)
						for (auto& kv : o->GetExports())
							if (!kv.second.isHidden)
								merged.try_emplace(kv.first, kv.second);
				if (impLibName.empty()) // SONAME: fall back to the output filename
				{
					StringBuffer<> n;
					n.AppendFileName(impLibFile.c_str());
					char buf[256];
					n.Parse(buf, sizeof(buf));
					impLibName = buf;
				}
				if (!ImportStubWriterElf::Write(logger, merged, impLibName.c_str(), elfMachine(), impLibFile.c_str()))
					return -1;
			}
			else
			{
				ImportLibWriter writer;
				writer.Write(logger, objFiles, impLibName.c_str(), impLibFile.c_str());
			}
		}
		else
		{
			if (objFile.empty())
				return PrintHelp(TC("No obj, lib or rsp file provided"));

			ObjectFile* objectFile = ObjectFile::OpenAndParse(logger, ObjectFileParseMode_All, objFile.c_str());
			if (!objectFile)
				return -1;
			auto g = MakeGuard([&](){ delete objectFile; });

			if (printSymbols)
			{
				for (auto& symbol : objectFile->GetImports())
					logger.Info(TC("I %S"), symbol.c_str());

				for (auto& kv : objectFile->GetExports())
					logger.Info(TC("E %S%S"), kv.second.symbol.c_str(), kv.second.isData ? ",DATA" : "");
			}
		}
		return 0;
	}
}

#if PLATFORM_WINDOWS
int wmain(int argc, wchar_t* argv[])
{
	return uba::WrappedMain(argc, argv);
}
#else
int main(int argc, char* argv[])
{
	return uba::WrappedMain(argc, argv);
}
#endif
