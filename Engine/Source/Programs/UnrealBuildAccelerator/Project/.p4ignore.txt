Intermediate/
Saved/
.vsconfig
All-Verse.code-workspace
/*.sln
