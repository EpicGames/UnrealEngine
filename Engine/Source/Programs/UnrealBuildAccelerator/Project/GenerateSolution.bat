@echo off
setlocal

cd "%~dp0"

call ..\..\..\..\Build\BatchFiles\GenerateProjectFiles.bat -Project="%~dp0\Uba.uproject" -Game -Platforms=Win64+Linux+LinuxArm64 -DEBUGCONFIGS -SHIPPINGCONFIGS
move /Y Uba_LinuxLinuxArm64Win64.sln Uba.sln

call ..\..\..\..\Build\BatchFiles\GenerateProjectFiles.bat -Project="%~dp0\Uba.uproject" -Game -Platforms=Linux -VisualStudioLinux

