{
  "solution": {
    "path": "Uba.sln",
    "projects": [
      "..\\..\\..\\..\\Intermediate\\Build\\BuildRulesProjects\\UE5ProgramRules\\UE5ProgramRules.csproj",
      "..\\..\\..\\..\\Intermediate\\Build\\BuildRulesProjects\\UE5Rules\\UE5Rules.csproj",
      "..\\..\\Shared\\EpicGames.Build\\EpicGames.Build.csproj",
      "..\\..\\Shared\\EpicGames.Core\\EpicGames.Core.csproj",
      "..\\..\\Shared\\EpicGames.Horde\\EpicGames.Horde.csproj",
      "..\\..\\Shared\\EpicGames.IoHash\\EpicGames.IoHash.csproj",
      "..\\..\\Shared\\EpicGames.MsBuild\\EpicGames.MsBuild.csproj",
      "..\\..\\Shared\\EpicGames.OIDC\\EpicGames.OIDC.csproj",
      "..\\..\\Shared\\EpicGames.Oodle\\EpicGames.Oodle.csproj",
      "..\\..\\Shared\\EpicGames.Serialization\\EpicGames.Serialization.csproj",
      "..\\..\\Shared\\EpicGames.UBA\\EpicGames.UBA.csproj",
      "..\\..\\Shared\\EpicGames.UHT\\EpicGames.UHT.csproj",
      "..\\..\\UnrealBuildTool\\UnrealBuildTool.csproj",
      "Intermediate\\ProjectFilesLinuxLinuxArm64Win64\\UbaAgent.vcxproj",
      "Intermediate\\ProjectFilesLinuxLinuxArm64Win64\\UbaCli.vcxproj",
      "Intermediate\\ProjectFilesLinuxLinuxArm64Win64\\UbaCommon.vcxproj",
      "Intermediate\\ProjectFilesLinuxLinuxArm64Win64\\UbaCoordinatorHorde.vcxproj",
      "Intermediate\\ProjectFilesLinuxLinuxArm64Win64\\UbaCore.vcxproj",
      "Intermediate\\ProjectFilesLinuxLinuxArm64Win64\\UbaDetours.vcxproj",
      "Intermediate\\ProjectFilesLinuxLinuxArm64Win64\\UbaHost.vcxproj",
      "Intermediate\\ProjectFilesLinuxLinuxArm64Win64\\UbaStorageProxy.vcxproj",
      "Intermediate\\ProjectFilesLinuxLinuxArm64Win64\\UbaTest.vcxproj",
      "Intermediate\\ProjectFilesLinuxLinuxArm64Win64\\UbaTestApp.vcxproj",
      "Intermediate\\ProjectFilesLinuxLinuxArm64Win64\\UbaVisualizer.vcxproj"
    ]
  }
}