{
	"FileVersion": 3,
	"EngineAssociation": "",
	"Category": "",
	"Description": "",
	"AdditionalRootDirectories": [
		".."
	]
}