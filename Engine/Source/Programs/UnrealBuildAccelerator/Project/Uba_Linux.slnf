{
  "solution": {
    "path": "Uba_Linux.sln",
    "projects": [
      "..\\..\\..\\..\\Intermediate\\Build\\BuildRulesProjects\\UE5ProgramRules\\UE5ProgramRules.csproj",
      "..\\..\\..\\..\\Intermediate\\Build\\BuildRulesProjects\\UE5Rules\\UE5Rules.csproj",
      "..\\..\\Shared\\EpicGames.Build\\EpicGames.Build.csproj",
      "..\\..\\Shared\\EpicGames.Core\\EpicGames.Core.csproj",
      "..\\..\\Shared\\EpicGames.Horde\\EpicGames.Horde.csproj",
      "..\\..\\Shared\\EpicGames.IoHash\\EpicGames.IoHash.csproj",
      "..\\..\\Shared\\EpicGames.MsBuild\\EpicGames.MsBuild.csproj",
      "..\\..\\Shared\\EpicGames.OIDC\\EpicGames.OIDC.csproj",
      "..\\..\\Shared\\EpicGames.Oodle\\EpicGames.Oodle.csproj",
      "..\\..\\Shared\\EpicGames.Serialization\\EpicGames.Serialization.csproj",
      "..\\..\\Shared\\EpicGames.UBA\\EpicGames.UBA.csproj",
      "..\\..\\Shared\\EpicGames.UHT\\EpicGames.UHT.csproj",
      "..\\..\\UnrealBuildTool\\UnrealBuildTool.csproj",
      "Intermediate\\ProjectFilesLinux\\UbaAgent.vcxproj",
      "Intermediate\\ProjectFilesLinux\\UbaCli.vcxproj",
      "Intermediate\\ProjectFilesLinux\\UbaCommon.vcxproj",
      "Intermediate\\ProjectFilesLinux\\UbaCoordinatorHorde.vcxproj",
      "Intermediate\\ProjectFilesLinux\\UbaCore.vcxproj",
      "Intermediate\\ProjectFilesLinux\\UbaDetours.vcxproj",
      "Intermediate\\ProjectFilesLinux\\UbaHost.vcxproj",
      "Intermediate\\ProjectFilesLinux\\UbaStorageProxy.vcxproj",
      "Intermediate\\ProjectFilesLinux\\UbaTest.vcxproj",
      "Intermediate\\ProjectFilesLinux\\UbaTestApp.vcxproj",
      "Intermediate\\ProjectFilesLinux\\UbaVisualizer.vcxproj"
    ]
  }
}