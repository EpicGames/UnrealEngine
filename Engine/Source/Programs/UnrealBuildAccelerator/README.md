# UBA

WIP

## Building on Windows

Windows is our primary development platform and is generally the easiest to debug and profile on. For
debugging we recommend Visual Studio 2022 or later, and for profiling you may want to use a high 
frequency sampling profiler such as Superluminal Performance.

### Windows Setup

To build the code you will need Visual Studio 2022 (we use c++20 features), git and vcpkg.

* Install Visual Studio 2022
* Install [git](https://git-scm.com/download/win)
  * You can also use `winget install git` if you have winget installed
  * You may want to install the github CLI to manage credentials etc - `winget install github.cli`
* Install vcpkg (see below)

#### Installing vcpkg

We use vcpkg to manage some libraries. Right now it's not set up on a project local
basis and requires manual bootstrap so you will need to do the following at least once:

* open up a command line window
  * create a `git`/`github` directory somewhere for you to clone repos into
  * issue `git clone https://github.com/microsoft/vcpkg.git` and build it using the `bootstrap-vcpkg.bat` script
  * run `vcpkg integrate install` to make sure Visual Studio can locate the vcpkg install
* optional: add the `vcpkg` directory you cloned to your PATH to allow invoking vcpkg on the command line

Now you are ready to start building!
