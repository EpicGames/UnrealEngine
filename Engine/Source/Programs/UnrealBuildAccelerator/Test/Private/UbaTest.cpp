// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaTest.h"
#include "UbaEnvironment.h"
#include "UbaEvent.h"
#include "UbaFileAccessor.h"
#include "UbaLogger.h"

namespace uba
{
	bool CreateTestFile(Logger& logger, StringView testRootDir, StringView fileName, StringView content, u32 attributes)
	{
		StringBuffer<> testFileName;
		return CreateTestFile(testFileName, logger, testRootDir, fileName, content, attributes);
	}

	bool CreateTestFile(StringBufferBase& outFile, Logger& logger, StringView testRootDir, StringView fileName, StringView content, u32 attributes)
	{
		outFile.Clear().Append(testRootDir).EnsureEndsWithSlash().Append(fileName).FixPathSeparators();

		if (fileName.Contains(PathSeparator) || fileName.Contains(NonPathSeparator))
		{
			StringBuffer<> testFileDir;
			testFileDir.AppendDir(outFile);
			DirectoryCache().CreateDirectory(logger, testFileDir.data);
		}

		u64 bytes = content.count*sizeof(tchar);
		FileAccessor file(logger, outFile.data);
		CHECK_TRUEF(file.CreateMemoryWrite(false, attributes, bytes), TC("Failed to create file for write"));
		memcpy(file.GetData(), content.data, bytes);
		return file.Close();
	}

	bool DeleteTestFile(Logger& logger, StringView testRootDir, StringView fileName)
	{
		StringBuffer<> testFileName;
		testFileName.Append(testRootDir).EnsureEndsWithSlash().Append(fileName).FixPathSeparators();
		CHECK_TRUEF(DeleteFileW(testFileName.data), TC("Failed to delete test file %s"), fileName.data);
		return true;
	}

	bool FileExists(Logger& logger, StringView dir, StringView fileName)
	{
		StringBuffer<> testFileName;
		testFileName.Append(dir).EnsureEndsWithSlash().Append(fileName).FixPathSeparators();
		return FileExists(logger, testFileName.data);
	}

	bool FilesEqual(Logger& logger, const tchar* file1, const tchar* file2)
	{
		FileAccessor f1(logger, file1);
		CHECK_TRUE(f1.OpenMemoryRead());
		FileAccessor f2(logger, file2);
		CHECK_TRUE(f2.OpenMemoryRead());
		CHECK_TRUE(f1.GetSize() == f2.GetSize());
		return memcmp(f1.GetData(), f2.GetData(), f1.GetSize()) == 0;
	}

	TestGroup* g_firstTestGroup = nullptr;

	u32 SubTestContext::FindOrInsert(const tchar* subName)
	{
		for (u32 i = 0; i != seenCount; ++i)
			if (TStrcmp(seen[i], subName) == 0)
				return i;
		if (seenCount < MaxSubTests)
			seen[seenCount++] = subName;
		return seenCount - 1;
	}

	bool RunTestWithSubTests(const tchar* fullName, LoggerWithWriter& logger, const StringBufferBase& testRootDir, TestFuncWithSubCtx impl)
	{
		SubTestContext ctx;
		do
		{
			if (!impl(logger, testRootDir, ctx))
				return false;
			++ctx.activeIndex;
		}
		while (ctx.sawAnySubTest && ctx.activeIndex < ctx.seenCount);
		return true;
	}

	bool RunRegisteredTests(const tchar* filter, LoggerWithWriter& logger, LoggerWithWriter& testLogger, const StringBufferBase& testRootDir)
	{
		// Collect groups and sort by sortOrder (stable — ties preserve registration order).
		constexpr u32 MaxGroups = 64;
		TestGroup* groups[MaxGroups];
		u32 groupCount = 0;
		for (TestGroup* g = g_firstTestGroup; g && groupCount < MaxGroups; g = g->nextGroup)
			groups[groupCount++] = g;
		for (u32 i = 1; i < groupCount; ++i)
		{
			TestGroup* key = groups[i];
			u32 j = i;
			while (j > 0 && groups[j-1]->sortOrder > key->sortOrder)
			{
				groups[j] = groups[j-1];
				--j;
			}
			groups[j] = key;
		}

		for (u32 gi = 0; gi < groupCount; ++gi)
		{
			TestGroup* group = groups[gi];
			for (Test* test = group->firstTest; test; test = test->next)
			{
				StringBuffer<256> fullName;
				fullName.Append(group->name).Append(TC(".")).Append(test->name);
				if (filter && !fullName.Contains(filter))
					continue;
				if (!test->func)
				{
					logger.Info(TC("Running %s (disabled: %s)"), fullName.data, test->disabledReason);
					continue;
				}
				DeleteAllFiles(logger, testRootDir.data, false);
				StringBuffer<320> running;
				running.Appendf(TC("Running %s..."), fullName.data);
				LogToConsoleNoLineFeed(LogEntryType_Info, running.data, running.count);
				u64 logsBefore = GetConsoleLogLineCount();
				bool ok = test->func(testLogger, testRootDir);
				bool testLogged = GetConsoleLogLineCount() != logsBefore;
				if (!ok)
				{
					if (testLogged)
						return logger.Error(TC("  %s failed"), fullName.data);
					return logger.Error(TC(" failed"));
				}
				if (testLogged)
					logger.Info(TC("  %s success!"), fullName.data);
				else
					logger.Info(TC(" success!"));
			}
		}
		return true;
	}

	bool RunTests(int argc, tchar* argv[])
	{
		LoggerWithWriter logger(g_consoleLogWriter, TC(""));

		FilteredLogWriter filteredWriter(g_consoleLogWriter, LogEntryType_Warning);
		LoggerWithWriter testLogger(filteredWriter, TC("   "));

		StringBuffer<512> testRootDir;

		#if PLATFORM_WINDOWS
		StringBuffer<> temp;
		temp.count = GetTempPathW(temp.capacity, temp.data);
		testRootDir.count = GetLongPathNameW(temp.data, testRootDir.data, testRootDir.capacity);
		testRootDir.EnsureEndsWithSlash().Append(L"UbaTest");
		#else
		testRootDir.count = GetFullPathNameW("~/UbaTest", testRootDir.capacity, testRootDir.data, nullptr);
		#endif
		CreateDirectoryW(testRootDir.data);
		testRootDir.EnsureEndsWithSlash();

		logger.Info(TC("Running tests (Test rootdir: %s)"), testRootDir.data);

		const tchar* filter = nullptr;
		if (argc > 1)
			filter = argv[1];

		if (!RunRegisteredTests(filter, logger, testLogger, testRootDir))
			return false;

		logger.Info(TC("Tests finished successfully!"));
		return true;
	}

	bool WrappedMain(int argc, tchar* argv[])
	{
		AddExceptionHandler();

		#if !PLATFORM_WINDOWS
		SetupOrphanReaper();
		#endif

		return RunTests(argc, argv);
	}

	TrackSystemUsage::TrackSystemUsage(Logger& l) : logger(l)
	{
		Event::ClearCache();
		#if PLATFORM_WINDOWS
		GetProcessHandleCount(GetCurrentProcess(), (DWORD*)&startHandleCount);
		#endif
	}

	TrackSystemUsage::~TrackSystemUsage()
	{
		Event::ClearCache();
		#if PLATFORM_WINDOWS
		u32 endHandleCount = 0;
		GetProcessHandleCount(GetCurrentProcess(), (DWORD*)&endHandleCount);
		if (startHandleCount != endHandleCount)
			logger.Error(TC("Handles were leaked while running test (%u)"), endHandleCount - startHandleCount);
		#endif
	}
}

#if PLATFORM_WINDOWS
int wmain(int argc, wchar_t* argv[])
{
	return uba::WrappedMain(argc, argv) ? 0 : -1;
}
#else
int main(int argc, char* argv[])
{
	return uba::WrappedMain(argc, argv) ? 0 : -1;
}
#endif
