// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaFile.h"
#include "UbaStringBuffer.h"

#define CHECK_TRUE(x) \
	do { \
	if (!(x)) \
		return logger.Error(TC("Failed %s (%s:%u)"), TC(#x), TC("") __FILE__, __LINE__); \
	} while (false)

#define CHECK_TRUEF(x, ...) \
	do { \
	if (!(x)) \
		return logger.Logf(LogEntryType_Error, ##__VA_ARGS__).ToFalse(); \
	} while (false)

namespace uba
{
	class Logger;
	class LoggerWithWriter;
	class ProcessHandle;
	class SessionServer;
	struct ProcessStartInfo;

	bool CreateTestFile(Logger& logger, StringView testRootDir, StringView fileName, StringView content, u32 attributes = DefaultAttributes());
	bool CreateTestFile(StringBufferBase& outFile, Logger& logger, StringView testRootDir, StringView fileName, StringView content, u32 attributes = DefaultAttributes());
	bool DeleteTestFile(Logger& logger, StringView testRootDir, StringView fileName);
	bool FileExists(Logger& logger, StringView dir, StringView fileName);
	bool FilesEqual(Logger& logger, const tchar* file1, const tchar* file2);

	using RunProcessFunction = Function<ProcessHandle(ProcessStartInfo&)>;
	using TestSessionFunction = Function<bool(LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)>;
	bool RunLocal(LoggerWithWriter& logger, const StringBufferBase& testRootDir, const TestSessionFunction& testFunc, uintptr_t extraData = 0, bool enableDetour = true);
	bool RunRemote(LoggerWithWriter& logger, const StringBufferBase& testRootDir, const TestSessionFunction& testFunc, bool deleteAll = true, bool serverShouldListen = true);
	void GetTestAppPath(LoggerWithWriter& logger, StringBufferBase& out);
	const tchar* GetSystemApplication();
	const tchar* GetSystemArguments();

	struct TrackSystemUsage
	{
		TrackSystemUsage(Logger& logger);
		~TrackSystemUsage();
		Logger& logger;
		u32 startHandleCount;
	};

	struct Test;
	struct TestGroup;
	extern TestGroup* g_firstTestGroup;

	using TestFunc = bool(*)(LoggerWithWriter& logger, const StringBufferBase& testRootDir);

	struct Test
	{
		const tchar* name;
		TestFunc func;
		const tchar* disabledReason;
		Test* next;
	};

	struct TestGroup
	{
		const tchar* name;
		Test* firstTest;
		Test* lastTest;
		int sortOrder;
		TestGroup* nextGroup;
	};

	struct TestGroupChain
	{
		TestGroupChain(TestGroup* group)
		{
			group->nextGroup = g_firstTestGroup;
			g_firstTestGroup = group;
		}
	};

	struct TestRegistrar
	{
		TestRegistrar(TestGroup& group, Test* test)
		{
			if (group.lastTest)
				group.lastTest->next = test;
			else
				group.firstTest = test;
			group.lastTest = test;
		}
	};

	struct SubTestContext
	{
		static constexpr u32 MaxSubTests = 64;
		const tchar* seen[MaxSubTests];
		u32 seenCount = 0;
		u32 activeIndex = 0;
		bool sawAnySubTest = false;

		bool Enter(const tchar* subName)
		{
			sawAnySubTest = true;
			return FindOrInsert(subName) == activeIndex;
		}

	private:
		u32 FindOrInsert(const tchar* subName);
	};

	using TestFuncWithSubCtx = bool(*)(LoggerWithWriter& logger, const StringBufferBase& testRootDir, SubTestContext& subCtx);
	bool RunTestWithSubTests(const tchar* fullName, LoggerWithWriter& logger, const StringBufferBase& testRootDir, TestFuncWithSubCtx impl);

	bool RunRegisteredTests(const tchar* filter, LoggerWithWriter& logger, LoggerWithWriter& testLogger, const StringBufferBase& testRootDir);
}

#define UBA_TEST_CONCAT_INNER(a, b) a##b
#define UBA_TEST_CONCAT(a, b) UBA_TEST_CONCAT_INNER(a, b)

#define UBA_DECLARE_TEST_GROUP(groupName) \
	namespace uba { extern TestGroup g_##groupName; }

#define UBA_DEFINE_TEST_GROUP(groupName, sortOrder) \
	uba::TestGroup g_##groupName{ TC(#groupName), nullptr, nullptr, sortOrder, nullptr }; \
	static uba::TestGroupChain UBA_TEST_CONCAT(g_##groupName##Chain, __LINE__){ &g_##groupName };

#define UBA_TEST(groupName, testName) \
	static bool Run##groupName##testName##Impl(uba::LoggerWithWriter& logger, const uba::StringBufferBase& testRootDir, uba::SubTestContext& subCtx); \
	static bool Run##groupName##testName(uba::LoggerWithWriter& logger, const uba::StringBufferBase& testRootDir) \
		{ return uba::RunTestWithSubTests(TC(#groupName) TC(".") TC(#testName), logger, testRootDir, &Run##groupName##testName##Impl); } \
	static uba::Test UBA_TEST_CONCAT(Run##groupName##testName##Node, __LINE__){ TC(#testName), &Run##groupName##testName, nullptr, nullptr }; \
	static uba::TestRegistrar UBA_TEST_CONCAT(Run##groupName##testName##Reg, __LINE__){ g_##groupName, &UBA_TEST_CONCAT(Run##groupName##testName##Node, __LINE__) }; \
	static bool Run##groupName##testName##Impl(uba::LoggerWithWriter& logger, const uba::StringBufferBase& testRootDir, uba::SubTestContext& subCtx)

#define UBA_TEST_IF(groupName, testName, enabledExpr) \
	static bool Run##groupName##testName##Impl(uba::LoggerWithWriter& logger, const uba::StringBufferBase& testRootDir, uba::SubTestContext& subCtx); \
	static bool Run##groupName##testName(uba::LoggerWithWriter& logger, const uba::StringBufferBase& testRootDir) \
		{ return uba::RunTestWithSubTests(TC(#groupName) TC(".") TC(#testName), logger, testRootDir, &Run##groupName##testName##Impl); } \
	static uba::Test UBA_TEST_CONCAT(Run##groupName##testName##Node, __LINE__){ TC(#testName), (enabledExpr) ? &Run##groupName##testName : nullptr, (enabledExpr) ? nullptr : TC("requires ") TC(#enabledExpr), nullptr }; \
	static uba::TestRegistrar UBA_TEST_CONCAT(Run##groupName##testName##Reg, __LINE__){ g_##groupName, &UBA_TEST_CONCAT(Run##groupName##testName##Node, __LINE__) }; \
	static bool Run##groupName##testName##Impl(uba::LoggerWithWriter& logger, const uba::StringBufferBase& testRootDir, uba::SubTestContext& subCtx)

#define UBA_TEST_DISABLED(groupName, testName, reason) \
	[[maybe_unused]] static bool Run##groupName##testName##Impl(uba::LoggerWithWriter& logger, const uba::StringBufferBase& testRootDir, uba::SubTestContext& subCtx); \
	static uba::Test UBA_TEST_CONCAT(Run##groupName##testName##Node, __LINE__){ TC(#testName), nullptr, TC(reason), nullptr }; \
	static uba::TestRegistrar UBA_TEST_CONCAT(Run##groupName##testName##Reg, __LINE__){ g_##groupName, &UBA_TEST_CONCAT(Run##groupName##testName##Node, __LINE__) }; \
	static bool Run##groupName##testName##Impl(uba::LoggerWithWriter& logger, const uba::StringBufferBase& testRootDir, uba::SubTestContext& subCtx)

#define UBA_SUB_TEST(subName) if (subCtx.Enter(TC(#subName)))

#define UBA_TEST_LOCAL_AND_REMOTE(groupName, testName) \
	static bool Run##groupName##testName##Body(uba::LoggerWithWriter& logger, uba::SessionServer& session, const uba::tchar* workingDir, const uba::RunProcessFunction& runProcess, uintptr_t extraData); \
	static bool Run##groupName##testName##Local(uba::LoggerWithWriter& logger, const uba::StringBufferBase& testRootDir) \
		{ return uba::RunLocal(logger, testRootDir, &Run##groupName##testName##Body); } \
	static bool Run##groupName##testName##Remote(uba::LoggerWithWriter& logger, const uba::StringBufferBase& testRootDir) \
		{ return uba::RunRemote(logger, testRootDir, &Run##groupName##testName##Body); } \
	static uba::Test UBA_TEST_CONCAT(Run##groupName##testName##LocalNode, __LINE__){ TC(#testName), &Run##groupName##testName##Local, nullptr, nullptr }; \
	static uba::Test UBA_TEST_CONCAT(Run##groupName##testName##RemoteNode, __LINE__){ TC(#testName) TC("Remote"), &Run##groupName##testName##Remote, nullptr, nullptr }; \
	static uba::TestRegistrar UBA_TEST_CONCAT(Run##groupName##testName##LocalReg, __LINE__){ g_##groupName, &UBA_TEST_CONCAT(Run##groupName##testName##LocalNode, __LINE__) }; \
	static uba::TestRegistrar UBA_TEST_CONCAT(Run##groupName##testName##RemoteReg, __LINE__){ g_##groupName, &UBA_TEST_CONCAT(Run##groupName##testName##RemoteNode, __LINE__) }; \
	static bool Run##groupName##testName##Body