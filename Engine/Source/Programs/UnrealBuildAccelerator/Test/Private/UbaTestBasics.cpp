// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaTest.h"
#include "UbaBinaryParser.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaDefinitions.h"
#include "UbaFileAccessor.h"
#include "UbaFileMappingBuffer.h"
#include "UbaPathUtils.h"
#include "UbaPlatform.h"
#include "UbaProcessStartInfoHolder.h"
#include "UbaProcessUtils.h"
#include "UbaEvent.h"
#include "UbaFile.h"
#include "UbaLogger.h"
#include "UbaRootPaths.h"
#include "UbaSharedMemoryAllocator.h"
#include "UbaThread.h"
#include "UbaTimer.h"
#include "UbaDirectoryIterator.h"

UBA_DEFINE_TEST_GROUP(Basic, 10)

#if 0 // PLATFORM_WINDOWS
#define DEBUG_LOG_DETOURED(...)
namespace uba
{
	#include "../../Detours/Private/Windows/UbaDetoursFunctionsImagehlp.inl"
}
#include "ImageHlp.h"
#pragma comment (lib, "imagehlp.lib")
#endif

#define VA_ARGS(...) , ##__VA_ARGS__
#define UBA_TEST_CHECK(expr, fmt, ...) if (!(expr)) return logger.Error(TC(fmt) VA_ARGS(__VA_ARGS__));


namespace uba
{
	bool GetCpuTime(u64& outTotalTime, u64& outIdleTime);

	UBA_TEST(Basic, Encodings)
	{
		StringBuffer<> buf;

		u64 v = 0;
		buf.Clear().AppendBase62(v);
		CHECK_TRUE(v == StringToValueBase62(buf.data, buf.count));

		v = 5473789457;
		buf.Clear().AppendBase62(v);
		CHECK_TRUE(v == StringToValueBase62(buf.data, buf.count));

		v = ~u64(0);
		buf.Clear().AppendBase62(v);
		CHECK_TRUE(v == StringToValueBase62(buf.data, buf.count));
		return true;
	}

	UBA_TEST(Basic, Time)
	{
#if 0
		LoggerWithWriter consoleLogger(g_consoleLogWriter); (void)consoleLogger;
		u64 time1 = GetSystemTimeUs();
		Sleep(1000);
		u64 time2 = GetSystemTimeUs();
		u64 ms = (time2 - time1) / 1000;
		consoleLogger.Info(TC("Slept ms: %llu"), ms);

		time1 = GetTime();
		Sleep(1000);
		time2 = GetTime();
		ms = (time2 * 1000 / GetFrequency()) - (time1 * 1000 / GetFrequency());
		consoleLogger.Info(TC("Slept ms: %llu"), ms);
#endif

		u64 seconds = 15;
		u64 fileTime = GetSecondsAsFileTime(seconds);
		u64 seconds2 = GetFileTimeAsSeconds(fileTime);
		UBA_TEST_CHECK(seconds == seconds2, "GetSecondsAsFileTime does not match GetFileTimeAsSeconds");


		u64 totalTime;
		u64 maxTime;
		CHECK_TRUEF(GetCpuTime(totalTime, maxTime), TC("GetCpuTime failed"));

		return true;
	}

	template<class EventType>
	bool TestEventsImpl(LoggerWithWriter& logger, const StringBufferBase& testRootDir, EventType& ev, EventType& ev2)
	{
		Thread t([&]()
			{
				Sleep(500);
				//logger.Info(TC("Setting event"));
				ev2.Set();
				Sleep(500);
				return true;
			});

		CHECK_TRUEF(!ev.IsSet(1), TC("Event was set after 1ms timeout where it should take 500ms"));

		CHECK_TRUEF(!ev.IsSet(0), TC("Event was set after no timeout where it should take 500ms"));

		//logger.Info(TC("Waiting for event"));
		CHECK_TRUEF(ev.IsSet(2000), TC("Event was not set after 2000ms where it should take 500ms"));
		//logger.Info(TC("Event was set"));

		CHECK_TRUEF(!t.Wait(0), TC("Thread wait timed out. Should already be done after 2000ms"));

		CHECK_TRUEF(t.Wait(2000), TC("Thread wait did not timed out should be done after 2000ms"));

		#if 0 // Long time test... disabled by default
		u64 time = GetTime();
		EventType longEv(true);
		longEv.IsSet(10 * 60 * 1000);
		CHECK_TRUEF(!TimeToMs((GetTime() - time) < 9 * 60 * 1000), TC("Event timeout was way too fast"));
		#endif
		return true;
	}

	UBA_TEST(Basic, Events)
	{
		Event ev;
		CHECK_TRUEF(ev.Create(EventResetType_Manual), TC("Failed to create event"));
		CHECK_TRUE(TestEventsImpl(logger, testRootDir, ev, ev));
		return true;
	}

	UBA_TEST(Basic, SharedEvents)
	{
		#if !PLATFORM_WINDOWS
		if (IsRunningDarling()) // bug in darling, this does not work
			return true;
		FileMappingBackend backend;
		SharedMemoryAllocator allocator(logger, backend);
		allocator.Init(1024 * 1024);
		FileMappingBuffer mappingBuffer(logger, allocator);
		mappingBuffer.Init(TC(""), 64*1024, 64 * 1024);
		MappedView view = mappingBuffer.AllocAndMapView(1024, 1, TC("Foo"));
		SharedEvent* ev = new (view.memory) SharedEvent();
		CHECK_TRUEF(ev->Create(EventResetType_Manual), TC("Failed to create event"));
		CHECK_TRUE(TestEventsImpl(logger, testRootDir, *ev, *ev));
		ev->~SharedEvent();
		mappingBuffer.UnmapView(view, TC("Foo"));
		#endif
		return true;
	}

	UBA_TEST(Basic, SharedEvents2)
	{
		#if !PLATFORM_WINDOWS
		SharedEvent ev;
		ev.Create(TCV("uba_tse"), true);
		StringBuffer<64> tmp;
		ev.ToString(tmp);
		SharedEvent ev2;
		ev2.Create(tmp, false);
		CHECK_TRUE(TestEventsImpl(logger, testRootDir, ev, ev2));
		#endif
		return true;
	}

	UBA_TEST(Basic, Paths)
	{
		const tchar* workingDir = IsWindows ? TC("e:\\dev\\") : TC("/dev/bar/");
		tchar buffer[1024];
		u32 lengthResult;

		auto TestPath = [&](const tchar* path) { return FixPath2(path, ~0u, workingDir, TStrlen(workingDir), buffer, sizeof_array(buffer), &lengthResult); };

#if PLATFORM_WINDOWS
		CHECK_TRUEF(FixPath2(TC("\"e:\\temp\""), ~0u, workingDir, TStrlen(workingDir), buffer, sizeof_array(buffer), &lengthResult), TC("FixPath2 (1) failed"));
		CHECK_TRUEF(!(!FixPath2(TC("\\??\\c:\\windows:::"), 14, workingDir, TStrlen(workingDir), buffer, sizeof_array(buffer), &lengthResult) || lengthResult != 10 || TStrlen(buffer) != 10), TC("FixPath2 (2) failed"));
		CHECK_TRUEF(!(!FixPath2(TC("Z:/UEVFS/Root/Engine/Binaries/Win64/UnrealEditor-InterchangeCore-Win64-Debug.pdb::"), 80, workingDir, TStrlen(workingDir), buffer, sizeof_array(buffer), &lengthResult) || lengthResult != 80 || TStrlen(buffer) != 80), TC("FixPath2 (2) failed"));
		CHECK_TRUEF(!FixPath2(TC("'Small vector' optimization: store up to a small number of items on the stack"), ~0u, workingDir, TStrlen(workingDir), buffer, sizeof_array(buffer), &lengthResult), TC("FixPath2 (2) failed"));
		#if 0
		const tchar longPath[] = TC("c:\\Program Files (x86)\\Microsoft Visual Studio\\Shared\\Entity Framework Tools\\NuGet Packages");
		tchar shortPath[MAX_PATH];
		DWORD len = GetShortPathNameW(longPath, shortPath, MAX_PATH);
		CHECK_TRUEF(!(len == 0 || len >= sizeof_array(longPath)), TC("GetShortPathNameW failed"));
		CHECK_TRUEF(!(!FixPath2(shortPath, len, workingDir, TStrlen(workingDir), buffer, sizeof_array(buffer), &lengthResult) || lengthResult != sizeof_array(longPath) - 1), TC("FixPath2 for short path failed"));
		#endif


#else

		CHECK_TRUEF(TestPath(TC("/..")), TC("FixPath2 should have failed"));
		UBA_TEST_CHECK(Equals(buffer, TC("/")), "Should not contain ..");

		CHECK_TRUEF(FixPath2(TC("/../Foo"), ~0u, workingDir, TStrlen(workingDir), buffer, sizeof_array(buffer), &lengthResult), TC("FixPath2 should have failed"));
		UBA_TEST_CHECK(Equals(buffer, TC("/Foo")), "Should not contain ..");

		if (!FixPath2(TC("/usr/bin//clang++"), ~0u, workingDir, TStrlen(workingDir), buffer, sizeof_array(buffer), &lengthResult))
			return logger.Error(TC("FixPath2 should have failed"));
		UBA_TEST_CHECK(!Contains(buffer, TC("//")), "Should not contain //");
#endif

		CHECK_TRUEF(FixPath2(TC("../Foo"), ~0u, workingDir, TStrlen(workingDir), buffer, sizeof_array(buffer), &lengthResult), TC("FixPath2 (1) failed"));
		UBA_TEST_CHECK(!Contains(buffer, TC("..")), "Should not contain ..");

		CHECK_TRUEF(FixPath2(TC("@../Foo"), ~0u, workingDir, TStrlen(workingDir), buffer, sizeof_array(buffer), &lengthResult), TC("FixPath2 (1) failed"));
		UBA_TEST_CHECK(Contains(buffer, TC("..")), "Should contain ..");

		CHECK_TRUEF(FixPath2(TC("..@/Foo"), ~0u, workingDir, TStrlen(workingDir), buffer, sizeof_array(buffer), &lengthResult), TC("FixPath2 (1) failed"));
		UBA_TEST_CHECK(Contains(buffer, TC("..")), "Should contain ..");

		return true;
	}

	UBA_TEST(Basic, Files)
	{
		StringBuffer<> testFileName(testRootDir);
		testFileName.Append(TCV("UbaTestFile"));

		FileAccessor fileHandle(logger, testFileName.data);
		CHECK_TRUEF(fileHandle.CreateWrite(), TC("Failed to create file for write"));

		u8 byte = 'H';
		CHECK_TRUE(fileHandle.Write(&byte, 1));

		CHECK_TRUE(fileHandle.Close());

		FileHandle fileHandle2;
		CHECK_TRUEF(OpenFileSequentialRead(logger, testFileName.data, fileHandle2), TC("Failed to create file for read"));

		u64 writeTime = 0;
		CHECK_TRUEF(GetFileLastWriteTime(writeTime, fileHandle2), TC("Failed to get last written time"));

		u64 writeTime2 = 0;
		TraverseDir(logger, testRootDir, [&](const DirectoryEntry& de)
			{
				if (Equals(de.name, TC("UbaTestFile")))
					writeTime2 = de.lastWritten;
			});

		CHECK_TRUEF(!(writeTime != writeTime2), TC("GetFileLastWriteTime and TraverseDir are returning different last write time for same file"));

		u64 systemTime = GetSystemTimeAsFileTime();
		CHECK_TRUEF(!(systemTime < writeTime), TC("System time is lower than last written time"));
		CHECK_TRUEF(!(GetFileTimeAsSeconds(systemTime) - GetFileTimeAsSeconds(writeTime) > 3), TC("System time or last written time is wrong (system: %llu, write: %llu, diffInSec: %llu)"), systemTime, writeTime, GetFileTimeAsSeconds(systemTime) - GetFileTimeAsSeconds(writeTime));


		u8 byte2 = 0;
		CHECK_TRUE(ReadFile(logger, testFileName.data, fileHandle2, &byte2, 1));

		CHECK_TRUE(CloseFile(testFileName.data, fileHandle2));

		FileHandle fileHandle3;
		CHECK_TRUEF(OpenFileSequentialRead(logger, TC("NonExistingFile"), fileHandle3, false), TC("OpenFileSequentialRead failed with non existing file"));
		CHECK_TRUEF(!(fileHandle3 != InvalidFileHandle), TC("OpenFileSequentialRead found file that doesn't exist"));

		CHECK_TRUEF(!RemoveDirectoryW(TC("TestDir")), TC("Did not fail to remove non-existing TestDir (or were things not cleaned before test)"));
		CHECK_TRUEF(GetLastError() == ERROR_FILE_NOT_FOUND, TC("GetLastError did not return correct error failing to remove non-existing directory TestDir"));

		CHECK_TRUEF(CreateDirectoryW(TC("TestDir")), TC("Failed to create dir"));

		CHECK_TRUEF(!CreateDirectoryW(TC("TestDir")), TC("Did not fail to create existing dir"));

		FileHandle fileHandle4;
		CHECK_TRUEF(!OpenFileSequentialRead(logger, TC("TestDir"), fileHandle4), TC("This should return fail"));

		CHECK_TRUEF(RemoveDirectoryW(TC("TestDir")), TC("Fail to remove TestDir"));

		u64 size = 0;
		CHECK_TRUEF(FileExists(logger, testFileName.data, &size) || size != 1, TC("UbaTestFile not found"));

		StringBuffer<> testFileName2(testRootDir);
		testFileName2.Append(TCV("UbaTestFile2"));

		DeleteFileW(testFileName2.data);

		CHECK_TRUEF(!DeleteFileW(testFileName2.data), TC("Did not fail to delete non-existing UbaTestFile2 (or were things not cleaned before test)"));
		CHECK_TRUEF(GetLastError() == ERROR_FILE_NOT_FOUND, TC("GetLastError did not return correct error failing to delete non-existing file UbaTestFile2"));

		CHECK_TRUEF(CreateHardLinkW(testFileName2.data, testFileName.data), TC("Failed to create hardlink from UbaTestFile to UbaTestFile2"));

		CHECK_TRUEF(DeleteFileW(testFileName.data), TC("Failed to delete UbaTestFile"));

		CHECK_TRUEF(!FileExists(logger, testFileName.data), TC("Found non-existing file UbaTestFile"));

		// CreateHardLinkW is a symbolic link on non-windows.. need to revisit
		#if PLATFORM_WINDOWS
		CHECK_TRUEF(FileExists(logger, testFileName2.data), TC("Failed to find file UbaTestFile2"));

		StringBuffer<> currentDir;
		CHECK_TRUEF(GetCurrentDirectoryW(currentDir), TC("GetCurrentDirectoryW failed"));

		bool foundFile = false;
		if (!TraverseDir(logger, testRootDir, [&](const DirectoryEntry& de) { foundFile |= TStrcmp(de.name, TC("UbaTestFile2")) == 0; }, true))
			return logger.Error(TC("Failed to TraverseDir '.'"));

		CHECK_TRUEF(foundFile, TC("Did not find UbaTestFile2 with TraverseDir"));

		CHECK_TRUE(DeleteFileW(testFileName2.data));
		#endif

		return true;
	}

	UBA_TEST(Basic, TraverseDir)
	{
		LoggerWithWriter nullLogger(g_nullLogWriter);
		CHECK_TRUE(!TraverseDir(nullLogger, AsView(TC("TestDir2")), [&](const DirectoryEntry&) {}, true));
		u32 foundCount = 0;

		StringBuffer<> testDir(testRootDir);
		testDir.Append(TCV("TraverseDir")).EnsureEndsWithSlash();
		CHECK_TRUE(CreateDirectoryW(testDir.data));

		CHECK_TRUE(TraverseDir(nullLogger, testDir, [&](const DirectoryEntry&) { ++foundCount; }, true));
		CHECK_TRUE(foundCount == 0);

		bool isFile = false;
		StringBuffer<> entry(testDir);
		entry.Append(TCV("Entry"));
		FileAccessor fileHandle(logger, entry.data);
		CHECK_TRUE(fileHandle.CreateWrite(false));
		CHECK_TRUE(fileHandle.Close());
		CHECK_TRUE(TraverseDir(nullLogger, testDir, [&](const DirectoryEntry& e) { ++foundCount; isFile = !IsDirectory(e.attributes); }, true));
		CHECK_TRUE(foundCount == 1);
		CHECK_TRUE(isFile);
		CHECK_TRUE(DeleteFileW(entry.data));
		CHECK_TRUE(CreateDirectoryW(entry.data));
		foundCount = 0;
		CHECK_TRUE(TraverseDir(nullLogger, testDir, [&](const DirectoryEntry& e) { ++foundCount; isFile = !IsDirectory(e.attributes); }, true));
		CHECK_TRUE(foundCount == 1);
		CHECK_TRUE(!isFile);
		return true;
	}

	UBA_TEST(Basic, OverlappedIO)
	{
		if (!EventIsNative)
			return true;

		StringBuffer<> testFileName(testRootDir);
		testFileName.Append(TCV("UbaTestFile"));

		u64 left = 2*1024*1024;

		FileAccessor fileHandle(logger, testFileName.data);
		CHECK_TRUEF(fileHandle.CreateWrite(false, DefaultAttributes() | FILE_FLAG_OVERLAPPED, left), TC("Failed to create file for write"));

		constexpr u64 oddSize = 277872ull;
		u8 buffer[oddSize];

		while (left)
		{
			u64 toWrite = Min(left, oddSize);
			CHECK_TRUEF(fileHandle.Write(buffer, toWrite), TC("Failed to create file for write"));
			left -= toWrite;
		}
		CHECK_TRUE(fileHandle.Close());
		return true;
	}

	UBA_TEST(Basic, MemoryBlock)
	{
		{
			MemoryBlock block(TC("TestMemoryBlock"), 1024 * 1024);
			u64* mem = (u64*)block.Allocate(8, 1, TC("Foo"));
			*mem = 0x1234;
			block.Free(mem);
		}

		if (GetHugePageCount())
		{
			MemoryBlock block(TC("TestHugePage"));
			CHECK_TRUEF(block.Init(1024 * 1024, nullptr, true), TC("Failed to allocate huge pages even though system says they exists"));
			u64* mem = (u64*)block.Allocate(8, 1, TC("Foo"));
			*mem = 0x1234;
			block.Free(mem);
		}

		return true;
	}

	UBA_TEST(Basic, ParseArguments)
	{
		auto ParseArguments = [](Vector<TString>& a, const tchar* args) { uba::ParseArguments(args, [&](const tchar* arg, u32 argLen) { a.push_back({arg, argLen}); }); };

		Vector<TString> arguments;
		ParseArguments(arguments, TC("foo bar"));
		UBA_TEST_CHECK(arguments.size() == 2, "ParseArguments 1 failed (%llu)", arguments.size());

		Vector<TString> arguments2;
		ParseArguments(arguments2, TC("\"foo\" bar"));
		UBA_TEST_CHECK(arguments2.size() == 2, "ParseArguments 2 failed");

		Vector<TString> arguments3;
		ParseArguments(arguments3, TC("\"foo meh\" bar"));
		UBA_TEST_CHECK(arguments3.size() == 2, "ParseArguments 3 failed");
		UBA_TEST_CHECK(Contains(arguments3[0].data(), TC(" ")), "ParseArguments 3 failed");

		Vector<TString> arguments4;
		ParseArguments(arguments4, TC("\"app\" @\"rsp\""));
		UBA_TEST_CHECK(arguments4.size() == 2, "ParseArguments 4 failed");
		UBA_TEST_CHECK(!Contains(arguments4[1].data(), TC("\"")), "ParseArguments 4 failed");

		Vector<TString> arguments5;
		ParseArguments(arguments5, TC("\"app\" @\"rsp foo\""));
		UBA_TEST_CHECK(arguments5.size() == 2, "ParseArguments 4 failed");
		UBA_TEST_CHECK(!Contains(arguments5[1].data(), TC("\"")), "ParseArguments 5 failed");
		UBA_TEST_CHECK(Contains(arguments5[1].data(), TC(" ")), "ParseArguments 5 failed");

		Vector<TString> arguments6;
		ParseArguments(arguments6, TC("\"app\"\"1\" @\"rsp foo\""));
		UBA_TEST_CHECK(arguments6.size() == 2, "ParseArguments 6 failed");
		UBA_TEST_CHECK(Equals(arguments6[0].data(), TC("app1")), "ParseArguments 6 failed");

		Vector<TString> arguments7;
		ParseArguments(arguments7, TC("app \" \\\"foo\\\" bar\""));
		UBA_TEST_CHECK(arguments7.size() == 2, "ParseArguments 7 failed");
		UBA_TEST_CHECK(Contains(arguments7[1].data(), TC("\"")), "ParseArguments 7 failed");

		Vector<TString> arguments8;
		ParseArguments(arguments8, TC("\nline1\r\nline2\r\nline3\n\r\n"));
		UBA_TEST_CHECK(arguments8.size() == 3, "ParseArguments 8 failed");
		UBA_TEST_CHECK(Equals(arguments8[0].data(), TC("line1")), "ParseArguments 8 failed");
		UBA_TEST_CHECK(Equals(arguments8[1].data(), TC("line2")), "ParseArguments 8 failed");
		UBA_TEST_CHECK(Equals(arguments8[2].data(), TC("line3")), "ParseArguments 8 failed");

		Vector<TString> arguments9;
		ParseArguments(arguments9, TC("\"foo\\\\\" \"bar\\\\\""));
		UBA_TEST_CHECK(arguments9.size() == 2, "ParseArguments 9 failed");
		UBA_TEST_CHECK(Equals(arguments9[0].data(), TC("foo\\\\")), "ParseArguments 9 failed");
		UBA_TEST_CHECK(Equals(arguments9[1].data(), TC("bar\\\\")), "ParseArguments 9 failed");

		Vector<TString> arguments10;
		ParseArguments(arguments10, TC("-i \\\"foo\\\""));
		UBA_TEST_CHECK(arguments10.size() == 2, "ParseArguments 10 failed");
		UBA_TEST_CHECK(Equals(arguments10[1].data(), TC("\"foo\"")), "ParseArguments 10 failed");

		#if PLATFORM_WINDOWS
		Vector<TString> arguments11;
		ParseArguments(arguments11, TC("\\\"a\\\\b\\\" \\\"c\\\\d\\\" meh"));
		UBA_TEST_CHECK(arguments11.size() == 3, "ParseArguments 11 failed");
		UBA_TEST_CHECK(Equals(arguments11[1].data(), TC("\"c\\\\d\"")), "ParseArguments 11 failed");
		UBA_TEST_CHECK(Equals(arguments11[2].data(), TC("meh")), "ParseArguments 11 failed");
		#else
		// POSIX: backslash-escaped apostrophe outside quotes is literal, must not toggle single-quote mode.
		// Regression for UbaNinja truncating Chromium rustc env args at d\'Antras.
		Vector<TString> arguments12;
		ParseArguments(arguments12, TC("--env CARGO=Alex\\ d\\'Antras FOO=bar"));
		UBA_TEST_CHECK(arguments12.size() == 3, "ParseArguments 12 failed (size=%llu)", arguments12.size());
		UBA_TEST_CHECK(Equals(arguments12[0].data(), TC("--env")), "ParseArguments 12 failed");
		UBA_TEST_CHECK(Equals(arguments12[1].data(), TC("CARGO=Alex d'Antras")), "ParseArguments 12 failed");
		UBA_TEST_CHECK(Equals(arguments12[2].data(), TC("FOO=bar")), "ParseArguments 12 failed");
		#endif
		return true;
	}

	// Drive ProcessStartInfoHolder::Expand() against shell command-line shapes from real builds.
	// Each row asserts whether Expand wraps to `sh -c '...'` and that the wrapped payload byte-exactly equals the original fullCmd.
	UBA_TEST(Basic, ProcessStartInfoExpand)
	{
	#if !PLATFORM_WINDOWS
		struct Case
		{
			const tchar* label;
			const tchar* app;        // empty → full cmdline in args
			const tchar* args;
			bool expectWrap;
			const tchar* expectFull; // sh argv[2] after `'\''`-unescape (wrapped), or applicationStr + " " + argumentsStr (unwrapped)
		};

		const Case cases[] = {
			{ TC("printf-glued-redirect"), TC(""), TC("/bin/printf '#!/bin/sh\\nexec foo $@\\n'> out/path.sh"), true, TC("/bin/printf '#!/bin/sh\\nexec foo $@\\n'> out/path.sh") },
			{ TC("redirect-spaced"),       TC(""), TC("cmd arg > out"),      true, TC("cmd arg > out") },
			{ TC("redirect-fused-right"),  TC(""), TC("cmd arg >out"),       true, TC("cmd arg >out") },
			{ TC("redirect-fused-both"),   TC(""), TC("cmd>out"),            true, TC("cmd>out") },
			{ TC("append-spaced"),         TC(""), TC("cmd >> out"),         true, TC("cmd >> out") },
			{ TC("append-fused"),          TC(""), TC("cmd>>out"),           true, TC("cmd>>out") },
			{ TC("stdin-spaced"),          TC(""), TC("cmd < in"),           true, TC("cmd < in") },
			{ TC("stdin-fused"),           TC(""), TC("cmd <in"),            true, TC("cmd <in") },
			{ TC("stderr-to-stdout"),      TC(""), TC("cmd 2>&1"),           true, TC("cmd 2>&1") },
			{ TC("background"),            TC(""), TC("cmd &"),              true, TC("cmd &") },
			{ TC("pipe-spaced"),           TC(""), TC("cmd | wc -l"),        true, TC("cmd | wc -l") },
			{ TC("pipe-fused"),            TC(""), TC("cmd|wc"),             true, TC("cmd|wc") },
			{ TC("andlist-spaced"),        TC(""), TC("cmd1 && cmd2"),       true, TC("cmd1 && cmd2") },
			{ TC("andlist-fused"),         TC(""), TC("cmd1&&cmd2"),         true, TC("cmd1&&cmd2") },
			{ TC("orlist-spaced"),         TC(""), TC("cmd1 || cmd2"),       true, TC("cmd1 || cmd2") },
			{ TC("orlist-fused"),          TC(""), TC("cmd1||cmd2"),         true, TC("cmd1||cmd2") },
			{ TC("seq-spaced"),            TC(""), TC("cmd1 ; cmd2"),        true, TC("cmd1 ; cmd2") },
			{ TC("seq-fused"),             TC(""), TC("cmd1;cmd2"),          true, TC("cmd1;cmd2") },
			{ TC("dollar-paren"),          TC(""), TC("echo $(date +%s)"),   true, TC("echo $(date +%s)") },
			{ TC("backtick"),              TC(""), TC("echo `date +%s`"),    true, TC("echo `date +%s`") },
			{ TC("if-keyword"),            TC(""), TC("if [ -f x ]; then echo; fi"), true, TC("if [ -f x ]; then echo; fi") },
			{ TC("for-keyword"),           TC(""), TC("for i in a b c; do echo $i; done"), true, TC("for i in a b c; do echo $i; done") },
			{ TC("cd-keyword"),            TC(""), TC("cd /tmp && ls"),      true, TC("cd /tmp && ls") },
			{ TC("export-keyword"),        TC(""), TC("export X=1"),         true, TC("export X=1") },
			{ TC("env-var-prefix"),        TC(""), TC("PWD=/proc/self/cwd /usr/bin/clang -c x.c"), true, TC("PWD=/proc/self/cwd /usr/bin/clang -c x.c") },
			{ TC("subshell-leading"),      TC(""), TC("(cd /tmp; ls)"),      true, TC("(cd /tmp; ls)") },
			{ TC("dotsh-application"),     TC("/usr/bin/setup.sh"), TC("--foo bar"), true, TC("/usr/bin/setup.sh --foo bar") },
			{ TC("hash-comment-trailer"),  TC(""), TC("/bin/cmd arg # hash of input list: a b c"), true, TC("/bin/cmd arg # hash of input list: a b c") },

			// Shouldn't wrap (no shell metachars outside quotes)
			{ TC("plain-app-args"),        TC("/bin/echo"), TC("hello world"), false, TC("/bin/echo hello world") },
			// Explicit application -> arguments are literal argv; metachars in them must not wrap.
			{ TC("explicit-app-literal-semicolon"), TC("/bin/UbaTestApp"), TC("write=sp,d spawn=read=sp,d;size=sp,9 unlink=sp"), false, TC("/bin/UbaTestApp write=sp,d spawn=read=sp,d;size=sp,9 unlink=sp") },
			{ TC("explicit-app-literal-pipe"),      TC("/bin/UbaTestApp"), TC("a|b"), false, TC("/bin/UbaTestApp a|b") },
			{ TC("plain-empty-app"),       TC(""), TC("/bin/echo hello"),     false, TC("/bin/echo hello") },
			{ TC("gt-inside-singlequotes"),TC(""), TC("/bin/grep -e '^>' file"), false, TC("/bin/grep -e '^>' file") },
			{ TC("gt-inside-doublequotes"),TC(""), TC("/bin/grep -e \">>>\" file"), false, TC("/bin/grep -e \">>>\" file") },
			// Quoted single-char metachars become argLen==1 after ParseArguments strips quotes, so they still trigger wrap (safe — sh re-parses with quotes intact).
			{ TC("semi-quoted-becomes-token"),TC(""), TC("/bin/grep ';' file"),  true, TC("/bin/grep ';' file") },
			{ TC("gt-quoted-becomes-token"),  TC(""), TC("/bin/grep '>' file"),  true, TC("/bin/grep '>' file") },
			{ TC("pipe-quoted-becomes-token"),TC(""), TC("/bin/grep '|' file"),  true, TC("/bin/grep '|' file") },
			{ TC("hash-mid-token"),        TC(""), TC("/bin/grep arg#foo file"), false, TC("/bin/grep arg#foo file") },
			// `if_something` starts with shell keyword as substring but tokLen
			// differs — must not match.
			{ TC("if-prefix-not-keyword"), TC(""), TC("if_check_thing -v"),     false, TC("if_check_thing -v") },
		};

		const u32 caseCount = sizeof(cases) / sizeof(cases[0]);
		for (u32 ci = 0; ci < caseCount; ++ci)
		{
			const Case& c = cases[ci];

			ProcessStartInfoHolder h;
			h.applicationStr = c.app;
			h.application = h.applicationStr.c_str();
			h.argumentsStr = c.args;
			h.arguments = h.argumentsStr.c_str();

			h.Expand();

			bool gotWrap = Equals(h.applicationStr.c_str(), TC("sh"));
			UBA_TEST_CHECK(gotWrap == c.expectWrap,
				"case[%u] %s: expected wrap=%i, got wrap=%i (app='%s', args='%s')",
				ci, c.label, (int)c.expectWrap, (int)gotWrap,
				h.applicationStr.c_str(), h.argumentsStr.c_str());

			if (c.expectWrap)
			{
				// argumentsStr is `-c '<escaped>'`. Strip outer `-c '` and trailing `'`.
				UBA_TEST_CHECK(h.argumentsStr.size() >= 5 &&
					h.argumentsStr.compare(0, 4, TC("-c '")) == 0 &&
					h.argumentsStr.back() == TC('\''),
					"case[%u] %s: wrapped form not `-c '...'`: '%s'",
					ci, c.label, h.argumentsStr.c_str());

				// Reverse the `'\''` escape applied by Expand to recover the
				// original fullCmd byte-for-byte.
				TString recovered;
				const tchar* p = h.argumentsStr.c_str() + 4; // after `-c '`
				const tchar* end = h.argumentsStr.c_str() + h.argumentsStr.size() - 1; // strip trailing `'`
				while (p < end)
				{
					if (end - p >= 4 && p[0] == TC('\'') && p[1] == TC('\\') &&
						p[2] == TC('\'') && p[3] == TC('\''))
					{
						recovered += TC('\'');
						p += 4;
					}
					else
					{
						recovered += *p++;
					}
				}
				UBA_TEST_CHECK(Equals(recovered.c_str(), c.expectFull),
					"case[%u] %s: wrapped payload mismatch\n  expected: %s\n  got:      %s",
					ci, c.label, c.expectFull, recovered.c_str());
			}
			else
			{
				// Without wrap, applicationStr is the first argv and
				// argumentsStr is the remainder. Compare the recomposed line.
				TString full = h.applicationStr;
				if (!h.argumentsStr.empty())
				{
					full += TC(" ");
					full += h.argumentsStr;
				}
				UBA_TEST_CHECK(Equals(full.c_str(), c.expectFull),
					"case[%u] %s: post-Expand recomposed line mismatch\n  expected: %s\n  got:      %s",
					ci, c.label, c.expectFull, full.c_str());
			}
		}

		// === idempotency: a second Expand() must be a no-op ===
		// (Guaranteed by the `expanded` flag; verify the flag's behavior.)
		{
			ProcessStartInfoHolder h;
			h.applicationStr = TC("");
			h.application = h.applicationStr.c_str();
			h.argumentsStr = TC("/bin/printf 'a'>out");
			h.arguments = h.argumentsStr.c_str();
			h.Expand();
			TString afterFirstApp = h.applicationStr;
			TString afterFirstArgs = h.argumentsStr;
			bool secondReturn = h.Expand();
			UBA_TEST_CHECK(!secondReturn, "second Expand() should return false (already expanded)");
			UBA_TEST_CHECK(Equals(h.applicationStr.c_str(), afterFirstApp.c_str()),
				"second Expand() must not change applicationStr ('%s' -> '%s')",
				afterFirstApp.c_str(), h.applicationStr.c_str());
			UBA_TEST_CHECK(Equals(h.argumentsStr.c_str(), afterFirstArgs.c_str()),
				"second Expand() must not change argumentsStr");
		}

		// === single quote inside payload — `'\''` escaping ===
		// Real-world: printf '...d'\''Antras...' must round-trip through the
		// wrap and unwrap without corruption.
		{
			ProcessStartInfoHolder h;
			h.applicationStr = TC("");
			h.application = h.applicationStr.c_str();
			h.argumentsStr = TC("/bin/echo 'has '\\''quote'\\'' inside' > out");
			h.arguments = h.argumentsStr.c_str();
			h.Expand();
			UBA_TEST_CHECK(Equals(h.applicationStr.c_str(), TC("sh")),
				"single-quote-in-payload: expected wrap to sh, got app='%s'",
				h.applicationStr.c_str());
		}
	#endif
		return true;
	}

	UBA_TEST(Basic, BinaryWriter)
	{
		auto testString = [&](const tchar* str)
		{
			u8 mem[1024];
			BinaryWriter writer(mem);
			writer.WriteString(str);
			BinaryReader reader(mem);
			TString s = reader.ReadString();
			CHECK_TRUEF(!(s.size() != TStrlen(str)), TC("Serialized string '%s' has wrong strlen"), str);
			CHECK_TRUEF(!(s != str), TC("Serialized string '%s' is different from source"), str);
			return true;
		};

		CHECK_TRUE(testString(TC("Foo")));

		#if PLATFORM_WINDOWS
		tchar str1[] = { 54620, 44544, 0 };
		CHECK_TRUE(testString(str1));
		tchar str2[] = { 'f', 54620, 'o', 44544, 0 };
		CHECK_TRUE(testString(str2));
		#endif

		if (!IsDebug)
		{
			u8 mem[1024];
			BinaryWriter writer(mem);
			writer.WriteString(TC("Foo"));
			writer.WriteString(TC("Bar"));
			BinaryReader reader(mem);
			StringBuffer<3> str;
			reader.ReadString(str);
			CHECK_TRUE(str.Equals(TC("Fo")));
			reader.ReadString(str.Clear());
			CHECK_TRUE(str.Equals(TC("Ba")));
		}

		return true;
	}

	#if PLATFORM_WINDOWS
	UBA_TEST(Basic, KnownSystemFiles)
	{
		for (auto systemFile : g_knownSystemFiles)
			CHECK_TRUEF(IsKnownSystemFile(systemFile), TC("IsKnownSystemFile returned false for %s which is a system file"), systemFile);
		CHECK_TRUEF(!IsKnownSystemFile(TC("Fooo.dll")), TC("IsKnownSystemFile returned true for Fooo.dll which is not a system file"));
		return true;
	}
	#endif

	UBA_TEST(Basic, RootPaths)
	{
		#if PLATFORM_WINDOWS
		const tchar root1[] = TC("c:\\temp\\");
		const tchar root2[] = TC("e:\\temp\\");
		const tchar str[] = TC("e:\\temp\\foo");
		#else
		const tchar root1[] = TC("/mnt/c/");
		const tchar root2[] = TC("/mnt/e/");
		const tchar str[] = TC("/mnt/e/foo");
		#endif

		RootPaths paths;
		CHECK_TRUE(paths.RegisterRoot(logger, root1));
		CHECK_TRUE(paths.RegisterRoot(logger, root2));

		bool success = true;
		StringBuffer<> temp;
		u32 rootPos = ~0u;
		bool res = paths.NormalizeString(logger, str, sizeof_array(str), [&](const tchar* str, u64 strLenIncTerm, u32 rp)
			{
				if (rp != ~0u)
				{
					if (strLenIncTerm != 1)
						success = false;
					if (str[0] != RootPaths::RootStartByte + PathsPerRoot + (IsWindows ? 1 : 0)) // Add one for windows because second entry is backslash
						success = false;
					rootPos = str[0];
				}
				else
				{
					temp.Append(str, strLenIncTerm - 1);
					if (!temp.Equals(TCV("foo")))
						success = false;
				}
			}, false, TC(""));

		CHECK_TRUE(res);
		CHECK_TRUE(success);

		StringBuffer<> newStr;
		auto& root = paths.GetRoot(rootPos - RootPaths::RootStartByte);
		newStr.Append(root.c_str()).Append(temp);
		CHECK_TRUE(newStr.Equals(str));

		#if PLATFORM_WINDOWS
		const tchar str2[] = TC("file://e:/temp/");
		bool foundPath = false;
		res = paths.NormalizeString(logger, str2, sizeof_array(str2), [&](const tchar* str, u64 strLenIncTerm, u32 rp)
			{
				if (rp != ~0)
				{
					if (str[0] != RootPaths::RootStartByte + PathsPerRoot) // Add one for windows because second entry is backslash
						success = false;
					foundPath = true;
				}
				else
				{
					if (!((strLenIncTerm == 1 && (!str[0] || str[0] == '/')) || (strLenIncTerm == 6 && Equals(str, TC("file:/"), 6, false))))
						success = false;
				}

			}, false, TC(""));
		CHECK_TRUE(res && foundPath && success);
		#endif

		return true;
	}


#if 0//PLATFORM_MAC
	Set<TString> g_visited;
	void LogImports(const tchar* import, bool isKnown)
	{
		if (!g_visited.insert(import).second)
			return;
		LoggerWithWriter logger(g_consoleLogWriter);
		logger.Info(TC("IMPORT: %s"), import);
		StringBuffer<> path(TC("/Users/henrik.karlsson/p4/fn/Engine/Binaries/Mac"));
		path.EnsureEndsWithSlash().Append(import);
		StringBuffer<> error;
		FindImportsMac(path.data, LogImports, error);
	}
#endif

	UBA_TEST(Basic, BinDependencies)
	{

#if PLATFORM_WINDOWS
		StringBuffer<> path;
		GetDirectoryOfCurrentModule(logger, path);
		path.EnsureEndsWithSlash().Append(TCV("UbaTestApp.exe"));
		bool importKernel = false;
		StringBuffer<> error;
		BinaryInfo info;
		ParseBinary(path, {}, info, [&](const tchar* import, bool isKnown, const tchar* const* importLoaderPaths)
		{
			importKernel |= isKnown && Contains(import, TC("KERNEL32.dll"));
		}, error);
		CHECK_TRUEF(importKernel, TC("Failed to find Kernel32 as import"));
#elif PLATFORM_MAC
		//StringBuffer<> error;
		//FindImportsMac(TC("/Users/henrik.karlsson/p4/fn/Engine/Binaries/Mac/ShaderCompileWorker"), LogImports, error);
#endif
		return true;
	}

	UBA_TEST_IF(Basic, VolumeCache, IsWindows)
	{
		VolumeCache cache;
		CHECK_TRUE(cache.Init(logger));
		return true;
	}

	UBA_NOINLINE void TestFunctionForThread(Event& ev1, Event& traverseDone)
	{
		ev1.Set();
		traverseDone.IsSet();
	}

	UBA_TEST(Basic, Threads)
	{
		Event traverseDone(EventResetType_Manual);

		Event ev1(EventResetType_Manual);
		Thread t1([&]()
			{
				TestFunctionForThread(ev1, traverseDone);
				return true;
			});
		ev1.IsSet();

		Event ev2(EventResetType_Manual);
		Thread t2([&]()
			{
				TestFunctionForThread(ev2, traverseDone);
				return true;
			});
		ev2.IsSet();

		TraverseAllThreads([&](u32 tid, void** callstack, u32 callstackCount, const tchar* desc)
			{
				static u8* writerMem = new u8[4096];
				BinaryWriter writer(writerMem, 0, 4096);
				WriteCallstackInfo(writer, callstack, callstackCount);
				BinaryReader reader(writerMem, 0, writer.GetPosition());
				StringBuffer<16*1024> sb;
				tchar executable[512] = TC("UbaTest");//{ 0 };
				
				StringView searchPaths[3];
				StringBuffer<512> currentModuleDir;
				LoggerWithWriter logger(g_nullLogWriter);
				GetDirectoryOfCurrentModule(logger, currentModuleDir);
				StringBuffer<512> alternativePath;
				u32 searchPathIndex = 0;
				if (GetAlternativeUbaPath(logger, alternativePath, currentModuleDir, IsWindows && IsArmBinary))
					searchPaths[searchPathIndex++] = alternativePath;
				searchPaths[searchPathIndex] = currentModuleDir;

				ParseCallstackInfo(sb, reader, executable, searchPaths);
				LoggerWithWriter(g_consoleLogWriter, TC("")).Info(TC("THREAD %u%s"), tid, sb.data);
			},
			[&](const StringView& error)
			{
				logger.Info(error.data);
			});


		traverseDone.Set();
		return true;
	}

	UBA_TEST(Basic, ImageDigestStream)
	{
		#if 0//PLATFORM_WINDOWS

		const tchar* fileName = TC("e:\\dev\\fn\\Engine\\Binaries\\ThirdParty\\ShaderConductor\\Win64\\dxcompiler.dll");
		HANDLE file = ::CreateFileW(fileName, GENERIC_READ, FILE_SHARE_READ, nullptr, OPEN_EXISTING, 0, nullptr);

		auto digestHash = [](DIGEST_HANDLE Handle, PBYTE Data, DWORD Length) -> BOOL
		{
			printf("SEG: %u\n", Length);
			return true;
		};

		printf("Native:\n");
		if (!ImageGetDigestStream(file, CERT_PE_IMAGE_DIGEST_ALL_IMPORT_INFO | CERT_PE_IMAGE_DIGEST_RESOURCES, digestHash, nullptr))
			return logger.Error(TC("ImageGetDigestStream failed"))

		printf("Detoured:\n");
		if (!Detoured_ImageGetDigestStream(file, CERT_PE_IMAGE_DIGEST_ALL_IMPORT_INFO | CERT_PE_IMAGE_DIGEST_RESOURCES, digestHash, nullptr))
			return logger.Error(TC("Detoured_ImageGetDigestStream failed"))

		#endif
		return true;
	}
}
