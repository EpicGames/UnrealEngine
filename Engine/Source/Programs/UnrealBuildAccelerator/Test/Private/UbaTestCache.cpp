// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaTest.h"
#include "UbaCacheClient.h"
#include "UbaCacheBucket.h"
#include "UbaCacheServer.h"
#include "UbaCompactTables.h"
#include "UbaFileAccessor.h"
#include "UbaHashMap.h"
#include "UbaNetworkBackendTcp.h"
#include "UbaNetworkClient.h"
#include "UbaNetworkServer.h"
#include "UbaProcessStartInfo.h"
#include "UbaRootPaths.h"
#include "UbaSessionServer.h"
#include "UbaStorageServer.h"

UBA_DEFINE_TEST_GROUP(Cache, 80)

namespace uba
{
	template<typename Key, typename Value, typename Hash = std::hash<Key>, typename EqualTo = std::equal_to<Key>>
	using GrowingNoLockUnorderedMap = UnorderedMap<Key, Value, Hash, EqualTo, GrowingAllocatorNoLock>;

	struct TestRecord
	{
		Set<u32> offsets;
		u32 expectedRangeCount;
		u32 expectedExtraCount;
	};

	bool CheckCacheEntry(Logger& logger, CacheEntries& entries, CacheEntry& entry, const TestRecord* rec, bool checkOffsets, bool checkExpectedCounts)
	{
		Set<u32> sharedOffsets;
		{
			BinaryReader sharedReader(entries.sharedInputCasKeyOffsets);
			while (sharedReader.GetLeft())
				CHECK_TRUE(sharedOffsets.insert(u32(sharedReader.Read7BitEncoded())).second);
		}

		Set<u32> entryOffsets;

		u32 rangeCount = 0;
		BinaryReader rangeReader(entry.sharedInputCasKeyOffsetRanges);
		while (rangeReader.GetLeft())
		{
			++rangeCount;
			u64 begin = rangeReader.Read7BitEncoded();
			u64 end = rangeReader.Read7BitEncoded();
			BinaryReader sharedReader(entries.sharedInputCasKeyOffsets.data(), begin, end);
			while (sharedReader.GetLeft())
			{
				u64 offset;
				CHECK_TRUE(sharedReader.TryRead7BitEncoded(offset));
				CHECK_TRUE(entryOffsets.insert(u32(offset)).second);
			}
		}

		u32 extraCount = 0;
		BinaryReader extraReader(entry.extraInputCasKeyOffsets);
		while (extraReader.GetLeft())
		{
			++extraCount;
			u32 offset = u32(extraReader.Read7BitEncoded());
			CHECK_TRUE(sharedOffsets.find(offset) == sharedOffsets.end());
			CHECK_TRUE(entryOffsets.insert(offset).second);
		}

		if (rec)
		{
			if (checkExpectedCounts)
			{
				CHECK_TRUE(rangeCount == rec->expectedRangeCount);
				CHECK_TRUE(extraCount == rec->expectedExtraCount);
			}

			if (checkOffsets)
			{
				CHECK_TRUE(rec->offsets == entryOffsets);
			}
			else
			{
				CHECK_TRUE(rec->offsets.size() == entryOffsets.size());
			}
		}
		return true;
	}

	void UpdateCacheEntries(LoggerWithWriter& logger, CacheEntries& entries, u32 multiplier)
	{
		HashMap2<u32, u32> oldToNewCasKeyOffset;
		MemoryBlock memory(TC("UpdateCacheEntries"), 64*1024);
		oldToNewCasKeyOffset.Init(memory, 100);
		if (multiplier)
			for (u32 i=0; i!=200; ++i)
				oldToNewCasKeyOffset.Insert(i) = i * multiplier;
		Vector<u32> temp;
		Vector<u8> temp2;
		Vector<u8> temp3;
		entries.UpdateEntries(logger, oldToNewCasKeyOffset, temp, temp2, temp3);
	}

	UBA_TEST(Cache, Entry)
	{
		CacheEntries entries;

		using Record = TestRecord;
		auto CheckEntry = [&](CacheEntry& entry, const Record* rec, bool checkOffsets, bool checkExpectedCounts) { return CheckCacheEntry(logger, entries, entry, rec, checkOffsets, checkExpectedCounts); };

		auto AddEntry = [&](const Record& rec, bool checkExpectedCount)
			{
				Vector<u8> inputOffsets;
				u64 bytes = 0;
				for (u32 i : rec.offsets)
					bytes += Get7BitEncodedCount(i);
				inputOffsets.resize(bytes);
				BinaryWriter writer(inputOffsets.data(), 0, inputOffsets.size());
				for (u32 i : rec.offsets)
					writer.Write7BitEncoded(i);

				CacheEntry entry;
				entries.BuildInputs(entry, rec.offsets);
				entries.entries.push_back(entry);
				return CheckEntry(entry, &rec, true, checkExpectedCount);
			};

		auto UpdateEntries = [&](u32 multiplier) { UpdateCacheEntries(logger, entries, multiplier); };

		auto Clear = [&]()
			{
				entries.entries.clear();
				entries.sharedInputCasKeyOffsets.clear();
				entries.sharedLogLines.clear();
				entries.idCounter = 0;
				entries.primaryId = ~0u; // Id of entry that shared offsets was made from
				entries.inputsThatAreOutputs.clear();
			};

		#if 0
		StorageImpl* storageImpl  = nullptr;
		CompactCasKeyTable* cckt = nullptr;
		#if PLATFORM_WINDOWS
		FileAccessor fa(logger, TC("e:\\temp\\CacheEntry.bin"));
		#else
		FileAccessor fa(logger, TC("/mnt/e/temp/CacheEntry.bin"));
		#endif
		fa.OpenMemoryRead();
		BinaryReader fileReader(fa.GetData(), 0, fa.GetSize());
		entries.ReadFromDisk(logger, fileReader, 9, *storageImpl, *cckt);
		fa.Close();
		//UpdateEntries(0);
		for (auto& entry : entries.entries)
			CHECK_TRUE(CheckEntry(entry, nullptr, false, false));
		return true;
		#endif

		Vector<Record> records0 = 
		{
			{ { 1 }, 1, 0 },
			{ { 0 }, 0, 1 },
		};

		Vector<Record> records1 = 
		{
			{ { 1, 4 }, 1, 0 },
			{ { 1, 2, 3, 4 }, 1, 2 },
			{ { 1, 2, 3, 4, 5 }, 1, 3 },
			{ { 1, 2, 3, 4, 5, 6 }, 1, 4 },
		};

		Vector<Record> records2 = 
		{
			{ { 1, 4, 6 }, 1, 0 },
			{ { 0, 4, 6 }, 1, 1 },
			{ { 2, 4, 6 }, 1, 1 },
			{ { 1, 4, 5 }, 1, 1 },
			{ { 1, 4, 7 }, 1, 1 },
			{ { 1, 3, 6 }, 2, 1 },
			{ { 1, 5, 6 }, 2, 1 },
			{ { 1, 5, 7 }, 1, 2 },
			{ { 1, 3, 5, 7 }, 1, 3 },
			{ { 1, 3, 5, 7, 8 }, 1, 4 },
			{ { 1, 4, 6, 7 }, 1, 1 },
			{ { 1, 4, 6, 7, 8 }, 1, 2 },
			{ { 0, 1, 4, 6 }, 1, 1 },
			{ { 1, 4, 5, 6 }, 1, 1 },
			{ { 0, 1 }, 1, 1 },
			{ { 0, 2 }, 0, 2 },
			{ { 0, 4 }, 1, 1 },
			{ { 0, 5 }, 0, 2 },
			{ { 0, 6 }, 1, 1 },
			{ { 0, 7 }, 0, 2 },
			{ { 1, 2 }, 1, 1 },
			{ { 1, 4 }, 1, 0 },
			{ { 1, 5 }, 1, 1 },
			{ { 1, 6 }, 2, 0 },
			{ { 1, 7 }, 1, 1 },
			{ { 2, 4 }, 1, 1 },
			{ { 2, 3 }, 0, 2 },
			{ { 7, 8 }, 0, 2 },
			{ { 0 }, 0, 1 },
			{ { 1 }, 1, 0 },
			{ { 2 }, 0, 1 },
			{ { 3 }, 0, 1 },
			{ { 4 }, 1, 0 },
			{ { 5 }, 0, 1 },
			{ { 6 }, 1, 0 },
			{ { 7 }, 0, 1 },
		};

		Vector<Record> records3 = 
		{
			{ { 2, 4, 6, 10, 14, 18 }, 1, 0 },
			{ { 2, 4, 6, 10, 14, 18 }, 1, 0 },
			{ { 2, 4, 5, 10, 15, 18 }, 3, 2 },
			{ { 2, 4, 6, 10, 19, 20 }, 1, 2 },
			{ { 0, 1 }, 0, 2 },
			{ { 4, 10, 18 }, 3, 0 },
			{ { 7, 8 }, 0, 2 },
			{ { 6, 7, 8 }, 1, 2 },
			{ { 5, 6, 7, 8 }, 1, 3 },
			{ { 2, 4, 6, 7, 8, 10, 14 }, 1, 2 },
			{ { 2, 4, 6, 7, 8, 10, 14, 18 }, 1, 2 },
			{ { 7, 8, 10 }, 1, 2 },
			{ { 7, 8, 10, 14, 18 }, 1, 2 },
			{ { 4, 7, 14, 18 }, 2, 1 },
		};

		Vector<Record> records4 = 
		{
			{ { 1, 4, 7 }, 1, 0 },
			{ { 1, 5, 6 }, 1, 2 },
		};

		Vector<Record> records5 = 
		{
			{ { 1, 3, 6 }, 1, 0 },
			{ { 1, 3, 5, 7 }, 1, 2 },
		};

		Vector<Record>* recordGroups[] =
		{
			&records0,
			&records1,
			&records2,
			&records3,
			&records4,
			&records5,
		};

		for (auto recordsPtr : recordGroups)
		{
			auto& records = *recordsPtr;
			Clear();
			for (auto& rec : records)
				CHECK_TRUE(AddEntry(rec, true));
			for (u32 i=0; i!=4; ++i)
			{
				UpdateEntries(i);
				u32 index = 0;
				for (auto& entry : entries.entries)
					CHECK_TRUE(CheckEntry(entry, &records[index++], i < 2, true));
			}
		}

		for (auto recordsPtr : recordGroups)
		{
			auto& records = *recordsPtr;

			for (u32 i=0;i!=records.size(); ++i)
			{
				Clear();
				CHECK_TRUE(AddEntry(records[i], false));

				for (u32 j=0;j!=records.size(); ++j)
				{
					u32 index = (i + j + 1) % records.size();
					CHECK_TRUE(AddEntry(records[index], false));
				}
			}

		}

		return true;
	}

	void GetTestAppPath(LoggerWithWriter& logger, StringBufferBase& out);
	void InvalidateCachedInfo(StorageImpl& storage, StringBufferBase& fileName)
	{
		StringBuffer<> fixedFilePath;
		storage.InvalidateCachedFileInfo(GetKeyAndFixedName(fixedFilePath, fileName.data));
	}

	UBA_TEST(Cache, CompactPath)
	{
		for (u32 useCommon=0; useCommon<=1; ++useCommon)
		{
			const tchar* pathsStr[] =
			{
				TC("Foo/Bar/Meh.h"),
				TC("Foo/Bar/Meh.cpp"),
				TC("Foo/Bar/Moo.h"),
				TC("Foo/Boo/Rud.h"),
				TC("Foo/Boo/Rud.cpp"),
				TC(")/Boo/Rud.cpp"),
				TC("%/cl.cpp"),
				TC(")/Boo/Rud.inl"),
			};
			constexpr u32 PathCount = sizeof_array(pathsStr);

			StringBuffer<128> paths[PathCount];
			for (u32 i=0;i!=PathCount; ++i)
				paths[i].Append(pathsStr[i]).FixPathSeparators();

			CompactPathTable table(CaseInsensitiveFs, 0, 0, 3);
			table.InitMem();
			if (useCommon)
				table.AddCommonStringSegments();

			u32 offsets[PathCount];
			for (u32 i=0;i!=PathCount; ++i)
				offsets[i] = table.AddNoLock(paths[i].data, paths[i].count);

			CompactPathTable table2(CaseInsensitiveFs, 0, 0, 3);
			table2.InitMem();
			if (useCommon)
				table2.AddCommonStringSegments();

			CompactPathTable::AddContext context{table};
			for (u32 i=0;i!=PathCount; ++i)
			{
				u32 offsets2 = table2.AddNoLock(context, offsets[i]);

				StringBuffer<> temp;
				CHECK_TRUEF(table2.GetString(temp, offsets2), TC("Error getting offset %u from table2"), offsets2);
				CHECK_TRUEF(temp.Equals(paths[i].data), TC("Error adding %s to table2. Found %s"), paths[i].data, temp.data);
			}
		}

		for (u32 version=0; version<=CacheBucketVersion; ++version)
		{
			CompactPathTable table(CaseInsensitiveFs, 0, 0, version);

			StringBuffer<> str;
			u32 offset;

			auto testStr = [&](const StringView& str)
				{
					StringBuffer<> str2;
					offset = table.Add(str.data, str.count);
					table.GetString(str2, offset);
					return str.Equals(str2.data);
				};

			CHECK_TRUE(testStr(str.Append("foo")));
			CHECK_TRUE(testStr(str.Clear().Append("foo").EnsureEndsWithSlash().Append("bar.h")));
			CHECK_TRUE(testStr(str.Clear().Append(PathSeparator).Append("foo").Append(PathSeparator).Append("bar.h")));

			CompactPathTable table2(CaseInsensitiveFs, 0, 0, version);
			BinaryReader reader(table.GetMemory(), 0, table.GetSize());
			table2.ReadMem(reader, true);
			u32 offset2 = table2.Add(str.data, str.count);
			CHECK_TRUE(offset == offset2);
		}
		return true;
	}

	UBA_TEST(Cache, CompactCasKey)
	{
		CompactCasKeyTable table;
		u32 offset0 = table.AddNoLock(CasKeyZero, 0);
		u32 offset1 = table.AddNoLock(CasKeyZero, 1);
		u32 offset2 = table.AddNoLock(CasKeyZero, 2);

		CompactCasKeyTable table2;
		BinaryReader reader(table.GetMemory(), 0, table.GetSize());
		table2.ReadMem(reader, true);
		CHECK_TRUE(table2.AddNoLock(CasKeyZero, 0) == offset0);
		CHECK_TRUE(table2.AddNoLock(CasKeyZero, 1) == offset1);
		CHECK_TRUE(table2.AddNoLock(CasKeyZero, 2) == offset2);

		for (u32 i=0;i!=32;++i)
		{
			CompactCasKeyTable table3(table.GetKeyCount());
			for (u32 j=0;j!=i;++j)
				table3.AddNoLock(CasKeyZero, j);
		}
		return true;
	}

	UBA_TEST(Cache, HashTable)
	{
		{
			MemoryBlock memoryBlock(TC("TestHashTable"), 1024*1024);
			HashMap<u32, u32> casMap;
			casMap.Init(memoryBlock, 3);
			CHECK_TRUE(!casMap.Find(1));
			casMap.Insert(1) = 2;
			CHECK_TRUE(*casMap.Find(1) == 2);
			casMap.Insert(1) = 3;
			CHECK_TRUE(*casMap.Find(1) == 3);

			HashMap<u32, u32, true> casMap2;
			casMap2.Init(4);
			for (u32 i=0;i!=4; ++i)
			{
				casMap2.Insert(i) = i;
				CHECK_TRUE(*casMap2.Find(i) == i);
			}
			for (u32 i=0;i!=4; ++i)
				CHECK_TRUE(*casMap2.Find(i) == i);

			casMap2.Insert(4) = 4;
			for (u32 i=0;i!=5; ++i)
				CHECK_TRUE(*casMap2.Find(i) == i);

			for (u32 i=5;i!=1000; ++i)
			{
				CHECK_TRUE(!casMap2.Find(i));
				casMap2.Insert(i) = i;
				CHECK_TRUE(*casMap2.Find(i) == i);
			}
		}

#if 0
		struct CasFileInfo { CasFileInfo(u32 s = 0) : size(s) {} u32 size; bool isUsed; }; // These are compressed cas, should never be over 4gb
		constexpr u64 memoryReserveSize = 192*1024*1024;
		struct ProfileScope
		{
			ProfileScope() : startTime(GetTime()) {}
			~ProfileScope() { u64 duration = GetTime() - startTime; LoggerWithWriter(g_consoleLogWriter, TC("")).Info(TC("Time: %s"), TimeToText(duration).str); }
			u64 startTime;
		};

		u32 totalCasCount = 1'800'000;

		{
			ProfileScope _;
			MemoryBlock memoryBlock;
			if (!memoryBlock.Init(memoryReserveSize, nullptr, true))
				memoryBlock.Init(memoryReserveSize);

			HashMap<CasKey, CasFileInfo> casMap;
			casMap.Init(memoryBlock, totalCasCount);

			CasKey key;

			for (u32 i=0; i!=totalCasCount; ++i)
			{
				key.a = i;
				casMap.Insert(key);
			}

			for (u32 i=0; i!=totalCasCount; ++i)
			{
				key.a = i;
				casMap.Find(key);
			}
		}

		{
			ProfileScope _;
			MemoryBlock memoryBlock;
			if (!memoryBlock.Init(memoryReserveSize, nullptr, true))
				memoryBlock.Init(memoryReserveSize);

			GrowingNoLockUnorderedMap<CasKey, CasFileInfo> casMap(&memoryBlock);
			casMap.reserve(totalCasCount);

			CasKey key;

			for (u32 i=0; i!=totalCasCount; ++i)
			{
				key.a = i;
				casMap.try_emplace(key);
			}

			for (u32 i=0; i!=totalCasCount; ++i)
			{
				key.a = i;
				auto it = casMap.find(key);
			}
		}
#endif
		return true;
	}

	UBA_TEST(Cache, ClientAndServer)
	{
		LogWriter& logWriter = logger.m_writer;
		NetworkBackendTcp tcpBackend(logWriter);

		StringBuffer<MaxPath> rootDir;
		rootDir.Append(testRootDir).Append(TCV("Uba"));

		StringBuffer<MaxPath> inputFile;
		StringBuffer<MaxPath> outputFile;

		{
			bool ctorSuccess = true;
			NetworkServer server(ctorSuccess, { logWriter });

			CHECK_TRUE(DeleteAllFiles(logger, rootDir.data));

			StorageServerCreateInfo storageServerInfo(server, rootDir.data, logWriter);
			storageServerInfo.casCapacityBytes = 1024ull * 1024 * 1024;
			auto& storageServer = *new StorageServer(storageServerInfo);
			auto ssg = MakeGuard([&]() { delete &storageServer; });

			CacheServerCreateInfo csci(storageServer, rootDir.data, logWriter);
			CacheServer cacheServer(csci);
			CHECK_TRUE(cacheServer.Load(false));

			SessionServerCreateInfo sessionInfo(storageServer, server, logWriter);
			sessionInfo.rootDir = rootDir.data;
			auto& session = *new SessionServer(sessionInfo);
			auto sg = MakeGuard([&]() { delete &session; });

			u16 port = 1356;
			CHECK_TRUE(server.StartListen(tcpBackend, port));
			auto disconnectServer = MakeGuard([&]() { server.DisconnectClients(); });

			StringBuffer<MaxPath> workingDir;
			workingDir.Append(testRootDir).Append(TCV("WorkingDir"));
			CHECK_TRUE(DeleteAllFiles(logger, workingDir.data));
			CHECK_TRUE(storageServer.CreateDirectory(workingDir.data));
			CHECK_TRUE(DeleteAllFiles(logger, workingDir.data, false));

			StringBuffer<> testApp;
			GetTestAppPath(logger, testApp);

			StringView content(TCV("Foo"));

			CHECK_TRUE(CreateTestFile(inputFile, logger, workingDir, TCV("Input.txt"), content));

			Vector<tchar> data;
			data.resize(sizeof(Guid)*100000);
			for (u32 i=0;i!=100000; ++i)
				CreateGuid(((Guid*)data.data())[i]);
			content = StringView(data.data(), u32(data.size()));

			CHECK_TRUE(CreateTestFile(outputFile, logger, workingDir, TCV("Output.exe"), content, DefaultAttributes(true)));

			StackBinaryWriter<256> inputs;
			inputs.WriteString(inputFile);

			StackBinaryWriter<256> outputs;
			outputs.WriteString(outputFile);

			StackBinaryWriter<256> logLines;
			logLines.WriteString(TC("Hello"));
			logLines.WriteByte(1);

			ProcessStartInfo psi;
			psi.application = testApp.data;

			{
				NetworkClient client(ctorSuccess, { logWriter });
				CacheClientCreateInfo ccci(logWriter, storageServer, client, session);
				ccci.useRoots = false;
				ccci.useMappingForCreatedCas = false; // Needed since cacheclient and server shares same cas and we don't want cas to be evicted
				CacheClient cacheClient(ccci);

				CHECK_TRUE(client.Connect(tcpBackend, TC("127.0.0.1"), port));
				auto disconnectClient = MakeGuard([&]() { client.Disconnect(); });

				//cacheClient.RegisterPathHash(workingDir.data, CasKey(1, 2, 3));

				{
					CacheResult result;
					CHECK_TRUE(!cacheClient.FetchFromCache(result, RootPaths(), 0, psi));
					CHECK_TRUE(!result.hit);
					CHECK_TRUE(cacheClient.WriteToCache(RootPaths(), 0, psi, inputs.GetData(), inputs.GetPosition(), outputs.GetData(), outputs.GetPosition(), logLines.GetData(), logLines.GetPosition()));
					CHECK_TRUE(DeleteFileW(outputFile.data));
					CHECK_TRUE(!FileExists(logger, outputFile.data));
					CHECK_TRUE(cacheClient.FetchFromCache(result, RootPaths(), 0, psi));
					CHECK_TRUE(FileExists(logger, outputFile.data));
					CHECK_TRUE(result.logLines.size() == 1);
					CHECK_TRUE(result.logLines[0].text == TC("Hello"));
				}

				{
					CHECK_TRUE(DeleteFileW(inputFile.data));
					CHECK_TRUE(CreateTestFile(inputFile, logger, workingDir, TCV("Input.txt"), TCV("Bar")));
					InvalidateCachedInfo(storageServer, inputFile);

					CacheResult result;
					CHECK_TRUE(!cacheClient.FetchFromCache(result, RootPaths(), 0, psi));
					CHECK_TRUE(!result.hit);

					CHECK_TRUE(cacheClient.WriteToCache(RootPaths(), 0, psi, inputs.GetData(), inputs.GetPosition(), outputs.GetData(), outputs.GetPosition(), logLines.GetData(), logLines.GetPosition()));

					CHECK_TRUE(DeleteFileW(outputFile.data));
					CHECK_TRUE(!FileExists(logger, outputFile.data));
					CHECK_TRUE(cacheClient.FetchFromCache(result, RootPaths(), 0, psi));
					CHECK_TRUE(FileExists(logger, outputFile.data));

					#if !PLATFORM_WINDOWS
					CHECK_TRUE(IsExecutable(GetFileAttributesW(outputFile.data)));
					#endif

					CHECK_TRUE(result.logLines.size() == 1);
					CHECK_TRUE(result.logLines[0].text == TC("Hello"));
				}
			}

			CHECK_TRUE(cacheServer.RunMaintenance(true, true, []() { return false; }));

			{
				NetworkClient client(ctorSuccess, { logWriter });
				CacheClientCreateInfo ccci(logWriter, storageServer, client, session);
				ccci.useRoots = false;
				CacheClient cacheClient(ccci);

				CHECK_TRUE(client.Connect(tcpBackend, TC("127.0.0.1"), port));
				auto disconnectClient = MakeGuard([&]() { client.Disconnect(); });

				{
					CacheResult result;
					CHECK_TRUE(cacheClient.FetchFromCache(result, RootPaths(), 0, psi));
					CHECK_TRUE(result.logLines.size() == 1);
					CHECK_TRUE(result.logLines[0].text == TC("Hello"));
				}
			}

			CHECK_TRUE(cacheServer.Save());
			CHECK_TRUE(storageServer.SaveCasTable(true));
		}

		{
			bool ctorSuccess = true;
			NetworkServer server(ctorSuccess, { logWriter });
			StorageServerCreateInfo storageServerInfo(server, rootDir.data, logWriter);
			storageServerInfo.casCapacityBytes = 1024ull * 1024 * 1024;
			StorageServer storageServer(storageServerInfo);
			storageServer.LoadCasTable();

			StorageImpl::FileEntry* inputFileEntry;
			StorageImpl::FileEntry* outputFileEntry;
			CHECK_TRUE(storageServer.GetFileEntry(inputFileEntry, inputFile.data));
			CHECK_TRUE(storageServer.GetFileEntry(outputFileEntry, outputFile.data));

			#if UBA_TRACK_IS_EXECUTABLE
			CHECK_TRUE(!inputFileEntry->isExecutable);
			CHECK_TRUE(outputFileEntry->isExecutable);
			#endif

			CacheServerCreateInfo csci(storageServer, rootDir.data, logWriter);
			CacheServer cacheServer(csci);
			CHECK_TRUE(cacheServer.Load(false));
		}
		return true;
	}

#if 0
	UBA_TEST(Cache, Load)
	{
		#if PLATFORM_WINDOWS
		const tchar* fileName = TC("e:\\temp\\uploads\\114629205754");
		#else
		const tchar* fileName = TC("/mnt/e/temp/uploads/114629205754");
		#endif

		FileAccessor bucketFile(logger, fileName);
		CHECK_TRUE(bucketFile.OpenMemoryRead());
		BinaryReader reader(bucketFile.GetData(), 0, bucketFile.GetSize());
		u32 bucketVersion = reader.ReadU32();
		CacheBucket::LoadStats stats;
		StorageServer* storage = nullptr;

		CacheBucket bucket(0, 0);
		CHECK_TRUE(bucket.Load(logger, reader, bucketVersion, stats, *storage));
		bucketFile.Close();

		LoggerWithWriter lg(g_consoleLogWriter);
		u32 entriesIndex = 0;
		for (auto& kv : bucket.m_cacheEntryLookup)
		{
			lg.Info(TC("Entries %u"), entriesIndex++);
			auto& entries = kv.second;

			if (entriesIndex == 358)
			{
				FileAccessor fa(logger, TC("e:\\temp\\CacheEntry.bin"));
				fa.CreateMemoryWrite(false, DefaultAttributes(), entries.GetTotalSize(CacheNetworkVersion, true));
				BinaryWriter w(fa.GetData(), 0, fa.GetSize());
				entries.Write(w, CacheNetworkVersion, true);
				fa.Close();
			}

			u32 entryIndex = 0;
			for (auto& entry : entries.entries)
				CHECK_TRUE(CheckCacheEntry(entries, entry, nullptr, false, false));
		}
		return true;
	}
#endif
}