// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaConfig.h"
#include "UbaLogger.h"
#include "UbaPlatform.h"
#include "UbaTest.h"

UBA_DEFINE_TEST_GROUP(Config, 30)

namespace uba
{
	UBA_TEST(Config, Load)
	{
		static const char* configText =
			"RootDir = \"e:\\foo\"\r\n"
			"[CacheClient]\r\n"
			"UseDirectoryPreparsing = true\r\n"
			"# Comment = true\r\n"
			"";

		Config config;
		CHECK_TRUE(config.LoadFromText(logger, configText, strlen(configText)));

		const ConfigTable* tablePtr = config.GetTable(TC("CacheClient"));
		CHECK_TRUE(tablePtr);
		const ConfigTable& table = *tablePtr;
		bool test = false;
		CHECK_TRUE(table.GetValueAsBool(test, TC("UseDirectoryPreparsing")));
		CHECK_TRUE(test == true);
		const tchar* str = nullptr;
		CHECK_TRUE(table.GetValueAsString(str, TC("RootDir")));
		CHECK_TRUE(TStrcmp(str, TC("e:\\foo")) == 0);
		CHECK_TRUE(!table.GetValueAsBool(test, TC("Comment")));
		return true;
	}

	UBA_TEST(Config, Save)
	{
		Vector<char> data;
		{
			Config config;
			ConfigTable& table = config.AddTable(TC("TestTable"));
			table.AddValue(TC("Foo"), 42);
			config.SaveToText(logger, data);
		}

		Config config;
		CHECK_TRUE(config.LoadFromText(logger, data.data(), data.size()));

		const ConfigTable* tablePtr = config.GetTable(TC("TestTable"));
		CHECK_TRUE(tablePtr);
		const ConfigTable& table = *tablePtr;
		int foo = 0;
		CHECK_TRUE(table.GetValueAsInt(foo, TC("Foo")));
		CHECK_TRUE(foo == 42);
		return true;
	}
}