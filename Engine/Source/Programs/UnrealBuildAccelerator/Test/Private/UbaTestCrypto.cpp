// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaCrypto.h"
#include "UbaLogger.h"
#include "UbaStringBuffer.h"
#include "UbaTest.h"

UBA_DEFINE_TEST_GROUP(Crypto, 20)

namespace uba
{
	UBA_TEST(Crypto, Basic)
	{
		u64 key128[] = { 0x1234567812345678llu, 0x1234567812345678llu };

		CryptoKey encryptKey = Crypto::CreateKey(logger, (const u8*)key128);

		for (u32 dataSize=1;dataSize!=135; ++dataSize)
		{
			CryptoKey decryptKey = encryptKey;

			u8* originalData = new u8[dataSize];
			for (u32 i=0;i!=dataSize; ++i)
				originalData[i] = u8(rand() % 256);

			u8* encryptedData = new u8[dataSize];
			memcpy(encryptedData, originalData, dataSize);

			for (u32 i=0; i!=3; ++i)
			{
				Guid iv;
				CHECK_TRUE(Crypto::Encrypt(logger, encryptKey, encryptedData, dataSize, iv));

				CHECK_TRUE(memcmp(encryptedData, originalData, dataSize) != 0);

				Guid iv2;
				CHECK_TRUE(Crypto::Decrypt(logger, decryptKey, encryptedData, dataSize, iv2));

				CHECK_TRUE(memcmp(encryptedData, originalData, dataSize) == 0);

				if (i == 1)
					decryptKey = Crypto::DuplicateKey(logger, encryptKey);
			}
			delete[] encryptedData;
			delete[] originalData;
			Crypto::DestroyKey(decryptKey);
		}

		Crypto::DestroyKey(encryptKey);
		return true;
	}
}