// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaDependencyCrawler.h"
#include "UbaDirectoryIterator.h"
#include "UbaLogger.h"
#include "UbaTest.h"

UBA_DEFINE_TEST_GROUP(DependencyCrawler, 90)

namespace uba
{
	UBA_TEST_DISABLED(DependencyCrawler, Basic, "hard coded paths in body")
	{
#if 0 // Commented out for now because of hard coded paths below

		WorkManagerImpl workManager(128);

		DependencyCrawler crawler(logger, workManager);

		auto CreateFileFunc = [&](TrackWorkScope& tracker, const StringView& fileName, const DependencyCrawler::AccessFileFunc& func)
			{
				FileAccessor fa(logger, fileName.data);
				CHECK_TRUE(fa.OpenMemoryRead());
				return func(fa.GetData(), fa.GetSize());
			};

		auto FileExistsFunc = [&](const StringView& fileName, u32& outAttr)
			{
				return FileExists(logger, fileName.data, nullptr, &outAttr);
			};

		auto TraverseFilesFunc = [&](const StringView& path, const DependencyCrawler::FileFunc& fileFunc)
			{
				TraverseDir(logger, path, [&](const DirectoryEntry& entry)
					{
						StringBuffer<> name;
						name.Append(entry.name, entry.nameLen);
						if (CaseInsensitiveFs)
							name.MakeLower();
						fileFunc(name, IsDirectory(entry.attributes));
					});

			};

		crawler.Init(FileExistsFunc, TraverseFilesFunc);

		const tchar app[] = TC("c:\\sdk\\AutoSDK\\HostWin64\\Win64\\LLVM\\18.1.8\\bin\\clang-cl.exe");

		//DependencyCrawlerType type = DependencyCrawlerType_MsvcCompiler;
		//const tchar rsp[] = TC("e:\\dev\\fn\\Engine\\Intermediate\\Build\\Win64\\x64\\BlankProgram\\Development\\Core\\Module.Core.1.cpp.obj.rsp");
		
		DependencyCrawlerType type = DependencyCrawlerType_MsvcCompiler;
		//const tchar rsp[] = TC("e:\\dev\\fn\\Engine\\Intermediate\\Build\\Linux\\x64\\BlankProgram\\Development\\Core\\Module.Core.1.cpp.o.rsp");
		//const tchar rsp[] = TC("e:\\temp\\response-22b46d.txt");
		//const tchar rsp[] = TC("e:\\temp\\MiMalloc.c.o.rsp");
		const tchar rsp[] = TC("e:\\dev\\fn\\Sandbox\\DevTools\\Clang\\Intermediate\\Build\\Win64\\x64\\ClangGame\\Debug\\InputCore\\Module.InputCore.cpp.obj.rsp");
		//const tchar rsp[] = TC("e:\\dev\\fn\\engine\\plugins\\experimental\\geometrycollectionplugin\\intermediate\\build\\win64\\x64\\unrealeditor\\development\\geometrycollectionnodes\\module.geometrycollectionnodes.3.cpp.obj.rsp");
		//const tchar rsp[] = TC("e:\\dev\\fn\\Engine\\Plugins\\Runtime\\MeshModelingToolset\\Intermediate\\Build\\Win64\\x64\\UnrealEditor\\Development\\ModelingComponents\\Module.ModelingComponents.3.cpp.obj.rsp");
		//const tchar rsp[] = TC("e:\\dev\\fn\\Engine\\Intermediate\\Build\\Win64\\x64\\UnrealEditor\\Development\\AnimGraphRuntime\\Module.AnimGraphRuntime.3.cpp.obj.rsp");

		crawler.Add(rsp, TC("e:\\dev\\fn\\Engine\\Source"), CreateFileFunc, [](StringBufferBase&)
			{
				return true;
			}, app, type, 10);

		workManager.FlushWork();
#endif
		return true;
	}
}