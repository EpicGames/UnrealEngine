// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaTest.h"
#include "UbaDirectoryTableHolder.h"
#include "UbaFileMapping.h"

UBA_DEFINE_TEST_GROUP(DirectoryTable, 50)

namespace uba
{
	UBA_TEST(DirectoryTable, Basic)
	{
		DirectoryTableHolder holder(logger.m_writer, TC(""), false);
		holder.SetTreatTempDirAsEmpty();

		CHECK_TRUE(holder.RefreshDirectory(testRootDir.data, true));

		MemoryBlock dirMem(TC("Mem"));
		DirectoryTable dir(dirMem);
		dir.Init(holder.GetDirectoryTableMemory(), 0, holder.GetDirectoryTableSize(nullptr).main);
		
		CHECK_TRUE(dir.EntryExists(testRootDir, false) == DirectoryTable::Exists_Yes);
		CHECK_TRUE(dir.EntryExists(testRootDir, true) == DirectoryTable::Exists_Yes);

		StringBuffer<> fooFile;
		CHECK_TRUE(CreateTestFile(fooFile, logger, testRootDir, TCV("foo.txt"), TCV("foo")));
		CHECK_TRUE(holder.RegisterNewFile(fooFile.data));

		CHECK_TRUE(dir.EntryExists(fooFile, true) == DirectoryTable::Exists_No);
		dir.ParseDirectoryTable(holder.GetDirectoryTableSize(nullptr));
		CHECK_TRUE(dir.EntryExists(fooFile, true) == DirectoryTable::Exists_Yes);


		DirectoryTableOverlay overlay;
		holder.CreateOverlay(overlay);

		dir.InitOverlay(overlay.memory, overlay.size);

		StringBuffer<> barFile;
		CHECK_TRUE(CreateTestFile(barFile, logger, testRootDir, TCV("bar.txt"), TCV("bar")));
		CHECK_TRUE(holder.PopulateOverlayFile(overlay, barFile, [](FileInformation& out, StringKey, StringView) { out.size = sizeof(tchar)*3; out.attributes = DefaultAttributes(); return true; }));

		CHECK_TRUE(dir.EntryExists(barFile, true) == DirectoryTable::Exists_No);
		dir.ParseDirectoryTable(holder.GetDirectoryTableSize(&overlay));
		CHECK_TRUE(dir.EntryExists(barFile, true) == DirectoryTable::Exists_Yes);

		StringBuffer<> mehFile;
		CHECK_TRUE(CreateTestFile(mehFile, logger, testRootDir, TCV("meh.txt"), TCV("meh")));
		CHECK_TRUE(holder.RegisterNewFile(mehFile.data));

		CHECK_TRUE(dir.EntryExists(mehFile, true) == DirectoryTable::Exists_No);
		dir.ParseDirectoryTable(holder.GetDirectoryTableSize(&overlay));
		CHECK_TRUE(dir.EntryExists(mehFile, true) == DirectoryTable::Exists_Yes);

		u32 counter = 0;
		StringBuffer<> testRootDirForLookup(StringView(testRootDir.data, testRootDir.count - 1));
		if (CaseInsensitiveFs)
			testRootDirForLookup.MakeLower();

		auto findIt = dir.m_lookup.find(ToStringKey(testRootDirForLookup));
		CHECK_TRUE(findIt != dir.m_lookup.end());
		CHECK_TRUE(findIt->second.files.size() == 3);

		bool success = true;
		dir.TraverseFilesRecursiveNoLock(testRootDirForLookup, [&](const DirectoryTable::EntryInformation& info, StringView fileName, DirTableOffset fileOffset)
			{
				++counter;
				success = success && info.size == sizeof(tchar)*3;
				success = success && info.attributes != 0;
				success = success && fileName.GetFileName().count == 7;
			});
		CHECK_TRUE(success);
		CHECK_TRUE(counter == 3);

		// Remove file by writing entry with attributes=0
		{
			OverlayEditScope scope(overlay);
			CHECK_TRUE(holder.WriteOverlayEntry(scope, barFile, {}));
		}
		dir.ParseDirectoryTable(holder.GetDirectoryTableSize(&overlay));
		DirTableOffset entryOffset;
		CHECK_TRUE(dir.EntryExists(barFile, false, &entryOffset) == DirectoryTable::Exists_Yes);
		CHECK_TRUE(!dir.GetAttributes(entryOffset));

		return true;
	}

	static constexpr u64 TestSnapshotMappingSize = 16ull * 1024 * 1024;

	UBA_TEST(DirectoryTable, Snapshot)
	{
		DirectoryTableHolder holder(logger.m_writer, TC(""), false);
		holder.SetTreatTempDirAsEmpty();
		CHECK_TRUE(holder.RefreshDirectory(testRootDir.data, true));

		StringBuffer<> alphaFile, betaFile, gammaFile;
		CHECK_TRUE(CreateTestFile(alphaFile, logger, testRootDir, TCV("alpha.txt"), TCV("a")));
		CHECK_TRUE(holder.RegisterNewFile(alphaFile.data));
		CHECK_TRUE(CreateTestFile(betaFile,  logger, testRootDir, TCV("beta.txt"),  TCV("bb")));
		CHECK_TRUE(holder.RegisterNewFile(betaFile.data));
		CHECK_TRUE(CreateTestFile(gammaFile, logger, testRootDir, TCV("gamma.txt"), TCV("ccc")));
		CHECK_TRUE(holder.RegisterNewFile(gammaFile.data));
		holder.RegisterDeleteFile(betaFile.data);

		DirTableSize covered = holder.GetDirectoryTableSize(nullptr);
		DirTableSnapshot snap;
		CHECK_TRUE(CreateDirectoryTableSnapshot(snap, TestSnapshotMappingSize, nullptr, 0, holder.GetDirectoryTableMemory(), covered.main, nullptr, 0, TC("TestSnapshot"), logger));
		CHECK_TRUE(snap.handle.IsValid());
		CHECK_TRUE(snap.mappedMemory != nullptr);

		DirectoryTable& consumer = *(DirectoryTable*)snap.mappedMemory;

		CHECK_TRUE(consumer.EntryExists(alphaFile) == DirectoryTable::Exists_Yes);
		CHECK_TRUE(consumer.EntryExists(gammaFile) == DirectoryTable::Exists_Yes);
		CHECK_TRUE(consumer.EntryExists(betaFile)  == DirectoryTable::Exists_No);

		StringBuffer<> rootForLookup(StringView(testRootDir.data, testRootDir.count - 1));
		if (CaseInsensitiveFs)
			rootForLookup.MakeLower();
		auto findIt = consumer.m_lookup.find(ToStringKey(rootForLookup));
		CHECK_TRUE(findIt != consumer.m_lookup.end());
		CHECK_TRUE(findIt->second.files.size() == 2);

		FileMapping_Unmap(logger, snap.mappedMemory, TestSnapshotMappingSize, TC("TestSnapshot"), false);
		FileMapping_Close(logger, snap.handle, TC("TestSnapshot"));
		return true;
	}

	UBA_TEST(DirectoryTable, SnapshotTail)
	{
		DirectoryTableHolder holder(logger.m_writer, TC(""), false);
		holder.SetTreatTempDirAsEmpty();
		CHECK_TRUE(holder.RefreshDirectory(testRootDir.data, true));

		StringBuffer<> oneFile, twoFile;
		CHECK_TRUE(CreateTestFile(oneFile, logger, testRootDir, TCV("one.txt"), TCV("1")));
		CHECK_TRUE(holder.RegisterNewFile(oneFile.data));
		CHECK_TRUE(CreateTestFile(twoFile, logger, testRootDir, TCV("two.txt"), TCV("2")));
		CHECK_TRUE(holder.RegisterNewFile(twoFile.data));

		DirTableSize coveredAtSnapshot = holder.GetDirectoryTableSize(nullptr);

		DirTableSnapshot snap;
		CHECK_TRUE(CreateDirectoryTableSnapshot(snap, TestSnapshotMappingSize, nullptr, 0, holder.GetDirectoryTableMemory(), coveredAtSnapshot.main, nullptr, 0, TC("TestSnapshotTail"), logger));

		StringBuffer<> threeFile, fourFile;
		CHECK_TRUE(CreateTestFile(threeFile, logger, testRootDir, TCV("three.txt"), TCV("3")));
		CHECK_TRUE(holder.RegisterNewFile(threeFile.data));
		CHECK_TRUE(CreateTestFile(fourFile,  logger, testRootDir, TCV("four.txt"),  TCV("4")));
		CHECK_TRUE(holder.RegisterNewFile(fourFile.data));

		DirTableSize coveredNow = holder.GetDirectoryTableSize(nullptr);
		CHECK_TRUE(coveredNow.main > coveredAtSnapshot.main);

		DirectoryTable& consumer = *(DirectoryTable*)snap.mappedMemory;

		CHECK_TRUE(consumer.EntryExists(oneFile)   == DirectoryTable::Exists_Yes);
		CHECK_TRUE(consumer.EntryExists(twoFile)   == DirectoryTable::Exists_Yes);
		CHECK_TRUE(consumer.EntryExists(threeFile) == DirectoryTable::Exists_No);
		CHECK_TRUE(consumer.EntryExists(fourFile)  == DirectoryTable::Exists_No);

		consumer.ParseDirectoryTable(coveredNow);
		CHECK_TRUE(consumer.EntryExists(oneFile)   == DirectoryTable::Exists_Yes);
		CHECK_TRUE(consumer.EntryExists(twoFile)   == DirectoryTable::Exists_Yes);
		CHECK_TRUE(consumer.EntryExists(threeFile) == DirectoryTable::Exists_Yes);
		CHECK_TRUE(consumer.EntryExists(fourFile)  == DirectoryTable::Exists_Yes);

		FileMapping_Unmap(logger, snap.mappedMemory, TestSnapshotMappingSize, TC("TestSnapshotTail"), false);
		FileMapping_Close(logger, snap.handle, TC("TestSnapshotTail"));
		return true;
	}

	UBA_TEST(DirectoryTable, SnapshotIncremental)
	{
		DirectoryTableHolder holder(logger.m_writer, TC(""), false);
		holder.SetTreatTempDirAsEmpty();
		CHECK_TRUE(holder.RefreshDirectory(testRootDir.data, true));

		StringBuffer<> aFile, bFile;
		CHECK_TRUE(CreateTestFile(aFile, logger, testRootDir, TCV("a.txt"), TCV("a")));
		CHECK_TRUE(holder.RegisterNewFile(aFile.data));
		CHECK_TRUE(CreateTestFile(bFile, logger, testRootDir, TCV("b.txt"), TCV("b")));
		CHECK_TRUE(holder.RegisterNewFile(bFile.data));

		DirTableSize coveredAtSnap1 = holder.GetDirectoryTableSize(nullptr);
		DirTableSnapshot snap1;
		CHECK_TRUE(CreateDirectoryTableSnapshot(snap1, TestSnapshotMappingSize, nullptr, 0, holder.GetDirectoryTableMemory(), coveredAtSnap1.main, nullptr, 0, TC("TestSnapIncr_1"), logger));

		StringBuffer<> cFile, dFile;
		CHECK_TRUE(CreateTestFile(cFile, logger, testRootDir, TCV("c.txt"), TCV("c")));
		CHECK_TRUE(holder.RegisterNewFile(cFile.data));
		CHECK_TRUE(CreateTestFile(dFile, logger, testRootDir, TCV("d.txt"), TCV("d")));
		CHECK_TRUE(holder.RegisterNewFile(dFile.data));
		holder.RegisterDeleteFile(aFile.data);

		DirTableSize coveredAtSnap2 = holder.GetDirectoryTableSize(nullptr);

		DirTableSnapshot snap2;
		CHECK_TRUE(CreateDirectoryTableSnapshot(snap2, TestSnapshotMappingSize, snap1.mappedMemory, snap1.mappedSize, holder.GetDirectoryTableMemory(), coveredAtSnap2.main, nullptr, 0, TC("TestSnapIncr_2"), logger));

		DirectoryTable& consumer = *(DirectoryTable*)snap2.mappedMemory;

		CHECK_TRUE(consumer.EntryExists(aFile) == DirectoryTable::Exists_No);
		CHECK_TRUE(consumer.EntryExists(bFile) == DirectoryTable::Exists_Yes);
		CHECK_TRUE(consumer.EntryExists(cFile) == DirectoryTable::Exists_Yes);
		CHECK_TRUE(consumer.EntryExists(dFile) == DirectoryTable::Exists_Yes);

		StringBuffer<> rootForLookup(StringView(testRootDir.data, testRootDir.count - 1));
		if (CaseInsensitiveFs)
			rootForLookup.MakeLower();
		auto findIt = consumer.m_lookup.find(ToStringKey(rootForLookup));
		CHECK_TRUE(findIt != consumer.m_lookup.end());
		CHECK_TRUE(findIt->second.files.size() == 3);

		// snap1 must outlive snap2 during build (snap2 reads prev's m_lookup), but order at teardown is free.
		FileMapping_Unmap(logger, snap2.mappedMemory, snap2.mappedSize, TC("TestSnapIncr_2"), false);
		FileMapping_Close(logger, snap2.handle, TC("TestSnapIncr_2"));
		FileMapping_Unmap(logger, snap1.mappedMemory, snap1.mappedSize, TC("TestSnapIncr_1"), false);
		FileMapping_Close(logger, snap1.handle, TC("TestSnapIncr_1"));
		return true;
	}

	UBA_TEST(DirectoryTable, SnapshotSubdir)
	{
		DirectoryTableHolder holder(logger.m_writer, TC(""), false);
		holder.SetTreatTempDirAsEmpty();
		CHECK_TRUE(holder.RefreshDirectory(testRootDir.data, true));

		StringBuffer<MaxPath> subDirPath(testRootDir);
		subDirPath.Append(TCV("sub"));
		CHECK_TRUE(CreateDirectoryW(subDirPath.data));
		CHECK_TRUE(holder.RegisterNewDirectory(subDirPath.data));

		StringBuffer<MaxPath> subDirWithSlash(subDirPath);
		subDirWithSlash.EnsureEndsWithSlash();
		StringBuffer<> innerOne, innerTwo;
		CHECK_TRUE(CreateTestFile(innerOne, logger, subDirWithSlash, TCV("inner1.txt"), TCV("1")));
		CHECK_TRUE(holder.RegisterNewFile(innerOne.data));
		CHECK_TRUE(CreateTestFile(innerTwo, logger, subDirWithSlash, TCV("inner2.txt"), TCV("22")));
		CHECK_TRUE(holder.RegisterNewFile(innerTwo.data));

		StringBuffer<> rootFile;
		CHECK_TRUE(CreateTestFile(rootFile, logger, testRootDir, TCV("root.txt"), TCV("r")));
		CHECK_TRUE(holder.RegisterNewFile(rootFile.data));

		DirTableSize covered = holder.GetDirectoryTableSize(nullptr);
		DirTableSnapshot snap;
		CHECK_TRUE(CreateDirectoryTableSnapshot(snap, TestSnapshotMappingSize, nullptr, 0, holder.GetDirectoryTableMemory(), covered.main, nullptr, 0, TC("TestSnapshotSubdir"), logger));

		DirectoryTable& consumer = *(DirectoryTable*)snap.mappedMemory;

		CHECK_TRUE(consumer.EntryExists(rootFile) == DirectoryTable::Exists_Yes);
		CHECK_TRUE(consumer.EntryExists(innerOne) == DirectoryTable::Exists_Yes);
		CHECK_TRUE(consumer.EntryExists(innerTwo) == DirectoryTable::Exists_Yes);
		CHECK_TRUE(consumer.EntryExists(subDirPath, true) == DirectoryTable::Exists_Yes);

		StringBuffer<> rootForLookup(StringView(testRootDir.data, testRootDir.count - 1));
		StringBuffer<> subForLookup(subDirPath);
		if (CaseInsensitiveFs) { rootForLookup.MakeLower(); subForLookup.MakeLower(); }

		auto rootIt = consumer.m_lookup.find(ToStringKey(rootForLookup));
		CHECK_TRUE(rootIt != consumer.m_lookup.end());
		CHECK_TRUE(rootIt->second.files.size() == 2);
		CHECK_TRUE(rootIt->second.populatedOffset == rootIt->second.latestOffset);

		auto subIt = consumer.m_lookup.find(ToStringKey(subForLookup));
		CHECK_TRUE(subIt != consumer.m_lookup.end());
		CHECK_TRUE(subIt->second.files.size() == 2);
		CHECK_TRUE(subIt->second.populatedOffset == subIt->second.latestOffset);

		FileMapping_Unmap(logger, snap.mappedMemory, TestSnapshotMappingSize, TC("TestSnapshotSubdir"), false);
		FileMapping_Close(logger, snap.handle, TC("TestSnapshotSubdir"));
		return true;
	}

	UBA_TEST(DirectoryTable, SnapshotDeleteDirectory)
	{
		DirectoryTableHolder holder(logger.m_writer, TC(""), false);
		holder.SetTreatTempDirAsEmpty();
		CHECK_TRUE(holder.RefreshDirectory(testRootDir.data, true));

		StringBuffer<MaxPath> killDir(testRootDir); killDir.Append(TCV("toremove"));
		StringBuffer<MaxPath> keepDir(testRootDir); keepDir.Append(TCV("tokeep"));
		CHECK_TRUE(CreateDirectoryW(killDir.data));
		CHECK_TRUE(CreateDirectoryW(keepDir.data));
		CHECK_TRUE(holder.RegisterNewDirectory(killDir.data));
		CHECK_TRUE(holder.RegisterNewDirectory(keepDir.data));

		StringBuffer<MaxPath> killDirSlash(killDir); killDirSlash.EnsureEndsWithSlash();
		StringBuffer<MaxPath> keepDirSlash(keepDir); keepDirSlash.EnsureEndsWithSlash();
		StringBuffer<> killInner, keepInner;
		CHECK_TRUE(CreateTestFile(killInner, logger, killDirSlash, TCV("doomed.txt"), TCV("d")));
		CHECK_TRUE(holder.RegisterNewFile(killInner.data));
		CHECK_TRUE(CreateTestFile(keepInner, logger, keepDirSlash, TCV("alive.txt"), TCV("k")));
		CHECK_TRUE(holder.RegisterNewFile(keepInner.data));

		holder.RegisterDeleteFile(killDir.data);

		DirTableSize covered = holder.GetDirectoryTableSize(nullptr);
		DirTableSnapshot snap;
		CHECK_TRUE(CreateDirectoryTableSnapshot(snap, TestSnapshotMappingSize, nullptr, 0, holder.GetDirectoryTableMemory(), covered.main, nullptr, 0, TC("TestSnapshotDelDir"), logger));

		DirectoryTable& consumer = *(DirectoryTable*)snap.mappedMemory;

		StringBuffer<> rootForLookup(StringView(testRootDir.data, testRootDir.count - 1));
		if (CaseInsensitiveFs) rootForLookup.MakeLower();
		auto rootIt = consumer.m_lookup.find(ToStringKey(rootForLookup));
		CHECK_TRUE(rootIt != consumer.m_lookup.end());
		CHECK_TRUE(rootIt->second.files.size() == 1);

		CHECK_TRUE(consumer.EntryExists(keepDir, true) == DirectoryTable::Exists_Yes);
		CHECK_TRUE(consumer.EntryExists(keepInner)     == DirectoryTable::Exists_Yes);

		StringBuffer<> killDirForLookup(killDir);
		if (CaseInsensitiveFs) killDirForLookup.MakeLower();
		CHECK_TRUE(rootIt->second.files.find(ToStringKey(killDirForLookup)) == rootIt->second.files.end());

		FileMapping_Unmap(logger, snap.mappedMemory, TestSnapshotMappingSize, TC("TestSnapshotDelDir"), false);
		FileMapping_Close(logger, snap.handle, TC("TestSnapshotDelDir"));
		return true;
	}

#if UBA_SUPPORT_SYMLINKS
	// Overlay-only — verifies the journal round-trip (attributes + target string) without depending
	// on platform-specific on-disk symlink creation.
	UBA_TEST(DirectoryTable, SnapshotSymlink)
	{
		DirectoryTableHolder holder(logger.m_writer, TC(""), false);
		holder.SetTreatTempDirAsEmpty();
		CHECK_TRUE(holder.RefreshDirectory(testRootDir.data, true));

		DirectoryTableOverlay overlay;
		holder.CreateOverlay(overlay);

		StringBuffer<MaxPath> symlinkPath(testRootDir);
		symlinkPath.Append(TCV("mylink"));
		const StringView target(TCV("/some/other/file"));

		FileInformation info;
		info.attributes        = S_IFLNK | 0777;
		info.volumeSerialNumber = 1;
		info.index             = 42;
		info.lastWriteTime     = 0;
		info.size              = target.count; // WriteOverlayEntry asserts size == symlinkContent.count
		{
			OverlayEditScope scope(overlay);
			CHECK_TRUE(holder.WriteOverlayEntry(scope, symlinkPath, info, target));
		}

		StringBuffer<> plainFile;
		CHECK_TRUE(CreateTestFile(plainFile, logger, testRootDir, TCV("plain.txt"), TCV("p")));
		CHECK_TRUE(holder.RegisterNewFile(plainFile.data));

		DirTableSize covered = holder.GetDirectoryTableSize(&overlay);
		DirTableSnapshot snap;
		CHECK_TRUE(CreateDirectoryTableSnapshot(snap, TestSnapshotMappingSize, nullptr, 0, holder.GetDirectoryTableMemory(), covered.main, overlay.memory, covered.overlay, TC("TestSnapshotSymlink"), logger));

		DirectoryTable& consumer = *(DirectoryTable*)snap.mappedMemory;

		DirTableOffset symOffset;
		CHECK_TRUE(consumer.EntryExists(symlinkPath, false, &symOffset) == DirectoryTable::Exists_Yes);
		DirectoryTable::EntryInformation gotInfo;
		consumer.GetEntryInformation(gotInfo, symOffset);
		CHECK_TRUE(IsSymlink(gotInfo.attributes));
		CHECK_TRUE(gotInfo.size == target.count);

		StringBuffer<> retrievedTarget;
		CHECK_TRUE(consumer.GetEntrySymlink(retrievedTarget, symOffset));
		CHECK_TRUE(retrievedTarget.Equals(target.data));

		CHECK_TRUE(consumer.EntryExists(plainFile) == DirectoryTable::Exists_Yes);

		holder.DestroyOverlay(overlay);
		FileMapping_Unmap(logger, snap.mappedMemory, TestSnapshotMappingSize, TC("TestSnapshotSymlink"), false);
		FileMapping_Close(logger, snap.handle, TC("TestSnapshotSymlink"));
		return true;
	}
#endif

	// Snapshot at a small baseline, then add far more entries than the snapshot reserved for —
	// the consumer's tail parse forces both m_lookup (new subdirs) and root dir.files (new files)
	// to grow past their snapshot-time bucket counts. All entries must remain queryable.
	UBA_TEST(DirectoryTable, SnapshotRehash)
	{
		DirectoryTableHolder holder(logger.m_writer, TC(""), false);
		holder.SetTreatTempDirAsEmpty();
		CHECK_TRUE(holder.RefreshDirectory(testRootDir.data, true));

		StringBuffer<> seedFile;
		CHECK_TRUE(CreateTestFile(seedFile, logger, testRootDir, TCV("seed.txt"), TCV("s")));
		CHECK_TRUE(holder.RegisterNewFile(seedFile.data));

		DirTableSize coveredAtSnapshot = holder.GetDirectoryTableSize(nullptr);
		DirTableSnapshot snap;
		CHECK_TRUE(CreateDirectoryTableSnapshot(snap, TestSnapshotMappingSize, nullptr, 0, holder.GetDirectoryTableMemory(), coveredAtSnapshot.main, nullptr, 0, TC("TestSnapshotRehash"), logger));

		constexpr u32 NumExtraFiles = 200;
		constexpr u32 NumExtraSubdirs = 50;

		Vector<StringBuffer<>> extraFiles;
		extraFiles.resize(NumExtraFiles);
		for (u32 i = 0; i < NumExtraFiles; ++i)
		{
			StringBuffer<32> name; name.Appendf(TC("f%u.txt"), i);
			CHECK_TRUE(CreateTestFile(extraFiles[i], logger, testRootDir, name, TCV("x")));
			CHECK_TRUE(holder.RegisterNewFile(extraFiles[i].data));
		}

		Vector<StringBuffer<MaxPath>> extraSubdirs;
		Vector<StringBuffer<>> subdirFiles;
		extraSubdirs.resize(NumExtraSubdirs);
		subdirFiles.resize(NumExtraSubdirs);
		for (u32 i = 0; i < NumExtraSubdirs; ++i)
		{
			extraSubdirs[i].Append(testRootDir).Appendf(TC("d%u"), i);
			CHECK_TRUE(CreateDirectoryW(extraSubdirs[i].data));
			CHECK_TRUE(holder.RegisterNewDirectory(extraSubdirs[i].data));

			StringBuffer<MaxPath> subWithSlash(extraSubdirs[i]); subWithSlash.EnsureEndsWithSlash();
			CHECK_TRUE(CreateTestFile(subdirFiles[i], logger, subWithSlash, TCV("inside.txt"), TCV("i")));
			CHECK_TRUE(holder.RegisterNewFile(subdirFiles[i].data));
		}

		DirTableSize coveredNow = holder.GetDirectoryTableSize(nullptr);

		DirectoryTable& consumer = *(DirectoryTable*)snap.mappedMemory;

		consumer.ParseDirectoryTable(coveredNow);

		CHECK_TRUE(consumer.EntryExists(seedFile) == DirectoryTable::Exists_Yes);
		for (u32 i = 0; i < NumExtraFiles; ++i)
			CHECK_TRUE(consumer.EntryExists(extraFiles[i]) == DirectoryTable::Exists_Yes);
		for (u32 i = 0; i < NumExtraSubdirs; ++i)
		{
			CHECK_TRUE(consumer.EntryExists(extraSubdirs[i], true) == DirectoryTable::Exists_Yes);
			CHECK_TRUE(consumer.EntryExists(subdirFiles[i]) == DirectoryTable::Exists_Yes);
		}

		StringBuffer<> rootForLookup(StringView(testRootDir.data, testRootDir.count - 1));
		if (CaseInsensitiveFs) rootForLookup.MakeLower();
		auto rootIt = consumer.m_lookup.find(ToStringKey(rootForLookup));
		CHECK_TRUE(rootIt != consumer.m_lookup.end());
		CHECK_TRUE(rootIt->second.files.size() == 1 + NumExtraFiles + NumExtraSubdirs);

		FileMapping_Unmap(logger, snap.mappedMemory, TestSnapshotMappingSize, TC("TestSnapshotRehash"), false);
		FileMapping_Close(logger, snap.handle, TC("TestSnapshotRehash"));
		return true;
	}

	// Reproduces the "RegisterNewDirectory on a deep path without registering parents first"
	// scenario the user described: both intermediate ancestors get stuck at populatedOffset==2
	// because the parent's enumeration didn't list them. EnsurePath must repair the entire
	// chain back to populatedOffset==1.
	UBA_TEST(DirectoryTable, EnsurePath)
	{
		DirectoryTableHolder holder(logger.m_writer, TC(""), false);
		holder.SetTreatTempDirAsEmpty();
		CHECK_TRUE(holder.RefreshDirectory(testRootDir.data, true));

		// x/y exist on disk but were created AFTER refresh, so testRootDir's cached enumeration
		// doesn't list x.
		StringBuffer<MaxPath> xDir(testRootDir);
		xDir.Append(TCV("x"));
		CHECK_TRUE(CreateDirectoryW(xDir.data));

		StringBuffer<MaxPath> yDir(xDir);
		yDir.Append(PathSeparator).Append(TCV("y"));
		CHECK_TRUE(CreateDirectoryW(yDir.data));

		// Register y directly without first registering x — this is the "out of order" call
		// that triggers populatedOffset==2 on every ancestor between testRootDir and y.
		CHECK_TRUE(holder.RegisterNewDirectory(yDir.data, false));

		StringBuffer<> xLookup(xDir);
		if (CaseInsensitiveFs)
			xLookup.MakeLower();
		StringKey xKey = ToStringKey(xLookup);

		StringBuffer<> yLookup(yDir);
		if (CaseInsensitiveFs)
			yLookup.MakeLower();
		StringKey yKey = ToStringKey(yLookup);

		StringBuffer<> testRootForLookup(StringView(testRootDir.data, testRootDir.count - 1));
		if (CaseInsensitiveFs)
			testRootForLookup.MakeLower();
		StringKey testRootKey = ToStringKey(testRootForLookup);

		auto& dirTable = holder.m_directoryTable;

		// Pre-condition: both ancestors stuck at populatedOffset==2.
		{
			SCOPED_READ_LOCK(dirTable.m_lookupLock, lookupLock);
			auto xIt = dirTable.m_lookup.find(xKey);
			CHECK_TRUE(xIt != dirTable.m_lookup.end());
			CHECK_TRUE(xIt->second.populatedOffset == 2);
			auto yIt = dirTable.m_lookup.find(yKey);
			CHECK_TRUE(yIt != dirTable.m_lookup.end());
			CHECK_TRUE(yIt->second.populatedOffset == 2);
			auto rootIt = dirTable.m_lookup.find(testRootKey);
			CHECK_TRUE(rootIt != dirTable.m_lookup.end());
			CHECK_TRUE(rootIt->second.files.find(xKey) == rootIt->second.files.end());
		}

		// Call EnsurePath with the file we're about to write. The leaf doesn't have to exist on
		// disk — EnsurePath only inspects ancestor directories.
		StringBuffer<MaxPath> leafFile(yDir);
		leafFile.EnsureEndsWithSlash().Append(TCV("leaf.txt"));
		CHECK_TRUE(holder.EnsurePath(leafFile));

		// Post-condition: every ancestor populatedOffset==1 and listed in its parent's files.
		{
			SCOPED_READ_LOCK(dirTable.m_lookupLock, lookupLock);
			auto xIt = dirTable.m_lookup.find(xKey);
			CHECK_TRUE(xIt != dirTable.m_lookup.end());
			CHECK_TRUE(xIt->second.populatedOffset == 1);
			auto yIt = dirTable.m_lookup.find(yKey);
			CHECK_TRUE(yIt != dirTable.m_lookup.end());
			CHECK_TRUE(yIt->second.populatedOffset == 1);
			auto rootIt = dirTable.m_lookup.find(testRootKey);
			CHECK_TRUE(rootIt != dirTable.m_lookup.end());
			CHECK_TRUE(rootIt->second.files.find(xKey) != rootIt->second.files.end());
			CHECK_TRUE(xIt->second.files.find(yKey) != xIt->second.files.end());
		}

		// A subsequent RegisterNewFile under the repaired chain must succeed.
		StringBuffer<MaxPath> yDirSlash(yDir);
		yDirSlash.EnsureEndsWithSlash();
		StringBuffer<> leafFileCreated;
		CHECK_TRUE(CreateTestFile(leafFileCreated, logger, yDirSlash, TCV("leaf.txt"), TCV("L")));
		CHECK_TRUE(holder.RegisterNewFile(leafFileCreated.data));

		// And a consumer parsing the journal must resolve every level.
		MemoryBlock dirMem(TC("EnsurePathMem"));
		DirectoryTable consumer(dirMem);
		consumer.Init(holder.GetDirectoryTableMemory(), 0, holder.GetDirectoryTableSize(nullptr).main);
		consumer.ParseDirectoryTable(holder.GetDirectoryTableSize(nullptr));

		CHECK_TRUE(consumer.EntryExists(xDir, true) == DirectoryTable::Exists_Yes);
		CHECK_TRUE(consumer.EntryExists(yDir, true) == DirectoryTable::Exists_Yes);
		CHECK_TRUE(consumer.EntryExists(leafFile) == DirectoryTable::Exists_Yes);

		return true;
	}
}
