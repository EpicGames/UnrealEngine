// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaTest.h"
#include "UbaLogger.h"
#include "UbaFileMapping.h"
#include "UbaFileMappingBuffer.h"
#include "UbaSharedMemoryAllocator.h"

UBA_DEFINE_TEST_GROUP(FileMapping, 40)

namespace uba
{
	bool TestFileMappingInner(LoggerWithWriter& logger, const StringBufferBase& rootDir, FileMappingBackend& backend)
	{
		FileMappingHandle fmh = backend.Create(logger, PAGE_READWRITE|SEC_RESERVE, 1024*1024, nullptr, TC("Test"));
		CHECK_TRUE(fmh.IsValid());
		u8* memory = backend.Map(logger, fmh, FILE_MAP_READ|FILE_MAP_WRITE, 0, 64*1024, false);
		CHECK_TRUE(memory);
		CHECK_TRUE(backend.Unmap(logger, memory, 64*1024, TC(""), false));
		CHECK_TRUE(backend.Close(logger, fmh, TC("")));
		return true;
	}

	UBA_TEST(FileMapping, Basic)
	{
		FileMappingBackend backend;
		CHECK_TRUE(TestFileMappingInner(logger, testRootDir, backend));

		#if PLATFORM_WINDOWS
		if (!IsRunningWine())
			return true;
		HMODULE m = LoadLibraryW(L"UbaWine.dll.so");
		CHECK_TRUEF(m, TC("Failed to load UbaWine module"));
		FileMappingBackend wineBackend;
		CHECK_TRUE(FileMapping_GetWineBackend(wineBackend, logger, m));
		CHECK_TRUE(TestFileMappingInner(logger, testRootDir, wineBackend));
		#endif
		return true;
	}

	bool TestFileMappingBufferInner(LoggerWithWriter& logger, const StringBufferBase& rootDir, FileMappingBackend& backend)
	{
		TrackSystemUsage tsu(logger);
		SharedMemoryAllocator allocator(logger, backend);
		allocator.Init(64ull*1024*1024);
		FileMappingBuffer mappingBuffer(logger, allocator);

		mappingBuffer.Init(TC("Temp"), 4ull*1024*1024, 1024*1024);

		for (u32 i = 0; i != 65; ++i)
		{
			MappedView view = mappingBuffer.AllocAndMapView(1024, 1, TC("Foo"));

			u64 v = i + 1337;
			*(u64*)view.memory = v;
			u64 value = *(u64*)view.memory;
			CHECK_TRUE(value == v);
			mappingBuffer.UnmapView(view, TC("Foo"));

			view = mappingBuffer.MapView(view.handle, view.offset, 1024, TC("Foo"));
			value = *(u64*)view.memory;
			CHECK_TRUE(value == v);
			mappingBuffer.UnmapView(view, TC("Foo"));
		}

		{
			// Allow shrink
			MappedView view1 = mappingBuffer.AllocAndMapView(1024, 1, TC("Foo"));
			MappedView view2 = mappingBuffer.AllocAndMapView(1024, 1, TC("Foo"));
			mappingBuffer.UnmapView(view1, TC("Foo"), 1024);
			mappingBuffer.UnmapView(view2, TC("Foo"), 512);
		}

		{
			// Independent
			MappedView view = mappingBuffer.AllocAndMapView(1024, 1, TC("Foo"), true);
			*(u64*)view.memory = 1337;
			u64 value = *(u64*)view.memory;
			CHECK_TRUE(value == 1337);
			mappingBuffer.UnmapView(view, TC("Foo"), 1024);
			view = mappingBuffer.MapView(view.handle, view.offset, 1024, TC("Foo"));
			value = *(u64*)view.memory;
			CHECK_TRUE(value == 1337);
			mappingBuffer.UnmapView(view, TC("Foo"), 1024);
		}
		return true;
	}

	UBA_TEST(FileMapping, Buffer)
	{
		FileMappingBackend backend;
		CHECK_TRUE(TestFileMappingBufferInner(logger, testRootDir, backend));

#if PLATFORM_WINDOWS
		if (!IsRunningWine())
			return true;
		HMODULE m = LoadLibraryW(L"UbaWine.dll.so");
		CHECK_TRUEF(m, TC("Failed to load UbaWine module"));
		FileMappingBackend wineBackend;
		CHECK_TRUE(FileMapping_GetWineBackend(wineBackend, logger, m));
		CHECK_TRUE(TestFileMappingBufferInner(logger, testRootDir, wineBackend));
#endif
		return true;
	}
}
