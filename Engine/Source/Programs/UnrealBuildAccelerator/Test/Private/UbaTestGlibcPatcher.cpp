// Copyright Epic Games, Inc. All Rights Reserved.
//
// Unit test for glibc-static detection + patching path. Builds a synthetic ELF in memory
// (ELF64 header + one PT_LOAD + SHT_SYMTAB exposing the 9 MVP libc wrappers + 7-byte stub
// bodies). Fixture is written to testRootDir so the patcher's FileAccessor can read it.
// The full PatchGlibcStaticBinary flow runs only when the stub blob is present at
// Engine/Binaries/Linux/UnrealBuildAccelerator/UbaStaticStub.bin; otherwise detection-only.

#include "UbaTest.h"
#include "UbaFileAccessor.h"
#include "UbaPlatform.h"
#include "UbaStaticPatcher.h"
#include "UbaStringBuffer.h"

UBA_DEFINE_TEST_GROUP(GlibcPatcher, 120)

#include <stdint.h>
#include <string.h>

namespace uba
{
#if PLATFORM_LINUX

	namespace
	{
		// Minimal x86-64 ELF64 builder. File order: Ehdr(64), Phdr(56), .text, .shstrtab, .strtab,
		// .symtab (10 entries: null + 9 hooks), Shdrs(5*64). .text VA = 0x400000.
		struct SynthLayout
		{
			Vector<u8> bytes;
			u64        textVaddr;       // VA of .text in the synthetic ELF
			u64        textFileOff;     // file offset of .text inside `bytes`
		};

		// 7-byte body: 48 c7 c0 ii ii ii ii  (mov rax, imm32) + 1-byte c3 (ret).
		constexpr u32 kWrapperBodySize = 8;

		// MVP libc names the patcher hooks (must match first alias of each row in kGlibcAliasTable).
		struct LibcSym { const char* name; u32 immValue; };
		constexpr LibcSym kSyms[] = {
			{ "__libc_open",  0xA0 },
			{ "__openat",     0xA1 },
			{ "__libc_read",  0xA2 },
			{ "__libc_write", 0xA3 },
			{ "__libc_close", 0xA4 },
			{ "__libc_lseek", 0xA5 },
			{ "__fxstat",     0xA6 },
			{ "__mmap",       0xA7 },
			{ "__access",     0xA8 },
		};
		constexpr u32 kSymCount = sizeof(kSyms) / sizeof(kSyms[0]);

		template<typename T>
		void Append(Vector<u8>& v, T value)
		{
			u64 sz = v.size();
			v.resize(sz + sizeof(T));
			memcpy(v.data() + sz, &value, sizeof(T));
		}

		void AppendBytes(Vector<u8>& v, const void* data, u64 size)
		{
			u64 sz = v.size();
			v.resize(sz + size);
			memcpy(v.data() + sz, data, size);
		}

		void AlignTo(Vector<u8>& v, u64 align)
		{
			while (v.size() % align)
				v.push_back(0);
		}

		bool BuildSyntheticElf(SynthLayout& out)
		{
			// Reserve large enough buffer; we'll splice headers in afterwards.
			out.bytes.resize(64 + 56);   // Ehdr + 1 Phdr placeholder

			// .text body: 9 wrappers laid out contiguously.
			AlignTo(out.bytes, 16);
			out.textFileOff = out.bytes.size();
			out.textVaddr   = 0x400000ULL + out.textFileOff; // page-mapped
			for (u32 i = 0; i < kSymCount; ++i)
			{
				// 48 c7 c0 ii ii ii ii c3
				out.bytes.push_back(0x48);
				out.bytes.push_back(0xC7);
				out.bytes.push_back(0xC0);
				u32 imm = kSyms[i].immValue;
				out.bytes.push_back(u8(imm));
				out.bytes.push_back(u8(imm >> 8));
				out.bytes.push_back(u8(imm >> 16));
				out.bytes.push_back(u8(imm >> 24));
				out.bytes.push_back(0xC3);
			}
			u64 textEnd = out.bytes.size();
			u64 textSize = textEnd - out.textFileOff;

			// .shstrtab.
			AlignTo(out.bytes, 1);
			u64 shstrOff = out.bytes.size();
			auto putStr = [&](const char* s) -> u32 {
				u32 off = u32(out.bytes.size() - shstrOff);
				while (*s) out.bytes.push_back(u8(*s++));
				out.bytes.push_back(0);
				return off;
			};
			out.bytes.push_back(0); // strtab[0] = "\0"
			u32 shName_text     = putStr(".text");
			u32 shName_shstrtab = putStr(".shstrtab");
			u32 shName_symtab   = putStr(".symtab");
			u32 shName_strtab   = putStr(".strtab");
			u64 shstrSize = out.bytes.size() - shstrOff;

			// .strtab (symbol names).
			AlignTo(out.bytes, 1);
			u64 strOff = out.bytes.size();
			out.bytes.push_back(0); // strtab[0]
			u32 nameOffsets[kSymCount];
			for (u32 i = 0; i < kSymCount; ++i)
			{
				nameOffsets[i] = u32(out.bytes.size() - strOff);
				const char* s = kSyms[i].name;
				while (*s) out.bytes.push_back(u8(*s++));
				out.bytes.push_back(0);
			}
			u64 strSize = out.bytes.size() - strOff;

			// .symtab (Elf64_Sym entries: 24 bytes each — name(4) info(1) other(1) shndx(2) value(8) size(8)).
			AlignTo(out.bytes, 8);
			u64 symOff = out.bytes.size();
			// Index 0 — STN_UNDEF (zeros).
			for (u32 b = 0; b < 24; ++b)
				out.bytes.push_back(0);
			constexpr u8 STT_FUNC   = 2;
			constexpr u8 STB_GLOBAL = 1;
			constexpr u8 stInfo = (STB_GLOBAL << 4) | STT_FUNC;
			constexpr u16 shndxText = 1; // .text section index in our shdr table
			for (u32 i = 0; i < kSymCount; ++i)
			{
				Append<u32>(out.bytes, nameOffsets[i]);  // st_name
				out.bytes.push_back(stInfo);             // st_info
				out.bytes.push_back(0);                  // st_other
				Append<u16>(out.bytes, shndxText);       // st_shndx
				u64 vaddr = out.textVaddr + u64(i) * kWrapperBodySize;
				Append<u64>(out.bytes, vaddr);           // st_value
				Append<u64>(out.bytes, kWrapperBodySize); // st_size
			}
			u64 symSize = out.bytes.size() - symOff;

			// Section headers (5 entries: null, .text, .shstrtab, .symtab, .strtab).
			AlignTo(out.bytes, 8);
			u64 shoff = out.bytes.size();
			constexpr u32 SHT_NULL    = 0;
			constexpr u32 SHT_PROGBITS = 1;
			constexpr u32 SHT_SYMTAB  = 2;
			constexpr u32 SHT_STRTAB  = 3;
			constexpr u64 SHF_ALLOC = 2, SHF_EXECINSTR = 4;

			auto writeShdr = [&](u32 nameIdx, u32 type, u64 flags, u64 addr,
			                     u64 fileOff, u64 size, u32 link, u32 info,
			                     u64 addralign, u64 entsize)
			{
				Append<u32>(out.bytes, nameIdx);
				Append<u32>(out.bytes, type);
				Append<u64>(out.bytes, flags);
				Append<u64>(out.bytes, addr);
				Append<u64>(out.bytes, fileOff);
				Append<u64>(out.bytes, size);
				Append<u32>(out.bytes, link);
				Append<u32>(out.bytes, info);
				Append<u64>(out.bytes, addralign);
				Append<u64>(out.bytes, entsize);
			};

			// [0] NULL
			writeShdr(0, SHT_NULL, 0, 0, 0, 0, 0, 0, 0, 0);
			// [1] .text  (must be at index 1 — see shndxText)
			writeShdr(shName_text, SHT_PROGBITS, SHF_ALLOC | SHF_EXECINSTR,
			          out.textVaddr, out.textFileOff, textSize, 0, 0, 16, 0);
			// [2] .shstrtab
			writeShdr(shName_shstrtab, SHT_STRTAB, 0, 0, shstrOff, shstrSize, 0, 0, 1, 0);
			// [3] .symtab
			writeShdr(shName_symtab, SHT_SYMTAB, 0, 0, symOff, symSize,
			          /* sh_link = .strtab index */ 4, /* sh_info */ 1, 8, 24);
			// [4] .strtab
			writeShdr(shName_strtab, SHT_STRTAB, 0, 0, strOff, strSize, 0, 0, 1, 0);

			constexpr u16 kShnum    = 5;
			constexpr u16 kShstrndx = 2;

			// --- Fill in ELF header (64 bytes at offset 0) ---
			u8* eh = out.bytes.data();
			eh[0]=0x7F; eh[1]='E'; eh[2]='L'; eh[3]='F';
			eh[4]=2;           // EI_CLASS = ELFCLASS64
			eh[5]=1;           // EI_DATA  = ELFDATA2LSB
			eh[6]=1;           // EI_VERSION
			memset(eh + 7, 0, 9); // padding
			constexpr u16 ET_EXEC = 2;
			constexpr u16 EM_X86_64 = 62;
			memcpy(eh + 16, &ET_EXEC, 2);             // e_type
			memcpy(eh + 18, &EM_X86_64, 2);           // e_machine
			u32 ev = 1; memcpy(eh + 20, &ev, 4);      // e_version
			u64 entryVa = out.textVaddr;
			memcpy(eh + 24, &entryVa, 8);             // e_entry
			u64 phoff = 64;     memcpy(eh + 32, &phoff, 8);
			memcpy(eh + 40, &shoff, 8);
			u32 eflags = 0; memcpy(eh + 48, &eflags, 4);
			u16 ehsize = 64; memcpy(eh + 52, &ehsize, 2);
			u16 phentsize = 56; memcpy(eh + 54, &phentsize, 2);
			u16 phnum = 1; memcpy(eh + 56, &phnum, 2);
			u16 shentsize = 64; memcpy(eh + 58, &shentsize, 2);
			memcpy(eh + 60, &kShnum, 2);
			memcpy(eh + 62, &kShstrndx, 2);

			// --- Fill in PT_LOAD (1 Phdr at offset 64) ---
			// p_type, p_flags, p_offset, p_vaddr, p_paddr, p_filesz, p_memsz, p_align
			u8* ph = out.bytes.data() + 64;
			constexpr u32 PT_LOAD = 1;
			constexpr u32 PF_X = 1, PF_R = 4;
			u32 ptype = PT_LOAD; memcpy(ph + 0, &ptype, 4);
			u32 pflags = PF_R | PF_X; memcpy(ph + 4, &pflags, 4);
			memcpy(ph + 8,  &out.textFileOff, 8);
			memcpy(ph + 16, &out.textVaddr,   8);
			memcpy(ph + 24, &out.textVaddr,   8);
			memcpy(ph + 32, &textSize,        8);
			memcpy(ph + 40, &textSize,        8);
			u64 palign = 0x1000; memcpy(ph + 48, &palign, 8);

			return true;
		}

		bool WriteFile(Logger& logger, const tchar* path, const Vector<u8>& bytes)
		{
			FileAccessor fa(logger, path);
			if (!fa.CreateWrite(false, DefaultAttributes(), bytes.size()))
				return false;
			if (!fa.Write(bytes.data(), bytes.size(), 0, true))
				return false;
			return fa.Close();
		}

		bool ReadFileBytes(Logger& logger, const tchar* path, Vector<u8>& out)
		{
			FileAccessor fa(logger, path);
			if (!fa.OpenMemoryRead())
				return false;
			out.resize(fa.GetSize());
			memcpy(out.data(), fa.GetData(), fa.GetSize());
			return true;
		}
	}

#endif // PLATFORM_LINUX

	UBA_TEST(GlibcPatcher, Basic)
	{
#if !PLATFORM_LINUX
		(void)logger; (void)testRootDir;
		return true;
#else
		// Build the synthetic glibc-static ELF.
		SynthLayout layout;
		CHECK_TRUEF(BuildSyntheticElf(layout), TC("Failed to build synthetic ELF"));

		// Write it under testRootDir.
		StringBuffer<512> elfPath;
		elfPath.Append(testRootDir).EnsureEndsWithSlash().Append(TCV("UbaTestGlibcStaticInput.elf"));
		CHECK_TRUEF(WriteFile(logger, elfPath.data, layout.bytes), TC("Failed to write synthetic ELF to %s"), elfPath.data);

		// 1. IsGlibcStaticBinary should accept it (≥5 of 9 MVP symbols hit).
		const u8* elfData = layout.bytes.data();
		const u32  elfSize = (u32)layout.bytes.size();
		CHECK_TRUEF(IsGlibcStaticBinary(logger, elfPath, elfData, elfSize), TC("IsGlibcStaticBinary rejected synthetic glibc-static ELF"));

		// 2. IsGoStaticBinary should NOT accept it (no Syscall6 symbol).
		CHECK_TRUEF(!IsGoStaticBinary(logger, elfPath, elfData, elfSize), TC("IsGoStaticBinary erroneously accepted synthetic glibc-static ELF"));

		// 3. BinaryRequiresPatching dispatches to either; should accept.
		CHECK_TRUEF(BinaryRequiresPatching(logger, elfPath, elfData, elfSize), TC("BinaryRequiresPatching rejected synthetic glibc-static ELF"));

		// 4. Stripped variant: re-build the ELF without a symtab — must be rejected.
		{
			Vector<u8> stripped = layout.bytes;
			// Truncate at textFileOff + textSize and zero shoff/shnum/shstrndx in the EHDR.
			u64 textEnd = layout.textFileOff;
			// recover textSize from the phdr we wrote:
			u64 textSize = 0;
			memcpy(&textSize, stripped.data() + 64 + 32, 8);
			stripped.resize(textEnd + textSize);
			u64 zero64 = 0; u16 zero16 = 0;
			memcpy(stripped.data() + 40, &zero64, 8);   // e_shoff = 0
			memcpy(stripped.data() + 60, &zero16, 2);   // e_shnum = 0
			memcpy(stripped.data() + 62, &zero16, 2);   // e_shstrndx = 0

			StringBuffer<512> strippedPath;
			strippedPath.Append(testRootDir).EnsureEndsWithSlash().Append(TCV("UbaTestGlibcStaticStripped.elf"));
			CHECK_TRUEF(WriteFile(logger, strippedPath.data, stripped), TC("Failed to write stripped synthetic ELF"));
			CHECK_TRUEF(!IsGlibcStaticBinary(logger, strippedPath, stripped.data(), (u32)stripped.size()), TC("IsGlibcStaticBinary should reject stripped (no symtab) ELFs (musl-static case)"));
		}

		// 5. End-to-end patch step. Requires the stub blob with sentinels + stamped GlibcHandlerTable.
		StringBuffer<512> stubPath;
		stubPath.Append(testRootDir);
		const tchar* candidates[] = {
			TC("/mnt/d/devel/fn/Engine/Binaries/Linux/UnrealBuildAccelerator/UbaStaticStub.bin"),
			TC("/d/devel/fn/Engine/Binaries/Linux/UnrealBuildAccelerator/UbaStaticStub.bin"),
		};
		const tchar* foundStub = nullptr;
		for (const tchar* c : candidates)
		{
			if (FileExists(logger, c))
			{
				foundStub = c;
				break;
			}
		}
		if (!foundStub)
		{
			logger.Info(TC("  (skipping PatchGlibcStaticBinary end-to-end — stub blob not found at the conventional path)"));
			return true;
		}

		StringBuffer<512> outPath;
		outPath.Append(testRootDir).EnsureEndsWithSlash().Append(TCV("UbaTestGlibcStaticOutput.elf"));
		// foundStub is a const tchar* literal; wrap into a StringView with TStrlen.
		StringView stubView(foundStub, (u32)TStrlen(foundStub));
		CHECK_TRUEF(PatchGlibcStaticBinary(logger, elfPath, elfData, elfSize, outPath, stubView),
			TC("PatchGlibcStaticBinary failed against synthetic ELF"));

		// Read the output and check that each MVP symbol's first byte is 0xE9 (JMP rel32).
		Vector<u8> patched;
		CHECK_TRUEF(ReadFileBytes(logger, outPath.data, patched), TC("Failed to read patched ELF"));
		u32 jmpHits = 0;
		for (u32 i = 0; i < kSymCount; ++i)
		{
			u64 off = layout.textFileOff + u64(i) * kWrapperBodySize;
			CHECK_TRUEF(off < patched.size(), TC("Patched ELF too small to inspect %hs"), kSyms[i].name);
			if (patched[off] == 0xE9)
				++jmpHits;
		}
		CHECK_TRUEF(jmpHits == kSymCount, TC("Expected %u JMP hits in patched ELF; got %u"), kSymCount, jmpHits);

		logger.Info(TC("  TestGlibcPatcher: detection ok, %u/%u JMPs stamped"), jmpHits, kSymCount);

		// 6. Optional sanity-check against the AOSP qemu-system-x86_64 binary if present on the dev box.
		StringView qemuView = TCV("/home/honk/git/android/prebuilts/android-emulator/trusty-x86_64/bin/qemu-system-x86_64");
		if (FileExists(logger, qemuView.data))
		{
			Vector<u8> qemuBytes;
			CHECK_TRUEF(ReadFileBytes(logger, qemuView.data, qemuBytes), TC("Failed to read qemu binary"));
			const u8* qd = qemuBytes.data();
			const u32 qs = (u32)qemuBytes.size();
			CHECK_TRUEF(IsGlibcStaticBinary(logger, qemuView, qd, qs), TC("IsGlibcStaticBinary should accept real qemu-system-x86_64"));
			CHECK_TRUEF(!IsGoStaticBinary(logger, qemuView, qd, qs), TC("IsGoStaticBinary should reject qemu-system-x86_64 (no Syscall6 symbol)"));
			logger.Info(TC("  TestGlibcPatcher: qemu-system-x86_64 detection ok"));
		}
		return true;
#endif
	}
}
