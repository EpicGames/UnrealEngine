// Copyright Epic Games, Inc. All Rights Reserved.
//
// Behavior tests for the namespace-pid helpers in UbaDetoursSharedPosix.h (uba_real_clone3_pidfd, uba_get_host_pid_from_pidfd, uba_check_parent_in_non_init_pid_ns).
// UbaTest doesn't link the detour module, so the helpers are duplicated inline. Tests needing a fresh PID NS skip cleanly when CAP_SYS_ADMIN / unprivileged user NS is unavailable.

#include "UbaTest.h"
#include "UbaPlatform.h"

UBA_DEFINE_TEST_GROUP(NamespaceHostPid, 130)

#if PLATFORM_LINUX
#include <errno.h>
#include <fcntl.h>
#include <sched.h>
#include <signal.h>
#include <stdio.h>
#include <string.h>
#include <sys/syscall.h>
#include <sys/wait.h>
#include <unistd.h>

#ifndef CLONE_PIDFD
#define CLONE_PIDFD 0x00001000
#endif
#ifndef CLONE_NEWUSER
#define CLONE_NEWUSER 0x10000000
#endif
#ifndef CLONE_NEWPID
#define CLONE_NEWPID 0x20000000
#endif
#ifndef SYS_clone3
#define SYS_clone3 435 // Linux x86_64 nr; older glibc may lack the define
#endif

namespace uba
{
	namespace iter64_test
	{
		struct uba_test_clone_args // mirror of kernel UAPI struct clone_args (Linux 5.5+)
		{
			unsigned long long flags;
			unsigned long long pidfd;
			unsigned long long child_tid;
			unsigned long long parent_tid;
			unsigned long long exit_signal;
			unsigned long long stack;
			unsigned long long stack_size;
			unsigned long long tls;
		};

		// Returns >0 parent / 0 child / -errno.
		static long test_clone3_pidfd(int* outPidfd, unsigned long long extraFlags)
		{
			uba_test_clone_args args = {};
			args.flags = (unsigned long long)CLONE_PIDFD | extraFlags;
			args.pidfd = (unsigned long long)(uintptr_t)outPidfd;
			args.exit_signal = SIGCHLD;
			long r = syscall(SYS_clone3, &args, sizeof(args));
			if (r < 0)
				return -errno;
			return r;
		}

		// Returns FIRST integer on the NSpid: line (= host-NS pid).
		static unsigned int test_get_host_pid_from_pidfd(int pidfd)
		{
			if (pidfd < 0)
				return 0;
			char path[64];
			snprintf(path, sizeof(path), "/proc/self/fdinfo/%d", pidfd);
			int fd = ::open(path, O_RDONLY | O_CLOEXEC);
			if (fd < 0)
				return 0;
			char buf[1024];
			ssize_t r = ::read(fd, buf, sizeof(buf) - 1);
			::close(fd);
			if (r <= 0)
				return 0;
			buf[r] = 0;
			const char* p = buf;
			const char* end = buf + r;
			while (p < end)
			{
				if (p + 6 <= end && p[0] == 'N' && p[1] == 'S' && p[2] == 'p' && p[3] == 'i' && p[4] == 'd' && p[5] == ':')
				{
					p += 6;
					while (p < end && (*p == ' ' || *p == '\t'))
						++p;
					unsigned int hostPid = 0;
					while (p < end && *p >= '0' && *p <= '9')
					{
						hostPid = hostPid * 10u + (unsigned int)(*p - '0');
						++p;
					}
					return hostPid;
				}
				while (p < end && *p != '\n') ++p;
				if (p < end) ++p;
			}
			return 0;
		}

		// True iff the NSpid: line has > 1 numeric value.
		static bool test_check_in_non_init_pid_ns()
		{
			int fd = ::open("/proc/self/status", O_RDONLY | O_CLOEXEC);
			if (fd < 0)
				return false;
			char buf[4096];
			ssize_t r = ::read(fd, buf, sizeof(buf) - 1);
			::close(fd);
			if (r <= 0)
				return false;
			buf[r] = 0;
			const char* p = buf;
			const char* end = buf + r;
			while (p < end)
			{
				if (p + 6 <= end && p[0] == 'N' && p[1] == 'S' && p[2] == 'p' && p[3] == 'i' && p[4] == 'd' && p[5] == ':')
				{
					p += 6;
					unsigned int valueCount = 0;
					while (p < end && *p != '\n')
					{
						if (*p == ' ' || *p == '\t') { ++p; continue; }
						if (*p >= '0' && *p <= '9')
						{
							++valueCount;
							while (p < end && *p >= '0' && *p <= '9') ++p;
							continue;
						}
						++p;
					}
					return valueCount > 1;
				}
				while (p < end && *p != '\n') ++p;
				if (p < end) ++p;
			}
			return false;
		}

		// CLONE_NEWPID without CAP_SYS_ADMIN needs an enclosing user NS. Probe by clone3'ing one and bailing if the kernel rejects.
		static bool probe_user_ns_supported()
		{
			int childPidfd = -1;
			long r = test_clone3_pidfd(&childPidfd, (unsigned long long)CLONE_NEWUSER | (unsigned long long)CLONE_NEWPID);
			if (r == 0)
				_exit(0);
			if (r > 0)
			{
				int status = 0;
				waitpid((pid_t)r, &status, 0);
				if (childPidfd >= 0) ::close(childPidfd);
				return true;
			}
			return false;
		}
	}

	using namespace iter64_test;

	UBA_TEST(NamespaceHostPid, CheckInitNsIsFalse)
	{
		bool inNonInit = test_check_in_non_init_pid_ns();
		CHECK_TRUEF(!inNonInit, TC("uba_check_parent_in_non_init_pid_ns() returned true for init NS — UbaTest itself runs in init NS, so this means the parser is buggy"));
		return true;
	}

	// Init NS: clone3+CLONE_PIDFD parent's return MUST equal uba_get_host_pid_from_pidfd(pidfd) — divergence breaks host waitpid().
	UBA_TEST(NamespaceHostPid, HostPidMatchesClone3InInitNs)
	{
		int childPidfd = -1;
		long cloneRet = test_clone3_pidfd(&childPidfd, 0);
		if (cloneRet == 0)
			_exit(0);
		CHECK_TRUEF(cloneRet > 0, TC("clone3(CLONE_PIDFD) failed, returned %lld (errno %d). Likely SYS_clone3 unsupported on this kernel — would need a runtime skip"), (long long)cloneRet, (int)-cloneRet);
		CHECK_TRUEF(childPidfd >= 0, TC("clone3 reported success but pidfd was not set"));

		unsigned int hostPid = test_get_host_pid_from_pidfd(childPidfd);
		long expected = cloneRet; // Snapshot before reap; fdinfo reflects the child while alive or zombie

		int status = 0;
		pid_t reaped = waitpid((pid_t)cloneRet, &status, 0);
		::close(childPidfd);

		CHECK_TRUEF(reaped == (pid_t)cloneRet, TC("waitpid returned %d, expected %lld"), (int)reaped, (long long)cloneRet);
		CHECK_TRUEF(hostPid != 0, TC("uba_get_host_pid_from_pidfd returned 0 — /proc/self/fdinfo/<pidfd>'s NSpid: line was unparseable"));
		CHECK_TRUEF((long)hostPid == expected, TC("host-pid mismatch: clone3 returned %lld but uba_get_host_pid_from_pidfd reported %u — the parser is reading the wrong NSpid: column"), (long long)expected, hostPid);
		return true;
	}

	// uba_check_parent_in_non_init_pid_ns() returns true from inside a fresh PID NS. Skips when CAP_SYS_ADMIN / unprivileged user-NS clone is unavailable.
	UBA_TEST(NamespaceHostPid, CheckInsideNewPidNsIsTrue)
	{
		if (!probe_user_ns_supported())
		{
			logger.Info(TC("[SKIP] TestNsCheckInsideNewPidNsIsTrue: cannot create user+pid NS — need CAP_SYS_ADMIN or kernel.unprivileged_userns_clone=1"));
			return true;
		}

		int pipefd[2];
		CHECK_TRUEF(pipe(pipefd) == 0, TC("pipe() failed errno=%d"), errno);

		int childPidfd = -1;
		long cloneRet = test_clone3_pidfd(&childPidfd, (unsigned long long)CLONE_NEWUSER | (unsigned long long)CLONE_NEWPID);
		if (cloneRet == 0)
		{
			::close(pipefd[0]);
			struct
			{
				int   inNonInit;
				int   nsLocalPid;
			} verdict;
			verdict.inNonInit  = test_check_in_non_init_pid_ns() ? 1 : 0;
			verdict.nsLocalPid = (int)getpid();
			ssize_t w = ::write(pipefd[1], &verdict, sizeof(verdict));
			::close(pipefd[1]);
			_exit(w == (ssize_t)sizeof(verdict) ? 0 : 2);
		}
		CHECK_TRUEF(cloneRet > 0, TC("clone3(CLONE_NEWUSER|CLONE_NEWPID|CLONE_PIDFD) failed: %lld (errno %d)"), (long long)cloneRet, (int)-cloneRet);
		::close(pipefd[1]);

		struct
		{
			int   inNonInit;
			int   nsLocalPid;
		} verdict = {};
		ssize_t r = ::read(pipefd[0], &verdict, sizeof(verdict));
		::close(pipefd[0]);
		int status = 0;
		waitpid((pid_t)cloneRet, &status, 0);
		if (childPidfd >= 0) ::close(childPidfd);

		CHECK_TRUEF(r == (ssize_t)sizeof(verdict), TC("child sent %lld bytes (expected %llu); WIFEXITED=%d WEXITSTATUS=%d"), (long long)r, (unsigned long long)sizeof(verdict), WIFEXITED(status), WEXITSTATUS(status));
		CHECK_TRUEF(verdict.inNonInit == 1, TC("child inside CLONE_NEWPID reported uba_check_parent_in_non_init_pid_ns()=false — parser missed the second NSpid value"));
		CHECK_TRUEF(verdict.nsLocalPid == 1, TC("child's getpid() inside fresh PID NS returned %d, expected 1 — NS not actually entered"), verdict.nsLocalPid);
		return true;
	}

	// HOST pid (not NS-local) from uba_get_host_pid_from_pidfd when child is clone3'd into a fresh PID NS. Core iter64 invariant for InternalExitProcess.waitpid.
	UBA_TEST(NamespaceHostPid, HostPidIsHostNotNsLocal)
	{
		if (!probe_user_ns_supported())
		{
			logger.Info(TC("[SKIP] TestNsHostPidIsHostNotNsLocal: cannot create user+pid NS (see TestNsCheckInsideNewPidNsIsTrue for setup requirements)."));
			return true;
		}

		int pipefd[2];
		CHECK_TRUEF(pipe(pipefd) == 0, TC("pipe() failed errno=%d"), errno);

		int childPidfd = -1;
		long cloneRet = test_clone3_pidfd(&childPidfd, (unsigned long long)CLONE_NEWUSER | (unsigned long long)CLONE_NEWPID);
		if (cloneRet == 0)
		{
			::close(pipefd[0]);
			int nsLocal = (int)getpid();
			ssize_t w = ::write(pipefd[1], &nsLocal, sizeof(nsLocal));
			::close(pipefd[1]);
			_exit(w == (ssize_t)sizeof(nsLocal) ? 0 : 2);
		}
		CHECK_TRUEF(cloneRet > 0, TC("clone3(CLONE_NEWUSER|CLONE_NEWPID|CLONE_PIDFD) failed: %lld"), (long long)cloneRet);
		CHECK_TRUEF(childPidfd >= 0, TC("clone3 reported success but pidfd was not set"));
		::close(pipefd[1]);

		unsigned int hostPidFromHelper = test_get_host_pid_from_pidfd(childPidfd); // Read BEFORE reap; closing pidfd after wait invalidates the fdinfo entry.

		int nsLocalPid = -1;
		ssize_t r = ::read(pipefd[0], &nsLocalPid, sizeof(nsLocalPid));
		::close(pipefd[0]);
		int status = 0;
		waitpid((pid_t)cloneRet, &status, 0);
		::close(childPidfd);

		CHECK_TRUEF(r == (ssize_t)sizeof(nsLocalPid), TC("child failed to ship its NS-local pid (read=%lld)"), (long long)r);
		CHECK_TRUEF(nsLocalPid == 1, TC("child reports getpid()=%d inside fresh NEWPID NS, expected 1"), nsLocalPid);
		CHECK_TRUEF(hostPidFromHelper != 0, TC("uba_get_host_pid_from_pidfd returned 0 inside fresh NS"));
		CHECK_TRUEF((long)hostPidFromHelper == cloneRet, TC("host-pid mismatch: clone3 returned %lld, helper read %u (vs ns-local pid %d) — feeding NS-local back into host waitpid is the iter64 bug class"), (long long)cloneRet, hostPidFromHelper, nsLocalPid);
		CHECK_TRUEF((int)hostPidFromHelper != nsLocalPid, TC("helper returned %u, equal to child's NS-local pid %d — parser is reading the wrong column of NSpid:"), hostPidFromHelper, nsLocalPid);
		return true;
	}

	// TODO: needs a helper binary or SessionServer integration to exercise NS-pid plumbing under the actual UBA detour. AOSP nsjail trusty edges exercise the path in production.
	UBA_TEST_DISABLED(NamespaceHostPid, EndToEndUnderDetour, "needs setns helper binary or SessionServer integration. Production coverage: AOSP nsjail trusty_security_vm edges.")
	{
		return true;
	}
}

#else // !PLATFORM_LINUX

namespace uba
{
	UBA_TEST_DISABLED(NamespaceHostPid, CheckInitNsIsFalse,            "Linux only") { return true; }
	UBA_TEST_DISABLED(NamespaceHostPid, HostPidMatchesClone3InInitNs,  "Linux only") { return true; }
	UBA_TEST_DISABLED(NamespaceHostPid, CheckInsideNewPidNsIsTrue,     "Linux only") { return true; }
	UBA_TEST_DISABLED(NamespaceHostPid, HostPidIsHostNotNsLocal,       "Linux only") { return true; }
	UBA_TEST_DISABLED(NamespaceHostPid, EndToEndUnderDetour,           "Linux only") { return true; }
}

#endif
