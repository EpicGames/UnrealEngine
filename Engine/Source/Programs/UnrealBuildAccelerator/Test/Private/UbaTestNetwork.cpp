// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaBinaryReaderWriter.h"
#include "UbaPlatform.h"
#include "UbaNetworkClient.h"
#include "UbaNetworkServer.h"
#include "UbaNetworkMessage.h"
#include "UbaNetworkBackendTcp.h"
#include "UbaNetworkBackendMemory.h"
#include "UbaTest.h"

UBA_DEFINE_TEST_GROUP(Network, 60)

namespace uba
{
	UBA_TEST(Network, Sockets)
	{
		static constexpr u16 port = 1346;
		Atomic<u8> result = 0;
		NetworkBackendTcp tcp(logger.m_writer);
		Thread t([&]()
			{
				logger.Info(TC("Starting to Listen"));
				if (!tcp.StartListen(logger, port, TC("127.0.0.1"), [&](void* connection, const sockaddr& remoteSocketAddr)
					{
						logger.Info(TC("Listen got connection"));
						tcp.SetDisconnectCallback(connection, nullptr, [](void*, const Guid&, void*) {});
						tcp.SetRecvCallbacks(connection, &result, 1,
							[](void* context, const Guid& connectionUid, u8* headerData, void*& outBodyContext, u8*& outBodyData, u32& outBodySize)
							{
								*(Atomic<u8>*)context = *headerData;
								//wprintf(TC("Listen got data from peer: %u\n"), *headerData);
								return true;
							},
							nullptr, TC(""));
						return true;
					}))
					logger.Error(TC("Failed to listen"));

				return 0;
			});

		Sleep(100);
		logger.Info(TC("Starting to Connect"));
		if (!tcp.Connect(logger, TC("127.0.0.1"), [&](void* connection, const sockaddr& remoteSocketAddr, bool* timedOut)
			{
				tcp.SetDisconnectCallback(connection, nullptr, [](void*, const Guid&, void*) {});
				tcp.SetRecvCallbacks(connection, &tcp, 1, [](void* context, const Guid& connectionUid, u8* headerData, void*& outBodyContext, u8*& outBodyData, u32& outBodySize) { return true; }, nullptr, TC(""));
				u8 b = 42;
				NetworkBackend::SendContext sc;
				if (!tcp.Send(logger, connection, &b, 1, sc, TC("")))
					logger.Error(TC("Failed to send"));
				return true;
			}, port))
			return logger.Error(TC("Failed to connect"));

		Sleep(200);
		t.Wait();

		CHECK_TRUEF(!(result != 42), TC("Failed to receive data"));

		return true;
	}

	UBA_TEST(Network, ClientServer)
	{
		auto& logWriter = logger.m_writer;
		NetworkBackendTcp networkBackend(logWriter);

		bool ctorSuccess = true;
		NetworkServer server(ctorSuccess, logWriter);
		NetworkClient client(ctorSuccess, logWriter);

		server.RegisterService(1, [&](const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer)
			{
				logger.Info(TC("Got ping!"));
				UBA_ASSERT(messageInfo.type == SessionMessageType_Ping);
				writer.WriteByte(42);
				return true;
			});

		auto ds = MakeGuard([&]() { networkBackend.StopListen(); server.DisconnectClients(); });
		CHECK_TRUEF(server.StartListen(networkBackend, 1234), TC("Failed to listen"));
		Sleep(100);
		auto dg = MakeGuard([&]() { client.Disconnect(); });
		CHECK_TRUEF(client.Connect(networkBackend, TC("127.0.0.1"), 1234), TC("Failed to connect"));


		StackBinaryWriter<128> writer;
		NetworkMessage msg(client, 1, SessionMessageType_Ping, writer);
		writer.WriteU32(32);
		StackBinaryReader<32> reader;
		CHECK_TRUEF(msg.Send(reader), TC("Failed to get message"));

		u8 value = reader.ReadByte();
		logger.Info(TC("Got value %u"), value);

		CHECK_TRUEF(server.SendKeepAlive(), TC("Failed to send keep alive"));
		Sleep(100);
		return true;
	}

	UBA_TEST(Network, ClientServer2)
	{
		auto& logWriter = logger.m_writer;
		NetworkBackendTcp networkBackend(logWriter);

		bool ctorSuccess = true;
		NetworkServer server(ctorSuccess, logWriter);
		NetworkClient client(ctorSuccess, logWriter);

		server.RegisterService(1, [&](const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer)
			{
				logger.Info(TC("Got ping!"));
				UBA_ASSERT(messageInfo.type == SessionMessageType_Ping);
				writer.WriteByte(42);
				return true;
			});

		CHECK_TRUEF(client.StartListen(networkBackend, 1239), TC("Client failed to listen"));
		Sleep(100);
		auto ds = MakeGuard([&]() { networkBackend.StopListen(); server.DisconnectClients(); });
		auto dg = MakeGuard([&]() { client.Disconnect(); });
		CHECK_TRUEF(server.AddClient(networkBackend, TC("127.0.0.1"), 1239), TC("Server failed to connect"));
		CHECK_TRUEF(server.AddClient(networkBackend, TC("127.0.0.1"), 1239), TC("Server failed to connect second"));

		u64 time = GetTime();
		while (!client.GetConnectionCount())
		{
			CHECK_TRUEF(!(TimeToMs(GetTime() - time) > 4000), TC("Client failed to establish connection"));
			Sleep(100);
		}

		StackBinaryWriter<128> writer;
		NetworkMessage msg(client, 1, SessionMessageType_Ping, writer);
		writer.WriteU32(32);
		StackBinaryReader<32> reader;
		CHECK_TRUEF(msg.Send(reader), TC("Failed to get message"));

		u8 value = reader.ReadByte();
		logger.Info(TC("Got value %u"), value);
		return true;
	}

	UBA_TEST(Network, ClientServerMem)
	{
		auto& logWriter = logger.m_writer;
		NetworkBackendMemory networkBackend(logWriter);

		bool ctorSuccess = true;
		NetworkServer server(ctorSuccess, logWriter);
		NetworkClient client(ctorSuccess, logWriter);

		server.RegisterService(1, [&](const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer)
			{
				logger.Info(TC("Got ping!"));
				UBA_ASSERT(messageInfo.type == SessionMessageType_Ping);
				writer.WriteByte(42);
				return true;
			});

		auto ds = MakeGuard([&]() { server.DisconnectClients(); });
		CHECK_TRUEF(server.StartListen(networkBackend, 1234), TC("Failed to listen"));
		Sleep(100);
		auto dg = MakeGuard([&]() { client.Disconnect(); });
		CHECK_TRUEF(client.Connect(networkBackend, TC("127.0.0.1"), 1234), TC("Failed to connect"));


		StackBinaryWriter<128> writer;
		NetworkMessage msg(client, 1, SessionMessageType_Ping, writer);
		writer.WriteU32(32);
		StackBinaryReader<32> reader;
		CHECK_TRUEF(msg.Send(reader), TC("Failed to get message"));

		u8 value = reader.ReadByte();
		CHECK_TRUEF(!(value != 42), TC("Got wrong value %u, expected 42"), value);
		logger.Info(TC("Got value %u"), value);
		return true;
	}

	UBA_TEST(Network, ClientServerCrypto)
	{
		auto& logWriter = logger.m_writer;
		NetworkBackendTcp networkBackend(logWriter);

		constexpr u8 crypto[] = { 0x00, 0x01, 0x02, 0x03, 0x04, 0x05, 0x06, 0x07, 0x08, 0x09, 0x0a, 0x0b, 0x0c, 0x0d, 0x0e, 0x0f };

		NetworkClientCreateInfo ci(logWriter);
		ci.cryptoKey128 = crypto;

		bool ctorSuccess = true;
		NetworkServer server(ctorSuccess, logWriter);
		NetworkClient client(ctorSuccess, ci);

		server.RegisterCryptoKey(crypto);

		server.RegisterService(1, [&](const ConnectionInfo& connectionInfo, const WorkContext& workContext, MessageInfo& messageInfo, BinaryReader& reader, BinaryWriter& writer)
			{
				logger.Info(TC("Got ping!"));
				CHECK_TRUE(reader.ReadU32() == 32);
				UBA_ASSERT(messageInfo.type == SessionMessageType_Ping);
				writer.WriteByte(42);
				writer.WriteGuid({});
				return true;
			});

		auto ds = MakeGuard([&]() { networkBackend.StopListen(); server.DisconnectClients(); });
		CHECK_TRUEF(server.StartListen(networkBackend, 1234, TC("0.0.0.0"), true), TC("Failed to listen"));
		Sleep(100);
		auto dg = MakeGuard([&]() { client.Disconnect(); });
		CHECK_TRUEF(client.Connect(networkBackend, TC("127.0.0.1"), 1234), TC("Failed to connect"));


		{
			StackBinaryWriter<128> writer;
			NetworkMessage msg(client, 1, SessionMessageType_Ping, writer);
			writer.WriteU32(32);
			writer.WriteGuid({});
			StackBinaryReader<32> reader;
			CHECK_TRUEF(msg.Send(reader), TC("Failed to get message"));
			u8 value = reader.ReadByte();
			CHECK_TRUE(value == 42);
		}

		{
			StackBinaryWriter<128> writer;
			NetworkMessage msg(client, 1, SessionMessageType_Ping, writer);
			writer.WriteU32(32);
			StackBinaryReader<32> reader;
			Event ev(EventResetType_Manual);
			if (!msg.SendAsync(reader, [](bool error, void* userData) { ((Event*)userData)->Set(); }, &ev))
				return logger.Error(TC("Failed to get message"));
			ev.IsSet();
			CHECK_TRUE(msg.ProcessAsyncResults(reader));
			u8 value = reader.ReadByte();
			CHECK_TRUE(value == 42);
		}

		CHECK_TRUEF(server.SendKeepAlive(), TC("Failed to send keep alive"));
		Sleep(100);
		return true;
	}
}
