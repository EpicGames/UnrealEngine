// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaFile.h"
#include "UbaNetworkServer.h"
#include "UbaScheduler.h"
#include "UbaSessionServer.h"
#include "UbaTest.h"

UBA_DEFINE_TEST_GROUP(Scheduler, 140)

namespace uba
{
	UBA_TEST(Scheduler, Local)
	{
		return RunLocal(logger, testRootDir, [](LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
			{
				SchedulerCreateInfo info(session);
				Scheduler scheduler(info);

				ProcessStartInfo processInfo;
				processInfo.application = GetSystemApplication();
				processInfo.workingDir = workingDir;
				processInfo.arguments = GetSystemArguments();

				EnqueueProcessInfo epi(processInfo);
				scheduler.EnqueueProcess(epi);
				scheduler.Start();

				u32 queued, activeLocal, activeRemote, finished;
				do { scheduler.GetStats(queued, activeLocal, activeRemote, finished); } while (finished != 1);

				scheduler.Stop();
				return true;
			});
	}

	bool RunSchedulerReuseTest(LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, bool isLocal)
	{
		SchedulerCreateInfo info(session);
		info.enableProcessReuse = true;
		info.maxLocalProcessors = isLocal ? 1 : 0;
		Scheduler scheduler(info);

		StringBuffer<> testApp;
		GetTestAppPath(logger, testApp);

		ProcessStartInfo processInfo;
		processInfo.application = testApp.data;
		processInfo.workingDir = workingDir;
		processInfo.arguments = TC("-reuse");

		EnqueueProcessInfo epi(processInfo);
		scheduler.EnqueueProcess(epi);
		scheduler.Start();

		u32 queued, activeLocal, activeRemote, finished;
		bool addSecond = true;
		do
		{
			scheduler.GetStats(queued, activeLocal, activeRemote, finished);
			if (finished == 1 && addSecond)
			{
				Sleep(100);
				scheduler.EnqueueProcess(epi);
				addSecond = false;
			}
		} while (finished != 2);

		if (!isLocal)
			session.GetServer().DisconnectClients();

		scheduler.Stop();

		return true;
	}

	UBA_TEST(Scheduler, LocalReuse)
	{
		return RunLocal(logger, testRootDir, [](LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
			{
				return RunSchedulerReuseTest(logger, session, workingDir, true);
			});
	}

	UBA_TEST(Scheduler, RemoteReuse)
	{
		return RunRemote(logger, testRootDir, [](LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
			{
				return RunSchedulerReuseTest(logger, session, workingDir, false);
			});
	}
}
