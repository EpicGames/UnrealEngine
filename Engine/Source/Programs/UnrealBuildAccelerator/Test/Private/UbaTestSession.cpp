// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaConfig.h"
#include "UbaEnvironment.h"
#include "UbaFileAccessor.h"
#include "UbaNetworkBackendTcp.h"
#include "UbaProcess.h"
#include "UbaSessionServer.h"
#include "UbaSessionClient.h"
#include "UbaStorageServer.h"
#include "UbaStorageClient.h"
#include "UbaTest.h"

UBA_DEFINE_TEST_GROUP(Session, 150)

namespace uba
{
	u32 GetTimeoutTime()
	{
		#if PLATFORM_WINDOWS
		if (IsDebuggerPresent())
			return 10'000'000;
		#endif
		return 10'000;
	}

	bool RunLocal(LoggerWithWriter& logger, const StringBufferBase& testRootDir, const Config& config, const TestSessionFunction& testFunc, uintptr_t extraData = 0, bool enableDetour = true)
	{
		LogWriter& logWriter = logger.m_writer;

		StringBuffer<MaxPath> rootDir;
		rootDir.Append(testRootDir).Append(TCV("Uba"));
		CHECK_TRUE(DeleteAllFiles(logger, rootDir.data));

		bool ctorSuccess = true;
		NetworkServer server(ctorSuccess, { logWriter });

		StorageCreateInfo storageInfo(rootDir.data, logWriter, server);
		storageInfo.casCapacityBytes = 1024ull * 1024 * 1024;
		storageInfo.Apply(config);
		StorageImpl storage(storageInfo);

		SessionServerCreateInfo sessionServerInfo(storage, server, logWriter);
		sessionServerInfo.rootDir = rootDir.data;
		sessionServerInfo.readIntermediateFilesCompressed = true;
		sessionServerInfo.Apply(config);

		#if UBA_DEBUG_LOG_ENABLED
		sessionServerInfo.logToFile = true;
		#endif

		SessionServer session(sessionServerInfo);

		StringBuffer<MaxPath> workingDir;
		workingDir.Append(testRootDir).Append(TCV("WorkingDir"));
		CHECK_TRUE(DeleteAllFiles(logger, workingDir.data));

		CHECK_TRUE(storage.CreateDirectory(workingDir.data));
		CHECK_TRUE(DeleteAllFiles(logger, workingDir.data, false));
		workingDir.EnsureEndsWithSlash();
		return testFunc(logger, session, workingDir.data, [&](ProcessStartInfo& pi) { return session.RunProcess(pi, true, enableDetour); }, extraData);
	}

	bool RunLocal(LoggerWithWriter& logger, const StringBufferBase& testRootDir, const TestSessionFunction& testFunc, uintptr_t extraData, bool enableDetour)
	{
		return RunLocal(logger, testRootDir, {}, testFunc, extraData, enableDetour);
	}

	void GetTestAppPath(LoggerWithWriter& logger, StringBufferBase& out)
	{
		GetDirectoryOfCurrentModule(logger, out);
		out.EnsureEndsWithSlash();

		// UBA_TEST_STATIC=1 routes through patched static binary (catches LD_PRELOAD vs static-stub divergences).
		#if PLATFORM_LINUX
		{
			tchar buf[8] = {};
			if (GetEnvironmentVariableW(TC("UBA_TEST_STATIC"), buf, sizeof_array(buf)) && buf[0] == '1')
			{
				out.Append(TC("UbaTestAppStatic"));
				return;
			}
		}
		#endif

		out.Append(IsWindows ? TC("UbaTestApp.exe") : TC("UbaTestApp"));
	}

	bool RunTestApp(LoggerWithWriter& logger, const tchar* workingDir, const RunProcessFunction& runProcess, const tchar* arguments, ProcessHandle* outHandle = nullptr, const tchar* overlayFiles = TC(""), u32 expectedExitCode = 0)
	{
		StringBuffer<> testApp;
		GetTestAppPath(logger, testApp);

		ProcessStartInfo processInfo;
		processInfo.application = testApp.data;
		processInfo.workingDir = workingDir;
		processInfo.arguments = arguments;
		processInfo.overlayFiles = overlayFiles;
		processInfo.logLineFunc = [](void* userData, const tchar* line, u32 length, LogEntryType type)
			{
				LoggerWithWriter(g_consoleLogWriter, TC("")).Info(line);
			};

		ProcessHandle process = runProcess(processInfo);
		CHECK_TRUEF(process.IsValid(), TC("Failed to run UbaTestApp"));
		CHECK_TRUEF(process.WaitForExit(GetTimeoutTime()), TC("UbaTestApp did not exit in 10 seconds"));
		if (outHandle)
			*outHandle = process;
		u32 exitCode = process.GetExitCode();
		if (exitCode == expectedExitCode)
			return true;
		for (auto& logLine : process.GetLogLines())
			logger.Error(logLine.text.c_str());
		return logger.Error(TC("UbaTestApp returned exit code %i"), int(exitCode));
	}

	using TestServerSessionFunction = Function<bool(LoggerWithWriter& logger, const StringView& workingDir, SessionServer& sessionServer)>;
	bool SetupServerSession(LoggerWithWriter& logger, const StringBufferBase& testRootDir, bool deleteAll, bool serverShouldListen, const Config& serverConfig, const TestServerSessionFunction& testFunc)
	{
		LogWriter& logWriter = logger.m_writer;
		NetworkBackendTcpCreateInfo tcpInfo{logWriter};
		tcpInfo.Apply(serverConfig);
		NetworkBackendTcp tcpBackend(tcpInfo);

		bool ctorSuccess = true;
		NetworkServer server(ctorSuccess, { logWriter });

		StringBuffer<MaxPath> rootDir;
		rootDir.Append(testRootDir).Append(TCV("Uba"));

		StringBuffer<MaxPath> toDelete(rootDir);
		if (!deleteAll)
			toDelete.Append(PathSeparator).Append(TCV("sessions"));
		CHECK_TRUE(DeleteAllFiles(logger, toDelete.data));

		StorageServerCreateInfo storageServerInfo(server, rootDir.data, logWriter);
		storageServerInfo.casCapacityBytes = 1024ull * 1024 * 1024;
		storageServerInfo.Apply(serverConfig);
		auto& storageServer = *new StorageServer(storageServerInfo);
		auto ssg = MakeGuard([&]() { delete &storageServer; });

		SessionServerCreateInfo sessionServerInfo(storageServer, server, logWriter);
		sessionServerInfo.rootDir = rootDir.data;
		sessionServerInfo.useUniqueId = false;

		#if UBA_DEBUG_LOG_ENABLED
		sessionServerInfo.logToFile = true;
		sessionServerInfo.remoteLogEnabled = true;
		#endif
		sessionServerInfo.Apply(serverConfig);

		auto& sessionServer = *new SessionServer(sessionServerInfo);
		auto ssg2 = MakeGuard([&]() { delete &sessionServer; });

		auto sg = MakeGuard([&]() { server.DisconnectClients(); });

		sessionServer.SetRemoteProcessReturnedEvent([](Process& p) { p.Cancel(); });

		Config clientConfig;
		clientConfig.AddTable(TC("Storage")).AddValue(TC("CheckExistsOnServer"), true);
		server.SetClientsConfig(clientConfig);

		StringBuffer<MaxPath> workingDir;
		workingDir.Append(testRootDir).Append(TCV("WorkingDir"));
		CHECK_TRUE(!deleteAll || DeleteAllFiles(logger, workingDir.data));
		CHECK_TRUE(storageServer.CreateDirectory(workingDir.data));
		CHECK_TRUE(!deleteAll || DeleteAllFiles(logger, workingDir.data, false));

		workingDir.EnsureEndsWithSlash();
		return testFunc(logger, workingDir, sessionServer);
	}

	using TestClientSessionFunction = Function<bool(LoggerWithWriter& logger, SessionClient& sessionClient)>;
	bool SetupClientSession(LoggerWithWriter& logger, const StringBufferBase& testRootDir, bool deleteAll, bool serverShouldListen, NetworkServer& server, u16 port, const TestClientSessionFunction& testFunc)
	{
		Config serverConfig;


		LogWriter& logWriter = logger.m_writer;
		NetworkBackendTcpCreateInfo tcpInfo{logWriter};
		tcpInfo.Apply(serverConfig);
		NetworkBackendTcp tcpBackend(tcpInfo);

		bool ctorSuccess = true;
		NetworkClient client(ctorSuccess, { logWriter });

		if (serverShouldListen)
		{
			CHECK_TRUEF(server.StartListen(tcpBackend, port), TC("Failed to start listen"));
			CHECK_TRUEF(client.Connect(tcpBackend, TC("127.0.0.1"), port), TC("Failed to connect"));
			CHECK_TRUEF(client.Connect(tcpBackend, TC("127.0.0.1"), port), TC("Failed to connect"));
		}
		else
		{
			CHECK_TRUEF(client.StartListen(tcpBackend, port), TC("Failed to listen"));
			CHECK_TRUEF(server.AddClient(tcpBackend, TC("127.0.0.1"), port), TC("Failed to connect"));
			while (!client.IsConnected())
				Sleep(1);
		}
		auto disconnectGuard = MakeGuard([&]() { tcpBackend.StopListen(); client.Disconnect(); server.RemoveDisconnectedConnections(); });

		Config config;
		CHECK_TRUEF(client.FetchConfig(config), TC("Failed to fetch config"));

		StringBuffer<MaxPath> rootDir;
		rootDir.Append(testRootDir).Append(TCV("UbaClient")).AppendValue(port);

		StringBuffer<MaxPath> toDelete(rootDir);
		if (!deleteAll)
			toDelete.Append(PathSeparator).Append(TCV("sessions"));
		CHECK_TRUEF(DeleteAllFiles(logger, toDelete.data), TC("Failed to delete all files"));

		StorageClientCreateInfo storageClientInfo(client, rootDir.data);
		storageClientInfo.Apply(config);
		auto& storageClient = *new StorageClient(storageClientInfo);
		auto scg = MakeGuard([&]() { delete &storageClient; });

		SessionClientCreateInfo sessionClientInfo(storageClient, client, logWriter);
		sessionClientInfo.rootDir = rootDir.data;
		sessionClientInfo.useUniqueId = false;
		//sessionClientInfo.allowKeepFilesInMemory = false;

		#if UBA_DEBUG_LOG_ENABLED
		sessionClientInfo.logToFile = true;
		#endif

		auto& sessionClient = *new SessionClient(sessionClientInfo);
		auto scg2 = MakeGuard([&]() { delete &sessionClient; });

		auto cg = MakeGuard([&]() { sessionClient.Stop(); disconnectGuard.Execute(); });

		storageClient.Start();
		sessionClient.Start();
		return testFunc(logger, sessionClient);
	}

	using TestServerClientSessionFunction = Function<bool(LoggerWithWriter& logger, const StringView& workingDir, SessionServer& sessionServer, SessionClient& sessionClient)>;
	bool SetupServerClientSession(LoggerWithWriter& logger, const StringBufferBase& testRootDir, bool deleteAll, bool serverShouldListen, const Config& serverConfig, const TestServerClientSessionFunction& testFunc)
	{
		return SetupServerSession(logger, testRootDir, deleteAll, serverShouldListen, serverConfig, [&](LoggerWithWriter& logger, const StringView& workingDir, SessionServer& sessionServer)
			{
				return SetupClientSession(logger, testRootDir, deleteAll, serverShouldListen, sessionServer.GetServer(), 1356, [&](LoggerWithWriter& logger, SessionClient& sessionClient)
					{
						return testFunc(logger, workingDir, sessionServer, sessionClient);
					});
			});
	}

	bool RunRemote(LoggerWithWriter& logger, const StringBufferBase& testRootDir, const Config& serverConfig, const TestSessionFunction& testFunc, uintptr_t extraData = 0, bool deleteAll = true, bool serverShouldListen = true)
	{
		return SetupServerClientSession(logger, testRootDir, deleteAll, serverShouldListen, serverConfig,
			[&](LoggerWithWriter& logger, const StringView& workingDir, SessionServer& sessionServer, SessionClient& sessionClient)
			{
				return testFunc(logger, sessionServer, workingDir.data, [&](ProcessStartInfo& pi) { return sessionServer.RunProcessRemote(pi); }, extraData);
			});
	}

	bool RunRemote(LoggerWithWriter& logger, const StringBufferBase& testRootDir, const TestSessionFunction& testFunc, bool deleteAll, bool serverShouldListen)
	{
		return RunRemote(logger, testRootDir, {}, testFunc, 0, deleteAll, serverShouldListen);
	}

	template<typename CharType>
	bool CreateTextFile(StringBufferBase& outPath, LoggerWithWriter& logger, const tchar* workingDir, const tchar* fileName, const CharType* text)
	{
		outPath.Clear().Append(workingDir).EnsureEndsWithSlash().Append(fileName);
		FileAccessor fr(logger, outPath.data);
		CHECK_TRUE(fr.CreateWrite());
		u32 len = 0;
		for (const CharType* it=text; *it; ++it)
			++len;
		fr.Write(text, (len + 1)*sizeof(CharType));
		return fr.Close();
	}

	bool SetupForTestApp(LoggerWithWriter& logger, const tchar* workingDir)
	{
		StringBuffer<MaxPath> fileR;
		CHECK_TRUE(CreateTextFile(fileR, logger, workingDir, TC("FileR.h"), "Foo"));

		{
			StringBuffer<MaxPath> dir;
			dir.Append(workingDir).Append(TCV("Dir1"));
			CHECK_TRUEF(CreateDirectoryW(dir.data), TC("Failed to create dir %s"), dir.data);

			dir.Clear().Append(workingDir).Append(TCV("Dir2"));
			CHECK_TRUEF(CreateDirectoryW(dir.data), TC("Failed to create dir %s"), dir.data);
			dir.EnsureEndsWithSlash().Append(TCV("Dir3"));
			CHECK_TRUEF(CreateDirectoryW(dir.data), TC("Failed to create dir %s"), dir.data);
			dir.EnsureEndsWithSlash().Append(TCV("Dir4"));
			CHECK_TRUEF(CreateDirectoryW(dir.data), TC("Failed to create dir %s"), dir.data);
			dir.EnsureEndsWithSlash().Append(TCV("Dir5"));
			CHECK_TRUEF(CreateDirectoryW(dir.data), TC("Failed to create dir %s"), dir.data);
		}

		CHECK_TRUE(CreateTestFile(logger, ToView(workingDir), TCV("File4.out"), TCV("0")));
		CHECK_TRUE(CreateTestFile(logger, ToView(workingDir), TCV("File4.obj"), TCV("9")));
		CHECK_TRUE(CreateTestFile(logger, ToView(workingDir), TCV("File4.lib"), TCV("9")));
		return true;
	}

	bool FinishForTestApp(LoggerWithWriter& logger, const tchar* workingDir)
	{
		StringBuffer<MaxPath> fileW2;
		fileW2.Append(workingDir).Append(TCV("FileW2"));
		CHECK_TRUEF(FileExists(logger, fileW2.data), TC("Can't find file %s"), fileW2.data);
		StringBuffer<MaxPath> fileWF;
		fileWF.Append(workingDir).Append(TCV("FileWF"));
		CHECK_TRUEF(FileExists(logger, fileWF.data), TC("Can't find file %s"), fileWF.data);
		return true;
	}

	bool RunTestAppTests(LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
	{
		CHECK_TRUE(SetupForTestApp(logger, workingDir));
		CHECK_TRUE(RunTestApp(logger, workingDir, runProcess, TC("")));
		CHECK_TRUE(FinishForTestApp(logger, workingDir));
		return true;
	}

#if PLATFORM_MAC
	bool ExecuteCommand(LoggerWithWriter& logger, const tchar* command, StringBufferBase& commandOutput)
	{
		FILE* fpCommand = popen(command, "r");
		if (fpCommand == nullptr || fgets(commandOutput.data, commandOutput.capacity, fpCommand) == nullptr || pclose(fpCommand) != 0)
		{
			logger.Warning("Failed to run command '%s' or get a response", command);
			return false;
		}

		commandOutput.count = strlen(commandOutput.data);
		while (isspace(commandOutput.data[commandOutput.count-1]))
		{
			commandOutput.data[commandOutput.count-1] = 0;
			commandOutput.count--;
		}
		return true;
	}
#endif

	bool RunClang(LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
	{
		StringBuffer<MaxPath> sourceFile;
		sourceFile.Append(workingDir).Append(TCV("Code.cpp"));
		FileAccessor codeFile(logger, sourceFile.data);
		CHECK_TRUE(codeFile.CreateWrite());
		char code[] = "#include <stdio.h>\n int main() { printf(\"Hello world\\n\"); return 0; }";
		CHECK_TRUE(codeFile.Write(code, sizeof(code) - 1));
		CHECK_TRUE(codeFile.Close());

#if PLATFORM_WINDOWS
		const tchar* clangPath = TC("c:\\Program Files\\Microsoft Visual Studio\\2022\\Professional\\VC\\Tools\\Llvm\\bin\\clang-cl.exe");
#elif PLATFORM_MAC
		StringBuffer<MaxPath> xcodePath;
		const tchar* clangPath;
		if (IsRunningDarling())
		{
			// xcrun does not work in darling
			clangPath = "/Volumes/SystemRoot/usr/bin/clang++";
		}
		else
		{
			if (!ExecuteCommand(logger, "/usr/bin/xcrun --find clang++", xcodePath))
				return true;
			clangPath = xcodePath.data;
		}
#else
		const tchar* clangPath = TC("/usr/bin/clang++");
#endif

		if (!FileExists(logger, clangPath)) // Skipping if clang is not installed.
			return true;

		ProcessStartInfo processInfo;
		processInfo.application = clangPath;

		StringBuffer<MaxPath> args;

#if PLATFORM_WINDOWS
		args.Append("/Brepro ");
#elif PLATFORM_MAC
		StringBuffer<MaxPath> xcodeSDKPath;
		if (!IsRunningDarling())
		{
			if (!ExecuteCommand(logger, "xcrun --show-sdk-path", xcodeSDKPath))
				return true;
			args.Append("-isysroot ");
			args.Append(xcodeSDKPath.data).Append(' ');
		}
#endif
		args.Append(TCV("-o code Code.cpp"));

		processInfo.arguments = args.data;

		processInfo.workingDir = workingDir;
		//processInfo.logFile = TC("/mnt/e/temp/ttt/RunClang.log");
		ProcessHandle process = runProcess(processInfo);
		CHECK_TRUEF(process.WaitForExit(GetTimeoutTime()), TC("clang++ timed out"));
		u32 exitCode = process.GetExitCode();
		CHECK_TRUEF(!(exitCode != 0), TC("clang++ returned exit code %u"), exitCode);
		return true;
	}

	bool RunCustomService(LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
	{
		bool gotMessage = false;
		session.RegisterCustomService([&](uba::Process& process, const void* recv, u32 recvSize, void* send, u32 sendCapacity)
			{
				gotMessage = true;
				//wprintf(L"GOT MESSAGE: %.*s\n", recvSize / 2, (const wchar_t*)recv);
				const wchar_t* hello = L"Hello response from server";
				u64 helloBytes = wcslen(hello) * 2;
				memcpy(send, hello, helloBytes);
				return u32(helloBytes);
			});

		CHECK_TRUE(RunTestApp(logger, workingDir, runProcess, TC("Whatever")));
		CHECK_TRUEF(gotMessage, TC("Never got message from UbaTestApp"));
		return true;
	}

	// NOTE: This test is dependent on the UbaTestApp<Platform>
	// The purpose of this test is to validate that the platform specific detours are
	// working as expected.
	// Before running the actual UbaTestApp, RunLocal calls through a variety of functions
	// that sets up the various UbaSession Servers, Clients, etc. It creates some temporary
	// directories, e.g. Dir1 and eventually call ProcessImpl::InternalCreateProcess.
	// InternalCreateProcess will setup the shared memory, inject the Detour library
	// and setup any other necessary environment variables, and spawn the actual process
	// (in this case the UbaTestApp)
	// Once UbaTestApp has started, it will first check and validate that the detour library
	// is in the processes address space. With the detour in place, the test app will
	// exercise various file functions which will actually go through our detour library.
	UBA_TEST(Session, DetouredTestApp)
	{
		return RunLocal(logger, testRootDir, RunTestAppTests);
	}

	UBA_TEST(Session, DetouredTestAppRemote)
	{
		return RunRemote(logger, testRootDir, RunTestAppTests);
	}

	bool RunSnapshotTestAppTests(LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
	{
		CHECK_TRUE(SetupForTestApp(logger, workingDir));
		CHECK_TRUE(session.RefreshDirectory(workingDir, true));
		CHECK_TRUE(session.UpdateDirTableSnapshot(true));
		CHECK_TRUE(RunTestApp(logger, workingDir, runProcess, TC("")));
		CHECK_TRUE(FinishForTestApp(logger, workingDir));
		return true;
	}

	UBA_TEST(Session, DetouredSnapshot)
	{
		Config config;
		config.AddTable(TC("Session")).AddValue(TC("DirTableSnapshotEnabled"), true);
		return RunLocal(logger, testRootDir, config, RunSnapshotTestAppTests);
	}

	UBA_TEST(Session, DetouredSnapshotRemote)
	{
		Config config;
		config.AddTable(TC("Session")).AddValue(TC("DirTableSnapshotEnabled"), true);
		return RunRemote(logger, testRootDir, config, RunSnapshotTestAppTests);
	}

	UBA_TEST_IF(Session, CustomService, IsWindows)
	{
		return RunRemote(logger, testRootDir, RunCustomService);
	}

	UBA_TEST(Session, DetouredClang)
	{
		return RunLocal(logger, testRootDir, RunClang);
	}

	UBA_TEST_IF(Session, DetouredClangRemote, !IsMac)
	{
		// Run twice to test LoadCasTable/SaveCasTable etc
		CHECK_TRUE(RunRemote(logger, testRootDir, RunClang));
		return RunRemote(logger, testRootDir, RunClang, false);
	}

	UBA_TEST_IF(Session, DetouredTouch, IsLinux)
	{
		return RunLocal(logger, testRootDir, [](LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
			{
				StringBuffer<> file;
				file.Append(workingDir).Append(TCV("TouchFile.h"));
				FileAccessor fr(logger, file.data);

				CHECK_TRUE(fr.CreateWrite());
				CHECK_TRUE(fr.Write("Foo", 4));
				CHECK_TRUE(fr.Close());
				FileInformation oldInfo;
				CHECK_TRUE(GetFileInformation(oldInfo, logger, file));

				Sleep(100);

				ProcessStartInfo processInfo;
				processInfo.application = TC("/usr/bin/touch");
				processInfo.workingDir = workingDir;
				processInfo.arguments = file.data;
				processInfo.logFile = TC("/home/honk/Touch.log");
				ProcessHandle process = runProcess(processInfo);
				CHECK_TRUEF(process.WaitForExit(GetTimeoutTime()), TC("UbaTestApp did not exit in 10 seconds"));
				u32 exitCode = process.GetExitCode();
				CHECK_TRUE(exitCode == 0);

				FileInformation newInfo;
				CHECK_TRUE(GetFileInformation(newInfo, logger, file));
				CHECK_TRUEF(!(newInfo.lastWriteTime == oldInfo.lastWriteTime), TC("File time not changed after touch"));
				return true;
			});
	}

	UBA_TEST_IF(Session, DetouredPopen, IsLinux)
	{
		#if PLATFORM_LINUX
		return RunLocal(logger, testRootDir, [](LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
			{
				StringBuffer<> testApp;
				GetTestAppPath(logger, testApp);

				ProcessStartInfo processInfo;
				processInfo.application = testApp.data;
				processInfo.workingDir = workingDir;
				processInfo.arguments = "-popen";
				processInfo.logLineFunc = [](void* userData, const tchar* line, u32 length, LogEntryType type)
					{
						LoggerWithWriter(g_consoleLogWriter, TC("")).Info(line);
					};

				ProcessHandle process = runProcess(processInfo);
				CHECK_TRUEF(process.WaitForExit(GetTimeoutTime()), TC("UbaTestApp did not exit in 10 seconds"));
				u32 exitCode = process.GetExitCode();

				if (exitCode != 0)
				{
					for (auto& logLine : process.GetLogLines())
						logger.Error(logLine.text.c_str());
					return logger.Error(TC("UbaTestApp returned exit code %i"), int(exitCode));
				}
				return true;
			});
		#else
		return true;
		#endif
	}

	const tchar* GetSystemApplication()
	{
		#if PLATFORM_WINDOWS
		return TC("c:\\windows\\system32\\ping.exe");
		#elif PLATFORM_LINUX
		return TC("/usr/bin/cat");
		#else
		return TC("/usr/bin/zip");
		#endif
	}

	const tchar* GetSystemArguments()
	{
		#if PLATFORM_WINDOWS
		return TC("-n 1 localhost");
		#elif PLATFORM_LINUX
		return TC("--help");
		#else
		return TC("-help");
		#endif
	}

	const tchar* GetSystemExpectedLogLine()
	{
		#if PLATFORM_WINDOWS
		return TC("Pinging ");
		#elif PLATFORM_LINUX
		return TC("cat [OPTION]");
		#else
		return TC("zip [-options]");
		#endif
	}

	UBA_TEST_IF(Session, MultipleDetouredProcesses, !IsMac)
	{
		return RunLocal(logger, testRootDir, [](LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
			{
				ProcessStartInfo processInfo;
				processInfo.application = GetSystemApplication();
				processInfo.workingDir = workingDir;
				processInfo.arguments = GetSystemArguments();
				//processInfo.logFile = TC("e:\\temp\\ttt\\LogFile.log");
				Vector<ProcessHandle> processes;

				for (u32 i=0; i!=50; ++i)
					processes.push_back(runProcess(processInfo));

				for (auto& process : processes)
				{
					CHECK_TRUEF(process.WaitForExit(GetTimeoutTime()), TC("UbaTestApp did not exit in 10 seconds"));
					u32 exitCode = process.GetExitCode();
					CHECK_TRUEF(!(exitCode != 0), TC("UbaTestApp exited with code %u"), exitCode);
				}

				return true;
			});
	}

	bool RunSystemApplicationAndLookForLog(LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
	{
		ProcessStartInfo processInfo;
		processInfo.application = GetSystemApplication();
		processInfo.workingDir = workingDir;
		processInfo.arguments = GetSystemArguments();

		bool foundPingString = false;
		processInfo.logLineUserData = &foundPingString;
		processInfo.logLineFunc = [](void* userData, const tchar* line, u32 length, LogEntryType type)
			{
				*(bool*)userData |= Contains(line, GetSystemExpectedLogLine());
			};

		ProcessHandle process = runProcess(processInfo);

		CHECK_TRUEF(process.WaitForExit(GetTimeoutTime()), TC("UbaTestApp did not exit in 10 seconds"));
		u32 exitCode = process.GetExitCode();
		CHECK_TRUEF(!(exitCode != 0), TC("Got exit code %u"), exitCode);
		CHECK_TRUEF(foundPingString, TC("Did not log string containing \"%s\""), GetSystemExpectedLogLine());
		return true;
	}

	UBA_TEST_IF(Session, LogLines, !IsMac)
	{
		return RunLocal(logger, testRootDir, RunSystemApplicationAndLookForLog);
	}

	UBA_TEST_IF(Session, LogLinesNoDetour, !IsMac)
	{
		return RunLocal(logger, testRootDir, RunSystemApplicationAndLookForLog, 0, false);
	}

	bool CheckAttributes(LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
	{
		StringBuffer<MaxPath> testApp;
		GetTestAppPath(logger, testApp);
		ProcessStartInfo processInfo;
		processInfo.application = testApp.data;
		processInfo.workingDir = workingDir;
		processInfo.logLineFunc = [](void* userData, const tchar* line, u32 length, LogEntryType type)
			{
				LoggerWithWriter(g_consoleLogWriter, TC("")).Info(line);
			};

		auto GetAttributes = [&](const StringView& file) -> u32
			{
				StringBuffer<> arg(TC("-getFileAttributes="));
				arg.Append(file);
				processInfo.arguments = arg.data;
				ProcessHandle process = runProcess(processInfo);
				CHECK_TRUEF(process.WaitForExit(GetTimeoutTime()), TC("UbaTestApp did not exit in 10 seconds"));
				u32 exitCode =  process.GetExitCode();
				return exitCode == 255 ? INVALID_FILE_ATTRIBUTES : exitCode;
			};

		MemoryBlock temp(TC("CheckAttributes"));
		DirectoryTable dirTable(temp);
		dirTable.Init(session.GetDirectoryTableMemory(), 0, 0);

		CHECK_TRUE(session.RefreshDirectory(workingDir, true));
		CHECK_TRUE(session.RefreshDirectory(workingDir));
		CHECK_TRUE(dirTable.EntryExists(ToView(workingDir)) == DirectoryTable::Exists_Maybe);
		dirTable.ParseDirectoryTable(session.GetDirectoryTableSize(nullptr));
		CHECK_TRUE(dirTable.EntryExists(ToView(workingDir), true) == DirectoryTable::Exists_Yes);

		StringBuffer<MaxPath> sourceFile;
		sourceFile.Append(workingDir).Append(TCV("Code.cpp"));

		CHECK_TRUE(GetAttributes(sourceFile) == INVALID_FILE_ATTRIBUTES);
		FileAccessor codeFile(logger, sourceFile.data);
		CHECK_TRUE(codeFile.CreateWrite());
		CHECK_TRUE(codeFile.Close());
		CHECK_TRUE(session.RegisterNewFile(sourceFile.data));
		CHECK_TRUE(GetAttributes(sourceFile) != INVALID_FILE_ATTRIBUTES);

		CHECK_TRUE(dirTable.EntryExists(sourceFile) == DirectoryTable::Exists_No);
		dirTable.ParseDirectoryTable(session.GetDirectoryTableSize(nullptr));
		CHECK_TRUE(dirTable.EntryExists(sourceFile) == DirectoryTable::Exists_Yes);

		StringBuffer<MaxPath> newDir;
		newDir.Append(workingDir).Append(TCV("NewDir"));
		StringBuffer<MaxPath> newDirAndSlash(newDir);
		newDirAndSlash.Append('/');

		CHECK_TRUE(GetAttributes(newDir) == INVALID_FILE_ATTRIBUTES);
		CHECK_TRUE(CreateDirectoryW(newDir.data));
		CHECK_TRUE(session.RegisterNewFile(newDir.data));
		CHECK_TRUE(dirTable.EntryExists(newDir) == DirectoryTable::Exists_No);
		dirTable.ParseDirectoryTable(session.GetDirectoryTableSize(nullptr));
		CHECK_TRUE(dirTable.EntryExists(newDir) == DirectoryTable::Exists_Yes);
		CHECK_TRUE(GetAttributes(newDir) != INVALID_FILE_ATTRIBUTES);
		CHECK_TRUE(GetAttributes(newDirAndSlash) != INVALID_FILE_ATTRIBUTES);

		StringBuffer<MaxPath> newDir2;
		newDir2.Append(workingDir).Append(TCV("NewDir2"));
		CHECK_TRUE(CreateDirectoryW(newDir2.data));
		CHECK_TRUE(GetAttributes(newDir2) == INVALID_FILE_ATTRIBUTES);
		CHECK_TRUE(session.RefreshDirectory(workingDir));
		CHECK_TRUE(GetAttributes(newDir2) != INVALID_FILE_ATTRIBUTES);
		dirTable.ParseDirectoryTable(session.GetDirectoryTableSize(nullptr));
		CHECK_TRUE(dirTable.EntryExists(newDir2) == DirectoryTable::Exists_Yes);

		return true;
	}

	UBA_TEST(Session, RegisterChanges)
	{
		return RunLocal(logger, testRootDir, CheckAttributes);
	}

	UBA_TEST(Session, RegisterChangesRemote)
	{
		return RunRemote(logger, testRootDir, CheckAttributes);
	}

	UBA_TEST(Session, SharedReservedMemory)
	{
		return RunLocal(logger, testRootDir, [](LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
			{
				StringBuffer<MaxPath> testApp;
				GetTestAppPath(logger, testApp);

				ProcessStartInfo processInfo;
				processInfo.application = testApp.data;
				processInfo.workingDir = workingDir;
				processInfo.arguments = TC("sleep=50");
				Vector<ProcessHandle> processes;

				for (u32 i=0; i!=16; ++i)
					processes.push_back(runProcess(processInfo));

				u32 idx = 0;
				for (auto& process : processes)
				{
					CHECK_TRUEF(process.WaitForExit(GetTimeoutTime()), TC("UbaTestApp[%u] did not exit"), idx);
					u32 exitCode = process.GetExitCode();
					if (exitCode != 0)
					{
						for (auto& logLine : process.GetLogLines())
							logger.Error(logLine.text.c_str());
						return logger.Error(TC("UbaTestApp[%u] returned exit code %u"), idx, exitCode);
					}
					++idx;
				}

				return true;
			});
	}


	UBA_TEST(Session, DirectoryTableRemote)
	{
#if 0
		return SetupServerClientSession(logger, testRootDir, true, true, {},
			[&](LoggerWithWriter& logger, const StringView& workingDir, SessionServer& sessionServer, SessionClient& sessionClient)
			{
				u32 attributes;
				#if PLATFORM_WINDOWS
				CHECK_TRUE(sessionClient.Exists(TCV("c:\\"), attributes));
				CHECK_TRUE(sessionClient.Exists(TCV("c:\\windows"), attributes));
				CHECK_TRUE(IsDirectory(attributes));
				CHECK_TRUE(!sessionClient.Exists(TCV("q:\\"), attributes))
				CHECK_TRUE(!sessionClient.Exists(TCV("r:\\foo"), attributes))
				#else
				CHECK_TRUE(!sessionClient.Exists(TCV("/ergewrgergreg"), attributes));
				CHECK_TRUE(!sessionClient.Exists(TCV("/ergergreg/h5r6tyh"), attributes));
				#endif
				return true;
			});
#else
		return true;
#endif
	}

	struct RunCreateVirtualFileTestData
	{
		bool remote = false;
		bool transient = false;
		bool registerToDir = true;
	};

	bool RunCreateVirtualFileTest(LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
	{
		auto testData = *(RunCreateVirtualFileTestData*)extraData;

		StringBuffer<> inFile(workingDir);
		inFile.Append(TCV("VirtualFile.in"));
		CHECK_TRUE(session.CreateVirtualFile(inFile.data, "FOO", 3, testData.transient, testData.registerToDir));
		ProcessHandle ph;


		auto runProcess2 = [&](ProcessStartInfo& pi)
			{
				StringBuffer<> overlayFiles;
				if (!testData.registerToDir)
				{
					overlayFiles.Append(inFile.data).Append(';').Append(inFile.data);
					pi.overlayFiles = overlayFiles.data;
				}
				return testData.remote ? session.RunProcessRemote(pi) : session.RunProcess(pi, true, true);
			};

		CHECK_TRUE(RunTestApp(logger, workingDir, runProcess2, TC("-virtualFile"), &ph));

		StringBuffer<> outFile(workingDir);
		outFile.Append(TCV("VirtualFile.out"));

		bool success = false;
		ph.TraverseOutputFiles([&](StringView file) { success = file.Equals(outFile); });
		CHECK_TRUE(success);

		CHECK_TRUE(session.DeleteVirtualFile(inFile.data, testData.registerToDir));

		CHECK_TRUE(!FileExists(logger, outFile.data));
		u64 outSize;
		CHECK_TRUE(session.GetOutputFileSize(outSize, outFile.data));
		CHECK_TRUE(outSize == 3);
		u8 data[3];
		CHECK_TRUE(session.GetOutputFileData(data, outFile.data, false));
		CHECK_TRUE(memcmp(data, "BAR", 3) == 0);
		CHECK_TRUE(!FileExists(logger, outFile.data));
		CHECK_TRUE(session.WriteOutputFile(outFile.data, true));
		CHECK_TRUE(FileExists(logger, outFile.data));

		if (ph.IsRemote())
			CHECK_TRUE(session.GetStorage().DeleteCasForFile(inFile.data));

		CHECK_TRUE(!session.GetOutputFileSize(outSize, outFile.data));
		CHECK_TRUE(!session.GetOutputFileData(data, outFile.data, true));
		return true;
	}

	UBA_TEST_IF(Session, CreateVirtualFile, IsWindows)
	{
		Config config;
		config.AddTable(TC("Session")).AddValue(TC("ShouldWriteToDisk"), false);

		RunCreateVirtualFileTestData testData { false };
		CHECK_TRUE(RunLocal(logger, testRootDir, config, RunCreateVirtualFileTest, uintptr_t(&testData)));

		testData.transient = true;
		CHECK_TRUE(RunLocal(logger, testRootDir, config, RunCreateVirtualFileTest, uintptr_t(&testData)));

		testData.registerToDir = false;
		CHECK_TRUE(RunLocal(logger, testRootDir, config, RunCreateVirtualFileTest, uintptr_t(&testData)));
		return true;
	}

	UBA_TEST_IF(Session, CreateVirtualFileRemote, IsWindows)
	{
		Config config;
		config.AddTable(TC("Session")).AddValue(TC("ShouldWriteToDisk"), false);
		config.AddTable(TC("Storage")).AddValue(TC("CreateIndependentMappings"), true);

		RunCreateVirtualFileTestData testData { true };

		testData.transient = true;
		CHECK_TRUE(RunRemote(logger, testRootDir, config, RunCreateVirtualFileTest, uintptr_t(&testData)));
			
		testData.transient = false;
		CHECK_TRUE(RunRemote(logger, testRootDir, config, RunCreateVirtualFileTest, uintptr_t(&testData)));

		testData.registerToDir = false;
		CHECK_TRUE(RunRemote(logger, testRootDir, config, RunCreateVirtualFileTest, uintptr_t(&testData)));
		return true;
	}

	bool RunRegisterVirtualFileTest(LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
	{
		StringBuffer<> bigFile;
		CHECK_TRUE(CreateTestFile(bigFile, logger, ToView(workingDir), TCV("BigFile.txt"), TCV("FooBar")));
		CHECK_TRUE(session.RegisterVirtualFile(StringBuffer(workingDir).Append(TCV("First.txt")).data, bigFile.data, 0, sizeof(tchar)*3));
		CHECK_TRUE(session.RegisterVirtualFile(StringBuffer(workingDir).Append(TCV("Second.txt")).data, bigFile.data, sizeof(tchar) * 3, sizeof(tchar) * 3));
		CHECK_TRUE(RunTestApp(logger, workingDir, runProcess, TC("-testRegisterVirtualFile")));
		return true;
	}

	UBA_TEST_IF(Session, RegisterVirtualFile, IsWindows)
	{
		Config config;
		CHECK_TRUE(RunLocal(logger, testRootDir, config, RunRegisterVirtualFileTest));
		return true;
	}

	UBA_TEST_IF(Session, RegisterVirtualFileRemote, IsWindows)
	{
		Config config;
		CHECK_TRUE(RunRemote(logger, testRootDir, config, RunRegisterVirtualFileTest));
		return true;
	}

#if PLATFORM_MAC
	bool RunXCodeSelect(LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
	{
		#if UBA_DEBUG // Failing on farm for some reason... need to revisit
		StringBuffer<> xcodeSelect;
		if (!ExecuteCommand(logger, "which xcode-select", xcodeSelect))
			return true;
		return RunTestApp(logger, workingDir, runProcess, TC("-xcode-select"));
		#else
		return true;
		#endif
	}
	UBA_TEST(Session, XcodeSelect)
	{
		return RunLocal(logger, testRootDir, RunXCodeSelect);
	}
	UBA_TEST(Session, XcodeSelectRemote)
	{
		return RunRemote(logger, testRootDir, RunXCodeSelect);
	}
	#endif

	UBA_TEST_IF(Session, ProcessSpecialCase1Remote, IsWindows)
	{
		Config config;
		return SetupServerSession(logger, testRootDir, true, true, config,
			[&](LoggerWithWriter& logger, const StringView& workingDir, SessionServer& sessionServer)
			{
				auto runProcess = [&](ProcessStartInfo& pi) { return sessionServer.RunProcessRemote(pi); };
				CHECK_TRUE(CreateTestFile(logger, workingDir, TCV("SpecialFile1"), TCV("0")));

				const StringView casFile(TCV("UbaClient1357/cas/4d/4d067153ac729a4a7e8220c97935ffba67487800"));

				if (!SetupClientSession(logger, testRootDir, true, false, sessionServer.GetServer(), 1357, [&](LoggerWithWriter& logger, SessionClient& sessionClient)
					{
						CHECK_TRUE(CreateTestFile(logger, testRootDir, casFile, TCV("0")));
						return RunTestApp(logger, workingDir.data, runProcess, TC("-readwrite=0"));
					}))
					return false;

				CHECK_TRUE(DeleteTestFile(logger, testRootDir, casFile));

				if (!SetupClientSession(logger, testRootDir, false, false, sessionServer.GetServer(), 1357, [&](LoggerWithWriter& logger, SessionClient& sessionClient)
					{
						CHECK_TRUE(RunTestApp(logger, workingDir.data, runProcess, TC("-readwrite=1")));
						return true;
					}))
					return false;


				return true;
			});
	}

	UBA_TEST_IF(Session, SpecialCopy, IsWindows)
	{
		Config config;
		return SetupServerSession(logger, testRootDir, true, true, config,
			[&](LoggerWithWriter& logger, const StringView& workingDir, SessionServer& sessionServer)
			{
				CHECK_TRUE(CreateTestFile(logger, workingDir, TCV("File.h"), TCV("0")));
				ProcessStartInfo processInfo;
				processInfo.application = TC("cmd.exe");
				processInfo.workingDir = workingDir.data;
				processInfo.arguments = TC("/c copy /Y \"File.h\" \"File2.h\"");
				ProcessHandle process = sessionServer.RunProcess(processInfo);
				CHECK_TRUEF(process.IsValid(), TC("Failed to start process"));
				CHECK_TRUEF(process.WaitForExit(GetTimeoutTime()), TC("UbaTestApp did not exit in 10 seconds"));
				CHECK_TRUEF(Equals(process.GetStartInfo().application, TC("ubacopy")), TC("Special copy was not used"));
				u32 exitCode = process.GetExitCode();
				CHECK_TRUEF(exitCode == 0, TC("Special copy failed"));
				CHECK_TRUE(FileExists(logger, workingDir, TCV("File2.h")));
				return true;
			});
	}

	bool WriteFileMap(LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
	{
		StringBuffer<MaxPath> fileR;
		CHECK_TRUE(CreateTextFile(fileR, logger, workingDir, TC("File.h"), TC("Foo")));
		CHECK_TRUE(RunTestApp(logger, workingDir, runProcess, TC("-readFileMap=Foo")));
		CHECK_TRUE(RunTestApp(logger, workingDir, runProcess, TC("-writeFileMap=Foo")));
		CHECK_TRUE(RunTestApp(logger, workingDir, runProcess, TC("-writeFileMap=Bar")));
		CHECK_TRUE(RunTestApp(logger, workingDir, runProcess, TC("-readFileMap=Bar")));
		return true;
	}

	UBA_TEST_IF(Session, FileMapWrite, IsWindows)
	{
		return RunLocal(logger, testRootDir, WriteFileMap);
	}

	UBA_TEST_IF(Session, FileMapWriteRemote, IsWindows)
	{
		return RunRemote(logger, testRootDir, WriteFileMap);
	}

	bool RunOverlayAccessOrdering(LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
	{
		StringBuffer<> barFile;
		barFile.Append(workingDir).Append(TCV("bar.txt"));
		CHECK_TRUE(session.CreateVirtualFile(barFile.data, "", 1, true, false));

		StringBuffer<> fooFile;
		CHECK_TRUE(CreateTestFile(fooFile, logger, ToView(workingDir), TCV("foo.txt"), TCV("foo")));

		if (extraData == 1)
			CHECK_TRUE(RunTestApp(logger, workingDir, runProcess, TC("-getFileAttributes=foo.txt"), nullptr, TC(""), 32));
		else if (extraData == 2)
		{
			CHECK_TRUE(RunTestApp(logger, workingDir, runProcess, TC("-getFileAttributes=bar.txt"), nullptr, barFile.data, 128));
			CHECK_TRUE(RunTestApp(logger, workingDir, runProcess, TC("-getFileAttributes=foo.txt"), nullptr, barFile.data, 32));
		}
		else if (extraData == 3)
			CHECK_TRUE(RunTestApp(logger, workingDir, runProcess, TC("-createFileEmptyFile=bar.txt -getFileAttributes=foo.txt"), nullptr, TC(""), 32));

		return true;
	}

	UBA_TEST_IF(Session, OverlayAccessOrdering, IsWindows)
	{
		CHECK_TRUE(RunLocal(logger, testRootDir, RunOverlayAccessOrdering, 1));
		CHECK_TRUE(RunLocal(logger, testRootDir, RunOverlayAccessOrdering, 2));
		CHECK_TRUE(RunLocal(logger, testRootDir, RunOverlayAccessOrdering, 3));
		return true;
	}

	UBA_TEST_IF(Session, OverlayAccessOrderingRemote, IsWindows)
	{
		Config c;
		CHECK_TRUE(RunRemote(logger, testRootDir, c, RunOverlayAccessOrdering, 1));
		CHECK_TRUE(RunRemote(logger, testRootDir, c, RunOverlayAccessOrdering, 2));
		CHECK_TRUE(RunRemote(logger, testRootDir, c, RunOverlayAccessOrdering, 3));
		return true;
	}

	#if PLATFORM_LINUX
	bool RunSymlinks(LoggerWithWriter& logger, SessionServer& session, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t extraData)
	{
		CHECK_TRUE(RunTestApp(logger, workingDir, runProcess, TC("-createSymlink"), nullptr, TC("")));
		StringBuffer<> newLink(workingDir);
		newLink.EnsureEndsWithSlash().Append(TCV("new_file"));
		struct stat st;
		CHECK_TRUE(lstat(newLink.data, &st) == 0);
		CHECK_TRUE(S_ISLNK(st.st_mode));
		return true;
	}

	UBA_TEST(Session, DetouredSymlinks)
	{
		CHECK_TRUE(RunLocal(logger, testRootDir, RunSymlinks));
		return true;
	}

	UBA_TEST(Session, DetouredSymlinksRemote)
	{
		CHECK_TRUE(RunRemote(logger, testRootDir, RunSymlinks));
		return true;
	}
	#endif
}
