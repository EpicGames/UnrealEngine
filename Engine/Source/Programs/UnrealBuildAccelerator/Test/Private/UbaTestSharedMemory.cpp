// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaTest.h"
#include "UbaLogger.h"
#include "UbaBinaryReaderWriter.h"
#include "UbaFileMapping.h"
#include "UbaSharedMemoryAllocator.h"
#include "UbaSharedMemoryView.h"

UBA_DEFINE_TEST_GROUP(SharedMemory, 110)

namespace uba
{
	UBA_TEST(SharedMemory, Basic)
	{
		TrackSystemUsage tsu(logger);
		FileMappingBackend backend;

		{
			SharedMemoryAllocator mem(logger, backend);
			mem.Init(64ull * 1024 * 1024 * 1024);
			SharedMemoryAllocatorHandle handle = mem;

			SharedMemoryHandle viewHandle1 = mem.CreateHandle(logger, TC(""));
			SharedMemoryHandle viewHandle2 = mem.CreateHandle(logger, TC(""));

			SharedMemoryView view1(backend);
			SharedMemoryView view2(backend);

			view1.Init(handle, 1024 * 1024);
			view2.Init(handle, 1024 * 1024);

			for (u32 i = 0; i != 3; ++i)
			{
				{
					StackBinaryWriter<1024> writer;
					mem.ExtendMemory(writer, viewHandle1, PageSize);
					BinaryReader reader(writer.GetData(), 0, writer.GetPosition());
					view1.AddRequestedMemory(handle, reader, SharedMemoryMapType_ReadWrite, 0, TC(""), 0xff);
				}
				{
					StackBinaryWriter<1024> writer;
					mem.ExtendMemory(writer, viewHandle2, PageSize);
					BinaryReader reader(writer.GetData(), 0, writer.GetPosition());
					view2.AddRequestedMemory(handle, reader, SharedMemoryMapType_ReadWrite, 0, TC(""), 0xdd);
				}
			}

			u8* ptr = mem.MapView(viewHandle1, TC(""), SharedMemoryMapType_ReadOnly);
			mem.UnmapView(viewHandle1, ptr);

			SharedMemoryView view3(backend);
			mem.MapView(view3, viewHandle1, TC(""), SharedMemoryMapType_ReadOnly);

			mem.CloseHandle(logger, viewHandle2, TC(""));
			mem.CloseHandle(logger, viewHandle1, TC(""));


			SharedMemoryHandle viewHandle4 = mem.CreateHandle(logger, TC(""));
			SharedMemoryView view4(backend);
			view4.Init(handle, 1024 * 1024);
			mem.ExtendMemory(view4, viewHandle4, PageSize, TC(""), false);
			mem.ExtendMemory(view4, viewHandle4, PageSize, TC(""), false);
		}

		{
			SharedMemoryAllocator mem(logger, backend);
			mem.Init(1024 * 1024);
			SharedMemoryHandle viewHandle1 = mem.CreateHandle(logger, TC(""));
			SharedMemoryHandle viewHandle2 = mem.CreateHandle(logger, TC(""));

			StackBinaryWriter<1024> writer1;
			mem.ExtendMemory(writer1, viewHandle1, PageSize);

			StackBinaryWriter<1024> writer2;
			mem.ExtendMemory(writer2, viewHandle2, (1024-64) * 1024);

			mem.CloseHandle(logger, viewHandle1, TC(""));
			mem.CloseHandle(logger, viewHandle2, TC(""));
		}

		return true;
	}
}
