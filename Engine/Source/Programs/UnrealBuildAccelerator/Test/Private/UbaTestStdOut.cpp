// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaNetworkBackendTcp.h"
#include "UbaProcessStartInfo.h"
#include "UbaSessionServer.h"
#include "UbaStorageServer.h"
#include "UbaTest.h"

UBA_DEFINE_TEST_GROUP(StdOut, 100)

namespace uba
{
	void GetTestAppPath(LoggerWithWriter& logger, StringBufferBase& out);

	bool TestStdOut(LoggerWithWriter& logger, const StringBufferBase& testRootDir, bool remote, const tchar* app = nullptr, const tchar* arg = nullptr, const tchar* expectedOut = nullptr)
	{
		LogWriter& logWriter = logger.m_writer;
		NetworkBackendTcp networkBackend(logWriter);

		bool ctorSuccess = true;
		NetworkServer networkServer(ctorSuccess, { logWriter });

		StringBuffer<> rootDir;
		rootDir.Append(testRootDir).Append(TCV("Uba"));
		CHECK_TRUE(DeleteAllFiles(logger, rootDir.data));

		StorageServerCreateInfo storageServerInfo(networkServer, rootDir.data, logWriter);
		storageServerInfo.casCapacityBytes = 1024ull * 1024 * 1024;
		StorageServer storageServer(storageServerInfo);

		SessionServerCreateInfo sessionServerInfo(storageServer, networkServer, logWriter);
		sessionServerInfo.rootDir = rootDir.data;
		sessionServerInfo.traceEnabled = true;
		//sessionServerInfo.remoteLogEnabled = true;
		SessionServer sessionServer(sessionServerInfo);

		auto sg = MakeGuard([&]() { networkServer.DisconnectClients(); });

		StringBuffer<> workingDir;
		workingDir.Append(testRootDir).Append(TCV("WorkingDir"));
		CHECK_TRUE(DeleteAllFiles(logger, workingDir.data));
		CHECK_TRUE(storageServer.CreateDirectory(workingDir.data));
		CHECK_TRUE(DeleteAllFiles(logger, workingDir.data, false));
		workingDir.EnsureEndsWithSlash();

		CHECK_TRUEF(networkServer.StartListen(networkBackend), TC("Failed to listen"));

		StringBuffer<> testApp;
		if (!app)
		{
			GetTestAppPath(logger, testApp);
			app = testApp.data;
		}
		if (!arg)
		{
			arg = TC("-stdout=rootprocess");
			expectedOut = TC("rootprocess");
		}

		ProcessStartInfo pi;
		pi.application = app;
		pi.arguments = arg;
		pi.workingDir = workingDir.data;
		pi.description = TC("StdOutDesc");
		pi.logFile = TC("Log");

		auto ph = remote ? sessionServer.RunProcessRemote(pi) : sessionServer.RunProcess(pi);
		CHECK_TRUEF(ph.WaitForExit(5000), TC("Timed out waiting for process"));
		if (auto ec = ph.GetExitCode())
			return logger.Error(TC("Process exited with error code %u"), ec);

		networkBackend.StopListen();
		networkServer.DisconnectClients();
		sessionServer.WaitOnAllTasks();

		auto& logLines = ph.GetLogLines();
		CHECK_TRUEF(!(logLines.size() != 1), TC("Application %s produced %u log line(s) but expected 1"), app, u32(logLines.size()));
		CHECK_TRUEF(Equals(logLines[0].text.c_str(), expectedOut), TC("Application %s produced non-matching log line: %s"), app, logLines[0].text.c_str());

		return true;
	}

	UBA_TEST_IF(StdOut, Local, IsWindows)
	{
		return TestStdOut(logger, testRootDir, false);
	}

	UBA_TEST_DISABLED(StdOut, Remote, "depends on a running UbaAgent")
	{
		return TestStdOut(logger, testRootDir, true);
	}

	UBA_TEST_IF(StdOut, ViaCmd, IsWindows)
	{
		StringBuffer<> args;
		args.Append(TCV("/c \""));
		GetTestAppPath(logger, args);
		args.Append(TCV(" -stdout=foo\""));
		return TestStdOut(logger, testRootDir, false, TC("c:\\windows\\system32\\cmd.exe"), args.data, TC("foo"));
	}
}
