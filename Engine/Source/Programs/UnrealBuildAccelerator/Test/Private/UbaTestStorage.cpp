// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaTest.h"
#include "UbaFileAccessor.h"
#include "UbaNetworkBackendTcp.h"
#include "UbaStorageClient.h"
#include "UbaStorageServer.h"

UBA_DEFINE_TEST_GROUP(Storage, 70)

namespace uba
{
	UBA_TEST(Storage, Basic)
	{
		#if PLATFORM_LINUX
		if (true) // TODO: Revisit this... fails on farm but works locally
			return true;
		#endif

		WorkManagerImpl workManager(1);
		StringBuffer<> rootDir;
		rootDir.Append(testRootDir).Append(TCV("Uba"));
		StorageCreateInfo storageInfo(rootDir.data, logger.m_writer, workManager);
		storageInfo.casCapacityBytes = 1024ull * 1024 * 1024;
		StorageImpl storage(storageInfo);

		StringBuffer<> detoursLib;
		GetDirectoryOfCurrentModule(logger, detoursLib);
		detoursLib.EnsureEndsWithSlash().Append(UBA_DETOURS_LIBRARY);

		storage.LoadCasTable();

		CasKey key;
		bool deferCreation = false;
		CHECK_TRUEF(storage.StoreCasFile(key, detoursLib.data, CasKeyZero, deferCreation), TC("Failed to store file %s"), detoursLib.data);
		CHECK_TRUEF(!(key == CasKeyZero), TC("Failed to find file %s"), detoursLib.data);

		StringBuffer<> detoursLibCopy(detoursLib);
		detoursLibCopy.Append(TCV(".tmp"));

		auto deleteFile = MakeGuard([&]() { return DeleteFileW(detoursLibCopy.data); });

		CHECK_TRUEF(storage.CopyOrLink(key, detoursLibCopy.data, DefaultAttributes()), TC("Failed to copy cas to file %s"), detoursLibCopy.data);

		FileHandle original;
		CHECK_TRUEF(OpenFileSequentialRead(logger, detoursLib.data, original), TC("Failed to open %s for read"), detoursLib.data);
		auto closeOriginal = MakeGuard([&]() { return CloseFile(detoursLib.data, original); });

		FileHandle copy;
		CHECK_TRUEF(OpenFileSequentialRead(logger, detoursLibCopy.data, copy), TC("Failed to open %s for read"), detoursLibCopy.data);
		auto closeCopy = MakeGuard([&]() { return CloseFile(detoursLibCopy.data, copy); });

		u64 originalSize;
		CHECK_TRUEF(GetFileSizeEx(originalSize, original), TC("Failed to get size of %s"), detoursLib.data);
		u64 copySize;
		CHECK_TRUEF(GetFileSizeEx(copySize, copy), TC("Failed to get size of %s"), detoursLibCopy.data);
		CHECK_TRUEF(!(originalSize != copySize), TC("Size mismatch between %s and %s (%llu vs %llu)"), detoursLib.data, detoursLibCopy.data, originalSize, copySize);

		u8 originalBuffer[PageSize];
		u8 copyBuffer[PageSize];
		u64 left = originalSize;
		while (left)
		{
			u64 toRead = Min(left, (u64)sizeof(originalBuffer));
			CHECK_TRUEF(ReadFile(logger, detoursLib.data, original, originalBuffer, toRead), TC("Failed to read %u from %s"), detoursLib.data);
			CHECK_TRUEF(ReadFile(logger, detoursLibCopy.data, copy, copyBuffer, toRead), TC("Failed to read %u from %s"), detoursLibCopy.data);
			CHECK_TRUEF(!(memcmp(originalBuffer, copyBuffer, toRead) != 0), TC("Data mismatch between %s and %s"), detoursLib.data, detoursLibCopy.data);

			left -= toRead;
		}

		CHECK_TRUEF(closeOriginal.Execute(), TC("Failed to close %s"), detoursLib.data);

		CHECK_TRUEF(closeCopy.Execute(), TC("Failed to close %s"), detoursLibCopy.data);

		CHECK_TRUEF(deleteFile.Execute(), TC("Failed to delete %s"), detoursLibCopy.data);

		return true;
	}

	UBA_TEST(Storage, RemoteStore)
	{
		auto& logWriter = logger.m_writer;
		NetworkBackendTcp serverTcp(logWriter, TC("ServerTcp"));
		NetworkBackendTcp clientTcp(logWriter, TC("ClientTcp"));

		bool ctorSuccess = true;
		NetworkServer server(ctorSuccess, logWriter);
		NetworkClient client(ctorSuccess, logWriter);
		auto dg = MakeGuard([&]() { client.Disconnect(); });

		StringBuffer<> rootDir;
		rootDir.Append(testRootDir).Append(TCV("Uba"));
		CHECK_TRUE(DeleteAllFiles(logger, rootDir.data));

		StorageServerCreateInfo storageServerInfo(server, rootDir.data, logger.m_writer);
		storageServerInfo.casCapacityBytes = 1024ull * 1024 * 1024;
		StorageServer storageServer(storageServerInfo);

		auto g = MakeGuard([&]() { server.DisconnectClients(); });

		rootDir.Append(TCV("Client"));
		CHECK_TRUE(DeleteAllFiles(logger, rootDir.data));

		StorageClientCreateInfo storageClientInfo(client, rootDir.data);
		StorageClient storageClient(storageClientInfo);
		storageClient.Start();

		CHECK_TRUE(storageClient.LoadCasTable(true));

		rootDir.EnsureEndsWithSlash();

		CHECK_TRUEF(server.StartListen(serverTcp, 1234), TC("Failed to listen"));
		Sleep(100);
		CHECK_TRUEF(client.Connect(clientTcp, TC("127.0.0.1"), 1234), TC("Failed to connect"));
		auto cg = MakeGuard([&]() { client.Disconnect(); });

		StringBuffer<> fileName;
		{
			fileName.Append(rootDir).Append(TCV("UbaTestFile"));
			FileAccessor fileHandle(logger, fileName.data);
			CHECK_TRUEF(fileHandle.CreateWrite(), TC("Failed to create file for write"));
			u8 byte = 'H';
			CHECK_TRUE(fileHandle.Write(&byte, 1));
			CHECK_TRUE(fileHandle.Close());
		}

		CasKey key;
		bool storeCompressed = false;
		if (!storageClient.StoreCasFileClient(key, ToStringKeyLower(fileName), fileName.data, SharedMemoryHandle{}, 0, 0, TC("UbaTestFile"), false, storeCompressed))
			return logger.Error(TC("Failed to store file %s"), fileName.data);

		fileName.Clear().Append(testRootDir).Append(TCV("Uba")).Append(PathSeparator).Append(TCV("UbaTestFile"));
		CHECK_TRUEF(storageServer.CopyOrLink(key, fileName.data, DefaultAttributes()), TC("Failed to copy cas to file %s"), fileName.data);
		return true;
	}

	UBA_TEST(Storage, RemoteFetch)
	{
		auto& logWriter = logger.m_writer;
		NetworkBackendTcp serverTcp(logWriter, TC("ServerTcp"));
		NetworkBackendTcp clientTcp(logWriter, TC("ClientTcp"));

		bool ctorSuccess = true;
		NetworkServer server(ctorSuccess, logWriter);
		NetworkClient client(ctorSuccess, logWriter);
		auto dg = MakeGuard([&]() { client.Disconnect(); });

		StringBuffer<> rootDir;
		rootDir.Append(testRootDir).Append(TCV("Uba"));
		CHECK_TRUE(DeleteAllFiles(logger, rootDir.data));

		StorageServerCreateInfo storageServerInfo(server, rootDir.data, logger.m_writer);
		storageServerInfo.casCapacityBytes = 1024ull * 1024 * 1024;
		storageServerInfo.storeCompressed = false;
		StorageServer storageServer(storageServerInfo);

		auto g = MakeGuard([&]() { server.DisconnectClients(); });

		rootDir.Append(TCV("Client"));
		CHECK_TRUE(DeleteAllFiles(logger, rootDir.data));

		StorageClientCreateInfo storageClientInfo(client, rootDir.data);
		StorageClient storageClient(storageClientInfo);
		storageClient.Start();

		CHECK_TRUE(storageClient.LoadCasTable(true));

		rootDir.EnsureEndsWithSlash();

		CHECK_TRUEF(server.StartListen(serverTcp, 1234), TC("Failed to listen"));
		Sleep(100);
		CHECK_TRUEF(client.Connect(clientTcp, TC("127.0.0.1"), 1234), TC("Failed to connect"));
		auto cg = MakeGuard([&]() { client.Disconnect(); });

		StringBuffer<> fileName;
#if 1
		{
			fileName.Append(rootDir).Append(TCV("UbaTestFile"));
			FileAccessor fileHandle(logger, fileName.data);
			CHECK_TRUEF(fileHandle.CreateWrite(), TC("Failed to create file for write"));
			u8 byte = 'H';
			CHECK_TRUE(fileHandle.Write(&byte, 1));
			CHECK_TRUE(fileHandle.Close());
		}
#else
		fileName.Append(TCV("e:\\dev\\fn\\QAGame\\Saved\\StagedBuilds\\PS5_Temp\\Split\\qagame\\content\\paks\\qagame-ps5.ucas\\.copied"));
#endif

		CasKey casKey;
		if (!storageServer.CalculateCasKey(casKey, {}, fileName.data))
			return false;

		casKey = AsCompressed(casKey, false);

		Storage::RetrieveResult result;
		if (!storageClient.RetrieveCasFile(result, casKey, {}, fileName.data))
			return logger.Error(TC("Failed to retrieve file %s"), fileName.data);

		storageClient.DropCasFile(casKey, true, fileName.data, false);
		storageServer.DropCasFile(casKey, true, fileName.data, false);
		if (!storageClient.RetrieveCasFile(result, casKey, {}, fileName.data))
			return logger.Error(TC("Failed to retrieve file %s"), fileName.data);

		if constexpr (StorageNetworkVersion >= 5)
		{
			StringKey testFileNameKey = ToStringKeyLower(fileName);

			auto& sharedMem = storageServer.GetSharedMemory();
			auto memHandle = sharedMem.CreateHandle(logger, TC(""));
			sharedMem.ExtendMemory(memHandle, 1);
			u8* mem = sharedMem.MapView(memHandle, TC(""), SharedMemoryMapType_ReadWrite);
			*mem = 'H';
			sharedMem.UnmapView(memHandle, mem);

			storageServer.RegisterExternalFileMappingsProvider([&](Storage::ExternalFileMapping& out, StringKey fileNameKey, const tchar* fileName)
				{
					CHECK_TRUE(fileNameKey == testFileNameKey);
					out.handle = sharedMem.DuplicateHandle(memHandle, TC(""));
					out.size = 1;
					out.dropCasAfterUse = true;
					out.createIndependentMapping = true;
					return true;
				});

			storageClient.DropCasFile(casKey, true, fileName.data, false);
			storageServer.DropCasFile(casKey, true, fileName.data, false);
			CHECK_TRUEF(storageClient.RetrieveCasFile(result, casKey, testFileNameKey, TC("KnownInput")), TC("Failed to retrieve file %s"), fileName.data);

			storageClient.DropCasFile(casKey, true, fileName.data, false);
			storageServer.DropCasFile(casKey, true, fileName.data, false);
			CHECK_TRUEF(storageClient.RetrieveCasFile(result, casKey, testFileNameKey, TC("KnownInput")), TC("Failed to retrieve file %s"), fileName.data);
		}

		return true;
	}

	UBA_TEST(Storage, RemoteStore2)
	{
		auto& logWriter = logger.m_writer;
		NetworkBackendTcp serverTcp(logWriter, TC("ServerTcp"));
		NetworkBackendTcp clientTcp(logWriter, TC("ClientTcp"));

		bool ctorSuccess = true;
		NetworkServer server(ctorSuccess, logWriter);
		NetworkClient client(ctorSuccess, logWriter);
		auto dg = MakeGuard([&]() { client.Disconnect(); });

		StringBuffer<> rootDir;
		rootDir.Append(testRootDir).Append(TCV("Uba"));
		CHECK_TRUE(DeleteAllFiles(logger, rootDir.data));

		StorageServerCreateInfo storageServerInfo(server, rootDir.data, logger.m_writer);
		storageServerInfo.casCapacityBytes = 1024ull * 1024 * 1024;
		storageServerInfo.createIndependentMappings = true;
		StorageServer storageServer(storageServerInfo);

		auto g = MakeGuard([&]() { server.DisconnectClients(); });

		rootDir.Append(TCV("Client"));
		CHECK_TRUE(DeleteAllFiles(logger, rootDir.data));

		StorageClientCreateInfo storageClientInfo(client, rootDir.data);
		StorageClient storageClient(storageClientInfo);
		storageClient.Start();

		CHECK_TRUE(storageClient.LoadCasTable(true));

		rootDir.EnsureEndsWithSlash();

		CHECK_TRUEF(server.StartListen(serverTcp, 1234), TC("Failed to listen"));
		Sleep(100);
		CHECK_TRUEF(client.Connect(clientTcp, TC("127.0.0.1"), 1234), TC("Failed to connect"));
		auto cg = MakeGuard([&]() { client.Disconnect(); });

		StringBuffer<> sourceFileName;
		{
			sourceFileName.Append(rootDir).Append(TCV("UbaTestFile"));
			FileAccessor fileHandle(logger, sourceFileName.data);
			CHECK_TRUEF(fileHandle.CreateWrite(), TC("Failed to create file for write"));
			u8 byte = 'H';
			CHECK_TRUE(fileHandle.Write(&byte, 1));
			CHECK_TRUE(fileHandle.Close());
		}

		StringBuffer<> destFileName;
		CasKey key;
		auto store = [&]()
			{
				bool storeCompressed = true;
				if (!storageClient.StoreCasFileClient(key, ToStringKeyLower(sourceFileName), sourceFileName.data, SharedMemoryHandle{}, 0, 0, TC("UbaTestFile"), false, storeCompressed))
					return logger.Error(TC("Failed to store file %s"), sourceFileName.data);
				return true;
			};

		auto copy = [&]()
			{
				destFileName.Clear().Append(testRootDir).Append(TCV("Uba")).Append(PathSeparator).Append(TCV("UbaTestFile"));
				CHECK_TRUE(storageServer.WaitForCasFile(key, destFileName.data));
				if (!storageServer.CopyOrLink(key, destFileName.data, DefaultAttributes(), false, 1, {}, false, true))
					return logger.Error(TC("Failed to copy cas to file %s"), destFileName.data);
				CHECK_TRUEF(FilesEqual(logger, sourceFileName.data, destFileName.data), TC("Files %s and %s not equal"), sourceFileName.data, destFileName.data);
				return true;
			};

		CHECK_TRUE(store());
		CHECK_TRUE(copy());
		CHECK_TRUE(store());
		CHECK_TRUE(copy());

		CHECK_TRUE(store());
		CHECK_TRUE(store());
		CHECK_TRUE(copy());
		CHECK_TRUE(copy());

		CHECK_TRUE(storageServer.StoreCasFile(key, sourceFileName.data, {}, true));

		//client.Fetch
		CHECK_TRUE(store());
		//if (!copy())
		//	return false;

		// Large file
		{
			sourceFileName.Clear().Append(rootDir).Append(TCV("UbaTestFile2"));
			FileAccessor fileHandle(logger, sourceFileName.data);
			CHECK_TRUEF(fileHandle.CreateWrite(), TC("Failed to create file for write"));
			
			// 256*64*1024 = 16mb file
			u32 data[16*1024];
			for (u32 i=0;i!=256; ++i)
			{
				for (u32 j=0;j!=sizeof_array(data); ++j)
					data[j] = u32(rand());
				CHECK_TRUE(fileHandle.Write(data, sizeof(data)));
			}
			CHECK_TRUE(fileHandle.Close());
		}

		CHECK_TRUE(store());
		CHECK_TRUE(copy());

		return true;
	}

}
