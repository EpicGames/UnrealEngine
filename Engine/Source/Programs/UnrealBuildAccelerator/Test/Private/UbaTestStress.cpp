// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaTest.h"
#include "UbaCoordinatorWrapper.h"
#include "UbaFileAccessor.h"
#include "UbaNetworkBackendTcp.h"
#include "UbaScheduler.h"
#include "UbaSessionServer.h"
#include "UbaStorageServer.h"

UBA_DEFINE_TEST_GROUP(Stress, 160)

namespace uba
{
	void GetTestAppPath(LoggerWithWriter& logger, StringBufferBase& out);

	UBA_TEST_DISABLED(Stress, Basic, "depends on CoordinatorHorde and credentials")
	{
		LogWriter& logWriter = logger.m_writer;
		NetworkBackendTcp networkBackend(logWriter);

		bool ctorSuccess = true;
		NetworkServer networkServer(ctorSuccess, { logWriter });

		StringBuffer<> rootDir;
		rootDir.Append(testRootDir).Append(TCV("Uba"));
		CHECK_TRUE(DeleteAllFiles(logger, rootDir.data));

		StorageServerCreateInfo storageServerInfo(networkServer, rootDir.data, logWriter);
		storageServerInfo.casCapacityBytes = 1024ull * 1024 * 1024;
		StorageServer storageServer(storageServerInfo);

		SessionServerCreateInfo sessionServerInfo(storageServer, networkServer, logWriter);
		sessionServerInfo.rootDir = rootDir.data;
		sessionServerInfo.traceEnabled = true;
		//sessionServerInfo.remoteLogEnabled = true;
		SessionServer sessionServer(sessionServerInfo);

		auto sg = MakeGuard([&]() { networkServer.DisconnectClients(); });

		StringBuffer<> workingDir;
		workingDir.Append(testRootDir).Append(TCV("WorkingDir"));
		CHECK_TRUE(DeleteAllFiles(logger, workingDir.data));
		CHECK_TRUE(storageServer.CreateDirectory(workingDir.data));
		CHECK_TRUE(DeleteAllFiles(logger, workingDir.data, false));
		workingDir.EnsureEndsWithSlash();

		SchedulerCreateInfo info(sessionServer);
		//info.forceRemote = true;
		info.enableProcessReuse = true;
		info.maxLocalProcessors = 1;
		Scheduler scheduler(info);
		scheduler.Start();

		CHECK_TRUEF(networkServer.StartListen(networkBackend), TC("Failed to listen"));

		CoordinatorWrapper coordinator;

		bool useCoordinator = true;
		if (useCoordinator)
		{
			return logger.Error(TC("uri/pool/oidc must be populated for this test to work"));

			#if 0 // populate uri/pool/oidc on CoordinatorCreateInfo and remove the early return above to actually run
			StringBuffer<512> coordinatorWorkDir(testRootDir);
			coordinatorWorkDir.EnsureEndsWithSlash().Append(TCV("Horde"));
			StringBuffer<512> binariesDir;
			CHECK_TRUE(GetDirectoryOfCurrentModule(logger, binariesDir));

			CoordinatorCreateInfo cinfo;
			cinfo.workDir = coordinatorWorkDir.data;
			cinfo.binariesDir = binariesDir.data;

			cinfo.maxCoreCount = 400;
			cinfo.logging = true;
			CHECK_TRUE(coordinator.Create(logger, TC("Horde"), cinfo, networkBackend, networkServer, sessionServer.GetTrace()));
			#endif
		}
		auto cg = MakeGuard([&]() { coordinator.Destroy(); });

		StringBuffer<> testApp;
		GetTestAppPath(logger, testApp);

		u32 counter = 0;

		Atomic<u32> activeCount = 0;
		Atomic<bool> error = false;
		while (true)
		{
			StringBuffer<512> fileName;
			fileName.Append(workingDir).Append(TCV("File_")).AppendValue(counter++).Append(TCV(".in"));
			{
				FileAccessor fa(logger, fileName.data);
				fa.CreateWrite();
				Guid g;
				CreateGuid(g);
				fa.Write(&g, sizeof(g));
				fa.Close();
			}
			sessionServer.RegisterNewFile(fileName.data);
			StringBuffer<512> args;
			args.Append(TCV("-file=")).Append(fileName);
			ProcessStartInfo pi;
			pi.application = testApp.data;
			pi.arguments = args.data;
			pi.workingDir = workingDir.data;
			pi.description = fileName.Last(PathSeparator) + 6;

			struct Rec
			{
				Logger& logger;
				StringBuffer<256> inFile;
				Atomic<u32>& activeCount;
				Atomic<bool>& error;
			};
			pi.userData = new Rec { logger, fileName, activeCount, error };
			pi.exitedFunc = [](void* userData, const ProcessHandle& ph, ProcessExitedResponse&)
				{
					auto rec = (Rec*)userData;
					for (auto& line : ph.GetLogLines())
						rec->logger.Log(line.type, line.text.c_str(), u32(line.text.size()));

					if (u32 exitCode = ph.GetExitCode())
					{
						rec->logger.Error(TC("Process %s failed with exit code %u"), ph.GetStartInfo().GetDescription(), exitCode);
						rec->error = true;
					}
					else
					{
						auto& logger = rec->logger;
						if (!DeleteFileW(rec->inFile.data))
						{
							logger.Error(TC("Failed to delete file %s"), rec->inFile.data);
							rec->error = true;
						}
						StringBuffer<256> outFile;
						outFile.Append(rec->inFile.data, rec->inFile.count - 2).Append(TCV("out"));
						if (!FileExists(logger, outFile.data))
						{
							logger.Error(TC("Couldn't find output file %s"), outFile.data);
							rec->error = true;
						}
						if (!DeleteFileW(outFile.data))
						{
							logger.Error(TC("Failed to delete file %s"), outFile.data);
							rec->error = true;
						}
					}
					--rec->activeCount;
					delete rec;
				};
			EnqueueProcessInfo epi(pi);
			scheduler.EnqueueProcess(epi);
			++activeCount;

			while (activeCount > 800)
				Sleep(100);
			if (error)
				break;

			if (counter != 20000)
				continue;

			while (true)
			{
				u32 queued;
				u32 activeLocal;
				u32 activeRemote;
				u32 finished;
				scheduler.GetStats(queued, activeLocal, activeRemote, finished);
				if (finished == 20000)
					break;
			}
			break;
		}

		networkBackend.StopListen();
		networkServer.DisconnectClients();
		scheduler.Stop();

		return !error;
	}
}
