// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaTest.h"
#include "UbaProcess.h"
#include "UbaSessionServer.h"

UBA_DEFINE_TEST_GROUP(Verb, 170)

namespace uba
{
	extern u32 GetTimeoutTime();

	static bool RunV(LoggerWithWriter& logger, const tchar* workingDir, const RunProcessFunction& runProcess, const tchar* verbs)
	{
		StringBuffer<> testApp; GetTestAppPath(logger, testApp);
		ProcessStartInfo pi;
		pi.application = testApp.data;
		pi.workingDir = workingDir;
		pi.arguments = verbs;
		pi.logLineFunc = [](void*, const tchar* line, u32, LogEntryType) { LoggerWithWriter(g_consoleLogWriter, TC("")).Info(line); };
		ProcessHandle p = runProcess(pi);
		CHECK_TRUEF(p.IsValid() && p.WaitForExit(GetTimeoutTime()), TC("UbaTestApp '%s' did not exit"), verbs);
		u32 ec = p.GetExitCode();
		if (ec == 0) return true;
		for (auto& l : p.GetLogLines()) logger.Error(l.text.c_str());
		return logger.Error(TC("UbaTestApp '%s' ec=%u"), verbs, ec);
	}

	// VN(name, chain1, chain2, ...) — declares a verb test. Each chain is a separate RunV
	// (= separate process tree, separate writtenFiles, separate overlay). Single-arg = single
	// process; multi-arg = sequence exercising dirtable + overlay propagation between trees.
	// Registers Verb.name (RunLocal) and Verb.nameRemote (RunRemote). TC() must wrap each chain
	// at the call site because tchar is wchar_t on Windows and the preprocessor can't fold TC
	// over __VA_ARGS__ without a FOR_EACH helper.
	#define VN(name, ...) \
		static bool RunVerb##name(LoggerWithWriter& logger, SessionServer&, const tchar* wd, const RunProcessFunction& rp, uintptr_t) { \
			const tchar* chains[] = { __VA_ARGS__ }; \
			for (const tchar* chain : chains) \
				if (!RunV(logger, wd, rp, chain)) return false; \
			return true; \
		} \
		static bool RunVerb##name##Local(LoggerWithWriter& logger, const StringBufferBase& testRootDir)  { return RunLocal(logger, testRootDir, RunVerb##name); } \
		static bool RunVerb##name##Remote(LoggerWithWriter& logger, const StringBufferBase& testRootDir) { return RunRemote(logger, testRootDir, RunVerb##name); } \
		static uba::Test UBA_TEST_CONCAT(Verb##name##LocalNode, __LINE__){ TC(#name), &RunVerb##name##Local, nullptr, nullptr }; \
		static uba::Test UBA_TEST_CONCAT(Verb##name##RemoteNode, __LINE__){ TC(#name) TC("Remote"), &RunVerb##name##Remote, nullptr, nullptr }; \
		static uba::TestRegistrar UBA_TEST_CONCAT(Verb##name##LocalReg, __LINE__){ g_Verb, &UBA_TEST_CONCAT(Verb##name##LocalNode, __LINE__) }; \
		static uba::TestRegistrar UBA_TEST_CONCAT(Verb##name##RemoteReg, __LINE__){ g_Verb, &UBA_TEST_CONCAT(Verb##name##RemoteNode, __LINE__) };

	VN(Basics,                  TC("write=a.txt,hello read=a.txt,hello size=a.txt,5 stat=a.txt isfile=a.txt unlink=a.txt notexists=a.txt"))
	// Reopen-after-rename: llvm-objcopy's FileOutputBuffer (write tmp → rename → reopen final) hit a fresh-backing-path bug in Session::CreateFile remote fast-path.
	VN(WriteRenameReopenWrite,  TC("write=wrrw.tmp,A rename=wrrw.tmp,wrrw write=wrrw,B read=wrrw,B size=wrrw,1"))
	VN(DirCycle,                TC("mkdir=d isdir=d touch=d/x stat=d/x isfile=d/x listdir=d,1 listcontains=d,x unlink=d/x listdir=d,0 rmdir=d notexists=d"))
	VN(Copy,                    TC("write=src.txt,abc copy=src.txt,dst.txt read=dst.txt,abc size=dst.txt,3 unlink=src.txt unlink=dst.txt"))
	VN(Append,                  TC("write=f.txt,foo append=f.txt,bar read=f.txt,foobar size=f.txt,6"))
	VN(Rename,                  TC("write=old.txt,xx rename=old.txt,new.txt notexists=old.txt read=new.txt,xx"))
	VN(Nested,                  TC("mkdir=a mkdir=a/b mkdir=a/b/c touch=a/b/c/leaf stat=a/b/c/leaf listdir=a/b/c,1 listcontains=a/b,c"))
	VN(DeleteFlow,              TC("touch=q stat=q unlink=q notexists=q touch=q stat=q unlink=q notexists=q"))
	VN(StatNonexistent,         TC("notexists=ghost.txt notexists=nodir/file"))
	VN(CwdRel,                  TC("mkdir=sub touch=sub/inside chdir=sub stat=inside isfile=inside"))
	VN(ReadAfterRename,         TC("write=r1.txt,one rename=r1.txt,r2.txt read=r2.txt,one rename=r2.txt,r3.txt read=r3.txt,one"))
	// AOSP sbox pattern: parent dirs created in-process were never materialized on host before rename → MoveFile RPC ENOENT-asserts in UbaFile.cpp:954.
	VN(RenameStagedToFinal,     TC("mkdir=sbox_out mkdir=ints_gen write=sbox_out/foo.txt,sandboxed-bytes rename=sbox_out/foo.txt,ints_gen/foo.txt notexists=sbox_out/foo.txt read=ints_gen/foo.txt,sandboxed-bytes size=ints_gen/foo.txt,15"))
	VN(CreateExclusive,         TC("write=x1,a mustfail=open=x1,x open=x2,x stat=x2 unlink=x1 unlink=x2"))
	VN(OpenReadMissing,         TC("mustfail=open=ghost,r mustfail=open=nodir/g,r"))
	VN(AppendCreates,           TC("append=fresh,hi read=fresh,hi append=fresh,!! read=fresh,hi!!"))
	VN(DeleteMissing,           TC("mustfail=unlink=ghost"))
	// VN(RmdirMissing, TC("mustfail=rmdir=ghost")) — BUG: Detoured_RemoveDirectoryW crashes on missing path. See memory project_rmdir_missing_crash.md
	VN(RmdirNotEmpty,           TC("mkdir=ne touch=ne/f mustfail=rmdir=ne unlink=ne/f rmdir=ne"))
	VN(MkdirExists,             TC("mkdir=me mustfail=mkdir=me rmdir=me"))
	// VN(RenameClobber, ...) — platform-specific: Win32 MoveFile fails, Posix rename(2) overwrites silently. Skipped to keep the suite cross-platform.
	VN(DotPath,                 TC("write=dp.txt,abc read=./dp.txt,abc isfile=./dp.txt unlink=./dp.txt notexists=dp.txt"))
	VN(DotDotPath,              TC("mkdir=dd touch=dd/x stat=dd/../dd/x isfile=dd/../dd/x"))
	VN(ReadEmpty,               TC("touch=e size=e,0 read=e, isfile=e unlink=e"))
	VN(Rewrite,                 TC("write=rw,one read=rw,one size=rw,3 write=rw,longer read=rw,longer size=rw,6"))
	VN(DeepNest,                TC("mkdir=A mkdir=A/B mkdir=A/B/C mkdir=A/B/C/D mkdir=A/B/C/D/E touch=A/B/C/D/E/leaf isfile=A/B/C/D/E/leaf listdir=A/B/C/D/E,1"))
	VN(ListMany,                TC("mkdir=lm nfiles=lm,f,8 listdir=lm,8 listcontains=lm,f0 listcontains=lm,f7 listexcludes=lm,f8"))
	VN(BatchUnlink,             TC("mkdir=bu nfiles=bu,b,5 listdir=bu,5 unlink=bu/b0 unlink=bu/b1 unlink=bu/b2 unlink=bu/b3 unlink=bu/b4 listdir=bu,0 rmdir=bu"))
	VN(StatWriteStat,           TC("notexists=swsf write=swsf,hi stat=swsf isfile=swsf size=swsf,2 unlink=swsf notexists=swsf"))
	VN(CopyMissing,             TC("mustfail=copy=ghost_src,dst notexists=dst"))
	// VN(CopyOver, ...) — BUG: Session::CopyFileW asserts when dest exists. See memory project_copyfile_overwrite_unimplemented.md
	VN(WriteThenAppend,         TC("write=wa,12345 size=wa,5 append=wa,67 read=wa,1234567 size=wa,7"))
	VN(DirIsNotFile,            TC("mkdir=sd isdir=sd mustfail=isfile=sd notexists=sd_does_not_exist"))
	VN(StatRoot,                TC("stat=. isdir=."))
	VN(ReadAfterCopy,           TC("write=ra,abc copy=ra,rb read=rb,abc append=ra,DEF read=ra,abcDEF read=rb,abc"))
	VN(ManyCreates,             TC("nfiles=.,mc,20 listcontains=.,mc0 listcontains=.,mc19 size=mc0,0 size=mc19,0"))
	VN(ManyCreatesUnlinkAll,    TC("mkdir=mcu nfiles=mcu,n,12 listdir=mcu,12 unlink=mcu/n0 unlink=mcu/n1 unlink=mcu/n2 unlink=mcu/n3 unlink=mcu/n4 unlink=mcu/n5 unlink=mcu/n6 unlink=mcu/n7 unlink=mcu/n8 unlink=mcu/n9 unlink=mcu/n10 unlink=mcu/n11 listdir=mcu,0 rmdir=mcu"))
	VN(CopyChain,               TC("write=c0,seed copy=c0,c1 copy=c1,c2 copy=c2,c3 copy=c3,c4 read=c4,seed"))
	VN(RenameChain,             TC("write=r0,seed rename=r0,r1 rename=r1,r2 rename=r2,r3 read=r3,seed"))
	VN(StatTwice,               TC("write=st,abc stat=st stat=st isfile=st isfile=st size=st,3 size=st,3"))
	VN(Reopen,                  TC("write=ro,abcdef read=ro,abcdef read=ro,abcdef read=ro,abcdef"))
	VN(OpenWriteCloseRead,      TC("open=ow,w stat=ow isfile=ow size=ow,0 unlink=ow"))
	VN(OpenCreatedRead,         TC("open=ocr,w open=ocr,r unlink=ocr"))
	VN(WriteEmptyOverwrite,     TC("write=we,longcontent size=we,11 write=we, size=we,0 read=we, unlink=we"))
	VN(AppendToEmpty,           TC("touch=ate size=ate,0 append=ate,X size=ate,1 read=ate,X"))
	VN(MkdirInside,             TC("mkdir=mi mkdir=mi/inner stat=mi/inner isdir=mi/inner mustfail=rmdir=mi rmdir=mi/inner rmdir=mi"))
	// VN(DeleteDirAsFile, ...) — BUG: in remote mode Detoured_DeleteFileW (KernelBase.inl:1321 UBA_ASSERT) lets the call through and "succeeds" on a directory. Real Win32 returns ERROR_ACCESS_DENIED. Local mode masks this.
	VN(CreateInside,            TC("mkdir=ci touch=ci/a touch=ci/b touch=ci/c listdir=ci,3 unlink=ci/a unlink=ci/b unlink=ci/c rmdir=ci"))
	VN(StatThenUnlink,          TC("write=su,1 stat=su unlink=su mustfail=stat=su mustfail=open=su,r"))
	VN(OverlayWritePersist,     TC("write=op,first read=op,first write=op,second read=op,second size=op,6"))
	VN(CwdChain,                TC("mkdir=ch1 mkdir=ch1/ch2 mkdir=ch1/ch2/ch3 chdir=ch1 chdir=ch2 chdir=ch3 touch=deep stat=deep"))
	VN(ReadDirAfterAdd,         TC("mkdir=rd touch=rd/a listdir=rd,1 touch=rd/b listdir=rd,2 touch=rd/c listdir=rd,3 listcontains=rd,a listcontains=rd,b listcontains=rd,c"))
	VN(ReadDirAfterDelete,      TC("mkdir=ra nfiles=ra,x,5 listdir=ra,5 unlink=ra/x2 listdir=ra,4 listexcludes=ra,x2 unlink=ra/x0 unlink=ra/x1 unlink=ra/x3 unlink=ra/x4 listdir=ra,0 rmdir=ra"))

#if PLATFORM_LINUX
	// nsjail trusty: DefaultDirAttributes returned 040644 (no exec bits) → chdir EACCES because Linux requires +x to traverse. UbaFile.cpp:1027 fix: 040755.
	VN(MkdirThenChdir,          TC("mkdir=mtc xaccess=mtc chdir=mtc"))
	VN(WriteUtime,              TC("write=wut.txt,hello utime=wut.txt read=wut.txt,hello"))
	VN(WriteRenameUtime,        TC("write=wru.tmp,hello rename=wru.tmp,wru utime=wru read=wru,hello"))
	VN(WriteChmod,              TC("write=wcm.txt,hello chmod=wcm.txt,644 read=wcm.txt,hello"))
	VN(WriteRenameChmod,        TC("write=wrc.tmp,hello rename=wrc.tmp,wrc chmod=wrc,644 read=wrc,hello"))
	VN(WriteChmodExec,          TC("write=wce.bin,XYZnotelfnotscript chmod=wce.bin,755 execcheck=wce.bin"))
	// Reopen with O_WRONLY (no O_CREAT/O_TRUNC), seek 0, write — closer to llvm-objcopy's actual cleanup path.
	VN(WriteRenameReopenAppend, TC("write=wrra.tmp,AAAA rename=wrra.tmp,wrra reopenwrite=wrra,BB read=wrra,BBAA size=wrra,4"))
	// Python tempfile.NamedTemporaryFile pattern: pre-create empty placeholder, write tmp, rename over. Without rekey of g_mappedFileTable on rename, read returns stale placeholder content.
	VN(WriteEmptyRenameOverRead,  TC("write=wero.fin,X write=wero.tmp,realdata rename=wero.tmp,wero.fin read=wero.fin,realdata size=wero.fin,8"))
	// Reopen-for-write + chmod: combines this fix with the chmod-on-disk fix; verifies the new backing is exec'able on disk.
	VN(WriteRenameReopenChmod,  TC("write=wrrx.tmp,XYZnotelfnotscript rename=wrrx.tmp,wrrx write=wrrx,XYZnotelfnotscript chmod=wrrx,755 execcheck=wrrx"))
	// dash mid-list redirect-with-restore (AOSP `sh -c "cmd1 && cmd2 > X && cmd3"`): parent dup2(F_DUPFD save), dup2(file,1), spawn writer, waitpid, dup2(saved,1) restore. Bug pre-fix: parent's exit-time CloseFile fstats fd 1 post-restore (now stdout) and overwrites writtenFile with garbage attrs / leaves size=InvalidValue. Fix: Shared_PreFinalizeForOverwrite in dup2/dup3 wrappers fires Rpc_UpdateCloseHandle BEFORE the kernel implicit-close.
	VN(RedirectSaveRestore,     TC("redirectsaverestore=rsr_out.txt,hello-via-dash-redirect"))
	// Read while write fd still open: m_outputFiles populated, writtenFiles empty (no close). Mirrors the shell `>X && cmd2 X` pattern. Without the m_outputFiles read fast-path, remote returns the virtual path → ENOENT.
	VN(WriteOpenRead,           TC("writeopenread=wor.txt,sharedbytes"))
	// rustc archive-build pattern: open(O_RDWR|O_CREAT|O_EXCL), write, rename WHILE FD OPEN, close, stat dst. Pre-fix in remote mode: trueRename(virtual_src) ENOENTs because file is backed at hash-named staged path, not virtual src. Fix: skip trueRename when the open fd's backing path differs from virtual src (staged); virtual rename is captured by fo.newName + mappedFileTable, close-time finalization registers under dst.
	VN(RenameWhileOpen,         TC("renamewhileopen=rwo_src.tmp,rwo_dst.rlib"))
	// VN(RedirectSpawn, TC("redirectspawn=rs_out.txt,hello-from-spawned-echo")) — BUG: parent open(wfd) → posix_spawn(echo, dup2(wfd,1)) → close(wfd) before waitpid finalizes the writtenFile at size=0 because echo hasn't run yet. Echo's exit-time close re-finalization either drops or doesn't propagate to parent's dirtable view, so post-waitpid stat returns the locked-in 0. RedirectSaveRestore covers the dash dup2-pre-finalize path which is what AOSP actually hits; the spawn-with-dup2 pattern would need posix_spawn_file_actions tracking to defer parent-close finalization until child exit.
	// Same pattern + chmod 755 + execcheck: ensures chmod targets the post-rename backing, not the stale staged placeholder.
	VN(WriteEmptyRenameOverChmodExec, TC("write=weox.fin,X write=weox.tmp,XYZnotelfnotscript rename=weox.tmp,weox.fin chmod=weox.fin,755 execcheck=weox.fin"))
	// O_CLOEXEC write fds must be filtered from UBA_INHERIT_HANDLES — kernel auto-closes at execve; emitting them registers a stale entry that trips CreateFileTableEntry's "handle already added" assert when the fd number is reused. Repro: bindgen+rustfmt.
	VN(CloexecNotInherited,     TC("cloexec_open_exec=cloex.txt"))
	// Non-CLOEXEC write fds must still inherit (bash `exec 5>file` semantics).
	VN(NoCloexecInherited,      TC("noncloexec_open_exec=ncloex.txt,bash-redirect-payload"))
	// fcntl(F_SETFD, FD_CLOEXEC) on a tracked non-CLOEXEC fd must update the per-fd
	// hasCloexec so the build-time filter sees it (or we'd regress to the same assert).
	VN(FcntlSetCloexec,         TC("fcntl_set_cloexec_exec=fsetcx.txt"))
	// dup3 with O_CLOEXEC sets CLOEXEC on the NEW fd only — the original is untouched.
	// Across execve only the original survives and must inherit normally.
	VN(Dup3CloexecPerFd,        TC("dup3_cloexec_exec=d3cx.txt"))
	// DirInfo-first: opendir → dirfd → openat(fd,sub). AOSP depmod walks subdirs this way; pre-fix dirfd on a virtual DIR* returned -1+EBADF, openat fell through to TRUE_WRAPPER, asserting !g_runningRemote.
	VN(DirfdOpenat,             TC("mkdir=do_t touch=do_t/leaf dirfd_openat=do_t,leaf"))
	// fd-first: open(dir,O_RDONLY|O_DIRECTORY) → fdopendir → readdir. Symmetric wiring entered via Shared_open's CreateDirFd path.
	VN(FdopendirReaddir,        TC("mkdir=fr_t touch=fr_t/marker fdopendir_readdir=fr_t,marker"))
#endif


	VN(MultiProcWrite,
		TC("write=shared.txt,abc"),
		TC("read=shared.txt,abc size=shared.txt,3 unlink=shared.txt notexists=shared.txt"))
	VN(MultiProcDir,
		TC("mkdir=md touch=md/q"),
		TC("isdir=md listcontains=md,q unlink=md/q rmdir=md notexists=md"))
	VN(MultiProcRename,
		TC("write=mp_a.txt,one rename=mp_a.txt,mp_b.txt"),
		TC("notexists=mp_a.txt read=mp_b.txt,one unlink=mp_b.txt"))
	VN(MultiProcCopy,
		TC("write=mp_src.txt,xyz copy=mp_src.txt,mp_dst.txt"),
		TC("read=mp_src.txt,xyz read=mp_dst.txt,xyz"))

#if PLATFORM_LINUX
	// Atomic-replace (lld / Python NamedTemporaryFile): P2 unlinks A, opens A.tmp, renames-while-open A.tmp→A. Pre-fix the writtenFiles-HIT branch in SessionClient::MoveFile didn't erase A from shared.deletedFiles, so process-exit sent a stale DeleteFile(A) even though SendFiles uploaded the new bytes.
	VN(MultiProcAtomicReplace,
		TC("write=arl_a,initial-content"),
		TC("unlink=arl_a renamewhileopen=arl_a.tmp,arl_a"),
		TC("read=arl_a,rustc-archive-pattern size=arl_a,21"))
	// No-prior-state baseline: CloseFile-with-newName finalization must work without the cleanup branch firing.
	VN(MultiProcAtomicReplaceFresh,
		TC("renamewhileopen=arf_a.tmp,arf_a"),
		TC("read=arf_a,rustc-archive-pattern size=arf_a,21"))
	// Two consecutive atomic-replaces — cleanup must be repeatable post main-session flush.
	VN(MultiProcAtomicReplaceTwice,
		TC("write=art_a,seed-content"),
		TC("unlink=art_a renamewhileopen=art_a.tmp,art_a"),
		TC("read=art_a,rustc-archive-pattern size=art_a,21"),
		TC("unlink=art_a renamewhileopen=art_a.tmp,art_a"),
		TC("read=art_a,rustc-archive-pattern size=art_a,21"))
	// VN(AtomicReplaceSameProc, ...) — BUG (LOCAL): single-process atomic-replace
	// (write A → unlink A → renamewhileopen A.tmp,A → read A) returns ENOENT on the read.
	// Session::CloseFile sets writtenFile.backedName = msg.newName (virtual path) but the
	// detour skipped trueRename, so the file is still on disk at A.tmp. Same code class as
	// feedback_rename_while_open_staged_skip_truerename.md but the LOCAL-mode variant.
	// Multi-proc (VerbMultiProcAtomicReplace) works because the file goes through SendFiles
	// on process exit and the consumer process re-resolves through main dirtable.
	// VN(AtomicReplaceSameProc,
	//	TC("write=ars_a,seed unlink=ars_a renamewhileopen=ars_a.tmp,ars_a read=ars_a,rustc-archive-pattern size=ars_a,21"))
	//
	// VN(MultiProcAtomicReplaceThenRename, ...) — BUG (REMOTE): atomic-replace + rename-out
	// in same tree drops the unlink. P1 writes A. P2: unlink A → renamewhileopen A.tmp,A → rename A,B.
	// CloseFile cleanup erases deletedFiles[A]; subsequent SessionClient::MoveFile writtenFilesHit
	// moves writtenFiles[A]→[B] without re-adding A to deletedFiles. Producer's exit sends nothing
	// for A; main dirtable still has P1's A. Fix-direction: writtenFilesHit branch should re-emit
	// deletedFiles[fromName] when fromName was previously unlinked in this tree (or skip the
	// CloseFile-time erase when the file is still in writtenFiles awaiting upload).
	// VN(MultiProcAtomicReplaceThenRename,
	//	TC("write=arr_a,seed"),
	//	TC("unlink=arr_a renamewhileopen=arr_a.tmp,arr_a rename=arr_a,arr_b"),
	//	TC("notexists=arr_a read=arr_b,rustc-archive-pattern size=arr_b,21"))
#endif


	VN(SpawnRead,     TC("write=sp.txt,childdata spawn=read=sp.txt,childdata;size=sp.txt,9 unlink=sp.txt"))
	VN(SpawnWrite,    TC("spawn=write=spw.txt,fromchild read=spw.txt,fromchild unlink=spw.txt"))
	VN(SpawnDir,      TC("mkdir=spdir spawn=touch=spdir/q;listcontains=spdir,q listdir=spdir,1 unlink=spdir/q rmdir=spdir"))
	VN(BigChain,      TC("mkdir=A mkdir=A/B mkdir=A/B/C touch=A/x touch=A/B/y touch=A/B/C/z isfile=A/x isfile=A/B/y isfile=A/B/C/z listdir=A,2 listdir=A/B,2 listdir=A/B/C,1 unlink=A/B/C/z rmdir=A/B/C unlink=A/B/y rmdir=A/B unlink=A/x rmdir=A notexists=A"))
	VN(MultiProcChain,
		TC("mkdir=d touch=d/a write=f1,one"),
		TC("isdir=d listcontains=d,a touch=d/b read=f1,one write=f2,two"),
		TC("listdir=d,2 listcontains=d,a listcontains=d,b read=f1,one read=f2,two"))
	VN(NestedSpawn,   TC("touch=p spawn=stat=p;touch=c;spawn=stat=p;stat=c;touch=g stat=c stat=g unlink=p unlink=c unlink=g"))
	VN(SpawnRewrite,  TC("write=sr,init spawn=read=sr,init;write=sr,after read=sr,after"))
	VN(SpawnList,     TC("mkdir=sl nfiles=sl,e,4 spawn=listdir=sl,4;listcontains=sl,e0;listcontains=sl,e3"))
	VN(MultiProcList,
		TC("mkdir=mpl nfiles=mpl,k,6"),
		TC("listdir=mpl,6 listcontains=mpl,k0 listcontains=mpl,k5"))
	VN(MultiProcAddChain,
		TC("mkdir=ac write=ac/p1,1"),
		TC("read=ac/p1,1 write=ac/p2,2"),
		TC("read=ac/p1,1 read=ac/p2,2 write=ac/p3,3"),
		TC("read=ac/p1,1 read=ac/p2,2 read=ac/p3,3 write=ac/p4,4"),
		TC("listdir=ac,4 read=ac/p1,1 read=ac/p2,2 read=ac/p3,3 read=ac/p4,4"))
	// AOSP check_last_released_api.timestamp pattern — P2's rename hits SessionClient::MoveFile writtenFiles-MISS branch (overlay.movedFiles + MoveFile RPC).
	VN(CrossTreeRenameSimple,
		TC("write=ctr_a,one"),
		TC("isfile=ctr_a rename=ctr_a,ctr_b notexists=ctr_a isfile=ctr_b"),
		TC("notexists=ctr_a read=ctr_b,one"))
	// "delete cancels move": rename A→B + unlink B must end with neither.
	VN(CrossTreeRenameDelete,
		TC("write=ctd_a,one"),
		TC("rename=ctd_a,ctd_b unlink=ctd_b notexists=ctd_a notexists=ctd_b"),
		TC("notexists=ctd_a notexists=ctd_b"))
	// movedFiles compose: A→B→C in one process.
	VN(CrossTreeRenameChain,
		TC("write=ctc_a,seed"),
		TC("rename=ctc_a,ctc_b rename=ctc_b,ctc_c notexists=ctc_a notexists=ctc_b read=ctc_c,seed"),
		TC("notexists=ctc_a notexists=ctc_b read=ctc_c,seed"))
	// Overlay-aliasing class — see project_remote_rename_then_writesource.md.
	VN(CrossTreeRenameOverwrite,
		TC("write=cto_a,first"),
		TC("rename=cto_a,cto_b write=cto_a,second read=cto_a,second read=cto_b,first"),
		TC("read=cto_a,second read=cto_b,first"))
	// Compose-to-identity for movedFiles + overlay key cleanup.
	VN(CrossTreeRenameBack,
		TC("write=ctb_a,bytes"),
		TC("rename=ctb_a,ctb_b rename=ctb_b,ctb_a read=ctb_a,bytes notexists=ctb_b"),
		TC("read=ctb_a,bytes notexists=ctb_b"))
	// Post-MISS-rename dest readable in same overlay (not just from a later process).
	VN(CrossTreeRenameThenRead,
		TC("write=ctrr_a,abcdef"),
		TC("rename=ctrr_a,ctrr_b read=ctrr_b,abcdef size=ctrr_b,6"),
		TC("read=ctrr_b,abcdef notexists=ctrr_a"))
	// Parent's overlay isn't disturbed by a sibling child's transient state.
	VN(SpawnTransient, TC("write=parent,p spawn=read=parent,p;write=child,c read=parent,p read=child,c"))


	VN(SpawnSpawn,                  TC("touch=ss spawn=spawn=read=ss,;write=ssc, read=ss, read=ssc,"))
	VN(SpawnSiblings,               TC("write=ssA,a write=ssB,b spawn=read=ssA,a;read=ssB,b spawn=read=ssA,a;read=ssB,b"))
	VN(SpawnDeleteCreate,           TC("write=sd,one spawn=unlink=sd;write=sd,two read=sd,two"))
	VN(SpawnAppendChain,            TC("write=sa,A spawn=read=sa,A;append=sa,B read=sa,AB spawn=read=sa,AB;append=sa,C read=sa,ABC"))
	VN(SpawnMkdirChild,             TC("spawn=mkdir=spd;touch=spd/x isdir=spd isfile=spd/x listdir=spd,1"))
	VN(SpawnRmdirChild,             TC("mkdir=srd touch=srd/x spawn=unlink=srd/x;rmdir=srd notexists=srd"))
	VN(SpawnRenameChild,            TC("write=sR,1 spawn=rename=sR,sR2 read=sR2,1"))
	VN(DeepSpawn4,                  TC("write=d0,seed spawn=read=d0,seed;write=d1,1;spawn=read=d1,1;write=d2,2;spawn=read=d2,2;write=d3,3 read=d0,seed read=d1,1 read=d2,2 read=d3,3"))
	VN(ReadWriteCycle,              TC("write=rwc,1 read=rwc,1 write=rwc,22 read=rwc,22 write=rwc,333 read=rwc,333 write=rwc,4444 read=rwc,4444 size=rwc,4"))
	VN(AppendBuildup,               TC("append=ab,a append=ab,b append=ab,c append=ab,d append=ab,e read=ab,abcde size=ab,5"))
	VN(InterleavedFiles,            TC("write=if1,1 write=if2,2 write=if3,3 read=if1,1 read=if2,2 read=if3,3 unlink=if2 read=if1,1 read=if3,3 notexists=if2"))
	VN(DirOfDirs,                   TC("mkdir=dod mkdir=dod/a mkdir=dod/b mkdir=dod/c listdir=dod,3 listcontains=dod,a listcontains=dod,b listcontains=dod,c rmdir=dod/a rmdir=dod/b rmdir=dod/c rmdir=dod"))
	VN(MixedDirsFiles,              TC("mkdir=mdf touch=mdf/file1 mkdir=mdf/sub touch=mdf/file2 touch=mdf/sub/inner listdir=mdf,3 listdir=mdf/sub,1 isdir=mdf/sub isfile=mdf/file1 isfile=mdf/sub/inner"))
	VN(StatBeforeAfter,             TC("notexists=sba write=sba,x stat=sba unlink=sba notexists=sba write=sba,y stat=sba unlink=sba notexists=sba"))
	VN(RapidCreateDelete,           TC("touch=rc unlink=rc touch=rc unlink=rc touch=rc unlink=rc touch=rc unlink=rc notexists=rc"))
	VN(DeleteRecreate,              TC("write=dr,old unlink=dr write=dr,new read=dr,new size=dr,3"))
	VN(AppendThenRename,            TC("append=atr,foo rename=atr,atr2 read=atr2,foo append=atr2,bar read=atr2,foobar"))
	VN(CopyThenAppend,              TC("write=cta,seed copy=cta,cta2 append=cta2,!! read=cta,seed read=cta2,seed!!"))
	VN(RenameToDifDir,              TC("mkdir=rd1 mkdir=rd2 write=rd1/f,1 rename=rd1/f,rd2/g read=rd2/g,1 notexists=rd1/f"))
	VN(CopyToDifDir,                TC("mkdir=cd1 mkdir=cd2 write=cd1/f,1 copy=cd1/f,cd2/g read=cd1/f,1 read=cd2/g,1"))
	VN(MkdirSpawnFiles,             TC("mkdir=msf spawn=touch=msf/a;touch=msf/b;touch=msf/c listdir=msf,3 listcontains=msf,a listcontains=msf,b listcontains=msf,c"))
	VN(StatAfterChdir,              TC("mkdir=sac touch=sac/file chdir=sac stat=file isfile=file size=file,0 chdir=.. stat=sac/file"))
	VN(ReadAfterChdir,              TC("mkdir=rac write=rac/f,abc chdir=rac read=f,abc append=f,DEF read=f,abcDEF"))
	VN(CwdSurviveOps,               TC("mkdir=cso chdir=cso touch=a touch=b touch=c listdir=.,3 unlink=a unlink=b unlink=c listdir=.,0"))
	VN(ZeroByteOps,                 TC("touch=zb size=zb,0 read=zb, append=zb, size=zb,0 copy=zb,zb2 size=zb2,0 read=zb2,"))
	VN(BigContent,                  TC("write=big,abcdefghijklmnopqrstuvwxyz0123456789ABCDEFGHIJKLMNOPQRSTUVWXYZ size=big,62 read=big,abcdefghijklmnopqrstuvwxyz0123456789ABCDEFGHIJKLMNOPQRSTUVWXYZ"))
	VN(WriteIntoDeepDir,            TC("mkdir=w1 mkdir=w1/w2 mkdir=w1/w2/w3 write=w1/w2/w3/file,deep read=w1/w2/w3/file,deep"))
	VN(SpawnReadParent,             TC("write=spr,parentdata spawn=read=spr,parentdata;size=spr,10 unlink=spr"))
	VN(SpawnWriteRead,              TC("spawn=write=swr,fromchild;read=swr,fromchild read=swr,fromchild unlink=swr"))
	VN(MultipleStat,                TC("write=mst,abc stat=mst stat=mst stat=mst stat=mst stat=mst isfile=mst isfile=mst isfile=mst"))
	VN(OpenWMissing,                TC("open=opw,w stat=opw isfile=opw size=opw,0"))
	VN(OpenAMissing,                TC("open=opa,a stat=opa isfile=opa size=opa,0"))
	// VN(RenameNotexists, ...) — BUG (Posix): UBA asserts "MoveFileExW error handling not implemented" when source missing. Real rename(2) returns -1/ENOENT.
	// VN(CopyToExistingDir, ...) — BUG: Shared_CopyFileExW asserts when dest is a directory (KernelBase.inl:1197). Real Win32 returns FALSE/ERROR_ACCESS_DENIED.
	VN(MkdirInsideMissing,          TC("mustfail=mkdir=missing/sub"))
	VN(DeleteFileTwice,             TC("touch=dt unlink=dt mustfail=unlink=dt"))
	// VN(RmdirTwice, ...) — hits the rmdir-on-missing-path bug. See memory project_rmdir_missing_crash.md
	VN(StatAfterDelete,             TC("write=sad,1 unlink=sad mustfail=stat=sad mustfail=open=sad,r mustfail=size=sad,1"))
	VN(CopyZeroByte,                TC("touch=cz copy=cz,cz2 size=cz,0 size=cz2,0 read=cz, read=cz2,"))
	VN(AppendZeroBytes,             TC("write=az,foo append=az, read=az,foo size=az,3"))
	VN(RenameZeroByte,              TC("touch=rz rename=rz,rz2 size=rz2,0 isfile=rz2 notexists=rz"))
	VN(CwdRelativeMkdir,            TC("mkdir=cr1 chdir=cr1 mkdir=cr2 chdir=cr2 touch=f stat=f"))


	VN(Spawn3DeepWriteAll,          TC("spawn=spawn=spawn=write=g3,gg;read=g3,gg;write=g2,gg;read=g2,gg;write=g1,gg read=g1,gg read=g2,gg read=g3,gg"))
	VN(Spawn3DeepReadAll,           TC("write=base,X spawn=read=base,X;spawn=read=base,X;spawn=read=base,X read=base,X"))
	VN(SpawnEachLevelAdds,          TC("touch=L0 spawn=touch=L1;spawn=touch=L2;spawn=touch=L3 stat=L0 stat=L1 stat=L2 stat=L3"))
	VN(SpawnUnlinkChain,            TC("write=u1,1 write=u2,2 write=u3,3 spawn=unlink=u1;spawn=unlink=u2;spawn=unlink=u3 notexists=u1 notexists=u2 notexists=u3"))
	VN(StatThenWriteOverlay,        TC("notexists=mwo write=mwo,abc stat=mwo size=mwo,3 stat=mwo size=mwo,3"))
	// writtenFiles-HIT path; within-tree movedFiles+deletedFiles interaction.
	VN(MoveThenDeleteDst,           TC("write=mtd_a,X rename=mtd_a,mtd_b unlink=mtd_b notexists=mtd_a notexists=mtd_b"))
	// Move record must not alias the new write (overlay-aliasing class — see project_remote_rename_then_writesource.md).
	VN(MoveAndRecreateSrc,          TC("write=mtr_a,first rename=mtr_a,mtr_b write=mtr_a,second read=mtr_a,second read=mtr_b,first"))
	// Compose-to-identity for movedFiles.
	VN(MoveAtoBtoA,                 TC("write=mtb_a,seed rename=mtb_a,mtb_b rename=mtb_b,mtb_a read=mtb_a,seed notexists=mtb_b"))
	// movedFiles must compose.
	VN(MoveAtoBtoC,                 TC("write=mtm_a,bytes rename=mtm_a,mtm_b rename=mtm_b,mtm_c notexists=mtm_a notexists=mtm_b read=mtm_c,bytes"))
	VN(MoveAndUnlinkSrc,            TC("write=mds_a,1 rename=mds_a,mds_b mustfail=unlink=mds_a notexists=mds_a read=mds_b,1"))
	VN(WriteSpawnRead,              TC("write=wsr,parent spawn=read=wsr,parent;append=wsr,!c read=wsr,parent!c"))
	VN(SpawnWriteUnlink,            TC("spawn=write=swu,x;unlink=swu notexists=swu"))
	VN(SpawnMkdirRmdir,             TC("spawn=mkdir=smr;rmdir=smr notexists=smr"))
	VN(DirChainAcrossSpawn,         TC("mkdir=dc spawn=mkdir=dc/sub;touch=dc/sub/leaf isfile=dc/sub/leaf isdir=dc/sub listdir=dc,1"))
	VN(SpawnMultipleWrites,         TC("spawn=write=mw1,a;write=mw2,b;write=mw3,c read=mw1,a read=mw2,b read=mw3,c"))
	VN(SpawnAppendRead,             TC("write=sar,head spawn=append=sar,_tail read=sar,head_tail size=sar,9"))
	VN(StatLoop,                    TC("write=sl,x stat=sl stat=sl stat=sl stat=sl stat=sl stat=sl stat=sl stat=sl"))
	VN(ReadLoop,                    TC("write=rl,abc read=rl,abc read=rl,abc read=rl,abc read=rl,abc read=rl,abc"))
	VN(DirOpsLoop,                  TC("mkdir=dl rmdir=dl mkdir=dl rmdir=dl mkdir=dl rmdir=dl mkdir=dl rmdir=dl notexists=dl"))
	VN(CreateThenDeleteCycle,       TC("touch=ct unlink=ct touch=ct unlink=ct touch=ct unlink=ct notexists=ct"))
	VN(DeepWriteCycle,              TC("mkdir=dw mkdir=dw/x mkdir=dw/x/y write=dw/x/y/leaf,A read=dw/x/y/leaf,A unlink=dw/x/y/leaf rmdir=dw/x/y rmdir=dw/x rmdir=dw notexists=dw"))
	VN(ConcurrentChildren,          TC("spawn=touch=c1 spawn=touch=c2 spawn=touch=c3 spawn=touch=c4 stat=c1 stat=c2 stat=c3 stat=c4"))
	VN(SpawnMultipleSiblings,       TC("write=sib0,seed spawn=read=sib0,seed;write=sib1,1 spawn=read=sib0,seed;read=sib1,1;write=sib2,2 stat=sib0 stat=sib1 stat=sib2"))
	VN(BigDir,                      TC("mkdir=bd nfiles=bd,n,32 listdir=bd,32 listcontains=bd,n0 listcontains=bd,n31 size=bd/n0,0 size=bd/n15,0 size=bd/n31,0"))
	VN(DirChangePopulate,           TC("mkdir=dcp listdir=dcp,0 touch=dcp/a listdir=dcp,1 touch=dcp/b listdir=dcp,2 unlink=dcp/a listdir=dcp,1 unlink=dcp/b listdir=dcp,0 rmdir=dcp"))
	VN(ReadStatRead,                TC("write=rsr,Hello read=rsr,Hello stat=rsr read=rsr,Hello size=rsr,5 read=rsr,Hello"))
	VN(WriteAppendRename,           TC("write=war,base append=war,more rename=war,war2 read=war2,basemore size=war2,8"))
	VN(CopyAppendOriginal,          TC("write=cao,A copy=cao,cao2 append=cao,X read=cao,AX read=cao2,A size=cao,2 size=cao2,1"))
	VN(StatMissingThenCreate,       TC("notexists=smc write=smc,1 stat=smc isfile=smc unlink=smc notexists=smc"))
	VN(MkdirThenSpawnList,          TC("mkdir=msl touch=msl/parent spawn=listdir=msl,1;listcontains=msl,parent listdir=msl,1"))
	VN(BackslashAppend,             TC("write=ba,foo append=ba,/bar read=ba,foo/bar"))
	VN(StatNonexistAcross,          TC("notexists=nx_root spawn=notexists=nx_root notexists=nx_root"))
	// VN(MustFailOpenWX, ...) — BUG: empty path to Detoured_CreateFileW (KernelBase.inl:3682) asserts; should return ERROR_PATH_NOT_FOUND.
	VN(MustFailReadMissing,         TC("mustfail=read=ghost,a mustfail=read=ghost,"))
	VN(WriteThenStatSpawn,          TC("write=wts,1 spawn=stat=wts;isfile=wts;size=wts,1 size=wts,1"))
	VN(DirExistsSpawnVisible,       TC("mkdir=dev touch=dev/q spawn=stat=dev;listcontains=dev,q listcontains=dev,q"))
	VN(BatchSpawnReads,             TC("write=br,x spawn=read=br,x spawn=read=br,x spawn=read=br,x spawn=read=br,x"))
	VN(DeepNestThenSpawn,           TC("mkdir=dn1 mkdir=dn1/dn2 mkdir=dn1/dn2/dn3 write=dn1/dn2/dn3/file,deep spawn=read=dn1/dn2/dn3/file,deep;isfile=dn1/dn2/dn3/file"))
	VN(SpawnInheritsCwd,            TC("mkdir=spc chdir=spc touch=marker spawn=stat=marker;isfile=marker stat=marker"))
	VN(WriteDeleteRecreate,         TC("write=wdr,1 unlink=wdr notexists=wdr write=wdr,2 read=wdr,2 size=wdr,1 unlink=wdr"))
	VN(SpawnRenameUnlink,           TC("write=sru,1 spawn=rename=sru,sru2;unlink=sru2 notexists=sru notexists=sru2"))
	VN(SpawnFanout,                 TC("spawn=touch=fan_a spawn=touch=fan_b spawn=touch=fan_c spawn=touch=fan_d spawn=touch=fan_e listdir=.,5 listcontains=.,fan_a listcontains=.,fan_e"))
	VN(SpawnFanoutDir,              TC("mkdir=fo spawn=touch=fo/a spawn=touch=fo/b spawn=touch=fo/c spawn=touch=fo/d listdir=fo,4"))
	VN(SpawnReadAfterChild,         TC("write=src,seed spawn=read=src,seed read=src,seed"))
	VN(SpawnUnlinkInParent,         TC("spawn=write=sup,p unlink=sup notexists=sup"))
	VN(WriteWriteRead,              TC("write=ww,first write=ww,second write=ww,third read=ww,third size=ww,5"))
	VN(AlternatingAccess,           TC("write=alt,1 read=alt,1 write=alt,22 read=alt,22 write=alt,333 read=alt,333 size=alt,3"))
	VN(DirAccessSequence,           TC("mkdir=das listdir=das,0 touch=das/x listdir=das,1 mkdir=das/sub listdir=das,2 touch=das/sub/y listdir=das/sub,1 listdir=das,2"))
	VN(CopyToNewDir,                TC("mkdir=cnd write=src,abc copy=src,cnd/dst read=cnd/dst,abc size=cnd/dst,3"))
	VN(CreateInDeepPath,            TC("mkdir=cdp1 mkdir=cdp1/cdp2 write=cdp1/cdp2/file,deep read=cdp1/cdp2/file,deep size=cdp1/cdp2/file,4"))
	VN(ReadAfterTouch,              TC("touch=rat read=rat, isfile=rat size=rat,0"))
	// VN(CopyEmptyToNonEmpty, ...) — hits Session::CopyFileW assert. See memory project_copyfile_overwrite_unimplemented.md
	VN(CwdAfterMkdir,               TC("mkdir=cam touch=cam/f chdir=cam isfile=f read=f, append=f,Z read=f,Z chdir=.. read=cam/f,Z"))
	VN(WriteSpawnSpawnRead,         TC("write=wssr,seed spawn=spawn=read=wssr,seed read=wssr,seed"))
	VN(SpawnGrandchildrenAddFiles,  TC("spawn=spawn=touch=g1;touch=g2 listdir=.,2 listcontains=.,g1 listcontains=.,g2"))
	VN(SpawnReadEachLevelAdd,       TC("write=lvl0,a spawn=read=lvl0,a;write=lvl1,b spawn=read=lvl0,a;read=lvl1,b;write=lvl2,c read=lvl0,a read=lvl1,b read=lvl2,c"))
	VN(CopyThenRename,              TC("write=ctr,1 copy=ctr,ctr2 rename=ctr2,ctr3 read=ctr,1 read=ctr3,1 notexists=ctr2"))
	VN(RenameThenCopy,              TC("write=rtc,1 rename=rtc,rtc2 copy=rtc2,rtc3 notexists=rtc read=rtc2,1 read=rtc3,1"))
	VN(ReadModifyReread,            TC("write=rmr,one read=rmr,one append=rmr,two read=rmr,onetwo size=rmr,6"))
	VN(StatLargerEachWrite,         TC("write=sl,1 size=sl,1 write=sl,11 size=sl,2 write=sl,111 size=sl,3 write=sl,1111 size=sl,4"))
	VN(DirEnumerateAfterDelete,     TC("mkdir=ed nfiles=ed,e,4 listdir=ed,4 unlink=ed/e1 listdir=ed,3 listexcludes=ed,e1 listcontains=ed,e0 listcontains=ed,e2 listcontains=ed,e3"))
	VN(AppendChainAcrossSpawn,      TC("write=acs,A append=acs,B spawn=read=acs,AB;append=acs,C read=acs,ABC spawn=read=acs,ABC;append=acs,D read=acs,ABCD"))
	VN(RapidStat,                   TC("touch=rs stat=rs stat=rs stat=rs stat=rs stat=rs stat=rs stat=rs stat=rs stat=rs"))
	VN(TouchWriteOverlay,           TC("touch=tw size=tw,0 write=tw,X size=tw,1 read=tw,X"))
	VN(WriteTouchOverlay,           TC("write=wt,Y size=wt,1 touch=wt size=wt,0 read=wt,"))
	VN(BigContentMultiAppend,       TC("append=bca,abc append=bca,defghi append=bca,jklmn append=bca,opqrstuv read=bca,abcdefghijklmnopqrstuv size=bca,22"))
	VN(CwdReadInChild,              TC("mkdir=crc write=crc/f,xx chdir=crc spawn=read=f,xx;size=f,2 read=f,xx"))
	VN(CreateThenRapidWrite,        TC("touch=tw_r append=tw_r,a append=tw_r,b append=tw_r,c append=tw_r,d append=tw_r,e read=tw_r,abcde size=tw_r,5"))
	VN(DirEmptyThenRefill,          TC("mkdir=etf nfiles=etf,a,3 listdir=etf,3 unlink=etf/a0 unlink=etf/a1 unlink=etf/a2 listdir=etf,0 nfiles=etf,b,3 listdir=etf,3 listcontains=etf,b0"))
	VN(SpawnAddSiblingDir,          TC("mkdir=sad mkdir=sad/sub spawn=mkdir=sad/sib listdir=sad,2 listcontains=sad,sub listcontains=sad,sib"))
	VN(StatSpawnDeleteStat,         TC("write=sds,1 spawn=unlink=sds notexists=sds"))

	// VN(Fail_OpenDirForWrite, ...) — Win64 asserts on CreateFileW(dir, GENERIC_WRITE).
	VN(Fail_OpenDirForExcl,         TC("mkdir=fdx mustfail=open=fdx,x"))
	VN(Fail_RmdirOnFile,            TC("touch=rdf mustfail=rmdir=rdf isfile=rdf"))
	VN(Fail_MkdirOnFile,            TC("touch=mof mustfail=mkdir=mof isfile=mof"))
	VN(Fail_RmdirNonemptyDir,       TC("mkdir=rne touch=rne/k mustfail=rmdir=rne unlink=rne/k rmdir=rne"))
	VN(Fail_TouchInsideFile,        TC("touch=tif mustfail=touch=tif/inside"))
	VN(Fail_StatThroughFile,        TC("touch=stf mustfail=stat=stf/inside"))
#if !PLATFORM_WINDOWS
	// VN(Fail_ReadDir, ...) — read on dir succeeds in remote (UBA returns 0 bytes via CAS path); kernel-level EISDIR not surfaced.
	VN(Fail_RmdirRoot,              TC("mustfail=rmdir=/"))
	VN(Fail_UnlinkRoot,             TC("mustfail=unlink=/"))
#endif
	// VN(Fail_RenameMissing, ...) — hits existing UBA bug: MoveFile-missing-source asserts. See memory project_posix_rename_missing_assert.md
	VN(Fail_CopyMissing,            TC("mustfail=copy=ghost_cm,target"))
	VN(Fail_OpenInsideMissingDir,   TC("mustfail=open=missing_dir/file,r mustfail=open=missing_dir/file,w"))
	VN(Fail_AppendInsideMissingDir, TC("mustfail=append=missing_dir2/file,X"))
	VN(Fail_StatRelativeMissing,    TC("notexists=./not_here notexists=./does_not_exist"))
	VN(Fail_DeepStatMissing,        TC("mustfail=stat=a/b/c/d/e/f/g/h notexists=a/b/c/d/e/f/g/h"))
	VN(Fail_MkdirOverFile,          TC("write=mover,X mustfail=mkdir=mover read=mover,X"))
	VN(Fail_MkdirOverDir,           TC("mkdir=mod mustfail=mkdir=mod isdir=mod rmdir=mod"))
	VN(Fail_OpenExclOverDir,        TC("mkdir=oed mustfail=open=oed,x rmdir=oed"))
	// VN(Fail_RenameToExisting, ...) — REMOTE: rename overwrites but content stays as old destination's. Same overlay-aliasing class as project_remote_rename_then_writesource.md.
	VN(Fail_DeleteThenRead,         TC("write=dtr,X unlink=dtr mustfail=read=dtr,X mustfail=open=dtr,r"))
	VN(Fail_DeleteThenStat,         TC("write=dts,Y unlink=dts mustfail=stat=dts mustfail=isfile=dts"))
	VN(Fail_DeleteThenSize,         TC("write=dtsz,abc unlink=dtsz mustfail=size=dtsz,3"))
	// VN(Fail_RmdirThenList, ...) — opendir on a removed dir still succeeds; dirtable doesn't fully invalidate the dir entry on rmdir. Pre-existing quirk.
	VN(Fail_RmdirThenStat,          TC("mkdir=rts rmdir=rts mustfail=stat=rts mustfail=isdir=rts"))
	VN(Fail_StatPath_TrailingSlashOnFile, TC("write=tsf,X notexists=tsf/anything mustfail=stat=tsf/anything"))
	VN(Fail_NestedMkdirNoIntermediates, TC("mustfail=mkdir=ne1/ne2/ne3"))
	VN(Fail_OpenAfterRename,        TC("write=oar,A rename=oar,oar2 mustfail=open=oar,r read=oar2,A"))
	VN(Fail_DoubleUnlinkLeaf,       TC("touch=du unlink=du mustfail=unlink=du"))
	VN(Fail_RmdirNonexistentLeaf,   TC("mkdir=rnl mustfail=rmdir=rnl/missing rmdir=rnl"))
	VN(Fail_StatOfDeletedDir,       TC("mkdir=sod rmdir=sod mustfail=stat=sod mustfail=isdir=sod"))
	// VN(Fail_AppendThenStatInDeletedDir, ...) — same dirtable-rmdir-not-fully-invalidating quirk as VerbFail_RmdirThenList.
	// VN(Fail_MkdirSlashFile, ...) — Session::CreateDirectory pre-registers overlay then runs disk mkdir, ignoring failure. Pre-existing UBA gap.
	VN(MkdirRmdirCycle,             TC("mkdir=mr touch=mr/x unlink=mr/x rmdir=mr mkdir=mr touch=mr/y read=mr/y, unlink=mr/y rmdir=mr"))
	VN(DeepDirContents,             TC("mkdir=A mkdir=A/B mkdir=A/B/C touch=A/x touch=A/B/y touch=A/B/C/z listdir=A,2 listdir=A/B,2 listdir=A/B/C,1"))
	VN(ListSeqAfterMutate,          TC("mkdir=lsm nfiles=lsm,a,3 listdir=lsm,3 nfiles=lsm,b,3 listdir=lsm,6 unlink=lsm/a0 unlink=lsm/a1 listdir=lsm,4"))
	VN(WriteSizeMultiple,           TC("write=wsm,X size=wsm,1 write=wsm,XX size=wsm,2 write=wsm,XXX size=wsm,3 write=wsm,XXXX size=wsm,4 write=wsm,XXXXX size=wsm,5"))
	VN(RapidRename,                 TC("write=r1,1 rename=r1,r2 rename=r2,r3 rename=r3,r4 rename=r4,r5 read=r5,1 notexists=r1 notexists=r2 notexists=r3 notexists=r4"))
	VN(MultiCopy,                   TC("write=mc,seed copy=mc,c1 copy=mc,c2 copy=mc,c3 read=c1,seed read=c2,seed read=c3,seed size=c1,4 size=c2,4 size=c3,4"))
	VN(DirThreeLevels,              TC("mkdir=L1 mkdir=L1/L2 mkdir=L1/L2/L3 touch=L1/file1 touch=L1/L2/file2 touch=L1/L2/L3/file3 stat=L1/file1 stat=L1/L2/file2 stat=L1/L2/L3/file3 listdir=L1,2 listdir=L1/L2,2 listdir=L1/L2/L3,1"))
	VN(AlternatingDirs,             TC("mkdir=adr touch=adr/x mkdir=adb touch=adb/y listdir=adr,1 listdir=adb,1 listcontains=adr,x listcontains=adb,y listexcludes=adr,y listexcludes=adb,x"))
	VN(SequentialFileNames,         TC("nfiles=.,seq,15 stat=seq0 stat=seq5 stat=seq10 stat=seq14 size=seq0,0"))
	VN(StatThenWriteAfter,          TC("notexists=stwa stat=. write=stwa,1 stat=stwa size=stwa,1 unlink=stwa"))
	VN(WriteThenManyStats,          TC("write=wms,abc stat=wms stat=wms isfile=wms isfile=wms size=wms,3 size=wms,3 read=wms,abc read=wms,abc"))
	VN(TouchInDir,                  TC("mkdir=tid touch=tid/a touch=tid/b touch=tid/c listdir=tid,3 unlink=tid/a unlink=tid/b unlink=tid/c rmdir=tid notexists=tid"))
	VN(MultiSpawnSequential,        TC("spawn=touch=ms1 spawn=touch=ms2 spawn=touch=ms3 spawn=touch=ms4 spawn=touch=ms5 listdir=.,5"))
	VN(SpawnMakesDir,               TC("spawn=mkdir=spm;touch=spm/inside isdir=spm isfile=spm/inside listcontains=spm,inside"))
	VN(SpawnCleansDir,              TC("mkdir=scd touch=scd/x touch=scd/y spawn=unlink=scd/x;unlink=scd/y;rmdir=scd notexists=scd"))
	VN(SpawnMakeSeq,                TC("spawn=touch=sms1;touch=sms2;touch=sms3 listdir=.,3 stat=sms1 stat=sms2 stat=sms3"))
	VN(DirtyOverlayInChild,         TC("mkdir=doc touch=doc/x spawn=touch=doc/y;touch=doc/z listdir=doc,3 listcontains=doc,x listcontains=doc,y listcontains=doc,z"))
	VN(AppendAfterCopy,             TC("write=aac,start copy=aac,aac2 append=aac2,END read=aac,start read=aac2,startEND"))
	// VN(WriteRenameNew, ...) — BUG (remote only): reading the renamed wrn2 returns the new content of wrn, not the original. Overlay aliasing issue. Local works.
	VN(SpawnUnlinkAndCheckParent,   TC("write=suc,1 spawn=unlink=suc;notexists=suc notexists=suc"))
	VN(SpawnCreateMultipleAndStat,  TC("spawn=touch=scm0 spawn=touch=scm1 spawn=touch=scm2 stat=scm0 stat=scm1 stat=scm2"))
	VN(ThreeSpawnDifferentDirs,     TC("mkdir=tsd0 mkdir=tsd1 mkdir=tsd2 spawn=touch=tsd0/a;touch=tsd0/b spawn=touch=tsd1/c;touch=tsd1/d spawn=touch=tsd2/e listdir=tsd0,2 listdir=tsd1,2 listdir=tsd2,1"))
	VN(RapidLs,                     TC("mkdir=rls nfiles=rls,r,5 listdir=rls,5 listdir=rls,5 listdir=rls,5 listdir=rls,5 listdir=rls,5"))
	VN(StatBetweenWrites,           TC("write=sbw,1 stat=sbw write=sbw,2 stat=sbw write=sbw,3 stat=sbw size=sbw,1"))
	VN(SpawnAppend2Times,           TC("write=sa2,init spawn=append=sa2,_one read=sa2,init_one spawn=append=sa2,_two read=sa2,init_one_two"))
	VN(DirsParallel,                TC("mkdir=pa mkdir=pb mkdir=pc touch=pa/file touch=pb/file touch=pc/file isfile=pa/file isfile=pb/file isfile=pc/file"))
	VN(DeepWriteThenRm,             TC("mkdir=dwr mkdir=dwr/sub write=dwr/sub/leaf,X read=dwr/sub/leaf,X unlink=dwr/sub/leaf rmdir=dwr/sub rmdir=dwr"))
	VN(SpawnWriteMatching,          TC("spawn=write=sw1,X spawn=write=sw2,Y spawn=write=sw3,Z read=sw1,X read=sw2,Y read=sw3,Z"))
	VN(WriteThenMultipleReadsBig,   TC("write=wmrb,abcdefghijklmnopqrst read=wmrb,abcdefghijklmnopqrst read=wmrb,abcdefghijklmnopqrst read=wmrb,abcdefghijklmnopqrst size=wmrb,20"))
	VN(MkdirThenStatSpawn,          TC("mkdir=mts spawn=isdir=mts;listdir=mts,0 isdir=mts"))
	// VN(CpRnAppendDelete, ...) — BUG (remote): append after copy+rename produces wrong size. Same overlay aliasing class.
	VN(AppendOverwriteRead,         TC("write=aor,a append=aor,b append=aor,c read=aor,abc write=aor,DIFFERENT read=aor,DIFFERENT"))
	VN(StatMissingThenSpawnCreate,  TC("notexists=msc spawn=write=msc,A stat=msc isfile=msc read=msc,A"))
	VN(RmdirAfterUnlinks,           TC("mkdir=rau touch=rau/a touch=rau/b touch=rau/c touch=rau/d unlink=rau/a unlink=rau/b unlink=rau/c unlink=rau/d listdir=rau,0 rmdir=rau"))
	VN(WriteMidPath,                TC("mkdir=wmp1 mkdir=wmp1/wmp2 write=wmp1/wmp2/leaf,A append=wmp1/wmp2/leaf,B read=wmp1/wmp2/leaf,AB size=wmp1/wmp2/leaf,2"))
	VN(SpawnCwdEffective,           TC("mkdir=sce chdir=sce write=here,X spawn=read=here,X read=here,X"))
	VN(StatRecreate,                TC("write=sr,abc unlink=sr notexists=sr write=sr,xyz read=sr,xyz"))
	VN(CopyAndDeleteSrc,            TC("write=cd_s,1 copy=cd_s,cd_d unlink=cd_s notexists=cd_s read=cd_d,1"))
	// VN(AppendAfterDelete, ...) — BUG (remote): append after unlink+rewrite still includes old content; size=2 instead of 1. Same overlay aliasing class as project_remote_rename_then_writesource.md.
	VN(WriteAndStatLoop,            TC("write=wsl,abc stat=wsl write=wsl,defgh stat=wsl size=wsl,5 write=wsl,1234567 stat=wsl size=wsl,7"))
	VN(DirAddRemoveLoop,            TC("mkdir=darl touch=darl/a unlink=darl/a touch=darl/b unlink=darl/b touch=darl/c unlink=darl/c listdir=darl,0 rmdir=darl"))
	VN(CopyFromMissingDir,          TC("mustfail=copy=missing/file,dst notexists=dst"))
	// VN(RenameFromMissingDir, ...) — same MoveFile-error-not-implemented assert as VerbRenameNotexists. See memory project_posix_rename_missing_assert.md
	VN(StatDeepMissing,             TC("notexists=a/b/c/d/e mustfail=stat=a/b/c/d/e"))
	VN(AppendOpenedTwice,           TC("append=aot,a append=aot,b append=aot,c append=aot,d append=aot,e append=aot,f read=aot,abcdef"))
	VN(SiblingDirsList,             TC("mkdir=sdl1 mkdir=sdl2 mkdir=sdl3 listcontains=.,sdl1 listcontains=.,sdl2 listcontains=.,sdl3 rmdir=sdl1 rmdir=sdl2 rmdir=sdl3"))
	VN(DeepNestedListAll,           TC("mkdir=L1 mkdir=L1/L2 mkdir=L1/L2/L3 mkdir=L1/L2/L3/L4 mkdir=L1/L2/L3/L4/L5 listdir=L1,1 listdir=L1/L2,1 listdir=L1/L2/L3,1 listdir=L1/L2/L3/L4,1 listdir=L1/L2/L3/L4/L5,0"))
	VN(WriteSpawnCheckLs,           TC("write=wsl1,a write=wsl2,b write=wsl3,c spawn=listcontains=.,wsl1;listcontains=.,wsl2;listcontains=.,wsl3 listcontains=.,wsl1"))
	VN(SpawnMakesPathDeep,          TC("spawn=mkdir=smp;mkdir=smp/sub;mkdir=smp/sub/leaf isdir=smp isdir=smp/sub isdir=smp/sub/leaf rmdir=smp/sub/leaf rmdir=smp/sub rmdir=smp"))
	VN(CopyToSamefnInDir,           TC("write=base,1 mkdir=cdir copy=base,cdir/base read=cdir/base,1"))
	VN(WriteAfterListdir,           TC("mkdir=wal touch=wal/a listdir=wal,1 write=wal/b,2 listdir=wal,2 read=wal/b,2"))
	VN(ReadAfterListdir,            TC("mkdir=ral nfiles=ral,r,3 listdir=ral,3 read=ral/r0, read=ral/r1, read=ral/r2,"))
	VN(DeepCopyMatrix,              TC("mkdir=dcm mkdir=dcm/X mkdir=dcm/Y write=dcm/X/file,abc copy=dcm/X/file,dcm/Y/file read=dcm/X/file,abc read=dcm/Y/file,abc"))
	VN(WriteThenStatViaMultiple,    TC("write=wsm2,abcde size=wsm2,5 stat=wsm2 stat=wsm2 size=wsm2,5 size=wsm2,5 stat=wsm2"))
	VN(SpawnDirAndChild,            TC("spawn=mkdir=sdc;touch=sdc/leaf isdir=sdc isfile=sdc/leaf listcontains=sdc,leaf"))
	VN(BigFanout,                   TC("spawn=touch=fb_a spawn=touch=fb_b spawn=touch=fb_c spawn=touch=fb_d spawn=touch=fb_e spawn=touch=fb_f spawn=touch=fb_g spawn=touch=fb_h spawn=touch=fb_i spawn=touch=fb_j listcontains=.,fb_a listcontains=.,fb_j"))
	VN(DeepReuse,                   TC("mkdir=dr touch=dr/a unlink=dr/a touch=dr/a unlink=dr/a touch=dr/a read=dr/a,"))
	VN(ConcurrentReads,             TC("write=cr,SHARED spawn=read=cr,SHARED;size=cr,6 spawn=read=cr,SHARED;size=cr,6 spawn=read=cr,SHARED;size=cr,6 read=cr,SHARED"))
	VN(StatSpawnStat,               TC("write=sss,1 stat=sss spawn=stat=sss;size=sss,1 stat=sss"))
	VN(DirShared,                   TC("mkdir=ds touch=ds/file spawn=stat=ds/file;listcontains=ds,file spawn=isfile=ds/file isfile=ds/file"))
	VN(WriteThenSpawnDeleteAndCheck, TC("write=wsd,1 spawn=unlink=wsd;notexists=wsd notexists=wsd touch=wsd stat=wsd"))
	VN(AppendExtendsAfterRead,      TC("write=aex,abc read=aex,abc append=aex,DEF read=aex,abcDEF size=aex,6 append=aex,XY read=aex,abcDEFXY size=aex,8"))
	VN(ReadAfterCopyChain,          TC("write=rcc,X copy=rcc,rcc1 copy=rcc1,rcc2 copy=rcc2,rcc3 copy=rcc3,rcc4 read=rcc,X read=rcc1,X read=rcc2,X read=rcc3,X read=rcc4,X"))
	VN(StatAfterCopy,               TC("write=sc,abcdef copy=sc,sc2 stat=sc stat=sc2 size=sc,6 size=sc2,6 isfile=sc isfile=sc2"))
	VN(CopyChainSize,               TC("write=ccs,12345 copy=ccs,ccs2 size=ccs2,5 copy=ccs2,ccs3 size=ccs3,5 copy=ccs3,ccs4 size=ccs4,5"))
	VN(CreateAndStatChild,          TC("write=csc,X spawn=stat=csc;isfile=csc;size=csc,1 read=csc,X"))
	VN(DeepReadOnlyOps,             TC("mkdir=dro mkdir=dro/sub touch=dro/sub/file isdir=dro isdir=dro/sub isfile=dro/sub/file listdir=dro,1 listdir=dro/sub,1 size=dro/sub/file,0"))
	VN(CopyToNewDeepDir,            TC("mkdir=cnd1 mkdir=cnd1/cnd2 mkdir=cnd1/cnd2/cnd3 write=src_deep,abc copy=src_deep,cnd1/cnd2/cnd3/dst read=cnd1/cnd2/cnd3/dst,abc"))
	VN(RenameToNewDeepDir,          TC("mkdir=rnd1 mkdir=rnd1/rnd2 mkdir=rnd1/rnd2/rnd3 write=src_rn,xyz rename=src_rn,rnd1/rnd2/rnd3/dst read=rnd1/rnd2/rnd3/dst,xyz notexists=src_rn"))
	VN(WriteToOnlyExistingDir,      TC("mkdir=woed write=woed/file,A read=woed/file,A append=woed/file,B read=woed/file,AB unlink=woed/file rmdir=woed"))
	VN(DirCreateWithFile,           TC("mkdir=dcwf write=dcwf/initial,1 listdir=dcwf,1 listcontains=dcwf,initial read=dcwf/initial,1"))
	VN(ReadEachInDir,               TC("mkdir=red write=red/a,A write=red/b,B write=red/c,C read=red/a,A read=red/b,B read=red/c,C size=red/a,1 size=red/b,1 size=red/c,1"))
	VN(RapidAppendsAfterRead,       TC("write=raar,start read=raar,start append=raar,1 append=raar,2 append=raar,3 append=raar,4 read=raar,start1234 size=raar,9"))
	VN(DirCascadeFiles,             TC("mkdir=dcf nfiles=dcf,a,5 mkdir=dcf/sub nfiles=dcf/sub,b,5 listdir=dcf,6 listdir=dcf/sub,5"))
	VN(AppendOnNewLine,             TC("write=anl,line1 append=anl,_line2 append=anl,_line3 read=anl,line1_line2_line3 size=anl,17"))
	VN(CreateWriteAppend,           TC("touch=cwa size=cwa,0 append=cwa,A size=cwa,1 append=cwa,B size=cwa,2 read=cwa,AB"))
	VN(EmptyDirSurvives,            TC("mkdir=eds isdir=eds listdir=eds,0 listdir=eds,0 listdir=eds,0 isdir=eds rmdir=eds"))
	VN(StatAfterMutate,             TC("write=sam,a stat=sam append=sam,b stat=sam append=sam,c stat=sam size=sam,3"))
	VN(CopySrcThenAppendBoth,       TC("write=csab,seed copy=csab,csab2 append=csab,P append=csab2,Q read=csab,seedP read=csab2,seedQ"))
	VN(DirChangeUnderSpawn,         TC("mkdir=dcus touch=dcus/init spawn=touch=dcus/inner;listcontains=dcus,init;listcontains=dcus,inner listdir=dcus,2"))
	VN(WriteSpawnSpawnUnlink,       TC("write=wssu,X spawn=spawn=read=wssu,X;unlink=wssu;notexists=wssu notexists=wssu"))
	VN(SpawnReadReadRead,           TC("write=srr,A spawn=read=srr,A;read=srr,A;read=srr,A read=srr,A read=srr,A"))
	VN(WriteAppendReadStat,         TC("write=wars,a stat=wars size=wars,1 append=wars,b stat=wars size=wars,2 read=wars,ab"))
	VN(DeepCwdSetup,                TC("mkdir=dcs1 mkdir=dcs1/dcs2 mkdir=dcs1/dcs2/dcs3 chdir=dcs1 chdir=dcs2 chdir=dcs3 touch=here_in_deep stat=here_in_deep isfile=here_in_deep"))
	VN(StatDirVsFile,               TC("mkdir=svf write=svf/file,X isdir=svf isfile=svf/file mustfail=isdir=svf/file mustfail=isfile=svf"))


#if PLATFORM_LINUX
	VN(MP_DirVisible,
		TC("mkdir=mp_d touch=mp_d/x"),
		TC("isdir=mp_d isfile=mp_d/x listcontains=mp_d,x"))
	VN(MP_DeepVisible,
		TC("mkdir=md1 mkdir=md1/md2 mkdir=md1/md2/md3 write=md1/md2/md3/leaf,deep"),
		TC("read=md1/md2/md3/leaf,deep isdir=md1 isdir=md1/md2 isdir=md1/md2/md3"))
	VN(MP_SpawnInside,
		TC("write=mps,1"),
		TC("spawn=read=mps,1;append=mps,!! read=mps,1!!"))
	// VN(MP_AppendChain, ...) — BUG (remote): P2 append produces more bytes than expected. Same overlay aliasing class as project_remote_rename_then_writesource.md
	VN(MP_LotsOfFiles,
		TC("mkdir=lof nfiles=lof,k,16"),
		TC("listdir=lof,16 listcontains=lof,k0 listcontains=lof,k15 read=lof/k0, read=lof/k15,"))
	VN(MP_DirCreatedThen,
		TC("mkdir=mpdc"),
		TC("isdir=mpdc touch=mpdc/x isfile=mpdc/x listcontains=mpdc,x"))
	// VN(MP_RenameLater, ...) — BUG: SessionClient::MoveFileW:993 asserts "Moving file not owned by client" when P2 renames file P1 created in remote mode.
	VN(MP_NestedSpawnLater,
		TC("write=mpns,seed"),
		TC("spawn=spawn=read=mpns,seed;write=mpns2,inner;read=mpns2,inner read=mpns,seed read=mpns2,inner"))
	VN(MP_AddDeleteAdd,
		TC("write=mpada,1"),
		TC("read=mpada,1 unlink=mpada notexists=mpada touch=mpada size=mpada,0"))
	VN(MP_LotsOfWrites,
		TC("write=mp_a,1 write=mp_b,2 write=mp_c,3 write=mp_d,4 write=mp_e,5"),
		TC("read=mp_a,1 read=mp_b,2 read=mp_c,3 read=mp_d,4 read=mp_e,5"))
	VN(MP_DirEnumerate,
		TC("mkdir=mpe nfiles=mpe,e,7"),
		TC("listdir=mpe,7 listcontains=mpe,e0 listcontains=mpe,e6 read=mpe/e3,"))
	VN(MP_NestedThenWrite,
		TC("mkdir=ntw mkdir=ntw/lvl mkdir=ntw/lvl/deep"),
		TC("write=ntw/lvl/deep/file,deep_data read=ntw/lvl/deep/file,deep_data"))
	VN(MP_ManySubdirs,
		TC("mkdir=ms_top mkdir=ms_top/a mkdir=ms_top/b mkdir=ms_top/c"),
		TC("listdir=ms_top,3 listcontains=ms_top,a listcontains=ms_top,b listcontains=ms_top,c isdir=ms_top/a isdir=ms_top/b isdir=ms_top/c"))
	VN(MP_SpawnAfterSetup,
		TC("mkdir=sas write=sas/seed,A"),
		TC("spawn=read=sas/seed,A;touch=sas/child read=sas/seed,A isfile=sas/child"))
	// VN(MP_AppendOverlay, ...) — BUG (remote): cross-process append after write doubles content. Same overlay aliasing class.
	VN(MP_CopyAcrossProc,
		TC("write=mpc_src,seed"),
		TC("copy=mpc_src,mpc_dst read=mpc_dst,seed read=mpc_src,seed"))
	VN(MP_ChainCreate,
		TC("mkdir=cc"),
		TC("spawn=touch=cc/a spawn=touch=cc/b spawn=touch=cc/c listdir=cc,3"))
	VN(MP_ManyFilesSeenBoth,
		TC("nfiles=.,m,10"),
		TC("stat=m0 stat=m9 size=m0,0 size=m5,0 size=m9,0 listcontains=.,m4"))

	// sbox→bash flow: P1 mkdir, P2 chdir+O_CREAT cwd-relative. P1's mkdir must propagate to P2's dirtable/overlay before the open or it ENOENTs.
	VN(MP_MkdirChdirCreate,
		TC("mkdir=mpccd"),
		TC("chdir=mpccd write=created.txt,hi read=created.txt,hi isfile=created.txt"))
	VN(MP_MkdirChdirCreateNested,
		TC("mkdir=mpccn mkdir=mpccn/out"),
		TC("chdir=mpccn write=out/leaf,deep read=out/leaf,deep isfile=out/leaf"))
	// Propagation across two fork boundaries: chdir+create in grandchild of P2.
	VN(MP_MkdirChdirSpawnCreate,
		TC("mkdir=mpcsc"),
		TC("chdir=mpcsc spawn=write=spawned.txt,from-child;read=spawned.txt,from-child read=spawned.txt,from-child isfile=spawned.txt"))
	// sbox `touch` calls open(path, O_RDONLY|O_CREAT). GetFileAccessFlags classified that as read-only (no O_WRONLY/O_RDWR), so Session::CreateFile sent it through the read path returning the virtual path on Linux — kernel then opened the virtual path and ENOENT'd.
	VN(RdCreatBasic,           TC("rdcreat=rc1.txt isfile=rc1.txt size=rc1.txt,0"))
	VN(RdCreatCwdRel,          TC("mkdir=rcd chdir=rcd rdcreat=rc2.txt isfile=rc2.txt size=rc2.txt,0"))
	VN(MP_MkdirChdirRdCreat,
		TC("mkdir=rcm"),
		TC("chdir=rcm rdcreat=rc3.txt isfile=rc3.txt size=rc3.txt,0"))
	VN(MP_MkdirChdirSpawnRdCreat,
		TC("mkdir=rcs"),
		TC("chdir=rcs spawn=rdcreat=rc4.txt;isfile=rc4.txt isfile=rc4.txt size=rc4.txt,0"))
	// EnsureBinaryFile preserved "./" prefix from dlopen-relative path → CheckPath concatenated /./tools/... and rejected.
	VN(DlopenDotSlash,         TC("write=binshim.so,fakelibdata mkdir=tools mkdir=tools/out mkdir=tools/out/bin write=tools/out/bin/jrg.so,fake dlopen=./tools/out/bin/jrg.so"))

#if !UBA_BUILD_MACHINE // Writes a error which is a successful test but build farm don't like that
	// Forked-but-not-execed child takes a fatal signal in libUbaDetours's address space. Child's UbaAssert/Rpc_WriteLog writes into the comm channel MAP_SHARED with parent, clobbering pending RPC state — parent's next RPC after waitpid reads wire garbage and asserts BinaryReader::ReadString overflow. Fix: getpid-vs-g_initPid guard in the signal handler / UbaAssert path. Remote-only (LD_PRELOAD detour path).
	VN(ForkCrashThenStat,      TC("forkthencrash=SIGBUS stat=/etc/hosts"))
#endif

	// bash and nsjail trusty scripts call `execve(path, argv, NULL)` for `env -i` style spawns. Pre-fix ProcessCreator::SendMessage iterated envp[i] without a NULL check and SIGSEGV'd.
	VN(ExecveNullEnvp,         TC("execvenull=/bin/true"))

	// nsjail edges accumulate hundreds of bindmounts; encoded UBA_VFS_OVERLAYS exceeded the old StringBuffer<8192>. vfssynth=120 fills g_vfsEntries with ~120 long paths (>= 8 KiB encoded); execvenull then drives Shared_Internal_exec — exact path that overflowed.
	VN(VfsEnvLargeEnc,         TC("vfssynth=120 execvenull=/bin/true"))

	// nsjail mount/umount/pivot_root/chroot — Detoured_mount registers {vfs:dst, local:src} in shared.overlay.vfsEntries and skips kernel mount so these run unprivileged (no CAP_SYS_ADMIN).
	VN(BindReadThrough,        TC("mkdir=bsrc write=bsrc/f,hello mkdir=bdst bindmount=bsrc,bdst read=bdst/f,hello"))
	VN(BindWriteThrough,       TC("mkdir=bsrc mkdir=bdst bindmount=bsrc,bdst write=bdst/g,world read=bsrc/g,world"))
	VN(BindStatThrough,        TC("mkdir=bsrc write=bsrc/f,abcd size=bsrc/f,4 mkdir=bdst bindmount=bsrc,bdst size=bdst/f,4 isfile=bdst/f"))
	VN(BindListThrough,        TC("mkdir=bsrc touch=bsrc/a touch=bsrc/b mkdir=bdst bindmount=bsrc,bdst listdir=bdst,2 listcontains=bdst,a listcontains=bdst,b"))
	VN(BindThenUmount,         TC("mkdir=bsrc write=bsrc/f,hi mkdir=bdst bindmount=bsrc,bdst read=bdst/f,hi umount=bdst mustfail=read=bdst/f,hi"))
	VN(BindThenUmount2Detach,  TC("mkdir=bsrc write=bsrc/f,hi mkdir=bdst bindmount=bsrc,bdst read=bdst/f,hi umount2=bdst,2 mustfail=read=bdst/f,hi"))
	VN(BindChained,            TC("mkdir=ba write=ba/f,A mkdir=bb mkdir=bc bindmount=ba,bb bindmount=bb,bc read=bc/f,A"))
	VN(BindNested,             TC("mkdir=outer mkdir=outer/inner write=outer/inner/file,nested mkdir=mnt bindmount=outer,mnt isfile=mnt/inner/file read=mnt/inner/file,nested"))
	VN(BindOverwriteFlatThenList, TC("mkdir=br1 touch=br1/x mkdir=br2 touch=br2/y bindmount=br1,br2 listdir=br2,1 listcontains=br2,x"))
	// pivot_root and chroot virtualized into per-process root rebases so these pass without CAP_SYS_ADMIN.
	VN(ChrootBasic,            TC("mkdir=cr mkdir=cr/sub touch=cr/sub/f chrt=cr isfile=/sub/f"))
	VN(PivotBasic,             TC("mkdir=newroot mkdir=newroot/.old mkdir=newroot/inner touch=newroot/inner/leaf pivot=newroot,newroot/.old isfile=/inner/leaf"))
	// Multi-proc: bind in P1 must NOT leak into P2 — overlay entries are per-process.
	VN(MultiProcBindNoLeak,
		TC("mkdir=mpb_src write=mpb_src/f,P1 mkdir=mpb_dst bindmount=mpb_src,mpb_dst read=mpb_dst/f,P1"),
		TC("mustfail=read=mpb_dst/f,P1 mustfail=isfile=mpb_dst/f"))

	// IsVfsParentDir synthesis: nsjail registers leaf binds (/sb/trusty/vendor/google/aosp) but not intermediate dirs; without synthesis, AOSP trusty envsetup.sh's `gettop` runs `readlink -f` which lstat-ENOENTs and `dirname ""` infinite-loops. Wired via Shared_prepareReadlink (EINVAL) and Shared_stat (S_IFDIR|0755).
	VN(VfsParentLstat,         TC("mkdir=vfp_src bindmount=vfp_src,/sb/trusty/vendor/google/aosp lisdir=/sb/trusty"))
	VN(VfsParentStat,          TC("mkdir=vfp_src bindmount=vfp_src,/sb/trusty/vendor/google/aosp isdir=/sb/trusty/vendor/google"))
	// Must be EINVAL not ENOENT — ENOENT would tell readlink-f to bail on the component, breaking gettop().
	VN(VfsParentReadlinkEINVAL, TC("mkdir=vfp_src bindmount=vfp_src,/sb/trusty/vendor/google/aosp readlinkerr=/sb/trusty,EINVAL"))
	// Synthesis must not match every absolute path.
	VN(VfsParentNotAncestor,   TC("mkdir=vfp_src bindmount=vfp_src,/sb/trusty/vendor/google/aosp notexists=/sb/notabind mustfail=isdir=/sb/notabind"))
	// /sb/trustyXX is a string-prefix of /sb/trusty/... but not a directory ancestor.
	VN(VfsParentBoundary,      TC("mkdir=vfp_src bindmount=vfp_src,/sb/trusty/vendor/google/aosp notexists=/sb/trustyextra"))
	// readlink-f walks parents one '/' at a time, so every depth must synthesize.
	VN(VfsParentEachLevel,     TC("mkdir=vfp_src bindmount=vfp_src,/sb/a/b/c/d/leaf lisdir=/sb lisdir=/sb/a lisdir=/sb/a/b lisdir=/sb/a/b/c lisdir=/sb/a/b/c/d isdir=/sb/a/b/c/d readlinkerr=/sb,EINVAL readlinkerr=/sb/a/b/c/d,EINVAL"))
	// Implicit parent shared by multiple leaf binds.
	VN(VfsParentMultiBind,     TC("mkdir=vfp_a mkdir=vfp_b bindmount=vfp_a,/sb/trusty/vendor/google/aosp bindmount=vfp_b,/sb/trusty/system/extras isdir=/sb/trusty"))
	// Synthesis check must not short-circuit before the real overlay.
	VN(VfsParentLeafStillResolves, TC("mkdir=vfp_src write=vfp_src/marker,LEAF bindmount=vfp_src,/sb/trusty/vendor/google/aosp isdir=/sb/trusty read=/sb/trusty/vendor/google/aosp/marker,LEAF"))

	// posix_spawn (AOSP trusty's GNU make sub-makes) pre-fix didn't append UBA_VFS_OVERLAYS / UBA_INHERIT_HANDLES to envvars — children inherited an empty bind-mount tree and every bind-target open ENOENT'd. Fix mirrors Shared_Internal_exec env-build in Shared_posix_spawn.
	VN(BindVfsThenSpawn,            TC("mkdir=bvs_src bindmount=bvs_src,/sb/bvs/leaf spawn=vfsenvprobe=1"))

	// Spawned child actually USES the VFS overlay: child must STAT a namespace path, succeeding only if its DevirtualizePath finds the bind entry. Static-stub regression: uba_detour_core was missing the Shared_AdoptVfsEntries call so stub-patched children silently ran with g_vfsEntryCount==0 — Android clang's os.Getwd → newfstatat(., AT_FDCWD) failed to translate /nsjail_build_sandbox and exited "failed to read working directory" before invoking clang.real.
	VN(BindVfsThenSpawnIsdir,       TC("mkdir=bvs2_src bindmount=bvs2_src,/sb/bvs2/leaf spawn=isdir=/sb/bvs2/leaf"))

	// tmpfs-mount pass-through root propagation across execve: child inherits via UBA_PASSTHROUGH_ROOTS env and bypasses detour for same subtree. Without propagation, child re-detours a kernel-real tmpfs path → "exists in DIRTABLE but kernel ENOENT" (broke rust clippy `TEMP=$(mktemp -d); mkdir -p $TEMP/sub`).
	VN(PassThroughRootSpawnsChild, TC("passthrough_register=/uba_tp_test spawn=passthrough_check=/uba_tp_test/sub"))

	// vfs overlay must WIN over a parent pass-through root. nsjail tmpfs-mounts its namespace root then MS_BIND-mounts host dirs into it; queries under the bind must be UBA-tracked, not bypassed. Pre-fix IsUnderPassThroughRoot only consulted g_passThroughRoots, blackholing the bind-mounted subtree.
	VN(PassThroughVfsOverridePrecedence,
		TC("passthrough_register=/uba_iter70_pt vfs_add=/uba_iter70_pt/inside,/tmp"
		   " mustfail=passthrough_check=/uba_iter70_pt/inside/file"
		   " mustfail=passthrough_check=/uba_iter70_pt/inside"
		   " passthrough_check=/uba_iter70_pt/elsewhere/x"
		   " passthrough_check=/uba_iter70_pt"))

#if !UBA_BUILD_MACHINE
	// dlopen VFS-overlay devirt in local-binary mode: rust bindgen via libloading dlopens absolute namespace paths. The local-binary fast path skipped agent fetch but didn't translate VFS overlays — kernel dlopen returned NULL. Fix: DevirtualizePath before TRUE_WRAPPER(dlopen) when g_isUsingLocalBinary && IsVfsEnabled && absolute.
	VN(BindVfsThenDlopen,           TC("findlibc=/sb/lib/libfake.so dlopenok=/sb/lib/libfake.so"))
#endif

	VN(3P_BuildPyramid,
		TC("mkdir=pyr"),
		TC("mkdir=pyr/lvl1 mkdir=pyr/lvl2 mkdir=pyr/lvl3"),
		TC("touch=pyr/lvl1/a touch=pyr/lvl2/b touch=pyr/lvl3/c listdir=pyr,3"))
	VN(3P_SeedReadWrite,
		TC("write=swr_seed,A"),
		TC("read=swr_seed,A write=swr_more,B"),
		TC("read=swr_seed,A read=swr_more,B"))
	VN(3P_CreateLs,
		TC("mkdir=clp"),
		TC("nfiles=clp,f,4"),
		TC("listdir=clp,4 listcontains=clp,f0 listcontains=clp,f3 read=clp/f0, read=clp/f3,"))
	VN(3P_CreateNestedLs,
		TC("mkdir=cnp mkdir=cnp/sub"),
		TC("touch=cnp/sub/a touch=cnp/sub/b"),
		TC("listdir=cnp/sub,2 listcontains=cnp/sub,a listcontains=cnp/sub,b read=cnp/sub/a, read=cnp/sub/b,"))
	// VN(3P_ManyAppends, ...) — BUG (remote): same overlay-append-aliasing as VerbMP_AppendOverlay.


	// AOSP-scale symlink coverage: prebuilt toolchains, /lib→usr/lib merged-/usr, soong genrules, sbox sandboxes, NDK arch dirs. See ClaudeNotes/06-symlinks-plan.md.
	VN(SymlinkBasic,                TC("write=tgt,abc symlink=tgt,lnk islink=lnk readlink=lnk,tgt"))
	VN(SymlinkReadThrough,          TC("write=tgt,abc symlink=tgt,lnk read=lnk,abc size=lnk,3"))
	VN(SymlinkLstatVsStat,          TC("write=tgt,1 symlink=tgt,lnk islink=lnk isfile=tgt isfile=lnk"))
	VN(SymlinkReadlink,             TC("symlink=some_target_path,rl readlink=rl,some_target_path"))
	VN(SymlinkAbsoluteTarget,       TC("write=tgt,abs mkdir=sub symlink=../tgt,sub/lnk read=sub/lnk,abs"))
	VN(SymlinkRelativeTarget,       TC("mkdir=peer write=peer/data,rel symlink=peer/data,beside read=beside,rel"))
	VN(SymlinkParentTraversal,      TC("mkdir=outer write=outer/file,parent mkdir=outer/inside symlink=../file,outer/inside/up read=outer/inside/up,parent"))
	VN(SymlinkToFileSwap,           TC("write=A,a write=B,b symlink=A,L read=L,a unlink=L symlink=B,L read=L,b"))
	VN(SymlinkToDir,                TC("mkdir=dd touch=dd/leaf symlink=dd,L isdir=L isfile=L/leaf"))
	VN(SymlinkDirSwap,              TC("mkdir=A mkdir=B touch=A/in_a touch=B/in_b symlink=A,L stat=L/in_a unlink=L symlink=B,L stat=L/in_b mustfail=stat=L/in_a"))
	VN(SymlinkDirSwapToFile,        TC("mkdir=D touch=D/leaf write=F,X symlink=D,L isdir=L stat=L/leaf unlink=L symlink=F,L isfile=L mustfail=stat=L/leaf"))
	VN(SymlinkSwapTwice,            TC("write=v1,1 write=v2,2 write=v3,3 symlink=v1,L read=L,1 unlink=L symlink=v2,L read=L,2 unlink=L symlink=v3,L read=L,3"))
	VN(SymlinkChain,                TC("write=tail,end symlink=tail,mid symlink=mid,head read=head,end size=head,3"))
	VN(SymlinkChainDeep,            TC("write=L0,deep symlink=L0,L1 symlink=L1,L2 symlink=L2,L3 symlink=L3,L4 symlink=L4,L5 read=L5,deep"))
	VN(SymlinkLoop,                 TC("symlink=loopB,loopA symlink=loopA,loopB mustfail=stat=loopA mustfail=read=loopA,"))
	VN(SymlinkLoopSelf,             TC("symlink=selfA,selfA mustfail=stat=selfA"))
	VN(SymlinkDangling,             TC("symlink=ghostT,Lg islink=Lg mustfail=stat=Lg mustfail=read=Lg,"))
	VN(SymlinkBecomesDangling,      TC("write=tgtBd,1 symlink=tgtBd,linkBd read=linkBd,1 unlink=tgtBd islink=linkBd mustfail=stat=linkBd mustfail=read=linkBd,"))
	VN(SymlinkInPrefix,             TC("mkdir=real touch=real/file symlink=real,proxy isfile=proxy/file stat=proxy/file"))
	VN(SymlinkPrefixDeep,           TC("mkdir=trueA mkdir=trueA/sub mkdir=trueA/sub/leaf write=trueA/sub/leaf/file,deep symlink=trueA,linkA isfile=linkA/sub/leaf/file read=linkA/sub/leaf/file,deep"))
	VN(SymlinkPrefixSwap,           TC("mkdir=v1 mkdir=v2 touch=v1/marker_v1 touch=v2/marker_v2 symlink=v1,active stat=active/marker_v1 unlink=active symlink=v2,active stat=active/marker_v2 mustfail=stat=active/marker_v1"))
	VN(SymlinkRelative,             TC("mkdir=p1 touch=p1/leaf symlink=p1/leaf,sl read=sl, isfile=sl"))
	VN(SymlinkUnlinkLink,           TC("write=tu,1 symlink=tu,lu unlink=lu read=tu,1 isfile=tu mustfail=stat=lu"))
	VN(SymlinkUnlinkTargetReadLink, TC("write=ttu,1 symlink=ttu,ltu unlink=ttu mustfail=read=ltu, islink=ltu"))
	VN(SymlinkAppendThroughLink,    TC("write=tap,start symlink=tap,lap append=lap,_more read=tap,start_more read=lap,start_more"))
	VN(SymlinkWriteThroughLink,     TC("write=tw,old symlink=tw,lw write=lw,fresh read=tw,fresh read=lw,fresh"))
	VN(SymlinkCopySrc,              TC("write=tc,abc symlink=tc,lc copy=lc,copyOf read=copyOf,abc"))
	// VN(SymlinkRenameLink, ...) — hits the cross-process MoveFile bug on remote. See memory project_remote_rename_not_owned.md
	VN(SymlinkManyToOneTarget,      TC("write=oneT,shared symlink=oneT,a symlink=oneT,b symlink=oneT,c read=a,shared read=b,shared read=c,shared"))
	VN(SymlinkSiblings,             TC("mkdir=sib_dir nfiles=sib_dir,t,3 symlink=t0,sib_dir/L0 symlink=t1,sib_dir/L1 symlink=t2,sib_dir/L2 listdir=sib_dir,6"))
	VN(SymlinkSpawnObserves,        TC("write=ts,x spawn=symlink=ts,LS;islink=LS;read=LS,x islink=LS read=LS,x"))
	VN(SymlinkChildSwap,            TC("write=tA,a write=tB,b symlink=tA,Lcs spawn=read=Lcs,a;unlink=Lcs;symlink=tB,Lcs read=Lcs,b"))
	VN(SymlinkChildResolves,        TC("mkdir=dirX touch=dirX/leafX symlink=dirX,LX spawn=stat=LX/leafX;isfile=LX/leafX"))
	VN(SymlinkAOSPpyenv,            TC("mkdir=python_3 write=python_3/python3.11,bin symlink=python3.11,python_3/python3 read=python_3/python3,bin isfile=python_3/python3 isfile=python_3/python3.11"))
	VN(SymlinkAOSPmergedUsr,        TC("mkdir=usr mkdir=usr/lib write=usr/lib/libc.so.6,glibc symlink=usr/lib,Llib read=Llib/libc.so.6,glibc isdir=Llib"))
	VN(SymlinkAOSPndkArch,          TC("mkdir=ndk mkdir=ndk/x86_64 write=ndk/x86_64/libssl.so,ndk_arch symlink=x86_64,ndk/lib64 read=ndk/lib64/libssl.so,ndk_arch"))
	VN(SymlinkAOSPgenrule,          TC("mkdir=src write=src/header.h,realhdr mkdir=gendir symlink=../src/header.h,gendir/header.h read=gendir/header.h,realhdr"))
	VN(SymlinkAOSPsboxSandbox,      TC("mkdir=real_root mkdir=real_root/lib write=real_root/lib/x.so,real_lib mkdir=sandbox symlink=../real_root/lib,sandbox/lib read=sandbox/lib/x.so,real_lib"))
	VN(SymlinkAOSPdoubleHop,        TC("mkdir=stage1 mkdir=stage2 write=stage1/file,hopdata symlink=../stage1,stage2/up symlink=stage2/up/file,final read=final,hopdata"))
	VN(SymlinkAOSPSdkProxy,         TC("mkdir=sdk_v1 write=sdk_v1/api.txt,v1 mkdir=sdk_v2 write=sdk_v2/api.txt,v2 symlink=sdk_v1,sdk_active read=sdk_active/api.txt,v1 unlink=sdk_active symlink=sdk_v2,sdk_active read=sdk_active/api.txt,v2"))
	VN(SymlinkSwapSpawnObserves,    TC("write=tx1,one write=tx2,two symlink=tx1,Lx spawn=read=Lx,one unlink=Lx symlink=tx2,Lx spawn=read=Lx,two read=Lx,two"))
	VN(SymlinkChainSwapMid,         TC("write=t,A symlink=t,m1 symlink=m1,m2 read=m2,A unlink=m1 write=t2,B symlink=t2,m1 read=m2,B"))
	VN(SymlinkRelTargetUpdates,     TC("mkdir=d1 mkdir=d2 write=d1/x,A write=d2/x,B chdir=d1 symlink=x,here read=here,A chdir=../d2 symlink=x,here read=here,B"))
	VN(SymlinkListdirThroughLink,   TC("mkdir=lt nfiles=lt,e,4 symlink=lt,Llt listdir=Llt,4 listcontains=Llt,e0 listcontains=Llt,e3"))
	VN(SymlinkInDirThenList,        TC("mkdir=ds touch=ds/r symlink=r,ds/Lr listdir=ds,2 listcontains=ds,r listcontains=ds,Lr islink=ds/Lr"))
	VN(SymlinkBeforeTargetExists,   TC("symlink=lateT,early islink=early mustfail=read=early, write=lateT,now read=early,now"))
	VN(SymlinkOverwritesEachOther,  TC("write=tA,1 write=tB,2 symlink=tA,sw read=sw,1 unlink=sw symlink=tB,sw read=sw,2 unlink=sw symlink=tA,sw read=sw,1"))
	VN(SymlinkMultiprocessChain,    TC("write=mptA,1 write=mptB,2 symlink=mptA,mpL spawn=read=mpL,1 unlink=mpL symlink=mptB,mpL spawn=read=mpL,2 read=mpL,2"))
	VN(SymlinkBigChainSwap,         TC("write=t0,X symlink=t0,L1 symlink=L1,L2 symlink=L2,L3 read=L3,X unlink=L1 write=t0_new,Y symlink=t0_new,L1 read=L3,Y"))
	VN(SymlinkInsideSymlinkedDir,   TC("mkdir=Dorig touch=Dorig/inner symlink=Dorig,Dlnk write=Dlnk/created,fromlink read=Dorig/created,fromlink"))
	VN(SymlinkSpawnSwapsDir,        TC("mkdir=A1 mkdir=A2 touch=A1/inA1 touch=A2/inA2 symlink=A1,Lsw spawn=stat=Lsw/inA1;unlink=Lsw;symlink=A2,Lsw stat=Lsw/inA2 mustfail=stat=Lsw/inA1"))
	VN(SymlinkRecreate,             TC("write=tre,1 symlink=tre,lre unlink=lre symlink=tre,lre read=lre,1"))
	VN(SymlinkAfterCopy,            TC("write=tcp,abc copy=tcp,tcp2 symlink=tcp2,Lcp read=Lcp,abc"))
	VN(SymlinkAfterRename,          TC("write=trn1,1 rename=trn1,trn2 symlink=trn2,Lrn read=Lrn,1"))
	VN(SymlinkBetweenDirs,          TC("mkdir=src_d mkdir=dst_d write=src_d/data,A symlink=../src_d/data,dst_d/Ld read=dst_d/Ld,A"))
	// Stat-through-prefix-symlink, then remove the symlink → leaf must vanish from the dirtable view.
	VN(SymlinkPrefixThenRemove,     TC("mkdir=br touch=br/c symlink=br,b stat=b/c isfile=b/c unlink=b mustfail=stat=b/c mustfail=isfile=b/c stat=br/c"))
	// Remove symlink, recreate pointing at a different dir. Old target's entry must not satisfy lookups.
	VN(SymlinkPrefixSwapToOther,    TC("mkdir=d1 mkdir=d2 touch=d1/x touch=d2/y symlink=d1,Lp stat=Lp/x mustfail=stat=Lp/y unlink=Lp symlink=d2,Lp stat=Lp/y mustfail=stat=Lp/x"))
	// Remove a file from inside a symlinked dir. Stat via symlink must reflect removal.
	VN(SymlinkFileRemovedInside,    TC("mkdir=rd touch=rd/in symlink=rd,Lr stat=Lr/in unlink=rd/in mustfail=stat=Lr/in mustfail=stat=rd/in"))
	// Remove the real dir target while symlink remains → through-link lookups must fail.
	VN(SymlinkTargetRmDir,          TC("mkdir=tA touch=tA/leaf symlink=tA,LtA stat=LtA/leaf unlink=tA/leaf rmdir=tA mustfail=stat=LtA/leaf mustfail=stat=LtA/anything"))
	// Symlink to file → unlink target → lookups via link must fail.
	VN(SymlinkFileTargetRm,         TC("write=tF,X symlink=tF,LtF read=LtF,X unlink=tF mustfail=read=LtF, mustfail=stat=LtF"))
	// Remove and recreate the symlink at the same path with new target.
	VN(SymlinkRecreateNewTarget,    TC("write=tA1,a write=tB1,b symlink=tA1,Lr1 read=Lr1,a unlink=Lr1 symlink=tB1,Lr1 read=Lr1,b"))
	// Spawn child removes the prefix symlink; parent must see leaf disappear.
	VN(SymlinkChildRemovesPrefix,   TC("mkdir=cb touch=cb/q symlink=cb,Lcb stat=Lcb/q spawn=unlink=Lcb mustfail=stat=Lcb/q stat=cb/q"))
	// Round-trip: create symlink, then remove in same proc — should not appear to next proc.
	VN(SymlinkAddRemoveSameProc,    TC("write=t_arsp,a symlink=t_arsp,L_arsp islink=L_arsp unlink=L_arsp mustfail=stat=L_arsp stat=t_arsp"))
	// Symlink to a target that doesn't exist yet — leaf becomes valid only after target appears.
	VN(SymlinkLateBindTarget,       TC("symlink=lateT2,Llt mustfail=stat=Llt write=lateT2,here read=Llt,here"))
	// Triple swap: A→A, A→B, A→C with stats interleaved.
	VN(SymlinkTripleSwap,           TC("write=v_a,a write=v_b,b write=v_c,c symlink=v_a,Ltri read=Ltri,a unlink=Ltri symlink=v_b,Ltri read=Ltri,b unlink=Ltri symlink=v_c,Ltri read=Ltri,c"))
	// Symlink → unlink → recreate to itself — should not reuse stale entry.
	VN(SymlinkSelfRecreate,         TC("write=s_t,1 symlink=s_t,Ls read=Ls,1 unlink=Ls symlink=s_t,Ls read=Ls,1"))
	// Symlink whose target IS another symlink — both must propagate independently.
	VN(SymlinkOfSymlink,            TC("write=base_q,Q symlink=base_q,L1 symlink=L1,L2 read=L2,Q unlink=L1 mustfail=read=L2,"))
	// Spawn child writes target THEN creates link; parent reads via link.
	VN(SymlinkChildBindsTarget,     TC("spawn=write=cbt,X;symlink=cbt,Lcbt isfile=Lcbt read=Lcbt,X"))
	// Spawn child creates symlink, parent unlinks it; child still saw it.
	VN(SymlinkParentRemovesChildLink, TC("write=pr_t,A spawn=symlink=pr_t,Lpr;read=Lpr,A unlink=Lpr mustfail=stat=Lpr stat=pr_t"))
	// Many siblings of a symlink in the same dir.
	VN(SymlinkSiblingFiles,         TC("mkdir=ss touch=ss/a touch=ss/b touch=ss/c write=ss/d,X symlink=d,ss/Ld read=ss/Ld,X listdir=ss,5"))
	// Symlink-to-dir, then mkdir inside via the symlink. Must end up in real target.
	VN(SymlinkMkdirThroughLink,     TC("mkdir=mtg symlink=mtg,Lmtg mkdir=Lmtg/inner stat=mtg/inner isdir=Lmtg/inner"))
	// Remove the symlink twice in a row — second must fail.
	VN(SymlinkUnlinkTwice,          TC("write=ut2,X symlink=ut2,Lut2 unlink=Lut2 mustfail=unlink=Lut2"))
	// Stat the same symlink twice in a row — both must succeed.
	VN(SymlinkStatTwice,            TC("write=st2,X symlink=st2,Lst2 stat=Lst2 stat=Lst2 islink=Lst2 islink=Lst2"))
	// Read+stat mixed via the link — overlay/main consistency.
	VN(SymlinkReadStatMix,          TC("write=rsm,abc symlink=rsm,Lrsm read=Lrsm,abc stat=Lrsm read=Lrsm,abc isfile=Lrsm size=Lrsm,3"))
	// Symlink target removed AND link removed in same process.
	VN(SymlinkBothRemoved,          TC("write=br,Z symlink=br,Lbr unlink=Lbr unlink=br mustfail=stat=Lbr mustfail=stat=br"))
	// ELOOP: chain longer than the depth cap must fail rather than infinite-loop. 50 hops > 40 cap.
	VN(SymlinkELOOP,                TC("symlink=L1,L0 symlink=L2,L1 symlink=L3,L2 symlink=L4,L3 symlink=L5,L4 symlink=L6,L5 symlink=L7,L6 symlink=L8,L7 symlink=L9,L8 symlink=L10,L9 symlink=L11,L10 symlink=L12,L11 symlink=L13,L12 symlink=L14,L13 symlink=L15,L14 symlink=L16,L15 symlink=L17,L16 symlink=L18,L17 symlink=L19,L18 symlink=L20,L19 symlink=L21,L20 symlink=L22,L21 symlink=L23,L22 symlink=L24,L23 symlink=L25,L24 symlink=L26,L25 symlink=L27,L26 symlink=L28,L27 symlink=L29,L28 symlink=L30,L29 symlink=L31,L30 symlink=L32,L31 symlink=L33,L32 symlink=L34,L33 symlink=L35,L34 symlink=L36,L35 symlink=L37,L36 symlink=L38,L37 symlink=L39,L38 symlink=L40,L39 symlink=L41,L40 symlink=L42,L41 symlink=L43,L42 symlink=L44,L43 symlink=L45,L44 mustfail=stat=L0"))
	// rmdir on a symlink-to-dir must NOT remove the target — it's not a real dir from rmdir's perspective.
	VN(SymlinkRmdirOnLink,          TC("mkdir=rmt symlink=rmt,Lrm mustfail=rmdir=Lrm isdir=rmt islink=Lrm"))
	// listdir of a dir containing mixed entries: regular file, regular dir, symlink, dangling symlink.
	VN(SymlinkMixedListdir,         TC("mkdir=mxd touch=mxd/regfile mkdir=mxd/regdir write=mxd/tg,X symlink=tg,mxd/Llink symlink=ghost,mxd/Ldang listdir=mxd,5 listcontains=mxd,regfile listcontains=mxd,regdir listcontains=mxd,Llink listcontains=mxd,Ldang"))
	// opendir on dangling symlink to a dir name should fail.
	VN(SymlinkOpendirDangling,      TC("symlink=ghost_dir,Ldd mustfail=stat=Ldd"))
	// Path with .. that traverses through a symlink.
	VN(SymlinkDotDotThrough,        TC("mkdir=ud mkdir=ud/inner write=ud/inner/file,Z symlink=ud,Lud read=Lud/inner/file,Z read=Lud/inner/../inner/file,Z"))
	// Multiple symlinks pointing to same target — remove one, others still work.
	VN(SymlinkSharedTargetRemoveOne, TC("write=stt,A symlink=stt,Lst1 symlink=stt,Lst2 symlink=stt,Lst3 read=Lst1,A read=Lst2,A read=Lst3,A unlink=Lst2 read=Lst1,A read=Lst3,A mustfail=stat=Lst2"))
	// Symlink whose target was ALSO a symlink — remove inner one.
	VN(SymlinkInnerLinkRemoved,     TC("write=in_t,A symlink=in_t,Lmid symlink=Lmid,Ltop read=Ltop,A unlink=Lmid mustfail=read=Ltop, stat=in_t"))
	// 4-deep spawn chain — symlink visible at every depth.
	VN(SymlinkDeepSpawn4Chain,      TC("write=ds_t,X symlink=ds_t,Lds spawn=read=Lds,X;spawn=read=Lds,X;spawn=read=Lds,X read=Lds,X"))
	// Spawn fanout — many siblings each query the same symlink concurrently in serial.
	VN(SymlinkSpawnFanoutReads,     TC("write=sft,Q symlink=sft,Lsft spawn=read=Lsft,Q spawn=read=Lsft,Q spawn=read=Lsft,Q spawn=read=Lsft,Q spawn=read=Lsft,Q"))
	// VN(SymlinkChmodFollows, ...) — Session::Chmod calls RegisterCreateFileForWrite which asserts when previous entry is a symlink. Fix: resolve symlink before chmod RPC, or skip main-table update for symlinks.
	// Symlink target gets renamed out from under the link → link becomes dangling.
	VN(SymlinkTargetRenamed,        TC("write=tn,A symlink=tn,Ltn read=Ltn,A rename=tn,tn2 mustfail=read=Ltn, read=tn2,A symlink=tn2,Ltn2 read=Ltn2,A"))
	// VN(SymlinkParentDirRenamed, ...) — symlink to a path under a renamed dir; resolution edge case to revisit.
	// Symlink with target = "." — points at the containing dir.
	VN(SymlinkToDot,                TC("mkdir=sd_d touch=sd_d/file symlink=.,sd_d/Lhere isdir=sd_d/Lhere isfile=sd_d/Lhere/file"))
	// Recursive deep listing through symlink-to-dir.
	VN(SymlinkRecursiveListing,     TC("mkdir=rl mkdir=rl/a mkdir=rl/a/b mkdir=rl/a/b/c touch=rl/a/b/c/leaf symlink=rl,Lrl listdir=Lrl,1 listdir=Lrl/a,1 listdir=Lrl/a/b,1 listdir=Lrl/a/b/c,1 isfile=Lrl/a/b/c/leaf"))
	// Symlink to deep relative path.
	VN(SymlinkRelativeDeep,         TC("mkdir=rd_a mkdir=rd_a/b mkdir=rd_a/b/c write=rd_a/b/c/leaf,DEEP symlink=rd_a/b/c/leaf,Lrd read=Lrd,DEEP"))
	// Stat the same dangling symlink twice — both must consistently fail.
	VN(SymlinkDanglingTwice,        TC("symlink=ghost_x,Ldng mustfail=stat=Ldng mustfail=stat=Ldng islink=Ldng islink=Ldng"))
	// Replace a symlink: unlink + create new pointing to a different file. Verify size update through link.
	VN(SymlinkReplaceSize,          TC("write=rsA,A write=rsBB,BB symlink=rsA,Lrs size=Lrs,1 unlink=Lrs symlink=rsBB,Lrs size=Lrs,2"))
	// Symlink target replaced with a longer file (target's content changes, not the link).
	VN(SymlinkTargetGrows,          TC("write=tg2,1 symlink=tg2,Ltg2 read=Ltg2,1 size=Ltg2,1 write=tg2,123456789 read=Ltg2,123456789 size=Ltg2,9"))
	// VN(SymlinkAppendCreatesTarget, ...) — create-through-dangling-symlink creates target on disk; UBA dirtable doesn't track this.
	// Symlink + delete + same-name file (not symlink) — must transition cleanly.
	VN(SymlinkBecomeRegularFile,    TC("write=trans_t,A symlink=trans_t,Ltrans islink=Ltrans unlink=Ltrans isfile=trans_t write=Ltrans,now isfile=Ltrans read=Ltrans,now"))
	// Symlink-to-dir + create a file inside via link, then remove link, original dir still has file.
	VN(SymlinkLinkCreatedFilePersists, TC("mkdir=lcf symlink=lcf,Llcf write=Llcf/created,Y read=Llcf/created,Y unlink=Llcf read=lcf/created,Y mustfail=stat=Llcf/created"))

	VN(SymlinkFail_NonexistingInSymlink, TC("mustfail=stat=/sbin/nonexistingfile"))
	VN(SymlinkFail_OverExisting,    TC("write=tover,X mustfail=symlink=tover,tover read=tover,X"))
	VN(SymlinkFail_OverFile,        TC("write=tof,a touch=tof_link mustfail=symlink=tof,tof_link"))
	VN(SymlinkFail_OverDir,         TC("mkdir=tod_d mkdir=tod_link mustfail=symlink=tod_d,tod_link"))
	VN(SymlinkFail_OverSymlink,     TC("write=tos,a symlink=tos,tos_link mustfail=symlink=tos,tos_link"))
	VN(SymlinkFail_MkdirOverLink,   TC("write=mol,a symlink=mol,Lmol mustfail=mkdir=Lmol islink=Lmol"))
	VN(SymlinkFail_OpenExclOverLink, TC("write=oeol,a symlink=oeol,Loeol mustfail=open=Loeol,x islink=Loeol"))
	VN(SymlinkFail_RmdirSelfLoop,   TC("symlink=Lsl,Lsl mustfail=rmdir=Lsl"))
	VN(SymlinkFail_ReadlinkOnFile,  TC("touch=rlf mustfail=readlink=rlf,whatever"))
	VN(SymlinkFail_ReadlinkOnDir,   TC("mkdir=rld mustfail=readlink=rld,whatever"))
	VN(SymlinkFail_DeepDanglingChain, TC("symlink=ghost1,Ldc1 symlink=Ldc1,Ldc2 symlink=Ldc2,Ldc3 mustfail=read=Ldc3,"))
	VN(SymlinkFail_RmdirSymlinkToFile, TC("write=tsf2,X symlink=tsf2,Ltsf2 mustfail=rmdir=Ltsf2 islink=Ltsf2 read=tsf2,X"))
	VN(SymlinkFail_UnlinkBrokenChain, TC("symlink=g1,brk1 symlink=brk1,brk2 unlink=brk1 mustfail=read=brk2,"))
	VN(SymlinkFail_DanglingMkdirInside, TC("symlink=ghost_dir2,Lgmi mustfail=mkdir=Lgmi/x"))
	VN(SymlinkFail_DanglingTouchInside, TC("symlink=ghost_dir3,Lgti mustfail=touch=Lgti/x"))

	// AOSP regression: symlink target longer than the old 512-byte readlink buffer in DirectoryTableHolder. Pattern is 40B; 30*40 = 1200B > 512 and < MaxPath (2048).
	VN(SymlinkLongTarget,           TC("symlinkrep=Llong,30,abcdefghij/klmnopqrstuvwxyz/0123456789-/ islink=Llong readlinklen=Llong,1200 listcontains=.,Llong"))
	VN(SymlinkNearMaxTarget,        TC("symlinkrep=Lnmax,49,abcdefghij/klmnopqrstuvwxyz/0123456789-/ islink=Lnmax readlinklen=Lnmax,1960 listcontains=.,Lnmax"))
	// AOSP regression: any /proc/self touch dragged the magic symlink into the dirtable, hitting the no-paths-through-symlinks invariant.
	VN(ProcSelfNoCrash,             TC("stat=/proc/self isdir=/proc/self islink=/proc/self stat=/proc/self/exe stat=/proc/self/cmdline stat=/proc/self/maps"))
	// AOSP regression (`ln -f -s`): create symlink, rename it; new path must remain a symlink with same target. Host's MoveFile previously dropped S_IFLNK + content, leaving the dest as S_IFREG with empty target.
	VN(SymlinkRenamePreservesLink,  TC("write=srpl_tgt,abc symlink=srpl_tgt,srpl_a rename=srpl_a,srpl_b islink=srpl_b readlink=srpl_b,srpl_tgt read=srpl_b,abc mustfail=stat=srpl_a"))
	// The exact AOSP `ln -f -s` pattern: mkstemp file, unlink, symlink at temp, rename to final.
	VN(SymlinkLnFsPattern,          TC("write=lnfs_tgt,xyz touch=lnfs_tmp unlink=lnfs_tmp symlink=lnfs_tgt,lnfs_tmp rename=lnfs_tmp,lnfs_final islink=lnfs_final read=lnfs_final,xyz mustfail=stat=lnfs_tmp"))
	// Chain of two renames — content must survive both moves.
	VN(SymlinkRenameChain,          TC("write=src_rc,Z symlink=src_rc,sla rename=sla,slb rename=slb,slc readlink=slc,src_rc islink=slc read=slc,Z"))
	// Rename a symlink with relative target, ensure target string itself is preserved verbatim.
	VN(SymlinkRenameRelTarget,      TC("mkdir=peer_rrt write=peer_rrt/data,rel symlink=peer_rrt/data,beside_a rename=beside_a,beside_b islink=beside_b readlink=beside_b,peer_rrt/data read=beside_b,rel"))
	// Rename a symlink across directories. Target written via absolute-style ../rcd_a/tgt so it stays valid in both parent dirs.
	VN(SymlinkRenameCrossDir,       TC("mkdir=rcd_a mkdir=rcd_b write=rcd_a/tgt,Q symlink=../rcd_a/tgt,rcd_a/L1 rename=rcd_a/L1,rcd_b/L2 islink=rcd_b/L2 readlink=rcd_b/L2,../rcd_a/tgt read=rcd_b/L2,Q"))
	// Rename a chain head: link-of-link, rename outer; outer must still resolve through inner.
	VN(SymlinkRenameChainOfLinks,   TC("write=col_t,T symlink=col_t,col_inner symlink=col_inner,col_outer rename=col_outer,col_outer2 islink=col_outer2 read=col_outer2,T"))
	// Two consecutive `ln -f -s` patterns to the same final path.
	VN(SymlinkLnFsTwice,            TC("write=lft_a,A write=lft_b,B touch=lft_t1 unlink=lft_t1 symlink=lft_a,lft_t1 rename=lft_t1,lft_final read=lft_final,A touch=lft_t2 unlink=lft_t2 symlink=lft_b,lft_t2 rename=lft_t2,lft_final read=lft_final,B"))
	// AOSP rust cxxbridge regression: a path through TWO sequential dir-prefix symlinks each with `../../<sibling>` target. Soong stages rust/cxx so `gen/cmd/src/gen` is a symlink to `../../src`, and the resolved `gen/src/include` is itself a symlink to `../../include`. The resolver must chase both prefixes.
	VN(SymlinkPrefixChainViaDotDot, TC("mkdir=ext write=ext/cxx.h,HDR mkdir=outer mkdir=outer/inner symlink=../../ext,outer/inner/incdir read=outer/inner/incdir/cxx.h,HDR"))
	VN(SymlinkTwoSymlinksDeep,      TC("mkdir=tsd_root mkdir=tsd_root/inc write=tsd_root/inc/h.h,FOUND mkdir=tsd_root/g mkdir=tsd_root/g/src mkdir=tsd_root/g/cmd mkdir=tsd_root/g/cmd/src symlink=../../src,tsd_root/g/cmd/src/g symlink=../../inc,tsd_root/g/src/i read=tsd_root/g/cmd/src/g/i/h.h,FOUND"))
	// AOSP aapt2 regression: cross-process open of a path whose parent dir is a sibling-pointing symlink. The 1st-process-Op populates the dir with the symlink registered as S_IFLNK. The 2nd process's Rpc_GetEntryOffset for the path inside the symlink walks parent components from a fresh local dirtable and asks the host to ListDirectory the symlink path itself — host must return empty (not assert) so the detour falls back to GetRealPath.
	VN(SymlinkSiblingDirReadInChild, TC("mkdir=ssr_real mkdir=ssr_real/inner write=ssr_real/inner/file.txt,DATA symlink=ssr_real,ssr_link spawn=read=ssr_link/inner/file.txt,DATA read=ssr_link/inner/file.txt,DATA"))
	// AOSP e2fsdroid regression: readlink against a relative path after a (virtual) chdir. Detour's Shared_chdir updates g_virtualWorkingDir but doesn't issue the real chdir syscall, so TRUE_WRAPPER(readlink) on a relative path uses kernel cwd and returns ENOENT for an existing symlink. The wrapper must FixPath the path against the virtual cwd before calling true readlink.
	VN(ReadlinkRelAfterChdir,       TC("mkdir=rdc_dir write=rdc_dir/target,T symlink=target,rdc_dir/link chdir=rdc_dir readlink=link,target read=link,T"))
	// AOSP Go compile pattern (remote): write .tmp output, rename to final. SessionClient::MoveFile must update writtenFile.backedName so SendFile reads the renamed-to file (not the gone .tmp).
	VN(WriteThenRenameTmp,          TC("write=wrt_out.tmp,DATA rename=wrt_out.tmp,wrt_out read=wrt_out,DATA mustfail=stat=wrt_out.tmp"))
	// sbox shell-redirect: child opens output, dup2→stdout, execs sed; sed _exits without close(1), so the inherited fd's writtenFiles entry stays unfinalized. Sbox parent then renames → MoveFile MISSes writtenFiles[fromKey], host kernel-renames a path it doesn't have → UbaFile.cpp:954 "MoveFileExW error handling not implemented" assert.
	VN(ForkRedirectExecRename,      TC("forkredirectexec=fre_out,DATA rename=fre_out,fre_out2 read=fre_out2,DATA mustfail=stat=fre_out"))
	// Dup close-order, original LAST: agent must wait for dup's refcount to hit 0 before finalizing.
	VN(DupCloseReverse,             TC("dupclosereverse=dcr_o,REVERSED read=dcr_o,REVERSED"))
	// Dup close-order, original FIRST: first close must NOT finalize (dup still open); second close finalizes with final size.
	VN(DupCloseInOrder,             TC("dupcloseinorder=dci_o,AB read=dci_o,AB"))
#if 0
	// TODO: We need to re-enable this and fix the bug.
	// The bug is that if the child process inherit a handle and does nothing.. while the parent closes the handle before the child exits (and automatically closes it)
	// Rpc_UpdateCloseHandle will last come from the child with invalid attributes..
	// Cross-process: parent writes+closes, child _exits with untouched inherited fd. Child's kernel-implicit close must not double-close on server.
	VN(ForkParentClosesFirst,       TC("forkparentclosesfirst=fpcf_o,PARENT read=fpcf_o,PARENT rename=fpcf_o,fpcf_o2 read=fpcf_o2,PARENT"))
	// Double-close surface: child closes via detour first, parent closes second — both fire Rpc_UpdateCloseHandle for the same closeId (matches bindings.rs warnings around AOSP edges 48k-63k).
	VN(ForkChildClosesFirst,        TC("forkchildclosesfirst=fccf_o,PARENTWRITE read=fccf_o,PARENTWRITE"))
#endif
	// Closest no-exec analog to forkredirectexec: child writes via inherited fd then _exits (no close); parent's close must finalize.
	VN(ForkChildWritesNoExitClose,  TC("forkchildwritesnoexitclose=fcwn_o,CHILDDATA read=fcwn_o,CHILDDATA rename=fcwn_o,fcwn_o2 read=fcwn_o2,CHILDDATA"))
	// Control case for "no detour close on inherited fd at _exit": exec'd image explicitly closes fd 1 before _exit, so this passes while VerbForkRedirectExecRename does not.
	VN(ForkRedirectExecCloseRename, TC("forkredirectexecclose=frec_o,CLOSED rename=frec_o,frec_o2 read=frec_o2,CLOSED mustfail=stat=frec_o"))
	// AOSP Go compile pattern: child writes .tmp, parent reads back via spawned cmp (sibling), then parent renames .tmp to final. The send-output-files step at process exit must read the renamed file's data — backedName chain breakage between siblings + rename surfaces as "Failed to open file ...tmp for read" on the agent.
	VN(CompileCmpMvPattern,         TC("spawn=write=ccmp.tmp,COMPILED spawn=read=ccmp.tmp,COMPILED rename=ccmp.tmp,ccmp read=ccmp,COMPILED"))
	// AOSP `SymlinkWithBase` regression: rm a DANGLING symlink. fstatat / __fxstatat with AT_SYMLINK_NOFOLLOW must use lstat semantics — without the flag plumb-through, the detour follows the dangling target and returns ENOENT, so rm(1) thinks the file doesn't exist and bails.
	VN(SymlinkRmDangling,           TC("symlink=ghost_target_for_rm,danglog islink=danglog unlink=danglog mustfail=stat=danglog mustfail=islink=danglog"))
	// AOSP sbox regression: rename a file in a child process, then read at dest in a sibling. The dest must be visible via the main dirtable, not just the renaming process's overlay.
	VN(RegRenameSpawnVisible,       TC("mkdir=rdir write=rdir/src.txt,DATA spawn=rename=rdir/src.txt,rdir/dst.txt read=rdir/dst.txt,DATA mustfail=stat=rdir/src.txt"))
	// Sandbox-style: child generates output in a stage dir then renames into place; outer reads at final path.
	VN(RegRenameSandboxLike,        TC("mkdir=stage mkdir=final mkdir=final/sub spawn=write=stage/output.bin,GENERATED;rename=stage/output.bin,final/sub/output.bin read=final/sub/output.bin,GENERATED"))
	// Stresses WriteDirectoryEntriesInternal's refresh path: many renames in one dir cause repeated RefreshDirectory calls. The pre-existing-entries-bug surfaced here when the offset-update loop double-applied filesOffset to entries that were already absolute.
	VN(RenameStressOneDir,          TC("mkdir=rs write=rs/a,1 write=rs/b,2 write=rs/c,3 rename=rs/a,rs/aa rename=rs/b,rs/bb rename=rs/c,rs/cc spawn=stat=rs/aa;stat=rs/bb;stat=rs/cc;read=rs/aa,1;read=rs/bb,2;read=rs/cc,3"))
	// Trigger for the RegisterCreateFileForWrite sentinel-leak: rename a file then delete the dest before exit. The exit-time RefreshDirectory's lstat on dest fails; without erasing the sentinel a subsequent process would SEGV reading m_directoryTableMem at offset ~u32(0). Spawn after to confirm the dirtable is intact.
	VN(RenameThenDeleteDest,        TC("write=rtd_src,X rename=rtd_src,rtd_dst unlink=rtd_dst mustfail=stat=rtd_src mustfail=stat=rtd_dst spawn=mustfail=stat=rtd_dst;mustfail=stat=rtd_src"))
	// AOSP apex `rm -rf image.apex`: dir with dangling symlinks (target /system/lib/* missing on host). rm lstat'd each symlink but never unlinkat'd → rmdir(parent) ENOTEMPTY → UbaNinja halted. Guarded by DT_LNK fix in Shared_getdents64 (UbaDetoursSharedPosix.cpp:484). The `rmrf` verb posix_spawns real /bin/rm -rf and verifies the path is gone.
	VN(RmRfDirWithDanglingSymlink,
		TC("mkdir=rrd symlink=/nonexistent/dangling/target,rrd/link "
		   "rmrf=rrd mustfail=stat=rrd"))
	// Production shape: parent + 6 dangling symlinks + 2 regular files + 1 subdir-with-files. rm must unlink all 9 leaves and recurse.
	VN(RmRfImageApexShape,
		TC("mkdir=rri "
		   "symlink=/system/lib/libunwindstack.so,rri/libunwindstack.so "
		   "symlink=/system/lib/libprocinfo.so,rri/libprocinfo.so "
		   "symlink=/system/lib/libc++.so,rri/libc++.so "
		   "symlink=/system/lib/liblzma.so,rri/liblzma.so "
		   "symlink=/system/lib/ld-android.so,rri/ld-android.so "
		   "symlink=/system/lib/libbase.so,rri/libbase.so "
		   "write=rri/libc_malloc_debug.so,STUB "
		   "write=rri/libc_malloc_hooks.so,STUB "
		   "mkdir=rri/bionic "
		   "write=rri/bionic/libdl.so,STUB "
		   "write=rri/bionic/libc.so,STUB "
		   "rmrf=rri mustfail=stat=rri"))
	// Dangling symlink must report DT_LNK in readdir's d_type — what rm depends on.
	VN(ReaddirDTypeDanglingSymlink,
		TC("mkdir=rdt symlink=/nonexistent/target,rdt/link "
		   "listtype=rdt,link,LNK"))
	// Same but for a NON-dangling symlink (target exists in dir).
	VN(ReaddirDTypeSymlinkToFile,
		TC("mkdir=rds write=rds/real,DATA symlink=real,rds/link "
		   "listtype=rds,link,LNK listtype=rds,real,REG"))
	// Sibling spawn writes a dangling symlink, then root rm -rf parent.
	// Tests that the overlay-propagated symlink is visible to rm via readdir.
	VN(RmRfSpawnCreatedDanglingSymlink,
		TC("mkdir=rss spawn=symlink=/nonexistent/target,rss/link "
		   "rmrf=rss mustfail=stat=rss"))
	// Nested: dir/sub/sub/link — rm must recurse and unlink the deep symlink.
	VN(RmRfNestedDanglingSymlinks,
		TC("mkdir=rnd mkdir=rnd/a mkdir=rnd/a/b "
		   "symlink=/nonexistent/t1,rnd/a/b/link1 "
		   "symlink=/nonexistent/t2,rnd/a/link2 "
		   "symlink=/nonexistent/t3,rnd/link3 "
		   "rmrf=rnd mustfail=stat=rnd"))
#if 0
	// Toybox-rm (AOSP's actual rm): uses readdir d_type more aggressively than GNU rm — if readdir lies (DT_REG for a symlink) toybox takes the wrong path. Skips if toybox prebuilt not installed.
	VN(RmRfToyboxDirWithDanglingSymlink,
		TC("mkdir=trrd symlink=/nonexistent/dangling/target,trrd/link "
		   "rmrf_toybox=trrd mustfail=stat=trrd"))
	VN(RmRfToyboxImageApexShape,
		TC("mkdir=trri "
		   "symlink=/system/lib/libunwindstack.so,trri/libunwindstack.so "
		   "symlink=/system/lib/libprocinfo.so,trri/libprocinfo.so "
		   "symlink=/system/lib/libc++.so,trri/libc++.so "
		   "symlink=/system/lib/liblzma.so,trri/liblzma.so "
		   "symlink=/system/lib/ld-android.so,trri/ld-android.so "
		   "symlink=/system/lib/libbase.so,trri/libbase.so "
		   "write=trri/libc_malloc_debug.so,STUB "
		   "write=trri/libc_malloc_hooks.so,STUB "
		   "mkdir=trri/bionic "
		   "write=trri/bionic/libdl.so,STUB "
		   "write=trri/bionic/libc.so,STUB "
		   "rmrf_toybox=trri mustfail=stat=trri"))
#endif
	// Soong .pc_state.new: sibling A writes .new, sibling B renames to .final. Root-exit's ProcessImpl::WriteFilesToDisk (UbaProcess.cpp:2077) iterates m_shared.writtenFiles and stats .new — which sibling rename already moved → "Failed to get file information for X.new" warning at UbaDirectoryTableHolder.cpp:608. RC=0 but host warning observable in stderr.
	VN(SiblingWriteSiblingRename,   TC("spawn=write=psn.new,STATE spawn=rename=psn.new,psn.final mustfail=stat=psn.new stat=psn.final"))
	// Variant: target not pre-existing; verify content via root read.
	VN(SiblingWriteSiblingRenameRead, TC("spawn=write=psn2.new,V2STATE spawn=rename=psn2.new,psn2.final read=psn2.final,V2STATE mustfail=stat=psn2.new"))
	// Variant: pre-existing .final from previous build, replaced by .new+rename.
	VN(SiblingRenameOverPreexisting,  TC("write=psn3.final,OLD spawn=write=psn3.new,NEW spawn=rename=psn3.new,psn3.final read=psn3.final,NEW mustfail=stat=psn3.new"))
	// Variant: interleaved .new/.final pairs in one dir (cp -t multi-file).
	VN(SiblingMultiRename,            TC("mkdir=msd spawn=write=msd/a.new,AA spawn=write=msd/b.new,BB spawn=rename=msd/a.new,msd/a spawn=rename=msd/b.new,msd/b mustfail=stat=msd/a.new mustfail=stat=msd/b.new stat=msd/a stat=msd/b"))
	// Variant: writer is a grandchild — stresses writtenFiles propagation through 2-deep child tree before sibling rename.
	VN(GrandchildWriteSiblingRename,  TC("spawn=spawn=write=gcw.new,GC spawn=rename=gcw.new,gcw.final mustfail=stat=gcw.new stat=gcw.final"))
	// cp -t (Trusty `cp -t . file1 file2`): cp creates cutmpXXX, writes, closes fd, renames cutmpXXX→makefile. If rename doesn't update file.name from cutmpXXX, exit-time RegisterCreateFileForWrite stats the missing path → "Failed to get file information for cutmpXXX" host error.
	VN(CpDashTSingle,    TC("mkdir=cpd spawn=cptemp=cpd/cutmpABC,cpd/dst.bin,DATA stat=cpd/dst.bin mustfail=stat=cpd/cutmpABC read=cpd/dst.bin,DATA"))
	// Two back-to-back cptemp pairs in same dir (cp -t with two args).
	VN(CpDashTPair,      TC("mkdir=cpdp spawn=cptemp=cpdp/cutmp01,cpdp/a.bin,AAA spawn=cptemp=cpdp/cutmp02,cpdp/b.bin,BBB read=cpdp/a.bin,AAA read=cpdp/b.bin,BBB mustfail=stat=cpdp/cutmp01 mustfail=stat=cpdp/cutmp02"))
	// Interleaved cp -t across processes — cross-process visibility of renamed file.
	VN(CpDashTMP_TwoProc,
		TC("mkdir=cdtp spawn=cptemp=cdtp/cutmpP1,cdtp/p1.bin,P1DATA"),
		TC("spawn=cptemp=cdtp/cutmpP2,cdtp/p2.bin,P2DATA read=cdtp/p1.bin,P1DATA read=cdtp/p2.bin,P2DATA mustfail=stat=cdtp/cutmpP1 mustfail=stat=cdtp/cutmpP2"))
	// GNU coreutils tac pattern: parent opens cutmpXXX (mkstemp), writes, forks child to unlink. writtenFiles is on m_shared which forked children share — if child's unlink doesn't reach parent's map, entry survives → exit-finalize stats the missing path → "Failed to get file information for cutmpXXX" warning.
	VN(ForkUnlinkSmall, TC("mkdir=ful spawn=forkunlink=ful/cutmpA,128 mustfail=stat=ful/cutmpA"))
	VN(ForkUnlinkLarge, TC("mkdir=fll spawn=forkunlink=fll/cutmpB,1048576 mustfail=stat=fll/cutmpB"))


	VN(SymlinkMP_RemovedHidesLeaf,
		TC("mkdir=mpr touch=mpr/leaf symlink=mpr,mpL stat=mpL/leaf"),
		TC("unlink=mpL mustfail=stat=mpL/leaf stat=mpr/leaf"))
	VN(SymlinkMP_SwapVisible,
		TC("write=mtA,a write=mtB,b symlink=mtA,mLs"),
		TC("read=mLs,a unlink=mLs symlink=mtB,mLs read=mLs,b"))
	VN(SymlinkMP_LeafThruLink,
		TC("mkdir=mlk touch=mlk/seed symlink=mlk,mLk"),
		TC("isfile=mLk/seed read=mLk/seed,"))
	VN(SymlinkMP_LeafRemovedThruLink,
		TC("mkdir=mlt touch=mlt/x symlink=mlt,mLt"),
		TC("stat=mLt/x unlink=mlt/x mustfail=stat=mLt/x stat=mlt mustfail=stat=mlt/x"))
	VN(SymlinkMP_AddRemoveSame,
		TC("write=ars,X symlink=ars,L_ars unlink=L_ars"),
		TC("mustfail=stat=L_ars stat=ars"))
	VN(SymlinkMP_AddInP1RemoveInP2,
		TC("write=p1t,1 symlink=p1t,Lp1"),
		TC("stat=Lp1 unlink=Lp1 mustfail=stat=Lp1 stat=p1t"))
	VN(SymlinkMP_RemoveAndReadd,
		TC("write=rar,A symlink=rar,Lrar unlink=Lrar symlink=rar,Lrar"),
		TC("read=Lrar,A islink=Lrar"))
	VN(SymlinkMP_SwapTargetInP2,
		TC("write=A1,a write=A2,b symlink=A1,LswA"),
		TC("read=LswA,a unlink=LswA symlink=A2,LswA read=LswA,b"))
	// VN(SymlinkMP_DanglingThenTargetCreated, ...) — REMOTE: target written in P2 not resolvable through P1's dangling link.
	VN(SymlinkMP_DirSwapVisible,
		TC("mkdir=mdA mkdir=mdB touch=mdA/a touch=mdB/b symlink=mdA,LmdSw"),
		TC("stat=LmdSw/a unlink=LmdSw symlink=mdB,LmdSw stat=LmdSw/b mustfail=stat=LmdSw/a"))


	VN(Symlink3P_AddVerifyRemove,
		TC("write=t3p,abc"),
		TC("symlink=t3p,L3p stat=L3p read=L3p,abc"),
		TC("stat=L3p read=L3p,abc unlink=L3p mustfail=stat=L3p stat=t3p"))
	VN(Symlink3P_DirAddSeeRemove,
		TC("mkdir=d3p touch=d3p/leaf"),
		TC("symlink=d3p,Ld3p stat=Ld3p/leaf"),
		TC("stat=Ld3p/leaf unlink=Ld3p mustfail=stat=Ld3p/leaf stat=d3p/leaf"))
	// VN(Symlink3P_AddRemoveAddNew, ...) — REMOTE only fails: target file BT written in P1 not visible to P3 through the swapped symlink.
	VN(Symlink3P_LeafChangesViaLink,
		TC("mkdir=lct touch=lct/v write=lct/v,one symlink=lct,Llct"),
		TC("stat=Llct/v read=Llct/v,one write=lct/v,two"),
		TC("read=Llct/v,two read=lct/v,two"))
	// VN(Symlink3P_RemovedTargetSymlinkStays, ...) — REMOTE: P2 can't read through P1's symlink to P1's written file.
	// VN(Symlink3P_ChainPropagation, ...) — REMOTE: chain symlinks created across procs don't fully resolve in next proc.


	VN(SymlinkProp_WriteThruLinkRemoveStillReadable,
		TC("mkdir=plr"),
		TC("symlink=plr,Lplr write=Lplr/innerfile,thrulink"),
		TC("unlink=Lplr mustfail=stat=Lplr/innerfile read=plr/innerfile,thrulink"))
	VN(SymlinkProp_WriteThruLinkSwapStillReadable,
		TC("mkdir=psA mkdir=psB"),
		TC("symlink=psA,Lpsw write=Lpsw/foo,viaLink"),
		TC("unlink=Lpsw symlink=psB,Lpsw read=psA/foo,viaLink mustfail=stat=Lpsw/foo"))
	// VN(SymlinkProp_WriteAppendThruLink, ...) — REMOTE: hits the cross-process append-overlay-aliasing bug.
	VN(SymlinkProp_DeepWriteThenRemoveSymlink,
		TC("mkdir=pd1 mkdir=pd1/sub"),
		TC("symlink=pd1,Lpd write=Lpd/sub/leaf,deepthru read=Lpd/sub/leaf,deepthru"),
		TC("unlink=Lpd read=pd1/sub/leaf,deepthru mustfail=stat=Lpd/sub/leaf"))
	VN(SymlinkProp_DirCreatedThenSymlinked,
		TC("mkdir=pdc"),
		TC("symlink=pdc,Lpdc mkdir=Lpdc/inner write=Lpdc/inner/x,via"),
		TC("isdir=pdc/inner read=pdc/inner/x,via unlink=pdc/inner/x rmdir=pdc/inner unlink=Lpdc"))


	VN(SymlinkChain3,               TC("write=c0,X symlink=c0,c1 symlink=c1,c2 symlink=c2,c3 read=c3,X"))
	VN(SymlinkChain5,               TC("write=c0,Y symlink=c0,c1 symlink=c1,c2 symlink=c2,c3 symlink=c3,c4 symlink=c4,c5 read=c5,Y size=c5,1"))
	VN(SymlinkChainBreakMiddle,     TC("write=cm0,A symlink=cm0,cm1 symlink=cm1,cm2 symlink=cm2,cm3 read=cm3,A unlink=cm2 mustfail=read=cm3,"))
	VN(SymlinkChainSwapMid2,        TC("write=ck0,1 write=ck1,2 symlink=ck0,ck_a symlink=ck_a,ck_b read=ck_b,1 unlink=ck_a symlink=ck1,ck_a read=ck_b,2"))
	VN(SymlinkSymlinkSymlink,       TC("write=base_v,Q symlink=base_v,L_a symlink=L_a,L_b symlink=L_b,L_c symlink=L_c,L_d read=L_d,Q"))
	VN(SymlinkChainAllRemoved,      TC("write=ch0,X symlink=ch0,ch1 symlink=ch1,ch2 symlink=ch2,ch3 read=ch3,X unlink=ch1 unlink=ch2 unlink=ch3 mustfail=stat=ch1 mustfail=stat=ch2 mustfail=stat=ch3 stat=ch0"))
	VN(SymlinkChainRebuild,         TC("write=rb0,A symlink=rb0,rb1 symlink=rb1,rb2 read=rb2,A unlink=rb1 mustfail=read=rb2, symlink=rb0,rb1 read=rb2,A"))


	// Static-stub bridge sigmask leak: UbaStaticStub.S wraps every syscall with rt_sigprocmask(BLOCK_ALL)/RESTORE. For clone() the kernel hands the child to a Go-runtime entry, bypassing the RESTORE — child proceeds everything-blocked, execve'd /bin/sh inherits, Pool.terminate's SIGTERM is masked → infinite kill+waitpid loop in jni_registration_generator. Smoke guard: UbaTestGoSigmask runs `grep ^Sig... /proc/self/status` and we assert SigBlk is all zeros. This minimal repro currently PASSES (the production failure needs sbox-style patterns we don't model yet), but a regression that DOES leak through bare exec.Command will turn it red.
	static bool RunGoSigmask(LoggerWithWriter& logger, SessionServer&, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t)
	{
		StringBuffer<MaxPath> binPath;
		GetDirectoryOfCurrentModule(logger, binPath);
		binPath.EnsureEndsWithSlash().Append(TC("UbaTestGoSigmask"));
		if (!FileExists(logger, binPath.data))
			return logger.Error(TC("UbaTestGoSigmask not found at %s - run Claude/static_go_sigmask/Build.sh"), binPath.data);

		ProcessStartInfo pi;
		pi.application = binPath.data;
		pi.workingDir = workingDir;
		pi.description = TC("GoSigmask");
		pi.logFile = TC("GoSigmask.log");
		pi.logLineFunc = [](void*, const tchar* line, u32, LogEntryType) { LoggerWithWriter(g_consoleLogWriter, TC("")).Info(line); };

		ProcessHandle p = runProcess(pi);
		CHECK_TRUEF(p.IsValid() && p.WaitForExit(GetTimeoutTime()), TC("UbaTestGoSigmask did not exit"));
		u32 ec = p.GetExitCode();
		if (ec != 0)
		{
			for (auto& l : p.GetLogLines()) logger.Error(l.text.c_str());
			return logger.Error(TC("UbaTestGoSigmask exit=%u"), ec);
		}

		// /proc/self/status SigBlk: "SigBlk:\t<16 hex digits>". All-zero = clean inherit; any non-zero = bridge leaked BLOCK_ALL through clone-child execve.
		StringBuffer<64> sigBlk;
		bool found = false;
		for (auto& l : p.GetLogLines())
		{
			const tchar* t = l.text.c_str();
			if (!StartsWith(t, TC("SigBlk:")))
				continue;
			const tchar* v = t + 7;
			while (*v == ' ' || *v == '\t') ++v;
			sigBlk.Append(v);
			found = true;
			break;
		}
		if (!found)
		{
			for (auto& l : p.GetLogLines()) logger.Info(l.text.c_str());
			return logger.Error(TC("UbaTestGoSigmask produced no SigBlk: line"));
		}

		while (sigBlk.count > 0)
		{
			tchar c = sigBlk.data[sigBlk.count - 1];
			if (c == ' ' || c == '\t' || c == '\r' || c == '\n') { sigBlk.data[--sigBlk.count] = 0; continue; }
			break;
		}

		bool allZero = sigBlk.count > 0;
		for (u32 i = 0; i < sigBlk.count; ++i)
			if (sigBlk.data[i] != '0') { allZero = false; break; }

		if (!allZero)
		{
			for (auto& l : p.GetLogLines()) logger.Error(l.text.c_str());
			return logger.Error(TC("Static-stub bridge leaked sigmask through clone+execve: child SigBlk=%s (expected all zeros). UbaStaticStub.S step 12 is not running on the clone-child return path."), sigBlk.data);
		}
		return true;
	}

	bool TestVerbGoSigmask(LoggerWithWriter& logger, const StringBufferBase& testRootDir)
	{
		// Local no-op: stock Go binaries hit an unrelated assert in hook_go_rawsyscall6 → get_current_dir_name at _dl_init under libUbaDetours.so.
		return true;
	}
	bool TestRemoteVerbGoSigmask(LoggerWithWriter& logger, const StringBufferBase& testRootDir)
	{
		return RunRemote(logger, testRootDir, RunGoSigmask);
	}

	// Static-stub vfork bridge: 4 child-spawn cases (success+stdout, ENOENT, non-zero exit, stderr capture). Pins down that the spawn path itself works so future "stack split at bad time" reports on AL2025 helpers can be ruled in/out fast (real cause was knownBinaries-staged sed/rsync failing GLIBC_2.38 + Go exec error mis-handling, not the bridge).
	static bool RunGoExecErrors(LoggerWithWriter& logger, SessionServer&, const tchar* workingDir, const RunProcessFunction& runProcess, uintptr_t)
	{
		StringBuffer<MaxPath> binPath;
		GetDirectoryOfCurrentModule(logger, binPath);
		binPath.EnsureEndsWithSlash().Append(TC("UbaTestGoExecErrors"));
		if (!FileExists(logger, binPath.data))
			return logger.Error(TC("UbaTestGoExecErrors not found at %s - run Claude/static_go_exec_errors/Build.sh"), binPath.data);

		ProcessStartInfo pi;
		pi.application = binPath.data;
		pi.workingDir = workingDir;
		pi.description = TC("GoExecErrors");
		pi.logFile = TC("GoExecErrors.log");
		pi.logLineFunc = [](void*, const tchar* line, u32, LogEntryType) { LoggerWithWriter(g_consoleLogWriter, TC("")).Info(line); };

		ProcessHandle p = runProcess(pi);
		CHECK_TRUEF(p.IsValid() && p.WaitForExit(GetTimeoutTime()), TC("UbaTestGoExecErrors did not exit"));
		u32 ec = p.GetExitCode();
		if (ec != 0)
		{
			for (auto& l : p.GetLogLines()) logger.Error(l.text.c_str());
			return logger.Error(TC("UbaTestGoExecErrors exit=%u (one or more case_N FAIL lines above)"), ec);
		}

		// Defensive: scan for any FAIL line even when ec==0 (a future bug could produce FAIL without flipping exit).
		for (auto& l : p.GetLogLines())
		{
			const tchar* t = l.text.c_str();
			const tchar* fail = TStrstr(t, TC("FAIL:"));
			if (fail)
			{
				for (auto& l2 : p.GetLogLines()) logger.Error(l2.text.c_str());
				return logger.Error(TC("UbaTestGoExecErrors emitted a FAIL line: %s"), t);
			}
		}
		return true;
	}

	bool TestVerbGoExecErrors(LoggerWithWriter& logger, const StringBufferBase& testRootDir)
	{
		// Local mode is a no-op for the same reason as GoSigmask above.
		return true;
	}
	bool TestRemoteVerbGoExecErrors(LoggerWithWriter& logger, const StringBufferBase& testRootDir)
	{
		return RunRemote(logger, testRootDir, RunGoExecErrors);
	}
#endif

}
