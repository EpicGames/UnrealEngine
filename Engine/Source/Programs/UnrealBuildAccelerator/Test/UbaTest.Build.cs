// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaTest : ModuleRules
{
	public UbaTest(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivatePCHHeaderFile = "../Core/Public/UbaCorePch.h";
		IWYUSupport = IWYUSupport.None;
		
		bRequiresPlatformSDK = true;

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PrivateDependencyModuleNames.AddRange(new string[] {
			"UbaCommon",
		});

		PrivateDefinitions.AddRange(new string[] {
			"_CONSOLE",
		});
	}
}
