// Copyright Epic Games, Inc. All Rights Reserved.

// dlsym forwarders into libUbaDetours.so. TestApp is EXEC + -z now so weak undef refs to
// LD_PRELOAD'd symbols don't bind; dlsym(RTLD_DEFAULT) walks global scope including the preload.
#if defined(__linux__)
typedef int (*uba_AddPassThroughRoot_fn)(const char*, unsigned int);
typedef int (*uba_IsUnderPassThroughRoot_fn)(const char*, unsigned int);
typedef int (*uba_AddVfsEntry_fn)(const char*, unsigned int, const char*, unsigned int);
#endif

#include <iostream>
#include <fcntl.h>
#include <unistd.h>
#include <sys/stat.h>
#include <sys/syscall.h>
#include <stdio.h>
#include <stdlib.h>
#include <string.h>
#include <stdarg.h>
#include <errno.h>
#include <dirent.h>
#include <dlfcn.h>
#include <spawn.h>

#if PLATFORM_LINUX
#include <link.h>
#include <wait.h>
#include <signal.h>
#include <sys/mount.h>
extern "C" int pivot_root(const char* new_root, const char* put_old); // no glibc wrapper
#else
#include <mach-o/dyld.h>
extern char** environ;
#endif


int LogError(const char* format, ...)
{
	va_list args;
	va_start(args, format);
	char buffer[2048];
	vsnprintf(buffer, 2048, format, args);
	printf("%s\n", buffer);
	va_end(args);
	return -1;
}

#define CHECK_TRUE(x) \
	do { \
	if (!(x)) \
		return LogError("Failed %s (%s:%u)", #x, __FILE__, __LINE__); \
	} while (false)

#define CHECK_TRUEF(x, ...) \
	do { \
	if (!(x)) \
		return LogError(##__VA_ARGS__); \
	} while (false)

bool IsLibraryLoaded(const char* libraryToMatch)
	{
		bool foundLib = false;
#if PLATFORM_LINUX
		// The dl_iterate_phdr() function walks through the list of an
		// application's shared objects and calls the function callback once
		// for each object, until either all shared objects have been
		// processed or callback returns a nonzero value.
		// The last iteration of the callbacks return is propigated all the way up
		foundLib = dl_iterate_phdr([](struct dl_phdr_info* info, size_t size, void* libraryToMatch)
				{
					return strstr(info->dlpi_name, (const char*)libraryToMatch) != 0 ? 1 : 0;
				}, (void*)libraryToMatch);
#elif PLATFORM_MAC

		bool foundDetoursLib = false;
		unsigned int count = _dyld_image_count();
		for (int i = 0; i < count; i++)
		{
			const char* ImageName = _dyld_get_image_name(i);
			if (strstr(ImageName, libraryToMatch))
			{
				foundLib = true;
				break;
			}
		}
#endif
		return foundLib;
	}


// Verb dispatcher — see UbaTestSession.cpp / TestVerb* for the testing protocol.
static int VErr(const char* fmt, ...) { va_list a; va_start(a,fmt); char b[1024]; vsnprintf(b,1024,fmt,a); va_end(a); printf("verb fail: %s (errno=%d)\n", b, errno); return 1; }
static bool VEq(const char* a, int n, const char* w) { int x=(int)strlen(w); return n==x && strncmp(a,w,x)==0; }
static void VSplit(const char* v, char* a, size_t aCap, char* b, size_t bCap, char sep=',')
{
	const char* s = strchr(v, sep);
	size_t aL = s ? (size_t)(s - v) : strlen(v);
	if (aL >= aCap) aL = aCap - 1;
	memcpy(a, v, aL); a[aL] = 0;
	if (s) { strncpy(b, s+1, bCap-1); b[bCap-1] = 0; } else b[0] = 0;
}
static int VRunVerb(const char* arg)
{
	const char* eq = strchr(arg, '='); int nL = eq?(int)(eq-arg):(int)strlen(arg);
	const char* v = eq?eq+1:"";
	if (VEq(arg,nL,"mkdir"))    return mkdir(v, 0755) == 0 ? 0 : VErr("mkdir %s", v);
	if (VEq(arg,nL,"rmdir"))    return rmdir(v) == 0 ? 0 : VErr("rmdir %s", v);
	if (VEq(arg,nL,"unlink"))   return unlink(v) == 0 ? 0 : VErr("unlink %s", v);
	if (VEq(arg,nL,"touch"))
	{
		int fd = open(v, O_WRONLY|O_CREAT|O_TRUNC, 0644);
		if (fd < 0) return VErr("touch %s", v);
		close(fd); return 0;
	}
	if (VEq(arg,nL,"rdcreat"))
	{
		// GNU touch uses O_RDONLY|O_CREAT — must be classified as a write.
		int fd = open(v, O_RDONLY|O_CREAT, 0666);
		if (fd < 0) return VErr("rdcreat open %s", v);
		close(fd); return 0;
	}
	if (VEq(arg,nL,"dlopen"))
	{
		// Exercises detour dlopen pipeline; load is allowed to fail, we only care it doesn't assert.
		(void)dlopen(v, RTLD_LAZY);
		(void)dlerror();
		return 0;
	}
	if (VEq(arg,nL,"dlopenok"))
	{
		void* h = dlopen(v, RTLD_NOW);
		if (!h) { const char* e = dlerror(); return VErr("dlopenok %s failed: %s", v, e ? e : "(null)"); }
		dlclose(h);
		(void)dlerror();
		return 0;
	}
	if (VEq(arg,nL,"vfsenvprobe"))
	{
		const char* val = getenv("UBA_VFS_OVERLAYS");
		if (!val || !*val) return VErr("vfsenvprobe UBA_VFS_OVERLAYS missing/empty");
		return 0;
	}
#if defined(__linux__)
	if (VEq(arg,nL,"passthrough_register"))
	{
		static uba_AddPassThroughRoot_fn fn = (uba_AddPassThroughRoot_fn)dlsym(RTLD_DEFAULT, "uba_AddPassThroughRoot");
		if (!fn) return VErr("passthrough_register: libUbaDetours not loaded (dlsym failed)");
		int ok = fn(v, (unsigned int)strlen(v));
		return ok ? 0 : VErr("passthrough_register %s (table full)", v);
	}
	if (VEq(arg,nL,"passthrough_check"))
	{
		static uba_IsUnderPassThroughRoot_fn fn = (uba_IsUnderPassThroughRoot_fn)dlsym(RTLD_DEFAULT, "uba_IsUnderPassThroughRoot");
		if (!fn) return VErr("passthrough_check: libUbaDetours not loaded (dlsym failed)");
		int under = fn(v, (unsigned int)strlen(v));
		if (!under) return VErr("passthrough_check %s: NOT under any registered pass-through root (env propagation likely failed)", v);
		return 0;
	}
	if (VEq(arg,nL,"vfs_add"))
	{
		static uba_AddVfsEntry_fn fn = (uba_AddVfsEntry_fn)dlsym(RTLD_DEFAULT, "uba_AddVfsEntry");
		if (!fn) return VErr("vfs_add: libUbaDetours not loaded (dlsym failed)");
		char vfs[512], local[512]; VSplit(v, vfs, 512, local, 512);
		int ok = fn(vfs, (unsigned int)strlen(vfs), local, (unsigned int)strlen(local));
		return ok ? 0 : VErr("vfs_add %s,%s (table full)", vfs, local);
	}
#endif
	if (VEq(arg,nL,"findlibc"))
	{
		// Probe /usr/lib first — on Debian/Ubuntu /lib is a symlink and UBA's DirectoryTable asserts on traversing it.
		static const char* const kCandidates[] = {
			"/usr/lib/x86_64-linux-gnu/libc.so.6",
			"/usr/lib64/libc.so.6",
			"/usr/lib/aarch64-linux-gnu/libc.so.6",
			"/usr/lib/libc.so.6",
			"/lib/x86_64-linux-gnu/libc.so.6",
			"/lib64/libc.so.6",
			"/lib/aarch64-linux-gnu/libc.so.6",
			"/lib/libc.so.6",
		};
		const char* src = nullptr;
		for (const char* c : kCandidates)
		{
			struct stat st;
			if (lstat(c, &st) != 0) continue;
			char tmp[256]; strncpy(tmp, c, sizeof(tmp)-1); tmp[sizeof(tmp)-1] = 0;
			bool ok = true;
			for (char* slash = strchr(tmp+1, '/'); slash; slash = strchr(slash+1, '/'))
			{
				*slash = 0;
				struct stat ps;
				if (lstat(tmp, &ps) != 0 || S_ISLNK(ps.st_mode)) { ok = false; *slash = '/'; break; }
				*slash = '/';
			}
			if (ok) { src = c; break; }
		}
		if (!src) return VErr("findlibc: no real-path libc.so.6 found in known locations");
#if PLATFORM_LINUX
		return mount(src, v, nullptr, MS_BIND, nullptr) == 0 ? 0 : VErr("findlibc bindmount %s,%s errno=%d", src, v, errno);
#else
		return VErr("findlibc only on linux");
#endif
	}
	if (VEq(arg,nL,"write") || VEq(arg,nL,"append"))
	{
		bool app = arg[0]=='a';
		char p[512], c[2048]; VSplit(v, p, 512, c, 2048);
		int fd = open(p, O_WRONLY|O_CREAT|(app?O_APPEND:O_TRUNC), 0644);
		if (fd < 0) return VErr("write %s", p);
		size_t n = strlen(c);
		ssize_t w = (n == 0) ? 0 : write(fd, c, n);
		close(fd);
		return (size_t)w == n ? 0 : VErr("write %s", p);
	}
	if (VEq(arg,nL,"read"))
	{
		char p[512], expect[2048]; VSplit(v, p, 512, expect, 2048);
		int fd = open(p, O_RDONLY);
		if (fd < 0) return VErr("read open %s", p);
		char buf[2048]; ssize_t got = read(fd, buf, sizeof(buf)); close(fd);
		if (got < 0) return VErr("read %s", p);
		size_t eL = strlen(expect);
		if ((size_t)got != eL) return VErr("read %s len=%zd want=%zu", p, got, eL);
		if (memcmp(buf, expect, eL) != 0) return VErr("read %s mismatch", p);
		return 0;
	}
	if (VEq(arg,nL,"writeopenread"))
	{
		// Read of an open-for-write path: writtenFiles empty, m_outputFiles has the staged path.
		char p[512], c[2048]; VSplit(v, p, 512, c, 2048);
		int wfd = open(p, O_WRONLY|O_CREAT|O_TRUNC, 0644);
		if (wfd < 0) return VErr("writeopenread open-w %s", p);
		size_t n = strlen(c);
		if (n != 0 && (size_t)write(wfd, c, n) != n) { close(wfd); return VErr("writeopenread write %s", p); }
		fsync(wfd);
		int rfd = open(p, O_RDONLY);
		if (rfd < 0) { close(wfd); return VErr("writeopenread open-r %s", p); }
		char buf[2048]; ssize_t got = read(rfd, buf, sizeof(buf));
		close(rfd); close(wfd);
		if (got < 0) return VErr("writeopenread read %s", p);
		if ((size_t)got != n) return VErr("writeopenread %s len=%zd want=%zu", p, got, n);
		if (n != 0 && memcmp(buf, c, n) != 0) return VErr("writeopenread %s mismatch", p);
		return 0;
	}
	if (VEq(arg,nL,"size"))
	{
		char p[512], n[64]; VSplit(v, p, 512, n, 64);
		struct stat st; if (stat(p, &st) != 0) return VErr("size %s", p);
		long long want = atoll(n);
		return (long long)st.st_size == want ? 0 : VErr("size %s got=%lld want=%lld", p, (long long)st.st_size, want);
	}
	if (VEq(arg,nL,"stat"))      { struct stat st; return stat(v, &st) == 0 ? 0 : VErr("stat %s", v); }
	if (VEq(arg,nL,"notexists")) { struct stat st; return stat(v, &st) != 0 ? 0 : VErr("notexists %s", v); }
	if (VEq(arg,nL,"isdir"))     { struct stat st; return (stat(v, &st) == 0 && S_ISDIR(st.st_mode)) ? 0 : VErr("isdir %s", v); }
	if (VEq(arg,nL,"isfile"))    { struct stat st; return (stat(v, &st) == 0 && S_ISREG(st.st_mode)) ? 0 : VErr("isfile %s", v); }
	if (VEq(arg,nL,"islink"))    { struct stat st; return (lstat(v, &st) == 0 && S_ISLNK(st.st_mode)) ? 0 : VErr("islink %s", v); }
	if (VEq(arg,nL,"xaccess"))   return access(v, X_OK) == 0 ? 0 : VErr("xaccess %s", v);
	if (VEq(arg,nL,"rename"))    { char s[512],d[512]; VSplit(v,s,512,d,512); return rename(s,d) == 0 ? 0 : VErr("rename %s,%s", s, d); }
	if (VEq(arg,nL,"copy"))
	{
		char s[512], d[512]; VSplit(v, s, 512, d, 512);
		int rfd = open(s, O_RDONLY); if (rfd < 0) return VErr("copy open %s", s);
		int wfd = open(d, O_WRONLY|O_CREAT|O_TRUNC, 0644); if (wfd < 0) { close(rfd); return VErr("copy open %s", d); }
		char buf[8192]; ssize_t r;
		while ((r = read(rfd, buf, sizeof(buf))) > 0) if (write(wfd, buf, r) != r) { close(rfd); close(wfd); return VErr("copy write %s", d); }
		close(rfd); close(wfd);
		return r == 0 ? 0 : VErr("copy read %s", s);
	}
	// `cp -t` pattern: O_EXCL temp + write + close + rename(temp,final). Rename must update writtenFile.name.
	if (VEq(arg,nL,"cptemp"))
	{
		char tmp[512], dst[512];
		VSplit(v, tmp, 512, dst, 512);
		char data[2048];
		VSplit(dst, dst, 512, data, 2048);
		int fd = open(tmp, O_CREAT|O_EXCL|O_WRONLY, 0600);
		if (fd < 0) return VErr("cptemp open %s O_EXCL", tmp);
		size_t n = strlen(data);
		if (n && (size_t)write(fd, data, n) != n) { close(fd); return VErr("cptemp write %s", tmp); }
		close(fd);
		if (rename(tmp, dst) != 0) return VErr("cptemp rename %s,%s", tmp, dst);
		return 0;
	}
	// GNU tac pattern: parent opens+writes temp, fork(), child unlink+close+exit, parent waitpid+close.
	// Child's unlink must reach parent's writtenFiles map so process-exit finalize doesn't orphan it.
	if (VEq(arg,nL,"forkunlink"))
	{
		char path[512], szs[64];
		VSplit(v, path, 512, szs, 64);
		long size = atol(szs);
		int fd = open(path, O_RDWR|O_CREAT|O_EXCL, 0600);
		if (fd < 0) return VErr("forkunlink open %s", path);
		if (size > 0)
		{
			char buf[4096];
			memset(buf, 'X', sizeof(buf));
			long remaining = size;
			while (remaining > 0)
			{
				ssize_t want = remaining > (long)sizeof(buf) ? (ssize_t)sizeof(buf) : (ssize_t)remaining;
				if (write(fd, buf, want) != want) { close(fd); return VErr("forkunlink write %s", path); }
				remaining -= want;
			}
		}
		pid_t pid = fork();
		if (pid < 0) { close(fd); return VErr("forkunlink fork"); }
		if (pid == 0)
		{
			if (unlink(path) != 0) _exit(2);
			close(fd);
			_exit(0);
		}
		int status = 0;
		if (waitpid(pid, &status, 0) != pid) { close(fd); return VErr("forkunlink waitpid"); }
		close(fd);
		if (!WIFEXITED(status) || WEXITSTATUS(status) != 0)
			return VErr("forkunlink child exit %d (status=0x%x)", WEXITSTATUS(status), status);
		return 0;
	}
	if (VEq(arg,nL,"chdir"))     return chdir(v) == 0 ? 0 : VErr("chdir %s", v);
	if (VEq(arg,nL,"cwd"))       { char cur[1024]; if (!getcwd(cur, 1024)) return VErr("cwd get"); return strcmp(cur, v) == 0 ? 0 : VErr("cwd got=%s want=%s", cur, v); }
	if (VEq(arg,nL,"listdir"))
	{
		char p[512], cs[64]; VSplit(v, p, 512, cs, 64);
		int want = atoi(cs);
		DIR* d = opendir(p); if (!d) return VErr("listdir %s", p);
		int n = 0; dirent* e;
		while ((e = readdir(d))) if (strcmp(e->d_name, ".") && strcmp(e->d_name, "..")) ++n;
		closedir(d);
		return n == want ? 0 : VErr("listdir %s got=%d want=%d", p, n, want);
	}
	if (VEq(arg,nL,"listcontains") || VEq(arg,nL,"listexcludes"))
	{
		bool wantPresent = arg[4] == 'c';
		char p[512], name[256]; VSplit(v, p, 512, name, 256);
		DIR* d = opendir(p); if (!d) return VErr("list %s", p);
		bool found = false; dirent* e;
		while ((e = readdir(d))) if (strcmp(e->d_name, name) == 0) { found = true; break; }
		closedir(d);
		return (found == wantPresent) ? 0 : VErr("list %s want %s of %s", p, wantPresent?"contain":"exclude", name);
	}
	if (VEq(arg,nL,"dirfd_openat"))
	{
		// opendir → dirfd → openat(fd, subname). Virtual DIR* must mint a real fd backing dirfd.
		char p[512], name[256]; VSplit(v, p, 512, name, 256);
		DIR* d = opendir(p);
		if (!d) return VErr("dirfd_openat opendir %s", p);
		int fd = dirfd(d);
		if (fd < 0) { closedir(d); return VErr("dirfd_openat dirfd -> %i errno=%i", fd, errno); }
		int sub = openat(fd, name, O_RDONLY);
		closedir(d);
		if (sub < 0) return VErr("dirfd_openat openat(fd=%i,%s) errno=%i", fd, name, errno);
		close(sub);
		return 0;
	}
	if (VEq(arg,nL,"fdopendir_readdir"))
	{
		// open(O_DIRECTORY) → fd → fdopendir → readdir. Exercises Shared_open's CreateDirFd path.
		char p[512], wantName[256]; VSplit(v, p, 512, wantName, 256);
		int fd = open(p, O_RDONLY | O_DIRECTORY);
		if (fd < 0) return VErr("fdopendir_readdir open %s errno=%i", p, errno);
		DIR* d = fdopendir(fd);
		if (!d) { close(fd); return VErr("fdopendir_readdir fdopendir errno=%i", errno); }
		bool found = false; dirent* e;
		while ((e = readdir(d))) if (strcmp(e->d_name, wantName) == 0) { found = true; break; }
		closedir(d); // closes fd too per POSIX
		return found ? 0 : VErr("fdopendir_readdir %s missing in %s", wantName, p);
	}
	if (VEq(arg,nL,"sleep"))
	{
		int ms = atoi(v); struct timespec ts{ ms/1000, (ms%1000)*1000000 };
		nanosleep(&ts, nullptr); return 0;
	}
	if (VEq(arg,nL,"redirectspawn"))
	{
		// `cmd > X` via posix_spawn dup2: writtenFile attrs must finalize for child-inherited fd.
		char p[512], c[2048]; VSplit(v, p, 512, c, 2048);
		int wfd = open(p, O_WRONLY|O_CREAT|O_TRUNC, 0644);
		if (wfd < 0) return VErr("redirectspawn open %s", p);
		const char* prog = "/usr/bin/echo";
		const char* argv[] = { prog, "-n", c, nullptr };
		posix_spawn_file_actions_t fa;
		posix_spawn_file_actions_init(&fa);
		posix_spawn_file_actions_adddup2(&fa, wfd, 1);
		posix_spawn_file_actions_addclose(&fa, wfd);
		pid_t pid = 0;
		int r = posix_spawn(&pid, prog, &fa, nullptr, (char**)argv, environ);
		posix_spawn_file_actions_destroy(&fa);
		close(wfd);
		if (r != 0) return VErr("redirectspawn posix_spawn=%d", r);
		int status = 0;
		while (waitpid(pid, &status, 0) == -1) if (errno != EINTR) return VErr("redirectspawn waitpid");
		if (!WIFEXITED(status) || WEXITSTATUS(status) != 0) return VErr("redirectspawn child status=%d", status);
		struct stat st;
		if (stat(p, &st) != 0) return VErr("redirectspawn stat %s errno=%d", p, errno);
		if (!S_ISREG(st.st_mode)) return VErr("redirectspawn %s not regular (mode=0%o)", p, st.st_mode);
		size_t expected = strlen(c);
		if ((size_t)st.st_size != expected) return VErr("redirectspawn %s size=%lld want=%zu", p, (long long)st.st_size, expected);
		return 0;
	}
	if (VEq(arg,nL,"rmrf") || VEq(arg,nL,"rmrf_toybox"))
	{
		// GNU `/bin/rm -rf` vs AOSP prebuilt toybox rm; toybox d_type handling differs from GNU rm.
		bool useToybox = VEq(arg,nL,"rmrf_toybox");
		const char* prog = useToybox
			? "/home/honk/git/android/prebuilts/build-tools/path/linux-x86/rm"
			: "/bin/rm";
		struct stat st0;
		if (useToybox && stat(prog, &st0) != 0) return 0; // toybox prebuilt missing — skip

		const char* argv[] = { prog, "-rf", v, nullptr };
		pid_t pid = 0;
		int r = posix_spawn(&pid, prog, nullptr, nullptr, (char**)argv, environ);
		if (r != 0) return VErr("rmrf posix_spawn=%d", r);
		int status = 0;
		while (waitpid(pid, &status, 0) == -1) if (errno != EINTR) return VErr("rmrf waitpid");
		if (!WIFEXITED(status)) return VErr("rmrf child signalled status=%d", status);
		int ec = WEXITSTATUS(status);
		if (ec != 0) return VErr("rmrf %s ec=%d (rm couldn't fully remove)", v, ec);
		struct stat st;
		if (lstat(v, &st) == 0) return VErr("rmrf %s: rm reported success but path still exists (mode=0%o)", v, st.st_mode);
		if (errno != ENOENT) return VErr("rmrf %s: post-rm lstat errno=%d (want ENOENT)", v, errno);
		return 0;
	}
	if (VEq(arg,nL,"listtype"))
	{
		// Verify getdents64 d_type for a named entry (LNK/REG/DIR/UNKNOWN). Detour must preserve/synthesize correctly.
		char p[512], rest[512]; VSplit(v, p, 512, rest, 512);
		char name[256], typeStr[16]; VSplit(rest, name, 256, typeStr, 16);
		DIR* d = opendir(p);
		if (!d) return VErr("listtype opendir %s errno=%d", p, errno);
		int actualType = -1;
		dirent* e;
		while ((e = readdir(d)))
		{
			if (strcmp(e->d_name, name) == 0)
			{
				actualType = e->d_type;
				break;
			}
		}
		closedir(d);
		if (actualType < 0) return VErr("listtype %s/%s: entry not found in readdir", p, name);
		int wantType = -1;
		if      (!strcmp(typeStr, "LNK"))     wantType = DT_LNK;
		else if (!strcmp(typeStr, "REG"))     wantType = DT_REG;
		else if (!strcmp(typeStr, "DIR"))     wantType = DT_DIR;
		else if (!strcmp(typeStr, "UNKNOWN")) wantType = DT_UNKNOWN;
		else return VErr("listtype %s/%s: unknown TYPE %s (want LNK/REG/DIR/UNKNOWN)", p, name, typeStr);
		if (actualType != wantType) return VErr("listtype %s/%s: got d_type=%d want=%d (%s)", p, name, actualType, wantType, typeStr);
		return 0;
	}
	if (VEq(arg,nL,"renamewhileopen"))
	{
		// rustc archive pattern: rename src→dst with fd still open. Detour must propagate via fo.newName.
		char src[512], dst[512]; VSplit(v, src, 512, dst, 512);
		int fd = open(src, O_RDWR|O_CREAT|O_CLOEXEC|O_EXCL, 0644);
		if (fd < 0) return VErr("renamewhileopen open %s errno=%d", src, errno);
		static const char content[] = "rustc-archive-pattern";
		ssize_t wlen = sizeof(content) - 1;
		if (write(fd, content, wlen) != wlen) { close(fd); return VErr("renamewhileopen write"); }
		if (rename(src, dst) != 0) { int e = errno; close(fd); return VErr("renamewhileopen rename %s -> %s errno=%d", src, dst, e); }
		if (close(fd) != 0) return VErr("renamewhileopen close errno=%d", errno);
		struct stat st;
		if (stat(dst, &st) != 0) return VErr("renamewhileopen stat %s errno=%d", dst, errno);
		if (!S_ISREG(st.st_mode)) return VErr("renamewhileopen %s not regular (mode=0%o)", dst, st.st_mode);
		if ((size_t)st.st_size != (size_t)wlen) return VErr("renamewhileopen %s size=%lld want=%zd", dst, (long long)st.st_size, wlen);
		return 0;
	}
	if (VEq(arg,nL,"redirectsaverestore"))
	{
		// dash redirect-with-restore: F_DUPFD save, dup2 onto fd 1, spawn, dup2 restore.
		// dup2 wrapper must pre-finalize writtenFile attrs while fd 1 still points at the file.
		char p[512], c[2048]; VSplit(v, p, 512, c, 2048);
		int saved = fcntl(1, F_DUPFD, 10);
		if (saved < 0) return VErr("redirectsaverestore F_DUPFD");
		int wfd = open(p, O_WRONLY|O_CREAT|O_TRUNC, 0644);
		if (wfd < 0) { close(saved); return VErr("redirectsaverestore open %s", p); }
		if (dup2(wfd, 1) != 1) { close(wfd); close(saved); return VErr("redirectsaverestore dup2(wfd,1)"); }
		close(wfd);
		const char* prog = "/usr/bin/echo";
		const char* argv[] = { prog, "-n", c, nullptr };
		pid_t pid = 0;
		int r = posix_spawn(&pid, prog, nullptr, nullptr, (char**)argv, environ);
		if (r != 0) { dup2(saved, 1); close(saved); return VErr("redirectsaverestore posix_spawn=%d", r); }
		int status = 0;
		while (waitpid(pid, &status, 0) == -1) if (errno != EINTR) { dup2(saved, 1); close(saved); return VErr("redirectsaverestore waitpid"); }
		if (dup2(saved, 1) != 1) { close(saved); return VErr("redirectsaverestore dup2(saved,1)"); }
		close(saved);
		if (!WIFEXITED(status) || WEXITSTATUS(status) != 0) return VErr("redirectsaverestore child status=%d", status);
		struct stat st;
		if (stat(p, &st) != 0) return VErr("redirectsaverestore stat %s errno=%d", p, errno);
		if (!S_ISREG(st.st_mode)) return VErr("redirectsaverestore %s not regular (mode=0%o)", p, st.st_mode);
		size_t expected = strlen(c);
		if ((size_t)st.st_size != expected) return VErr("redirectsaverestore %s size=%lld want=%zu", p, (long long)st.st_size, expected);
		return 0;
	}
	// bash `cmd > path` before exec: child opens+dup2(1)+execve, never calls close(1) — detour must
	// still finalize the writtenFile entry across the kernel-implicit fd-1 close at process teardown.
	if (VEq(arg,nL,"forkredirectexec"))
	{
		char p[512], c[2048]; VSplit(v, p, 512, c, 2048);
		pid_t pid = fork();
		if (pid < 0) return VErr("forkredirectexec fork errno=%d", errno);
		if (pid == 0)
		{
			int wfd = open(p, O_WRONLY|O_CREAT|O_TRUNC, 0644);
			if (wfd < 0) _exit(71);
			if (dup2(wfd, 1) != 1) _exit(72);
			if (close(wfd) != 0) _exit(73);
			const char* prog = "/usr/bin/echo";
			char* argv2[] = { (char*)prog, (char*)"-n", c, nullptr };
			execve(prog, argv2, environ);
			_exit(74);
		}
		int status = 0;
		while (waitpid(pid, &status, 0) == -1) if (errno != EINTR) return VErr("forkredirectexec waitpid errno=%d", errno);
		if (!WIFEXITED(status)) return VErr("forkredirectexec child signalled status=%d", status);
		if (WEXITSTATUS(status) != 0) return VErr("forkredirectexec child ec=%d", WEXITSTATUS(status));
		struct stat st;
		if (stat(p, &st) != 0) return VErr("forkredirectexec stat %s errno=%d", p, errno);
		if (!S_ISREG(st.st_mode)) return VErr("forkredirectexec %s not regular (mode=0%o)", p, st.st_mode);
		size_t expected = strlen(c);
		if ((size_t)st.st_size != expected) return VErr("forkredirectexec %s size=%lld want=%zu", p, (long long)st.st_size, expected);
		return 0;
	}
	// dup refCount: open → dup → close(dup) → write(orig) → close(orig). Agent must not finalize at dup close.
	if (VEq(arg,nL,"dupclosereverse"))
	{
		char p[512], c[2048]; VSplit(v, p, 512, c, 2048);
		int fd = open(p, O_WRONLY|O_CREAT|O_TRUNC, 0644);
		if (fd < 0) return VErr("dupclosereverse open %s errno=%d", p, errno);
		int dfd = dup(fd);
		if (dfd < 0) { close(fd); return VErr("dupclosereverse dup errno=%d", errno); }
		if (close(dfd) != 0) { close(fd); return VErr("dupclosereverse close(dfd) errno=%d", errno); }
		size_t n = strlen(c);
		if (write(fd, c, n) != (ssize_t)n) { close(fd); return VErr("dupclosereverse write errno=%d", errno); }
		if (close(fd) != 0) return VErr("dupclosereverse close(fd) errno=%d", errno);
		struct stat st;
		if (stat(p, &st) != 0) return VErr("dupclosereverse stat errno=%d", errno);
		if (!S_ISREG(st.st_mode) || (size_t)st.st_size != n) return VErr("dupclosereverse %s mode=0%o size=%lld want=%zu", p, st.st_mode, (long long)st.st_size, n);
		return 0;
	}
	// dup refCount inverse: close orig first then write via dup then close dup; LAST close finalizes.
	if (VEq(arg,nL,"dupcloseinorder"))
	{
		char p[512], c[2048]; VSplit(v, p, 512, c, 2048);
		size_t total = strlen(c);
		if (total < 2) return VErr("dupcloseinorder content too short");
		size_t half = total / 2;
		int fd = open(p, O_WRONLY|O_CREAT|O_TRUNC, 0644);
		if (fd < 0) return VErr("dupcloseinorder open %s errno=%d", p, errno);
		int dfd = dup(fd);
		if (dfd < 0) { close(fd); return VErr("dupcloseinorder dup errno=%d", errno); }
		if (write(fd, c, half) != (ssize_t)half) { close(fd); close(dfd); return VErr("dupcloseinorder write(fd) errno=%d", errno); }
		if (close(fd) != 0) { close(dfd); return VErr("dupcloseinorder close(fd) errno=%d", errno); }
		if (write(dfd, c + half, total - half) != (ssize_t)(total - half)) { close(dfd); return VErr("dupcloseinorder write(dfd) errno=%d", errno); }
		if (close(dfd) != 0) return VErr("dupcloseinorder close(dfd) errno=%d", errno);
		struct stat st;
		if (stat(p, &st) != 0) return VErr("dupcloseinorder stat errno=%d", errno);
		if (!S_ISREG(st.st_mode) || (size_t)st.st_size != total) return VErr("dupcloseinorder %s mode=0%o size=%lld want=%zu", p, st.st_mode, (long long)st.st_size, total);
		return 0;
	}
	// fork: parent writes+closes before child exits; child's kernel-implicit close must not double-close.
	if (VEq(arg,nL,"forkparentclosesfirst"))
	{
		char p[512], c[2048]; VSplit(v, p, 512, c, 2048);
		int fd = open(p, O_WRONLY|O_CREAT|O_TRUNC, 0644);
		if (fd < 0) return VErr("forkparentclosesfirst open %s errno=%d", p, errno);
		pid_t pid = fork();
		if (pid < 0) { close(fd); return VErr("forkparentclosesfirst fork errno=%d", errno); }
		if (pid == 0) _exit(0); // child kernel-closes inherited fd; no detour close call
		size_t n = strlen(c);
		if (write(fd, c, n) != (ssize_t)n) { close(fd); return VErr("forkparentclosesfirst write errno=%d", errno); }
		if (close(fd) != 0) return VErr("forkparentclosesfirst close errno=%d", errno);
		int status = 0;
		while (waitpid(pid, &status, 0) == -1) if (errno != EINTR) return VErr("forkparentclosesfirst waitpid errno=%d", errno);
		if (!WIFEXITED(status) || WEXITSTATUS(status) != 0) return VErr("forkparentclosesfirst child status=%d", status);
		struct stat st;
		if (stat(p, &st) != 0) return VErr("forkparentclosesfirst stat errno=%d", errno);
		if (!S_ISREG(st.st_mode) || (size_t)st.st_size != n) return VErr("forkparentclosesfirst %s mode=0%o size=%lld want=%zu", p, st.st_mode, (long long)st.st_size, n);
		return 0;
	}
	// fork: child detour-closes first, parent closes after. Server must not log "unknown closeId" on second close.
	if (VEq(arg,nL,"forkchildclosesfirst"))
	{
		char p[512], c[2048]; VSplit(v, p, 512, c, 2048);
		int fd = open(p, O_WRONLY|O_CREAT|O_TRUNC, 0644);
		if (fd < 0) return VErr("forkchildclosesfirst open %s errno=%d", p, errno);
		pid_t pid = fork();
		if (pid < 0) { close(fd); return VErr("forkchildclosesfirst fork errno=%d", errno); }
		if (pid == 0) { if (close(fd) != 0) _exit(81); _exit(0); }
		size_t n = strlen(c);
		if (write(fd, c, n) != (ssize_t)n) { close(fd); return VErr("forkchildclosesfirst write errno=%d", errno); }
		if (close(fd) != 0) return VErr("forkchildclosesfirst close errno=%d", errno);
		int status = 0;
		while (waitpid(pid, &status, 0) == -1) if (errno != EINTR) return VErr("forkchildclosesfirst waitpid errno=%d", errno);
		if (!WIFEXITED(status) || WEXITSTATUS(status) != 0) return VErr("forkchildclosesfirst child status=%d", status);
		struct stat st;
		if (stat(p, &st) != 0) return VErr("forkchildclosesfirst stat errno=%d", errno);
		if (!S_ISREG(st.st_mode) || (size_t)st.st_size != n) return VErr("forkchildclosesfirst %s mode=0%o size=%lld want=%zu", p, st.st_mode, (long long)st.st_size, n);
		return 0;
	}
	// fork: child writes via inherited fd then _exit (no close); parent's later close must finalize.
	if (VEq(arg,nL,"forkchildwritesnoexitclose"))
	{
		char p[512], c[2048]; VSplit(v, p, 512, c, 2048);
		int fd = open(p, O_WRONLY|O_CREAT|O_TRUNC, 0644);
		if (fd < 0) return VErr("forkchildwritesnoexitclose open %s errno=%d", p, errno);
		pid_t pid = fork();
		if (pid < 0) { close(fd); return VErr("forkchildwritesnoexitclose fork errno=%d", errno); }
		size_t n = strlen(c);
		if (pid == 0)
		{
			if (write(fd, c, n) != (ssize_t)n) _exit(82);
			_exit(0);
		}
		int status = 0;
		while (waitpid(pid, &status, 0) == -1) if (errno != EINTR) return VErr("forkchildwritesnoexitclose waitpid errno=%d", errno);
		if (!WIFEXITED(status) || WEXITSTATUS(status) != 0) return VErr("forkchildwritesnoexitclose child status=%d", status);
		if (close(fd) != 0) return VErr("forkchildwritesnoexitclose parent close errno=%d", errno);
		struct stat st;
		if (stat(p, &st) != 0) return VErr("forkchildwritesnoexitclose stat errno=%d", errno);
		if (!S_ISREG(st.st_mode) || (size_t)st.st_size != n) return VErr("forkchildwritesnoexitclose %s mode=0%o size=%lld want=%zu", p, st.st_mode, (long long)st.st_size, n);
		return 0;
	}
	// Control case for forkredirectexec: child's shell EXPLICITLY closes fd 1 via `exec 1>&-` before _exit.
	if (VEq(arg,nL,"forkredirectexecclose"))
	{
		char p[512], c[2048]; VSplit(v, p, 512, c, 2048);
		pid_t pid = fork();
		if (pid < 0) return VErr("forkredirectexecclose fork errno=%d", errno);
		if (pid == 0)
		{
			int wfd = open(p, O_WRONLY|O_CREAT|O_TRUNC, 0644);
			if (wfd < 0) _exit(91);
			if (dup2(wfd, 1) != 1) _exit(92);
			if (close(wfd) != 0) _exit(93);
			char cmd[2200];
			snprintf(cmd, sizeof(cmd), "printf %%s %s; exec 1>&-", c);
			const char* prog = "/bin/sh";
			char* argv2[] = { (char*)prog, (char*)"-c", cmd, nullptr };
			execve(prog, argv2, environ);
			_exit(94);
		}
		int status = 0;
		while (waitpid(pid, &status, 0) == -1) if (errno != EINTR) return VErr("forkredirectexecclose waitpid errno=%d", errno);
		if (!WIFEXITED(status) || WEXITSTATUS(status) != 0) return VErr("forkredirectexecclose child status=%d", status);
		struct stat st;
		if (stat(p, &st) != 0) return VErr("forkredirectexecclose stat errno=%d", errno);
		size_t n = strlen(c);
		if (!S_ISREG(st.st_mode) || (size_t)st.st_size != n) return VErr("forkredirectexecclose %s mode=0%o size=%lld want=%zu", p, st.st_mode, (long long)st.st_size, n);
		return 0;
	}
	if (VEq(arg,nL,"spawn"))
	{
		char self[1024]; ssize_t sl = readlink("/proc/self/exe", self, sizeof(self)-1);
		if (sl <= 0) return VErr("spawn readlink self");
		self[sl] = 0;
		// split v on ';' -> argv tokens
		char buf[2048]; strncpy(buf, v, sizeof(buf)-1); buf[sizeof(buf)-1] = 0;
		const char* tok[64]; int tn = 1; tok[0] = self; tok[tn++] = buf;
		for (char* c = buf; *c; ++c) if (*c == ';') { *c = 0; if (tn < 63) tok[tn++] = c+1; }
		tok[tn] = nullptr;
		pid_t pid = 0;
		if (posix_spawn(&pid, self, nullptr, nullptr, (char**)tok, environ) != 0) return VErr("spawn posix_spawn");
		int status = 0;
		while (waitpid(pid, &status, 0) == -1) if (errno != EINTR) return VErr("spawn waitpid");
		if (!WIFEXITED(status)) return VErr("spawn child signalled");
		return WEXITSTATUS(status) == 0 ? 0 : VErr("spawn child ec=%d", WEXITSTATUS(status));
	}
	// Match uba::MaxPath on Linux/Mac. Same cap as UBA's path buffers — anything longer asserts in UBA.
	#define UBA_TEST_MAX_PATH 2048
	if (VEq(arg,nL,"symlink"))   { char t[UBA_TEST_MAX_PATH], l[UBA_TEST_MAX_PATH]; VSplit(v, t, UBA_TEST_MAX_PATH, l, UBA_TEST_MAX_PATH); return symlink(t, l) == 0 ? 0 : VErr("symlink %s,%s", t, l); }
	if (VEq(arg,nL,"symlinkrep"))
	{
		// linkname,N,pattern — symlink to (pattern repeated N times). Tests long readlink targets.
		char l[512], cs[16], pat[256];
		const char* c1 = strchr(v, ','); const char* c2 = c1 ? strchr(c1+1, ',') : nullptr;
		if (!c1 || !c2) return VErr("symlinkrep syntax %s", v);
		size_t lL = (size_t)(c1 - v); if (lL >= sizeof(l)) lL = sizeof(l)-1; memcpy(l, v, lL); l[lL] = 0;
		size_t nL2 = (size_t)(c2 - c1 - 1); if (nL2 >= sizeof(cs)) nL2 = sizeof(cs)-1; memcpy(cs, c1+1, nL2); cs[nL2] = 0;
		strncpy(pat, c2+1, sizeof(pat)-1); pat[sizeof(pat)-1] = 0;
		int n = atoi(cs); size_t pL = strlen(pat);
		if (n <= 0 || pL == 0 || (size_t)n * pL >= UBA_TEST_MAX_PATH - 32) return VErr("symlinkrep bad n/pat");
		char buf[UBA_TEST_MAX_PATH]; size_t off = 0;
		for (int i = 0; i < n; ++i) { memcpy(buf+off, pat, pL); off += pL; }
		buf[off] = 0;
		return symlink(buf, l) == 0 ? 0 : VErr("symlinkrep %s", l);
	}
	if (VEq(arg,nL,"readlinklen"))
	{
		// path,N — readlink and verify result is exactly N bytes (avoids embedding huge strings in verb).
		char p[512], cs[16]; VSplit(v, p, 512, cs, 16);
		int want = atoi(cs);
		char buf[UBA_TEST_MAX_PATH]; ssize_t r = readlink(p, buf, sizeof(buf)-1);
		if (r < 0) return VErr("readlinklen %s", p);
		return r == want ? 0 : VErr("readlinklen %s got=%zd want=%d", p, r, want);
	}
	if (VEq(arg,nL,"readlink"))
	{
		char p[UBA_TEST_MAX_PATH], expect[UBA_TEST_MAX_PATH]; VSplit(v, p, UBA_TEST_MAX_PATH, expect, UBA_TEST_MAX_PATH);
		char buf[UBA_TEST_MAX_PATH]; ssize_t r = readlink(p, buf, sizeof(buf)-1);
		if (r < 0) return VErr("readlink %s", p);
		buf[r] = 0;
		return strcmp(buf, expect) == 0 ? 0 : VErr("readlink %s got=%s want=%s", p, buf, expect);
	}
	// readlink MUST fail with the named errno. IsVfsParentDir synthesized dirs must return EINVAL for `readlink -f`.
	if (VEq(arg,nL,"readlinkerr"))
	{
		char p[UBA_TEST_MAX_PATH], en[16]; VSplit(v, p, UBA_TEST_MAX_PATH, en, 16);
		int want;
		if      (strcmp(en, "EINVAL") == 0) want = EINVAL;
		else if (strcmp(en, "ENOENT") == 0) want = ENOENT;
		else if (strcmp(en, "EACCES") == 0) want = EACCES;
		else if (strcmp(en, "ENOTDIR") == 0) want = ENOTDIR;
		else                                 want = atoi(en);
		char buf[UBA_TEST_MAX_PATH]; ssize_t r = readlink(p, buf, sizeof(buf)-1);
		if (r >= 0) return VErr("readlinkerr %s succeeded (expected fail errno=%d)", p, want);
		int got = errno;
		return got == want ? 0 : VErr("readlinkerr %s got errno=%d want=%d", p, got, want);
	}
	if (VEq(arg,nL,"lisdir")) { struct stat st; return (lstat(v, &st) == 0 && S_ISDIR(st.st_mode)) ? 0 : VErr("lisdir %s", v); }
	if (VEq(arg,nL,"chmod"))     { char p[512], m[16]; VSplit(v, p, 512, m, 16); return chmod(p, (mode_t)strtol(m, nullptr, 8)) == 0 ? 0 : VErr("chmod %s,%s", p, m); }
	if (VEq(arg,nL,"utime"))     { return utimensat(AT_FDCWD, v, nullptr, 0) == 0 ? 0 : VErr("utime %s", v); }
	if (VEq(arg,nL,"execcheck"))
	{
		// fork+execve and read child's errno. ENOEXEC=8 means exec bit reached disk; EACCES=13 means chmod did not.
		pid_t pid = fork();
		if (pid < 0) return VErr("execcheck fork %s", v);
		if (pid == 0) { char* a[] = { (char*)v, nullptr }; execve(v, a, environ); _exit(errno); }
		int st = 0; while (waitpid(pid, &st, 0) == -1) if (errno != EINTR) return VErr("execcheck waitpid");
		int ec = WIFEXITED(st) ? WEXITSTATUS(st) : -1;
		if (ec == 8) return 0;
		return VErr("execcheck %s ec=%d (expected ENOEXEC=8; EACCES=13 means chmod didn't reach disk)", v, ec);
	}
	if (VEq(arg,nL,"open"))
	{
		char p[512], m[16]; VSplit(v, p, 512, m, 16);
		int flags;
		if (m[0] == 'r') flags = O_RDONLY;
		else if (m[0] == 'w') flags = O_WRONLY | O_CREAT | O_TRUNC;
		else if (m[0] == 'a') flags = O_WRONLY | O_CREAT | O_APPEND;
		else if (m[0] == 'x') flags = O_WRONLY | O_CREAT | O_EXCL;
		else return VErr("open mode '%s'", m);
		int fd = open(p, flags, 0644);
		if (fd < 0) return VErr("open %s,%s", p, m);
		close(fd); return 0;
	}
	// llvm-objcopy reopen pattern: O_WRONLY (no CREAT/TRUNC) + lseek 0 + write. Exercises writtenFiles reuse.
	if (VEq(arg,nL,"reopenwrite"))
	{
		char p[512], c[2048]; VSplit(v, p, 512, c, 2048);
		int fd = open(p, O_WRONLY);
		if (fd < 0) return VErr("reopenwrite open %s", p);
		if (lseek(fd, 0, SEEK_SET) != 0) { close(fd); return VErr("reopenwrite lseek %s", p); }
		size_t n = strlen(c);
		ssize_t w = (n == 0) ? 0 : write(fd, c, n);
		close(fd);
		return (size_t)w == n ? 0 : VErr("reopenwrite %s", p);
	}
	if (VEq(arg,nL,"nfiles"))
	{
		char d[512], pre[64], cs[16];
		const char* c1 = strchr(v, ','); const char* c2 = c1 ? strchr(c1+1, ',') : nullptr;
		if (!c1 || !c2) return VErr("nfiles syntax %s", v);
		size_t dL = (size_t)(c1 - v); if (dL >= 512) dL = 511;
		memcpy(d, v, dL); d[dL] = 0;
		size_t pL = (size_t)(c2 - c1 - 1); if (pL >= 64) pL = 63;
		memcpy(pre, c1+1, pL); pre[pL] = 0;
		strncpy(cs, c2+1, 15); cs[15] = 0;
		int n = atoi(cs);
		for (int i = 0; i < n; ++i)
		{
			char f[768]; snprintf(f, 768, "%s/%s%d", d, pre, i);
			int fd = open(f, O_WRONLY|O_CREAT|O_TRUNC, 0644);
			if (fd < 0) return VErr("nfiles %s", f);
			close(fd);
		}
		return 0;
	}
#if PLATFORM_LINUX
	// Fork; child raises fatal SIG inside libUbaDetours. Parent's next RPC must survive without wire-format corruption.
	if (VEq(arg,nL,"forkthencrash"))
	{
		int sig = SIGBUS;
		if (strcmp(v, "SIGBUS") == 0)       sig = SIGBUS;
		else if (strcmp(v, "SIGSEGV") == 0) sig = SIGSEGV;
		else if (strcmp(v, "SIGABRT") == 0) sig = SIGABRT;
		else if (strcmp(v, "SIGILL") == 0)  sig = SIGILL;
		else if (strcmp(v, "SIGFPE") == 0)  sig = SIGFPE;
		else if (v[0] != 0) return VErr("forkthencrash unknown signal %s", v);
		pid_t pid = fork();
		if (pid < 0) return VErr("forkthencrash fork errno=%d", errno);
		if (pid == 0)
		{
			raise(sig);
			_exit(0);
		}
		int status = 0;
		while (waitpid(pid, &status, 0) == -1) if (errno != EINTR) return VErr("forkthencrash waitpid errno=%d", errno);
		if (!(WIFSIGNALED(status) || WIFEXITED(status))) return VErr("forkthencrash child status=%d (neither signalled nor exited)", status);
		return 0;
	}
	// execve with envp=NULL — bash `env -i` style. LD_PRELOAD wrapper must not deref NULL envp.
	if (VEq(arg,nL,"execvenull"))
	{
		pid_t pid = fork();
		if (pid < 0) return VErr("execvenull fork errno=%d", errno);
		if (pid == 0)
		{
			char* argv2[2] = { (char*)v, nullptr };
			execve(v, argv2, nullptr); // envp=NULL is the trigger
			_exit(0);
		}
		int status = 0;
		while (waitpid(pid, &status, 0) == -1) if (errno != EINTR) return VErr("execvenull waitpid errno=%d", errno);
		if (!WIFEXITED(status)) return VErr("execvenull child status=%d (not exited)", status);
		return 0;
	}
	// Register N synthetic VFS bind overlays. Synthetic paths ~100 chars each so the UBA_VFS_OVERLAYS encoding
	// buffer must handle nsjail-scale bind-mount trees (N=~80+ crosses an old 8 KiB encoding cap).
	if (VEq(arg,nL,"vfssynth"))
	{
		int n = atoi(v);
		if (n <= 0 || n > 200) return VErr("vfssynth N=%d out of range (1..200)", n);
		for (int i = 0; i < n; ++i)
		{
			char src[256], dst[256];
			snprintf(src, sizeof(src),
				"/uba_vfs_synth_src_padding_to_make_this_path_at_least_one_hundred_characters_long_xxxxxxxx/%05d", i);
			snprintf(dst, sizeof(dst),
				"/uba_vfs_synth_dst_padding_to_make_this_path_at_least_one_hundred_characters_long_yyyyyyyy/%05d", i);
			if (mount(src, dst, nullptr, MS_BIND, nullptr) != 0)
				return VErr("vfssynth mount #%d errno=%d", i, errno);
		}
		return 0;
	}
	if (VEq(arg,nL,"bindmount")) { char s[512],d[512]; VSplit(v,s,512,d,512); return mount(s, d, nullptr, MS_BIND, nullptr) == 0 ? 0 : VErr("bindmount %s,%s", s, d); }
	if (VEq(arg,nL,"umount"))    { return umount(v) == 0 ? 0 : VErr("umount %s", v); }
	if (VEq(arg,nL,"umount2"))   { char p[512],fs[16]; VSplit(v,p,512,fs,16); return umount2(p, atoi(fs)) == 0 ? 0 : VErr("umount2 %s,%s", p, fs); }
	if (VEq(arg,nL,"pivot"))     { char nr[512],po[512]; VSplit(v,nr,512,po,512); return pivot_root(nr, po) == 0 ? 0 : VErr("pivot %s,%s", nr, po); }
	if (VEq(arg,nL,"chrt"))      { return chroot(v) == 0 ? 0 : VErr("chrt %s", v); }
#endif
	// Helper: exit 0 iff path is NOT in UBA_INHERIT_HANDLES. Probe-opens exercise fd reuse to trip a regressed filter.
	if (VEq(arg,nL,"inherit_verify_absent"))
	{
		const char* env = getenv("UBA_INHERIT_HANDLES");
		if (env && strstr(env, v))
			return VErr("inherit_verify_absent: %s leaked to UBA_INHERIT_HANDLES=%s", v, env);
		int probes[16] = {0};
		char tmp[600];
		for (int i = 0; i < 16; ++i)
		{
			snprintf(tmp, sizeof(tmp), "%s_probe%02d", v, i);
			probes[i] = open(tmp, O_WRONLY|O_CREAT|O_TRUNC, 0644);
			if (probes[i] < 0) return VErr("inherit_verify_absent probe-open #%d errno=%d", i, errno);
		}
		for (int i = 0; i < 16; ++i) close(probes[i]);
		return 0;
	}

	// Helper: exit 0 iff path IS in UBA_INHERIT_HANDLES.
	if (VEq(arg,nL,"inherit_verify_present"))
	{
		const char* env = getenv("UBA_INHERIT_HANDLES");
		if (!env || !strstr(env, v))
			return VErr("inherit_verify_present: %s NOT in UBA_INHERIT_HANDLES=%s", v, env ? env : "<null>");
		return 0;
	}

	// Parent opens O_CLOEXEC, fork+execve self; child must NOT see the path in UBA_INHERIT_HANDLES.
	if (VEq(arg,nL,"cloexec_open_exec"))
	{
		int fd = open(v, O_WRONLY|O_CREAT|O_TRUNC|O_CLOEXEC, 0644);
		if (fd < 0) return VErr("cloexec_open_exec open %s errno=%d", v, errno);
		if (write(fd, "X", 1) != 1) { close(fd); return VErr("cloexec_open_exec write errno=%d", errno); }
		pid_t pid = fork();
		if (pid < 0) return VErr("cloexec_open_exec fork errno=%d", errno);
		if (pid == 0)
		{
			char vbuf[600]; snprintf(vbuf, sizeof(vbuf), "inherit_verify_absent=%s", v);
			char* argv2[] = { (char*)"/proc/self/exe", vbuf, nullptr };
			execve("/proc/self/exe", argv2, environ);
			_exit(99);
		}
		int status = 0;
		while (waitpid(pid, &status, 0) == -1) if (errno != EINTR) { close(fd); return VErr("cloexec_open_exec waitpid errno=%d", errno); }
		close(fd);
		if (!WIFEXITED(status)) return VErr("cloexec_open_exec child signalled status=0x%x", status);
		if (WEXITSTATUS(status) != 0) return VErr("cloexec_open_exec child ec=%d", WEXITSTATUS(status));
		return 0;
	}

	// Non-CLOEXEC write fd MUST still inherit (bash `exec 5>file` semantics).
	if (VEq(arg,nL,"noncloexec_open_exec"))
	{
		char p[512], c[2048]; VSplit(v, p, 512, c, 2048);
		int fd = open(p, O_WRONLY|O_CREAT|O_TRUNC, 0644);
		if (fd < 0) return VErr("noncloexec_open_exec open %s errno=%d", p, errno);
		size_t n = strlen(c);
		if (n && (size_t)write(fd, c, n) != n) { close(fd); return VErr("noncloexec_open_exec write errno=%d", errno); }
		pid_t pid = fork();
		if (pid < 0) return VErr("noncloexec_open_exec fork errno=%d", errno);
		if (pid == 0)
		{
			char vbuf[600]; snprintf(vbuf, sizeof(vbuf), "inherit_verify_present=%s", p);
			char* argv2[] = { (char*)"/proc/self/exe", vbuf, nullptr };
			execve("/proc/self/exe", argv2, environ);
			_exit(99);
		}
		int status = 0;
		while (waitpid(pid, &status, 0) == -1) if (errno != EINTR) { close(fd); return VErr("noncloexec_open_exec waitpid errno=%d", errno); }
		close(fd);
		if (!WIFEXITED(status) || WEXITSTATUS(status) != 0) return VErr("noncloexec_open_exec child ec=%d", WEXITSTATUS(status));
		return 0;
	}

	// Open without CLOEXEC then fcntl(F_SETFD, FD_CLOEXEC); fcntl wrapper must update hasCloexec on the tracked fd.
	if (VEq(arg,nL,"fcntl_set_cloexec_exec"))
	{
		int fd = open(v, O_WRONLY|O_CREAT|O_TRUNC, 0644);
		if (fd < 0) return VErr("fcntl_set_cloexec_exec open %s errno=%d", v, errno);
		if (write(fd, "Y", 1) != 1) { close(fd); return VErr("fcntl_set_cloexec_exec write errno=%d", errno); }
		int flags = fcntl(fd, F_GETFD);
		if (flags < 0) { close(fd); return VErr("fcntl_set_cloexec_exec F_GETFD errno=%d", errno); }
		if (fcntl(fd, F_SETFD, flags | FD_CLOEXEC) < 0) { close(fd); return VErr("fcntl_set_cloexec_exec F_SETFD errno=%d", errno); }
		pid_t pid = fork();
		if (pid < 0) return VErr("fcntl_set_cloexec_exec fork errno=%d", errno);
		if (pid == 0)
		{
			char vbuf[600]; snprintf(vbuf, sizeof(vbuf), "inherit_verify_absent=%s", v);
			char* argv2[] = { (char*)"/proc/self/exe", vbuf, nullptr };
			execve("/proc/self/exe", argv2, environ);
			_exit(99);
		}
		int status = 0;
		while (waitpid(pid, &status, 0) == -1) if (errno != EINTR) { close(fd); return VErr("fcntl_set_cloexec_exec waitpid errno=%d", errno); }
		close(fd);
		if (!WIFEXITED(status) || WEXITSTATUS(status) != 0) return VErr("fcntl_set_cloexec_exec child ec=%d", WEXITSTATUS(status));
		return 0;
	}

#if PLATFORM_LINUX
	// Open without CLOEXEC then dup3(fd, fd+10, O_CLOEXEC); per-fd CLOEXEC must be tracked separately on dup3.
	if (VEq(arg,nL,"dup3_cloexec_exec"))
	{
		int fd = open(v, O_WRONLY|O_CREAT|O_TRUNC, 0644);
		if (fd < 0) return VErr("dup3_cloexec_exec open %s errno=%d", v, errno);
		int dfd = fd + 10;
		int r = dup3(fd, dfd, O_CLOEXEC);
		if (r < 0) { close(fd); return VErr("dup3_cloexec_exec dup3 errno=%d", errno); }
		pid_t pid = fork();
		if (pid < 0) { close(fd); close(dfd); return VErr("dup3_cloexec_exec fork errno=%d", errno); }
		if (pid == 0)
		{
			char vbuf[600]; snprintf(vbuf, sizeof(vbuf), "inherit_verify_present=%s", v);
			char* argv2[] = { (char*)"/proc/self/exe", vbuf, nullptr };
			execve("/proc/self/exe", argv2, environ);
			_exit(99);
		}
		int status = 0;
		while (waitpid(pid, &status, 0) == -1) if (errno != EINTR) { close(fd); close(dfd); return VErr("dup3_cloexec_exec waitpid errno=%d", errno); }
		close(fd); close(dfd);
		if (!WIFEXITED(status) || WEXITSTATUS(status) != 0) return VErr("dup3_cloexec_exec child ec=%d", WEXITSTATUS(status));
		return 0;
	}
#endif
	if (VEq(arg,nL,"mustfail")) return VRunVerb(v) != 0 ? 0 : VErr("mustfail succeeded: %s", v); // Inverts inner result; inner's failure print is expected.
	return VErr("unknown verb %s", arg);
}

/******************** WARNING ********************/
// UbaTestAppPosix cannot be run standalone. It is extremely
// dependent on the UbaTest runner. Please see details in:
// UbaTestSession.h
int main(int argc, char* argv[])
{
	bool runningRemote = false;
	char* tmp = getenv("UBA_REMOTE");
	if (tmp && tmp[0] == '1')
		runningRemote = true;

	// UBA_TEST_STATIC=1 — harness launched the statically-patched binary; libUbaDetours.so isn't mapped.
	bool runningStatic = false;
	char* tmp2 = getenv("UBA_TEST_STATIC");
	if (tmp2 && tmp2[0] == '1')
		runningStatic = true;

	// Make the assumption that if we're running remote the detour lib will be there.
	bool foundDetoursLib = IsLibraryLoaded(UBA_DETOURS_LIBRARY) || runningRemote || runningStatic;


	if (!foundDetoursLib)
		return LogError("libUbaDetours not loaded. This app is designed to only start from inside UnrealBuildAccelerator.");

	if (argc > 1 && argv[1][0] != '-' && strchr(argv[1], '='))
	{
		for (int i = 1; i < argc; ++i)
		{
			int r = VRunVerb(argv[i]);
			if (r) return r;
		}
		return 0;
	}

	if (argc == 1)
	{
		char cwd[1024];
		if (!getcwd(cwd, sizeof(cwd)))
			return LogError("getcwd failed");

		struct stat attrR;
		if (stat("FileR.h", &attrR) == -1)
			return LogError("stat for FileR.h failed");
		if (S_ISREG(attrR.st_mode) == 0)
			return LogError("stat for FileR.h did not return normal file");

		int fdr = open("FileR.h", O_RDONLY);
		if (fdr == -1)
			return LogError("open FileR.h failed");
	
		char buf[4];
		if (read(fdr, buf, 4) != 4)
			return LogError("Failed to read FileR.h");

		if (strcmp(buf, "Foo") != 0)
			return LogError("FileR.h content was wrong");

		if (close(fdr) == -1)
			return LogError("close FileR.h failed");

		int fdw = open("FileW", O_WRONLY | O_CREAT | O_TRUNC, S_IRUSR | S_IWUSR);
		if (fdw == -1)
			return LogError("open FileW failed");

		if (write(fdw, "hello", 6) == -1)
			return LogError("write FileW failed");

		struct stat attrW1;
		if (fstat(fdw, &attrW1) == -1)
			return LogError("fstat FileW failed");

		if (close(fdw) == -1)
			return LogError("closed FileW failed");

		struct stat attrW2;
		if (stat("FileW", &attrW2) == -1)
			return LogError("stat for FileW failed");
		if (S_ISREG(attrW2.st_mode) == 0)
			return LogError("stat for FileW did not return normal file (0x%x)", attrW2.st_mode);

		if (rename("FileW", "FileW2") == -1)
			return LogError("rename for FileW to FileW2 failed");


		struct stat attrD1;
		if (stat("Dir1", &attrD1) == -1)
			return LogError("stat for Dir1 failed");
		if (S_ISREG(attrD1.st_mode) != 0)
			return LogError("stat for Dir1 did not return directory");
		
		if (mkdir("Dir2/Dir3", S_IRUSR | S_IWUSR) == 0)
			return LogError("mkdir for dir2 did not fail even though it exists");
		if (errno != EEXIST)
			return LogError("mkdir for dir2 did not return error that it exists");

		struct stat attrD2;
		if (stat("Dir2/Dir3/Dir4/Dir5", &attrD2) == -1)
			return LogError("stat for Dir2/Dir3 failed");

		struct stat attr3;
		if (stat("/usr", &attr3) == -1)
			return LogError("stat for /usr failed");

		FILE* f = fopen("FileWF", "w+");
		if (f == nullptr)
			return LogError("fopen FileWF failed");
		if (fwrite("Hello", 1, 6, f) != 6)
			return LogError("fwrite FileWF failed");
		if (fclose(f) != 0)
			return LogError("fclose FileWF failed");
		struct stat attrWF;
		if (stat("FileWF", &attrWF) == -1)
			return LogError("stat for FileW failed");

	#if !PLATFORM_LINUX // Farm linux machines do not have clang installed... need to revisit this
		char fullPath[PATH_MAX];
		if (realpath("/usr/bin/clang", fullPath) == nullptr)
			return LogError("realpath for 'clang' failed");
	#endif

		struct stat attrRoot;
		if (stat("/", &attrRoot) != 0)
			return LogError("stat for '/' failed");

		if (mkdir("FooDir", S_IRUSR | S_IWUSR) != 0)
			return LogError("mkdir 'FooDir' failed");

		struct stat attrFoo;
		if (stat("FooDir", &attrFoo) != 0)
			return LogError("stat for 'FooDir' failed");
		if (!S_ISDIR(attrFoo.st_mode))
			return LogError("stat for dir 'FooDir' returned wrong type");

		// Add file to FooDir
		// Try to rmdir while files are there
		// Delete file

		if (rmdir("FooDir") != 0)
			return LogError("rmdir 'FooDir' failed");

		if (stat("FooDir", &attrFoo) == 0)
			return LogError("stat for 'FooDir' failed to not find removed directory");

		auto dir = opendir(".");
		if (!dir)
			return LogError("opendir failed");
		while (true)
		{
			dirent* ent = readdir(dir);
			if (!ent)
				break;
		}
		closedir(dir);

		{
			char execPath[1024] = {}; // readlink doesn't null-terminate
			if (readlink("/proc/self/exe", execPath, sizeof(execPath) - 1) == -1)
				return LogError("readlink failed while getting executable path");

			auto Waitpid = [](pid_t childPid)
				{
					int status = 0;
					do
					{
						if (waitpid(childPid, &status, WUNTRACED | WCONTINUED) == -1)
							return LogError("waitpid on child process failed (pid %i)", childPid);
						if (WIFSIGNALED(status))
							return LogError("Child process killed by signal %d", WTERMSIG(status));
						if (WIFSTOPPED(status))
							return LogError("Child process stopped by signal %d", WSTOPSIG(status));
						if (WIFCONTINUED(status))
							return LogError("Child process continued");
					} while (!WIFEXITED(status));
					if (WEXITSTATUS(status) != 0)
						return LogError("Child process failed");
					return 0;
				};

			{
				const char* args[] { execPath, "-child", nullptr };
				pid_t childPid = 0;
				if (posix_spawn(&childPid, execPath, nullptr, nullptr, (char**)args, environ) != 0)
					return LogError("posix_spawn failed");
				if (Waitpid(childPid) == -1)
					return -1;
			}
			#if PLATFORM_LINUX
			{
				pid_t childPid = 0;
				childPid = fork();
				if (childPid == -1)
					return LogError("SYS_clone failed");
				if (childPid == 0)
				{
					const char* args[] = { execPath, "-child", nullptr };
					execve((char*)args[0], (char**)args, environ);  // NOTE: `environ` is implicitly declared	
					_exit(1);
				}
				else
				{
					if (Waitpid(childPid) == -1)
						return -1;
				}
			}
			{
				pid_t childPid = 0;
				childPid = syscall(SYS_clone, SIGCHLD, NULL);
				if (childPid == -1)
					return LogError("SYS_clone failed");
				if (childPid == 0)
				{
					const char* args[] = { execPath, "-child", nullptr };
					execve((char*)args[0], (char**)args, environ);  // NOTE: `environ` is implicitly declared	
					_exit(1);
				}
				else
				{
					if (Waitpid(childPid) == -1)
						return -1;
				}
			}
			#endif
		}

		return 0;
	}
	else if (strcmp(argv[1], "-child") == 0)
	{
		struct stat attr;
		if (stat("FileW2", &attr) != 0)
			return LogError("stat for 'FileW' in child process failed");
		if (stat("FileW", &attr) != -1)
			return LogError("stat for 'FileW' in child process failed");
		return 0;
	}
	else if (strncmp(argv[1], "-getFileAttributes=", 6) == 0)
	{
		const char* str = argv[1] + 19;
		struct stat attr;
		if (stat(str, &attr) == -1)
			return 255;
		return 1;
	}
	else if (strcmp(argv[1], "-popen") == 0)
	{
		FILE* file = popen("xdg-user-dir DOCUMENTS", "r");
		if (!file)
			return -3;
		char docPath[256];
		if (!fgets(docPath, 256, file))
			return -4;
		auto docLen = strlen(docPath) - 1;
		if (docLen <= 0)
			return -5;
		if (docPath[docLen] != '\n')
			return -6;
		pclose(file);
		return 0;
	}
	else if (strncmp(argv[1], "-file=", 6) == 0)
	{
		void* detoursHandle = dlopen(UBA_DETOURS_LIBRARY, RTLD_LAZY);
		if (!detoursHandle)
			return -3;
		using UbaRequestNextProcessFunc = bool(unsigned int prevExitCode, char* outArguments, unsigned int outArgumentsCapacity);
		static UbaRequestNextProcessFunc* requestNextProcess = (UbaRequestNextProcessFunc*)(void*)dlsym(detoursHandle, "UbaRequestNextProcess");
		if (!requestNextProcess)
			return -8;

		char arguments[1024];
		const char* file = argv[1] + 6;
		while (true)
		{
			int rh = open(file, O_RDONLY);
			if (rh == -1)
				return LogError("Failed to open file %s", file);
			if (close(rh) == -1)
				return LogError("Failed to close file %s", file);

			srand(getpid());
			int milliseconds = rand() % 2000;
			struct timespec ts;
			ts.tv_sec = milliseconds / 1000;
			ts.tv_nsec = (milliseconds % 1000) * 1000000;
			nanosleep(&ts, NULL);

			char outFile[1024];
			strcpy(outFile, file);
			outFile[strlen(file)-3] = 0;
			strcat(outFile, ".out");

			int wh = open(outFile, O_WRONLY | O_CREAT | O_TRUNC, S_IRUSR | S_IWUSR);
			if (wh == -1)
				return LogError("Failed to create file %s", file);
			if (close(wh) == -1)
				return LogError("Failed to close created file %s", file);

			// Request new process
			if (!requestNextProcess(0, arguments, 1024))
				break; // No process available, exit loop
			file = arguments + 6;
		}

		return 0;
	}
	else if (strncmp(argv[1], "-virtualFile=", 13) == 0)
	{
		const char* file = argv[1]+13;
		int rh = open(file, O_RDONLY);
		if (rh == -1)
			return LogError("Failed to open file %s", file);
		char data[4] = {};
		if (read(rh, data, 3) != 3)
			return LogError("Failed to read 3 bytes from file %s", file);
		close(rh);
		if (memcmp(data, "FOO", 3) != 0)
			return LogError("File %s has wrong content", file);
	}
	else if (strcmp(argv[1], "-createSymlink") == 0)
	{
		int wh = open("empty_file", O_WRONLY | O_CREAT | O_TRUNC, S_IRUSR | S_IWUSR);
		CHECK_TRUE(wh != -1);
		CHECK_TRUE(close(wh) != -1);
		CHECK_TRUE(symlink("empty_file", "new_file") != -1);
		return 0;
	}
	#if PLATFORM_MAC
	else if (strcmp(argv[1], "-xcode-select") == 0)
	{
		int pipefd[2];
		if (pipe(pipefd) != 0)
			return LogError("Failed to create pipe");

		posix_spawn_file_actions_t actions;
		posix_spawn_file_actions_init(&actions);

		// Close read end in child, redirect stdout to write end
		posix_spawn_file_actions_addclose(&actions, pipefd[0]);
		posix_spawn_file_actions_adddup2(&actions, pipefd[1], STDOUT_FILENO);
		posix_spawn_file_actions_addclose(&actions, pipefd[1]);

		pid_t childPid = 0;
		const char* execPath = "/usr/bin/xcode-select";
		const char* args[] { execPath, "--print-path", nullptr };
		if (posix_spawn(&childPid, execPath, &actions, nullptr, (char**)args, environ) != 0)
			return LogError("posix_spawn failed");
		
		posix_spawn_file_actions_destroy(&actions);
		close(pipefd[1]);  // Close write end in parent

		bool success = false;

		char buffer[256];
		int count;
		while ((count = read(pipefd[0], buffer, sizeof(buffer)-1)) > 0)
		{
			buffer[count] = '\0';
			if (strstr(buffer, "Xcode.app"))
				success = true;
		}

		int status = 0;
		do
		{
			if (waitpid(childPid, &status, WUNTRACED | WCONTINUED) == -1)
				return LogError("waitpid on child process failed (pid %i)", childPid);
			if (WIFSIGNALED(status))
				return LogError("Child process killed by signal %d", WTERMSIG(status));
			if (WIFSTOPPED(status))
				return LogError("Child process stopped by signal %d", WSTOPSIG(status));
			if (WIFCONTINUED(status))
				return LogError("Child process continued");
		} while (!WIFEXITED(status));
		if (WEXITSTATUS(status) != 0)
			return LogError("Child process failed");

		return success ? 0 : -1;
	}
	#endif

	return -2;
}
