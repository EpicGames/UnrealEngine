// Copyright Epic Games, Inc. All Rights Reserved.

#define WINDOWS_LEAN_AND_MEAN
#include <Windows.h>
#include <wchar.h> 
#include <stdarg.h>
#include <time.h>

int LogError(const wchar_t* format, ...)
{
	va_list args;
	va_start(args, format);
	wchar_t buffer[1024];
	_vsnwprintf_s(buffer, 1024, _TRUNCATE, format, args);
	wprintf(L"%s\n", buffer);
	va_end(args);
	return -1;
}

int ReadTestFile(void* outData, int capacity, const wchar_t* fileName)
{
	HANDLE fh = CreateFileW(fileName, GENERIC_READ, FILE_SHARE_READ, NULL, OPEN_EXISTING, 0, NULL);
	if (fh == INVALID_HANDLE_VALUE)
		return LogError(L"Failed to open %s for read", fileName);
	DWORD bytesRead;
	if (!ReadFile(fh, outData, capacity, &bytesRead, NULL))
		return LogError(L"Failed to read from file %s", fileName);
	CloseHandle(fh);
	return bytesRead;
}

int WriteTestFile(const void* data, int size, const wchar_t* fileName)
{
	HANDLE fh = CreateFileW(fileName, GENERIC_WRITE, FILE_SHARE_WRITE, NULL, CREATE_ALWAYS, 0, NULL);
	if (fh == INVALID_HANDLE_VALUE)
		return LogError(L"Failed to open %s for write", fileName);
	DWORD bytesWritten;
	if (!WriteFile(fh, data, size, &bytesWritten, NULL) || size != int(bytesWritten))
		return LogError(L"Failed to write %i bytes to file %s", size, fileName);
	CloseHandle(fh);
	return 0;
}

// Verb dispatcher — see UbaTestSession.cpp / TestVerb* for the testing protocol.
// Each verb is one argv entry "name=arg" or "name=a,b". Verbs self-verify; nonzero exit = fail.
static bool VEq(const wchar_t* a, int n, const wchar_t* w) { int x=(int)wcslen(w); return n==x && wcsncmp(a,w,x)==0; }
static int VErr(const wchar_t* fmt, ...) { va_list a; va_start(a,fmt); wchar_t b[1024]; _vsnwprintf_s(b,1024,_TRUNCATE,fmt,a); va_end(a); wprintf(L"verb fail: %s (gle=%u)\n", b, GetLastError()); return 1; }
static void VSplit(const wchar_t* v, wchar_t* a, size_t aCap, wchar_t* b, size_t bCap, wchar_t sep=L',')
{
	const wchar_t* s = wcschr(v, sep);
	size_t aL = s ? (size_t)(s - v) : wcslen(v);
	if (aL >= aCap) aL = aCap - 1;
	memcpy(a, v, aL*sizeof(wchar_t)); a[aL] = 0;
	if (s) wcsncpy_s(b, bCap, s+1, _TRUNCATE); else b[0] = 0;
}
static bool VAttrOK(DWORD a, DWORD must, DWORD mustnot=0) { return a != INVALID_FILE_ATTRIBUTES && (a & must) == must && (a & mustnot) == 0; }
static int VRunVerb(const wchar_t* arg)
{
	const wchar_t* eq = wcschr(arg, L'='); int nL = eq?(int)(eq-arg):(int)wcslen(arg);
	const wchar_t* v = eq?eq+1:L"";
	if (VEq(arg,nL,L"mkdir"))    return CreateDirectoryW(v, NULL) ? 0 : VErr(L"mkdir %s", v);
	if (VEq(arg,nL,L"rmdir"))    return RemoveDirectoryW(v) ? 0 : VErr(L"rmdir %s", v);
	if (VEq(arg,nL,L"unlink"))   return DeleteFileW(v) ? 0 : VErr(L"unlink %s", v);
	if (VEq(arg,nL,L"touch"))
	{
		HANDLE h = CreateFileW(v, GENERIC_WRITE, FILE_SHARE_READ|FILE_SHARE_WRITE, NULL, CREATE_ALWAYS, 0, NULL);
		if (h == INVALID_HANDLE_VALUE) return VErr(L"touch %s", v);
		CloseHandle(h); return 0;
	}
	if (VEq(arg,nL,L"write") || VEq(arg,nL,L"append"))
	{
		bool app = arg[0]==L'a';
		wchar_t p[512], c[2048]; VSplit(v, p, 512, c, 2048);
		HANDLE h = CreateFileW(p, GENERIC_WRITE, FILE_SHARE_READ|FILE_SHARE_WRITE, NULL, app?OPEN_ALWAYS:CREATE_ALWAYS, 0, NULL);
		if (h == INVALID_HANDLE_VALUE) return VErr(L"write %s", p);
		if (app) SetFilePointer(h, 0, NULL, FILE_END);
		DWORD len = (DWORD)wcslen(c); char buf[2048]; for (DWORD i=0;i<len;++i) buf[i]=(char)c[i];
		DWORD wn=0; BOOL ok = (len==0) || (WriteFile(h, buf, len, &wn, NULL) && wn==len);
		CloseHandle(h); return ok ? 0 : VErr(L"write %s", p);
	}
	if (VEq(arg,nL,L"read"))
	{
		wchar_t p[512], expect[2048]; VSplit(v, p, 512, expect, 2048);
		HANDLE h = CreateFileW(p, GENERIC_READ, FILE_SHARE_READ|FILE_SHARE_WRITE, NULL, OPEN_EXISTING, 0, NULL);
		if (h == INVALID_HANDLE_VALUE) return VErr(L"read open %s", p);
		char buf[2048]; DWORD got=0; BOOL ok = ReadFile(h, buf, sizeof(buf), &got, NULL); CloseHandle(h);
		if (!ok) return VErr(L"read %s", p);
		DWORD eL = (DWORD)wcslen(expect);
		if (got != eL) return VErr(L"read %s len=%u want=%u", p, got, eL);
		for (DWORD i=0;i<eL;++i) if (buf[i] != (char)expect[i]) return VErr(L"read %s mismatch@%u", p, i);
		return 0;
	}
	if (VEq(arg,nL,L"size"))
	{
		wchar_t p[512], n[64]; VSplit(v, p, 512, n, 64);
		WIN32_FILE_ATTRIBUTE_DATA d;
		if (!GetFileAttributesExW(p, GetFileExInfoStandard, &d)) return VErr(L"size %s", p);
		LARGE_INTEGER li; li.LowPart = d.nFileSizeLow; li.HighPart = d.nFileSizeHigh;
		long long want = _wtoi64(n);
		return li.QuadPart == want ? 0 : VErr(L"size %s got=%lld want=%lld", p, li.QuadPart, want);
	}
	if (VEq(arg,nL,L"stat"))      return GetFileAttributesW(v) != INVALID_FILE_ATTRIBUTES ? 0 : VErr(L"stat %s", v);
	if (VEq(arg,nL,L"notexists")) return GetFileAttributesW(v) == INVALID_FILE_ATTRIBUTES ? 0 : VErr(L"notexists %s", v);
	if (VEq(arg,nL,L"isdir"))     return VAttrOK(GetFileAttributesW(v), FILE_ATTRIBUTE_DIRECTORY) ? 0 : VErr(L"isdir %s", v);
	if (VEq(arg,nL,L"isfile"))    { DWORD a = GetFileAttributesW(v); return (a != INVALID_FILE_ATTRIBUTES && !(a & FILE_ATTRIBUTE_DIRECTORY)) ? 0 : VErr(L"isfile %s", v); }
	if (VEq(arg,nL,L"islink"))    return VAttrOK(GetFileAttributesW(v), FILE_ATTRIBUTE_REPARSE_POINT) ? 0 : VErr(L"islink %s", v);
	if (VEq(arg,nL,L"rename"))    { wchar_t s[512],d[512]; VSplit(v,s,512,d,512); return MoveFileW(s,d) ? 0 : VErr(L"rename %s,%s", s, d); }
	if (VEq(arg,nL,L"copy"))      { wchar_t s[512],d[512]; VSplit(v,s,512,d,512); return CopyFileW(s,d,FALSE) ? 0 : VErr(L"copy %s,%s", s, d); }
	if (VEq(arg,nL,L"chdir"))     return SetCurrentDirectoryW(v) ? 0 : VErr(L"chdir %s", v);
	if (VEq(arg,nL,L"cwd"))
	{
		wchar_t cur[1024]; if (!GetCurrentDirectoryW(1024, cur)) return VErr(L"cwd get");
		return _wcsicmp(cur, v) == 0 ? 0 : VErr(L"cwd got=%s want=%s", cur, v);
	}
	if (VEq(arg,nL,L"listdir"))
	{
		wchar_t p[512], cs[64]; VSplit(v, p, 512, cs, 64);
		int want = _wtoi(cs);
		wchar_t pat[1024]; _snwprintf_s(pat, 1024, _TRUNCATE, L"%s\\*", p);
		WIN32_FIND_DATAW fd; HANDLE h = FindFirstFileW(pat, &fd);
		if (h == INVALID_HANDLE_VALUE) return VErr(L"listdir %s", p);
		int n = 0;
		do { if (wcscmp(fd.cFileName, L".") && wcscmp(fd.cFileName, L"..")) ++n; } while (FindNextFileW(h, &fd));
		FindClose(h);
		return n == want ? 0 : VErr(L"listdir %s got=%d want=%d", p, n, want);
	}
	if (VEq(arg,nL,L"listcontains") || VEq(arg,nL,L"listexcludes"))
	{
		bool wantPresent = arg[4] == L'c';
		wchar_t p[512], name[256]; VSplit(v, p, 512, name, 256);
		wchar_t pat[1024]; _snwprintf_s(pat, 1024, _TRUNCATE, L"%s\\*", p);
		WIN32_FIND_DATAW fd; HANDLE h = FindFirstFileW(pat, &fd);
		if (h == INVALID_HANDLE_VALUE) return VErr(L"list %s", p);
		bool found = false;
		do { if (_wcsicmp(fd.cFileName, name) == 0) { found = true; break; } } while (FindNextFileW(h, &fd));
		FindClose(h);
		return (found == wantPresent) ? 0 : VErr(L"list %s want %s of %s", p, wantPresent?L"contain":L"exclude", name);
	}
	if (VEq(arg,nL,L"sleep")) { Sleep((DWORD)_wtoi(v)); return 0; }
	if (VEq(arg,nL,L"spawn"))
	{
		wchar_t self[1024]; GetModuleFileNameW(NULL, self, 1024);
		wchar_t cmd[2048]; _snwprintf_s(cmd, 2048, _TRUNCATE, L"\"%s\" ", self);
		size_t at = wcslen(cmd); wcscpy_s(cmd+at, 2048-at, v);
		for (wchar_t* p = cmd+at; *p; ++p) if (*p == L';') *p = L' ';
		STARTUPINFOW si{}; si.cb = sizeof(si); PROCESS_INFORMATION pi{};
		if (!CreateProcessW(NULL, cmd, NULL, NULL, TRUE, 0, NULL, NULL, &si, &pi)) return VErr(L"spawn %s", cmd);
		WaitForSingleObject(pi.hProcess, INFINITE);
		DWORD ec = 1; GetExitCodeProcess(pi.hProcess, &ec);
		CloseHandle(pi.hThread); CloseHandle(pi.hProcess);
		return ec == 0 ? 0 : VErr(L"spawn child ec=%u (%s)", ec, v);
	}
	if (VEq(arg,nL,L"symlink"))   { wchar_t t[512],l[512]; VSplit(v,t,512,l,512); return CreateSymbolicLinkW(l,t,0) ? 0 : VErr(L"symlink %s,%s", t, l); }
	if (VEq(arg,nL,L"open"))
	{
		wchar_t p[512], m[16]; VSplit(v, p, 512, m, 16);
		DWORD acc = 0, disp = 0;
		if (m[0] == L'r') { acc = GENERIC_READ;  disp = OPEN_EXISTING; }
		else if (m[0] == L'w') { acc = GENERIC_WRITE; disp = CREATE_ALWAYS; }
		else if (m[0] == L'a') { acc = GENERIC_WRITE; disp = OPEN_ALWAYS; }
		else if (m[0] == L'x') { acc = GENERIC_WRITE; disp = CREATE_NEW; }
		else return VErr(L"open mode '%s'", m);
		HANDLE h = CreateFileW(p, acc, FILE_SHARE_READ|FILE_SHARE_WRITE, NULL, disp, 0, NULL);
		if (h == INVALID_HANDLE_VALUE) return VErr(L"open %s,%s", p, m);
		CloseHandle(h); return 0;
	}
	if (VEq(arg,nL,L"nfiles"))
	{
		wchar_t d[512], pre[64], cs[16]; const wchar_t* c1 = wcschr(v, L','); const wchar_t* c2 = c1 ? wcschr(c1+1, L',') : nullptr;
		if (!c1 || !c2) return VErr(L"nfiles syntax %s", v);
		size_t dL = (size_t)(c1 - v); if (dL >= 512) dL = 511;
		memcpy(d, v, dL*sizeof(wchar_t)); d[dL] = 0;
		size_t pL = (size_t)(c2 - c1 - 1); if (pL >= 64) pL = 63;
		memcpy(pre, c1+1, pL*sizeof(wchar_t)); pre[pL] = 0;
		wcsncpy_s(cs, 16, c2+1, _TRUNCATE);
		int n = _wtoi(cs);
		for (int i = 0; i < n; ++i)
		{
			wchar_t f[768]; _snwprintf_s(f, 768, _TRUNCATE, L"%s\\%s%d", d, pre, i);
			HANDLE h = CreateFileW(f, GENERIC_WRITE, FILE_SHARE_READ|FILE_SHARE_WRITE, NULL, CREATE_ALWAYS, 0, NULL);
			if (h == INVALID_HANDLE_VALUE) return VErr(L"nfiles %s", f);
			CloseHandle(h);
		}
		return 0;
	}
	if (VEq(arg,nL,L"mustfail")) return VRunVerb(v) != 0 ? 0 : VErr(L"mustfail succeeded: %s", v); // Inverts inner result; inner's failure print is expected.
	return VErr(L"unknown verb %s", arg);
}

int wmain(int argc, wchar_t* argv[])
{
	HMODULE detoursHandle = GetModuleHandleW(L"UbaDetours.dll");

	using UbaRequestNextProcessFunc = bool(unsigned int prevExitCode, wchar_t* outArguments, unsigned int outArgumentsCapacity);
	static UbaRequestNextProcessFunc* requestNextProcess = (UbaRequestNextProcessFunc*)(void*)GetProcAddress(detoursHandle, "UbaRequestNextProcess");

	if (argc == 1)
	{
		bool runningRemote = false;

		if (detoursHandle)
		{
			using UbaRunningRemoteFunc = bool();
			UbaRunningRemoteFunc* runningRemoteFunc = (UbaRunningRemoteFunc*)GetProcAddress(detoursHandle, "UbaRunningRemote");
			if (!runningRemoteFunc)
				return LogError(L"Couldn't find UbaRunningRemote function in UbaDetours.dll");
			runningRemote = (*runningRemoteFunc)();
		}

		HMODULE modules[] = { 0, detoursHandle, GetModuleHandleW(L"UbaTestApp.exe") };
		for (HMODULE module : modules)
		{
			DWORD res1 = GetModuleFileNameW(module, NULL, 0);
			if (GetLastError() != ERROR_INSUFFICIENT_BUFFER)
				return LogError(L"Expected insufficient buffer");
			if (res1 != 0)
				return LogError(L"Expected zero");
			wchar_t name[512];
			memset(name, 123, sizeof(name));
			DWORD realLen = GetModuleFileNameW(module, name, 512);
			if (realLen == 0)
				return LogError(L"Did not expect this function to fail");
			if (GetLastError() == ERROR_INSUFFICIENT_BUFFER)
				return LogError(L"Expected sufficient buffer");
			memset(name, 123, sizeof(name));
			name[realLen] = 254;
			name[realLen+1] = 254;
			DWORD res2 = GetModuleFileNameW(module, name, realLen);
			if (GetLastError() != ERROR_INSUFFICIENT_BUFFER)
				return LogError(L"Expected insufficient buffer");
			if (res2 != realLen)
				return LogError(L"Expected to return same as sent in");
			if (name[realLen] != 254)
				return LogError(L"Overwrite");
			if (name[realLen-1] != 0)
				return LogError(L"Insufficient buffer not terminated: %s", name);
			memset(name, 123, sizeof(name));
			name[realLen] = 254;
			name[realLen + 1] = 254;
			DWORD res3 = GetModuleFileNameW(module, name, realLen+1);
			if (GetLastError() == ERROR_INSUFFICIENT_BUFFER)
				return LogError(L"Expected sufficient buffer");
			if (res3 != realLen)
				return LogError(L"Expected to return same as sent in");
			if (name[realLen+1] != 254)
				return LogError(L"Overwrite");
			if (name[realLen] != 0)
				return LogError(L"Not terminated: %s", name);
		}

		wchar_t currentDir[MAX_PATH];
		DWORD currentDirLen = GetCurrentDirectoryW(MAX_PATH, currentDir);
		if (!currentDirLen)
			return LogError(L"GetCurrentDirectoryW failed");
		currentDir[currentDirLen] = '\\';
		currentDir[currentDirLen + 1] = 0;

		wchar_t notepad[] = L"c:\\windows\\system32\\notepad.exe";
		wchar_t localNotepad[MAX_PATH];
		wcscpy_s(localNotepad, MAX_PATH, currentDir);
		wcscat_s(localNotepad, MAX_PATH, L"notepad.exe");

		if (!CopyFileW(notepad, localNotepad, false))
			return LogError(L"CopyFileW failed");

		{
			HANDLE fh = CreateFileW(localNotepad, GENERIC_READ, FILE_SHARE_READ, NULL, OPEN_EXISTING, 0, NULL);
			if (fh == INVALID_HANDLE_VALUE)
				return LogError(L"Failed to open %s for read", localNotepad);
			wchar_t path[MAX_PATH];
			DWORD res = GetFinalPathNameByHandleW(fh, path, MAX_PATH, 0);
			if (!res)
				return LogError(L"GetFinalPathNameByHandleW failed");
			if (res != wcslen(path))
				return LogError(L"GetFinalPathNameByHandleW did not return length of string");
			DWORD res2 = GetFinalPathNameByHandleW(fh, path, res, 0);
			if (res2 != res + 1)
				return LogError(L"GetFinalPathNameByHandleW should return full length plus terminating character");
			DWORD res3 = GetFinalPathNameByHandleW(fh, path, res+1, 0);
			if (res3 != res)
				return LogError(L"GetFinalPathNameByHandleW should return full length plus terminating character");
			// TODO: Test character after terminator char

			if (!runningRemote)
				GetFinalPathNameByHandleW(fh, path, MAX_PATH, VOLUME_NAME_NT); // Testing so it doesn't assert

			CloseHandle(fh);
		}

		{
			wchar_t testPath[] = L"R:.";
			wchar_t fullPathName[MAX_PATH];
			DWORD len = GetFullPathNameW(testPath, MAX_PATH, fullPathName, NULL);
			if (len != 3)
				return LogError(L"GetFullPathNameW failed");
			testPath[0] = currentDir[0];
			DWORD len2 = GetFullPathNameW(testPath, MAX_PATH, fullPathName, NULL);
			if (len2 != currentDirLen)
				return LogError(L"GetFullPathNameW returns length that does not match current dir");
			if (memcmp(fullPathName, currentDir, len*sizeof(wchar_t)) != 0)
				return LogError(L"GetFullPathNameW returned wrong path");
			// TODO: Test character after terminator char
		}

		{
			HANDLE fh = CreateFileW(L"FileW", GENERIC_WRITE, FILE_SHARE_WRITE, NULL, CREATE_ALWAYS, 0, NULL);
			if (fh == INVALID_HANDLE_VALUE)
				return LogError(L"Failed to create file File");
			CloseHandle(fh);
			if (!MoveFile(L"FileW", L"FileW2"))
				return LogError(L"Failed to move file from FileW to FileW2");

			if (!CopyFile(L"FileW2", L"FileWF", false))
				return LogError(L"Failed to copy file from FileW2 to FileWF");
			if (DeleteFile(L"FileW"))
				return LogError(L"Should fail to delete FileW");
		}

		{
			wchar_t fileW[512];
			GetTempPath(512, fileW);
			wcscat_s(fileW, 1024, L"FileW");

			if (GetFileAttributesW(L"fileW") != INVALID_FILE_ATTRIBUTES)
				return LogError(L"FileW Should not exist in temp yet");

			wchar_t tempPath[512];
			GetTempPath(512, tempPath);
			if (CreateDirectory(tempPath, NULL))
				return LogError(L"Should not be able to create temp path");
			if (GetLastError() != ERROR_ALREADY_EXISTS)
				return LogError(L"Temp path should already exist");



			wchar_t fileW2[512];
			GetTempPath(512, fileW2);
			wcscat_s(fileW2, 1024, L"FileW2");

			wchar_t fileWF[512];
			GetTempPath(512, fileWF);
			wcscat_s(fileWF, 1024, L"FileWF");

			HANDLE fh = CreateFileW(fileW, GENERIC_WRITE, FILE_SHARE_WRITE, NULL, CREATE_ALWAYS, 0, NULL);
			if (fh == INVALID_HANDLE_VALUE)
				return LogError(L"Failed to create file File");
			CloseHandle(fh);
			if (!MoveFile(fileW, fileW2))
				return LogError(L"Failed to move file from FileW to FileW2");

			if (!CopyFile(fileW2, fileWF, false))
				return LogError(L"Failed to copy file from FileW2 to FileWF");
			if (!DeleteFile(fileW2))
				return LogError(L"Failed to delete FileW2");
			if (!DeleteFile(fileWF))
				return LogError(L"Failed to delete FileWF");
			if (DeleteFile(fileW))
				return LogError(L"Should fail to delete FileW");
		}

		{
#ifdef _DEBUG
			if (CreateDirectoryW(L"c:\\", NULL))
				return LogError(L"Should fail to create root dir");
			if (GetLastError() != ERROR_ALREADY_EXISTS)
				return LogError(L"Expected ERROR_ALREADY_EXISTS for root dir");
			if (CreateDirectoryW(L"a:\\", NULL))
				return LogError(L"Should fail to create root dir");
#endif

			if (!CreateDirectoryW(L"DirA", NULL))
				return LogError(L"Failed to create directory");
			if (CreateDirectoryW(L"DirA", NULL))
				return LogError(L"Should not succeed creating directory that exists");

			if (GetFileAttributesW(L"DirA") == 0)
				return LogError(L"Failed to get attributes of directory DirA");

			if (WriteTestFile("", 0, L"DirA\\File"))
				return -1;

			if (RemoveDirectoryW(L"DirA"))
				return LogError(L"Should not succeed in removing directory with files in it");
			if (GetLastError() != ERROR_DIR_NOT_EMPTY)
				return LogError(L"Wrong last error code");
			if (!DeleteFileW(L"DirA\\File"))
				return LogError(L"Failed to delete file");

			if (!RemoveDirectoryW(L"DirA"))
				return LogError(L"Failed to remove directory");

			if (GetFileAttributesW(L"DirA") != INVALID_FILE_ATTRIBUTES)
				return LogError(L"Found attributes of deleted directory");

			if (CreateDirectoryW(L"Dir2\\Dir3", NULL))
				return LogError(L"Should not succeed creating directory inside directory that does not exist");
			if (detoursHandle)
			{
				if (GetLastError() != ERROR_ALREADY_EXISTS)
					return LogError(L"Did not get correct error when failing to create existing directory: %u", GetLastError());
				if (GetFileAttributesW(L"Dir2\\Dir3\\Dir4\\Dir5") == INVALID_FILE_ATTRIBUTES)
					return LogError(L"Failed to get attributes of directory Dir2\\Dir3\\Dir4\\Dir5");
			}
		}


		if (detoursHandle)
		{
			{
				HANDLE fh = CreateFileW(L"File4.out", GENERIC_WRITE, FILE_SHARE_READ, NULL, OPEN_EXISTING, 0, NULL);
				if (fh == INVALID_HANDLE_VALUE)
					return LogError(L"Failed to open File4.out for read with write permissions");
				DWORD bytesRead;
				wchar_t data[1];
				if (!ReadFile(fh, data, sizeof(data), &bytesRead, NULL) || bytesRead != sizeof(data) || data[0] != '0')
					return LogError(L"Failed to read one byte from File4.out");
				CloseHandle(fh);
			}

			auto createForRead = [](const wchar_t* fileName, wchar_t expectedData)
				{
					HANDLE fh = CreateFileW(fileName, GENERIC_READ, FILE_SHARE_READ, NULL, OPEN_EXISTING, 0, NULL);
					if (fh == INVALID_HANDLE_VALUE)
						return LogError(L"Failed to open %s for read with write permissions", fileName);
					DWORD bytesRead;
					wchar_t data[1];
					if (!ReadFile(fh, data, sizeof(data), &bytesRead, NULL) || bytesRead != sizeof(data) || data[0] != expectedData)
						return LogError(L"Failed to read one byte from %s", fileName);
					CloseHandle(fh);
					return 0;
				};

			if (createForRead(L"File4.obj", '9'))
				return -1;
			if (createForRead(L"File4.obj", '9'))
				return -1;
			if (createForRead(L"File4.lib", '9'))
				return -1;
		}
		
		if (WriteTestFile(L"Foo", 8, L"MemoryFile.out"))
			return -1;

		{
			STARTUPINFOW si;
			memset(&si, 0, sizeof(si));
			PROCESS_INFORMATION pi;
			memset(&pi, 0, sizeof(pi));
			wchar_t arg[1024];
			wcscpy_s(arg, 1024, argv[0]);
			wcscat_s(arg, 1024, L" -child");
			if (!CreateProcessW(nullptr, arg, nullptr, nullptr, TRUE, 0, nullptr, nullptr, &si, &pi))
				return LogError(L"Failed to create child process");
			CloseHandle(pi.hThread);
			
			if (WaitForSingleObject(pi.hProcess, 10000) != WAIT_OBJECT_0)
				return LogError(L"Failed waiting for child process");

			DWORD exitCode;
			if (!GetExitCodeProcess(pi.hProcess, &exitCode) || exitCode)
				return LogError(L"Child process failed");
			CloseHandle(pi.hProcess);
		}
		return 0;
	}

	for (int i = 1; i != argc; ++i)
	{
		wchar_t* arg = argv[i];

		if (arg[0] != L'-' && wcschr(arg, L'='))
		{
			int r = VRunVerb(arg);
			if (r) return r;
			continue;
		}

		if (wcscmp(arg, L"-child") == 0)
		{
			if (GetFileAttributes(L"FileW2") == INVALID_FILE_ATTRIBUTES)
				return LogError(L"Child process could not get attributes of FileW2");
			if (GetFileAttributes(L"FileWF") == INVALID_FILE_ATTRIBUTES)
				return LogError(L"Child process could not get attributes of FileWF");
			if (GetFileAttributes(L"FileW") != INVALID_FILE_ATTRIBUTES)
				return LogError(L"Child process found FileW which should not exist anymore");
		}
		else if (wcscmp(arg, L"-reuse") == 0)
		{
			if (!detoursHandle)
				return LogError(L"Did not find UbaDetours.dll in process!!!\n");

			wchar_t arguments[1024];
			if (requestNextProcess(0, arguments, sizeof(arguments)))
				return LogError(L"Didn't expect another process");
			while (true)
			{
				if (requestNextProcess(0, arguments, sizeof(arguments)))
					break;
				wchar_t temp[512];
				GetTempPath(512, temp);
				if (GetFileAttributesW(temp) == INVALID_FILE_ATTRIBUTES)
					return LogError(L"Temp should exist");
			}
		}
		else if (wcsncmp(arg, L"-file=", 6) == 0)
		{
			wchar_t arguments[1024];
			const wchar_t* file = arg + 6;
			while (true)
			{
				HANDLE rh = CreateFileW(file, GENERIC_READ, FILE_SHARE_READ, NULL, OPEN_EXISTING, 0, NULL);
				if (rh == INVALID_HANDLE_VALUE)
					return LogError(L"Failed to open file %s", file);
				char data[17] = {};
				DWORD bytesRead;
				if (!ReadFile(rh, data, 16, &bytesRead, NULL) || bytesRead != 16)
					return LogError(L"Failed to read 16 bytes from file %s", file);
				CloseHandle(rh);

				srand(GetProcessId(GetCurrentProcess()));
				Sleep(rand() % 2000);
				wchar_t outFile[1024];
				wcscpy_s(outFile, 1024, file);
				outFile[wcslen(file)-3] = 0;
				wcscat_s(outFile, 1024, L".out");

				HANDLE wh = CreateFileW(outFile, GENERIC_WRITE, FILE_SHARE_WRITE, NULL, CREATE_ALWAYS, 0, NULL);
				if (wh == INVALID_HANDLE_VALUE)
					return LogError(L"Failed to create file File");
				data[16] = 1;
				DWORD bytesWritten;
				if (!WriteFile(wh, data, 17, &bytesWritten, NULL) || bytesWritten != 17)
					return LogError(L"Failed to read 16 bytes from file %s", file);

				CloseHandle(wh);

				// Request new process
				if (!requestNextProcess(0, arguments, 1024))
					break; // No process available, exit loop
				file = arguments + 6;
			}

			return 0;
		}
		else if (wcsncmp(arg, L"-getFileAttributes=", 6) == 0)
		{
			const wchar_t* str = arg + 19;
			DWORD attr = GetFileAttributesW(str);
			return attr == INVALID_FILE_ATTRIBUTES ? 255 : attr;
		}
		else if (wcscmp(arg, L"-createTempPath") == 0)
		{
			wchar_t tempPath[512];
			GetTempPath(512, tempPath);
			if (CreateDirectory(tempPath, NULL))
				return 0;
			return GetLastError();
		}
		else if (wcsncmp(arg, L"-stdout=", 8) == 0)
		{
			const wchar_t* str = arg + 8;
			if (wcscmp(str, L"rootprocess") == 0)
			{
				STARTUPINFOW si;
				memset(&si, 0, sizeof(si));
				PROCESS_INFORMATION pi;
				memset(&pi, 0, sizeof(pi));
				wchar_t childArg[1024];
				wcscpy_s(childArg, 1024, argv[0]);
				wcscat_s(childArg, 1024, L" -stdout=childprocess");
				//wcscpy_s(childArg, 1024, L"\"c:\\sdk\\AutoSDK/HostWin64/Win64/MetalDeveloperTools/4.1/metal/macos/bin/metal.exe\" -v --target=air64-apple-darwin18.7.0 16384");

				SECURITY_ATTRIBUTES saAttr;
				saAttr.nLength = sizeof(SECURITY_ATTRIBUTES);
				saAttr.bInheritHandle = TRUE;
				saAttr.lpSecurityDescriptor = NULL;
				HANDLE readPipe;
				HANDLE writePipe;
				if (!CreatePipe(&readPipe, &writePipe, &saAttr, 0))
					return 1;

				if (!SetHandleInformation(readPipe, HANDLE_FLAG_INHERIT, 0))
					return 2;

				si.dwFlags = STARTF_USESTDHANDLES;
				si.hStdInput = GetStdHandle(STD_INPUT_HANDLE);
				si.hStdOutput = writePipe;
				si.hStdError = writePipe;

				DWORD flags = 0;//CREATE_NO_WINDOW;
				if (!CreateProcessW(nullptr, childArg, nullptr, nullptr, TRUE, flags, nullptr, nullptr, &si, &pi))
					return 3;
				CloseHandle(pi.hThread);
				CloseHandle(writePipe);

				char buf[4096] = { 0 };
				DWORD readCount = 0;
				if (!::ReadFile(readPipe, buf, sizeof(buf) - 1, &readCount, NULL))
				{
					LogError(L"Failed to read pipe %u %u", GetLastError(), readCount);
					return 4;
				}
				buf[readCount] = 0;
				if (strncmp(buf, "childprocess", 12) != 0)
					return 5;

				if (WaitForSingleObject(pi.hProcess, INFINITE) != WAIT_OBJECT_0)
					return 6;
				CloseHandle(pi.hProcess);
			}
			wprintf(L"%s\n", str);
		}
		else if (wcsncmp(arg, L"-virtualFile", 12) == 0)
		{
			const wchar_t* file = L"VirtualFile.in";
			HANDLE rh = CreateFileW(file, GENERIC_READ, FILE_SHARE_READ, NULL, OPEN_EXISTING, 0, NULL);
			if (rh == INVALID_HANDLE_VALUE)
				return LogError(L"Failed to open file %s", file);
			char data[4] = {};
			DWORD bytesRead;
			if (!ReadFile(rh, data, 4, &bytesRead, NULL) || bytesRead != 3)
				return LogError(L"Failed to read 3 bytes from file %s", file);
			CloseHandle(rh);
			if (memcmp(data, "FOO", 3) != 0)
				return LogError(L"File %s has wrong content", file);

			file = L"VirtualFile.out";
			HANDLE wh = CreateFileW(file, GENERIC_WRITE, FILE_SHARE_WRITE, NULL, CREATE_ALWAYS, 0, NULL);
			if (wh == INVALID_HANDLE_VALUE)
				return LogError(L"Failed to open file %s for write", file);
			if (!WriteFile(wh, "BAR", 3, NULL, NULL))
				return LogError(L"Failed to write 3 bytes to file %s", file);
			CloseHandle(wh);
		}
		else if (wcsncmp(arg, L"-read=", 6) == 0)
		{
			int callIndex = _wtoi(arg + 6);

			wchar_t buf[32];
			int read = ReadTestFile(buf, 32*2, L"SpecialFile1");
			if (read != 2)
				return LogError(L"Expected to read two bytes but got %i", read);
			buf[1] = 0;

			int readIndex = _wtoi(buf);
			if (callIndex != readIndex)
				return LogError(L"ReadTestFile callIndex (%i) different from readIndex (%i)", callIndex, readIndex);
			return 0;
		}
		else if (wcsncmp(arg, L"-write=", 7) == 0)
		{
			int callIndex = _wtoi(arg + 7);
			if (callIndex > 9)
				return LogError(L"CallIndex too large (%i)", callIndex);
			wchar_t buf[32];
			buf[0] = wchar_t('1' + callIndex);
			return WriteTestFile(buf, 2, L"SpecialFile1");
		}
		else if (wcsncmp(arg, L"-readwrite=", 11) == 0)
		{
			int callIndex = _wtoi(arg + 11);
			if (callIndex > 9)
				return LogError(L"CallIndex too large (%i)", callIndex);

			wchar_t buf[32];
			int read = ReadTestFile(buf, 32*2, L"SpecialFile1");
			if (read != 2)
				return LogError(L"Expected to read two bytes but got %i", read);
			buf[1] = 0;

			int readIndex = _wtoi(buf);
			if (callIndex != readIndex)
				return LogError(L"ReadWriteTestFile callIndex (%i) different from readIndex (%i)", callIndex, readIndex);

			buf[0] = wchar_t('1' + readIndex);
			return WriteTestFile(buf, 2, L"SpecialFile1");
		}
		else if (wcscmp(arg, L"-testRegisterVirtualFile") == 0)
		{
			wchar_t data[3];
			if (ReadTestFile(data, 6, L"First.txt") == -1)
				return -1;
			if (memcmp(data, L"Foo", 6) != 0)
				return -1;
			if (ReadTestFile(data, 6, L"Second.txt") == -1)
				return -1;
			if (memcmp(data, L"Bar", 6) != 0)
				return -1;
		}
		else if (wcsncmp(arg, L"-writeFileMap=", 14) == 0)
		{
			wchar_t* toWrite = arg + 14;
			if (WriteTestFile(toWrite, (wcslen(toWrite) + 1)*sizeof(wchar_t), L"File.h") == -1)
				return -1;
		}
		else if (wcsncmp(arg, L"-readFileMap=", 13) == 0)
		{
			wchar_t* toExpect = arg + 13;
			wchar_t buf[32];
			if (ReadTestFile(buf, sizeof(buf), L"File.h") == -1)
				return -1;
			if (wcscmp(toExpect, buf) != 0)
				return LogError(L"-readFileMap did not read what was expected (read %s, expected %s)", buf, toExpect);
		}
		else if (wcscmp(arg, L"-readWriteFileMap") == 0)
		{
			wchar_t buf[32];
			if (ReadTestFile(buf, 1, L"File.h") == -1)
				return -1;
			buf[0] = 'M';
			if (WriteTestFile(buf, 1, L"File.h") == -1)
				return -1;
			buf[0] = 0;
			if (ReadTestFile(buf, 1, L"File.h") == -1)
				return -1;
			if (buf[0] != 'M')
				return -1;
		}
		else if (wcsncmp(arg, L"-createFileEmptyFile=", 21) == 0)
		{
			wchar_t* fileName = arg + 22;
			if (WriteTestFile("", 0, fileName) == -1)
				return -1;
		}
		else
		{
			if (!detoursHandle)
				return LogError(L"Did not find UbaDetours.dll in process!!!\n");

			using u32 = unsigned int;
			using UbaSendCustomMessageFunc = u32(const void* send, u32 sendSize, void* recv, u32 recvCapacity);

			UbaSendCustomMessageFunc* sendMessage = (UbaSendCustomMessageFunc*)GetProcAddress(detoursHandle, "UbaSendCustomMessage");
			if (!sendMessage)
				return LogError(L"Couldn't find UbaSendCustomMessage function in UbaDetours.dll");

			const wchar_t* helloMsg = L"Hello from client";
			wchar_t response[256];
			u32 responseSize = (*sendMessage)(helloMsg, u32(wcslen(helloMsg)) * 2, response, 256 * 2);
			if (responseSize == 0)
				return LogError(L"Didn't get proper response from session");
			//wprintf(L"Recv: %.*s\n", responseSize / 2, response);
		}
	}
	return 0;
}
