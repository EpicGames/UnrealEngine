// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaTestApp : ModuleRules
{
	public UbaTestApp(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		
		bRequiresPlatformSDK = true;

		PrivateDefinitions.AddRange(new string[] {
			"_CONSOLE",
		});

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Linux))
		{
			PublicSystemLibraries.Add("dl");
		}
	}
}
