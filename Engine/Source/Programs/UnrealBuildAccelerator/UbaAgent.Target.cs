// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaAgentTarget : TargetRules
{
	const string UbaVersion = "Uba_v1.0.0";

	public static void CommonUbaSettings(TargetRules Rules, TargetInfo Target, bool ShouldExport = false)
	{
		Rules.Type = TargetType.Program;
		Rules.IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		Rules.LinkType = TargetLinkType.Monolithic;
		Rules.bDeterministic = true;
		Rules.bWarningsAsErrors = true;

		Rules.UndecoratedConfiguration = UnrealTargetConfiguration.Shipping;
		Rules.bHasExports = false;

		Rules.bRetainFramePointers = true;

		// Lean and mean
		Rules.bBuildDeveloperTools = false;

		// Editor-only is enabled for desktop platforms to run unit tests that depend on editor-only data
		// It's disabled in test and shipping configs to make profiling similar to the game
		Rules.bBuildWithEditorOnlyData = false;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		Rules.bCompileAgainstEngine = false;
		Rules.bCompileAgainstCoreUObject = false;
		Rules.bCompileAgainstApplicationCore = false;
		Rules.bCompileICU = false;

		Rules.GlobalDefinitions.Add("UBA_BUILD=1");

		// to build with automation tests:
		// bForceCompileDevelopmentAutomationTests = true;

		// to enable tracing:
		// bEnableTrace = true;

		// This app is a console application, not a Windows app (sets entry point to main(), instead of WinMain())
		Rules.bIsBuildingConsoleApplication = true;

		Rules.WindowsPlatform.TargetWindowsVersion = 0x0A00;
		Rules.WindowsPlatform.TargetWindowsMinorVersion = 0x0A000005;
		Rules.bUseStaticCRT = true;

		Rules.SolutionDirectory = "Programs/UnrealBuildAccelerator";

		var BinariesFolder = Path.Combine("Binaries", Target.Platform.ToString(), "UnrealBuildAccelerator");

		string BinaryName = null;
		bool usingSanitizer = false;

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			Rules.AdditionalLinkerArguments += " /NODEFAULTLIB:kernel32.lib";
			Rules.AdditionalLinkerArguments += " mincore.lib";

			string BinaryExt = Rules.bShouldCompileAsDLL ? ".dll" : ".exe";
			BinaryName = $"{Rules.LaunchModuleName}{BinaryExt}";
			Rules.OutputFile = Path.Combine(BinariesFolder, Target.Architectures.SingleArchitecture.bIsX64 ? "x64" : "arm64", BinaryName);

			Rules.GlobalDefinitions.AddRange(new string[] {
				"UBA_DETOURS_LIBRARY=L\"UbaDetours.dll\"",
				"UBA_AGENT_EXECUTABLE=L\"UbaAgent.exe\"",
				"UBA_DETOURS_LIBRARY_ANSI=\"UbaDetours.dll\"",
				"UBA_DETOURS_LIBRARY_LINUX=\"libUbaDetours.so\"",
			});
			usingSanitizer = Rules.WindowsPlatform.bEnableAddressSanitizer;
		}
		else
		{
			Rules.GlobalDefinitions.Add("PLATFORM_WINDOWS=0");
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Linux))
		{
			string BinaryPrefix = Rules.bShouldCompileAsDLL ? "lib" : string.Empty;
			string BinaryExt = Rules.bShouldCompileAsDLL ? ".so" : string.Empty;
			BinaryName = $"{BinaryPrefix}{Rules.LaunchModuleName}{BinaryExt}";
			Rules.OutputFile = Path.Combine(BinariesFolder, BinaryName);

			Rules.GlobalDefinitions.AddRange(new string[] {
				"UBA_DETOURS_LIBRARY=\"libUbaDetours.so\"",
				"UBA_AGENT_EXECUTABLE=\"UbaAgent\"",
				"__NO_INLINE__",
			});

			if (!Rules.bShouldCompileAsDLL)
			{
				// There is a hack in UBT that adds '-rdynamic' and no option to disable it.. by adding this at the end we override it
				// Reason is because we don't want to leak operator new/delete to loaded dlls that link them weak
				Rules.AdditionalLinkerArguments += " -Wl,--no-export-dynamic";
			}
			usingSanitizer = Rules.LinuxPlatform.bEnableAddressSanitizer || Rules.LinuxPlatform.bEnableThreadSanitizer;
		}
		else
		{
			Rules.GlobalDefinitions.Add("PLATFORM_LINUX=0");
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Apple))
		{
			if (!Rules.Architectures.Contains(UnrealArch.X64))
			{
				Rules.Architectures.Architectures.Add(UnrealArch.X64);
			}
			if (!Rules.Architectures.Contains(UnrealArch.Arm64))
			{
				Rules.Architectures.Architectures.Add(UnrealArch.Arm64);
			}
			if (Rules.LaunchModuleName != "UbaVisualizer") // Revisit this. UbaVisualizer fails packaging if name is changed
			{
				string BinaryPrefix = Rules.bShouldCompileAsDLL ? "lib" : string.Empty;
				string BinaryExt = Rules.bShouldCompileAsDLL ? ".dylib" : string.Empty;
				BinaryName = $"{BinaryPrefix}{Rules.LaunchModuleName}{BinaryExt}";
				Rules.OutputFile = Path.Combine(BinariesFolder, BinaryName);
			}
			else
			{
				Rules.ExeBinariesSubFolder = "UnrealBuildAccelerator";
				//Rules.OutputFile = Path.Combine(BinariesFolder, "UbaVisualizer");
			}
			Rules.GlobalDefinitions.AddRange(new string[] {
				"UBA_DETOURS_LIBRARY=\"libUbaDetours.dylib\"",
				"UBA_AGENT_EXECUTABLE=\"UbaAgent\"",
				"UBA_DETOURS_LIBRARY_LINUX=\"libUbaDetours.so\"",
			});
		}
		else
		{
			Rules.GlobalDefinitions.Add("PLATFORM_MAC=0");
		}

		Rules.GlobalDefinitions.Add($"UBA_BINARY=\"{BinaryName}\"");

		int useMiMalloc = 0;
		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) || (!Rules.bShouldCompileAsDLL && Target.Platform.IsInGroup(UnrealPlatformGroup.Linux)))
		{
			if (Rules.bUsePCHFiles && !usingSanitizer)
			{
				useMiMalloc = 1;
			}
		}
		Rules.GlobalDefinitions.Add($"UBA_USE_MIMALLOC={useMiMalloc}");


		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			if (ShouldExport)
			{
				Rules.GlobalDefinitions.Add("UBA_API=__declspec(dllexport)");
			}
		}
		else
		{
			if (ShouldExport)
			{
				Rules.GlobalDefinitions.Add("UBA_API=__attribute__ ((visibility(\"default\")))");
			}
		}

		if (!ShouldExport)
		{
			Rules.GlobalDefinitions.Add("UBA_API=");
		}

		if (Target.Configuration == UnrealTargetConfiguration.Debug)
		{
			Rules.GlobalDefinitions.Add("UBA_DEBUG=1");
		// 	Rules.bDebugBuildsActuallyUseDebugCRT = true;
		}
		else
		{
			Rules.GlobalDefinitions.Add("UBA_DEBUG=0");
		}

		if (Target.Configuration == UnrealTargetConfiguration.Shipping)
		{
			Rules.bAllowLTCG = true;
		}

		Rules.WindowsPlatform.bSetResourceVersions = true;
		Rules.BuildVersion = $"{UbaVersion}-{Rules.Version.Changelist}";

		Rules.CppCompileWarningSettings.CastFunctionTypeMismatchWarningLevel = WarningLevel.Off;
	}

	public UbaAgentTarget(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "UbaAgent";
		CommonUbaSettings(this, Target);
	}
}
