// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaCacheServiceTarget : TargetRules
{
	public UbaCacheServiceTarget(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "UbaCacheService";
		UbaAgentTarget.CommonUbaSettings(this, Target);
	}
}
