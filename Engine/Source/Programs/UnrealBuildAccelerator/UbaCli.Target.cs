// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaCliTarget : TargetRules
{
	public UbaCliTarget(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "UbaCli";
		UbaAgentTarget.CommonUbaSettings(this, Target);
	}
}
