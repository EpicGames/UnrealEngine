// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaCommonTarget : TargetRules
{
	public UbaCommonTarget(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "UbaCommon";
		bShouldCompileAsDLL = true;
		UbaAgentTarget.CommonUbaSettings(this, Target);
	}
}
