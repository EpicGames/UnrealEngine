// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaCoordinatorHordeTarget : TargetRules
{
	public UbaCoordinatorHordeTarget(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "UbaCoordinatorHorde";
		bShouldCompileAsDLL = true;

		Type = TargetType.Program;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LinkType = TargetLinkType.Monolithic;
		bDeterministic = true;
		bWarningsAsErrors = true;

		UndecoratedConfiguration = UnrealTargetConfiguration.Shipping;
		bHasExports = false;

		// Lean and mean
		bBuildDeveloperTools = false;

		// Editor-only is enabled for desktop platforms to run unit tests that depend on editor-only data
		// It's disabled in test and shipping configs to make profiling similar to the game
		bBuildWithEditorOnlyData = false;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;
		//bCompileAgainstApplicationCore = false;
		bCompileICU = false;

		bIsBuildingConsoleApplication = true;

		WindowsPlatform.TargetWindowsVersion = 0x0A00;
		//bUseStaticCRT = true;

		SolutionDirectory = "Programs/UnrealBuildAccelerator";

		var BinariesFolder = Path.Combine("Binaries", Target.Platform.ToString(), "UnrealBuildAccelerator");

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			OutputFile = Path.Combine(BinariesFolder, Target.Architectures.SingleArchitecture.bIsX64 ? "x64" : "arm64", $"{LaunchModuleName}{".dll"}");
		}
		else
		{
			GlobalDefinitions.Add("PLATFORM_WINDOWS=0");
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Apple))
		{
			OutputFile = Path.Combine(BinariesFolder, $"lib{LaunchModuleName}.dylib");
		}
		else
		{
			GlobalDefinitions.Add("PLATFORM_MAC=0");
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Linux))
		{
			OutputFile = Path.Combine(BinariesFolder, $"lib{LaunchModuleName}.so");
		}
		else
		{
			GlobalDefinitions.Add("PLATFORM_LINUX=0");
		}

		GlobalDefinitions.Add("UE_EXTERNAL_PROFILING_ENABLED=0");
		GlobalDefinitions.Add("STATS=0");
		GlobalDefinitions.Add("ENABLE_STATNAMEDEVENTS=0");
		GlobalDefinitions.Add("ALLOW_HITCH_DETECTION=0");
		GlobalDefinitions.Add("UE_USE_MALLOC_FILL_BYTES=0");
		GlobalDefinitions.Add("DISABLE_CWD_CHANGES=1");
		GlobalDefinitions.Add("UE_NO_ENGINE_OIDC=0");

		GlobalDefinitions.Add("UBA_COORDINATOR_HORDE_DLL");
	}
}
