// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaCoreTarget : TargetRules
{
	public UbaCoreTarget(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "UbaCore";
		bShouldCompileAsDLL = true;
		UbaAgentTarget.CommonUbaSettings(this, Target);
		bDisableLinking = true;
	}
}
