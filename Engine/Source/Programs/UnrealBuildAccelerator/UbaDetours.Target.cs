// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaDetoursTarget : TargetRules
{
	public UbaDetoursTarget(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "UbaDetours";
		bShouldCompileAsDLL = true;
		UbaAgentTarget.CommonUbaSettings(this, Target);
		GlobalDefinitions.Add("_CRT_SECURE_CPP_OVERLOAD_SECURE_NAMES=0");

		// For some reason xcode 26.2.0+ produce a bad dylib that fails to load when ltcg is enabled
		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Apple))
		{
			bAllowLTCG = false;
		}

		// Force EAGER (non-lazy) PLT binding on POSIX. With lazy binding, the
		// dynamic linker (ld.so) acquires an internal mutex on the first call
		// to each imported symbol. Inside our SIGSEGV/SIGBUS/SIGILL handler
		// (uba::segfault_sigaction in UbaPlatform.cpp), calls to libc helpers
		// (snprintf, write, etc.) would trip lazy binding the first time —
		// and if the signal arrived while the main thread was inside a
		// dlopen/dlsym/another lazy-bind, the handler deadlocks or crashes
		// trying to re-enter the same ld.so mutex. Observed symptom: rsync /
		// aapt2 / sh / clang++ cores all dying with SIGBUS at libUbaDetours.so
		// PLT entries (e.g. __errno_location@plt), no /tmp/uba_crash log
		// produced. With -z now, every PLT slot is resolved at load time,
		// so signal handlers can call any imported function without
		// touching ld.so. POSIX-only — Windows resolves imports differently.
		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Linux))
		{
			AdditionalLinkerArguments = "-Wl,-z,now";
		}
	}
}
