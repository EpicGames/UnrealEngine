

[Network]
MessageMaxSize = 262144  # 256*1024

[NetworkServer]
WorkerMaxCount = 64
ReceiveTimeoutSeconds = 0

[SessionServer]
# RootDir = "D:\UbaHost"
# DisableCustomAllocator = false
AllowMemoryMaps = true
ShouldWriteToDisk = true
TraceEnabled = true
DetailedTrace = true
StoreObjFilesCompressed = true
MemWaitLoadPercent = 85
MemKillLoadPercent = 95
ResetCas = false
RemoteExecutionEnabled = true
NameToHashTableEnabled = true
CheckMemory = true
AllowWaitOnMem = false
AllowKillOnMem = false
RemoteLogEnabled = false
RemoteTraceEnabled = false


[CacheClient]
ReportMissReason = false
UseDirectoryPreparsing = true
ValidateCacheWritesInput = false
ValidateCacheWritesOutput = true