// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaHostTarget : TargetRules
{
	public UbaHostTarget(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "UbaHost";
		bShouldCompileAsDLL = true;
		UbaAgentTarget.CommonUbaSettings(this, Target, true);
	}
}
