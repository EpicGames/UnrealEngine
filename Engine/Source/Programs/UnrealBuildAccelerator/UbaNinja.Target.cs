// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaNinjaTarget : TargetRules
{
	public UbaNinjaTarget(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "UbaNinja";
		UbaAgentTarget.CommonUbaSettings(this, Target);
	}
}
