// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaTestTarget : TargetRules
{
	public UbaTestTarget(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "UbaTest";
		UbaAgentTarget.CommonUbaSettings(this, Target);

		GlobalDefinitions.Add("UBA_TEST_TARGET=1");

		GlobalDefinitions.Add($"UBA_BUILD_MACHINE={(AdditionalProperties.GetProperty("IsBuildMachine") == "1" ? "1" : "0")}");
	}
}
