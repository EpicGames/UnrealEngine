// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaTestAppTarget : TargetRules
{
	public UbaTestAppTarget(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "UbaTestApp";
		UbaAgentTarget.CommonUbaSettings(this, Target);
	}
}
