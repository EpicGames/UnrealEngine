// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UbaVisualizerTarget : TargetRules
{
	public UbaVisualizerTarget(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "UbaVisualizer";
		UbaAgentTarget.CommonUbaSettings(this, Target);
		bIsBuildingConsoleApplication = false;
		WindowsPlatform.ApplicationIcon = "Programs/UnrealBuildAccelerator/Visualizer/Resources/Windows/UbaVisualizer.ico";
	}
}
