// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaVersion.h"

namespace uba
{
	const tchar* GetVersionString()
	{
		return UBA_VERSION;
	}
}