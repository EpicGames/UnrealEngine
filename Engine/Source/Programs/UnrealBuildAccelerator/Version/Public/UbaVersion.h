// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace uba
{
	#if PLATFORM_WINDOWS
	using tchar = wchar_t;
	#else
	using tchar = char;
	#endif

	const tchar* GetVersionString();
}