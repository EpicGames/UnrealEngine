// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaVersion : ModuleRules
{
	public UbaVersion(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		PCHUsage = PCHUsageMode.NoPCHs;

		string StringPrefix = Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) ? "L" : "";
		PrivateDefinitions.Add($"UBA_VERSION={StringPrefix}\"{Target.Version.MajorVersion}.{Target.Version.MinorVersion}.{Target.Version.PatchVersion}-{Target.BuildVersion}\"");
	}
}
