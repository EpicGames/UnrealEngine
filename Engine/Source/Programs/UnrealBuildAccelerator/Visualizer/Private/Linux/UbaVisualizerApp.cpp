// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaTraceChannel.h"
#include "UbaTraceReader.h"

int main(int argc, char* argv[])
{
	using namespace uba;

	LoggerWithWriter logger(g_consoleLogWriter);

	logger.Info(TC("UbaVisualizer (test implementation)"));

	TraceChannel channel(logger);
	if (!channel.Init())
		return -1;

	StringBuffer<> oldTrace;

	TraceView view;
	TraceReader reader(logger);

	while (true)
	{
		StringBuffer<> newTrace;
		if (!channel.Read(newTrace))
			return -1;
		if (oldTrace.Equals(newTrace))
		{
			Sleep(100);
			continue;
		}
		logger.Info(TC("New trace %s"), newTrace.data);
		oldTrace = newTrace;

		if (!newTrace.count)
			continue;
		
		if (!reader.StartReadNamed(view, newTrace.data))
			logger.Info(TC("StartReadNamed failed"));
		
		while (true)
		{
			//if (!view.sessions.empty())
			//	printf("%s\n", view.sessions[0].name.c_str());
			bool outChanged = false;
			if (!reader.UpdateReadNamed(view, ~0ull, outChanged))
				break;
			//if (outChanged)
			//	logger.Info(TC("Changed"));
			if (view.finished)
				break;
		}

	}

	return 0;
}