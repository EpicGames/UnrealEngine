// Copyright Epic Games, Inc. All Rights Reserved.

#import <Cocoa/Cocoa.h>

#include "UbaFile.h"
#include "UbaVisualizer.h"

namespace uba { class VisualizerMac; }

@interface VisualizerCanvasView : NSView
{
    NSTrackingArea* _tracking;
	NSTimer* _tickTimer;
	uba::VisualizerMac* _visualizer;
    CGFloat _magnifyAccum;
    CGFloat _magnifyTotal;
	bool _mouseInside;
}
- (void)startTimer;
- (void)stopTimer;
- (void)tick:(NSTimer *)timer;
@end

enum
{
	Popup_CopySessionInfo = 3,
	Popup_CopyProcessInfo,
	Popup_CopyProcessLog,
	Popup_CopyProcessBreadcrumbs,
	Popup_CopyWorkInfo,
	Popup_Replay,
	Popup_Pause,
	Popup_Play,
	Popup_JumpToEnd,

	#define UBA_VISUALIZER_FLAG(name, defaultValue, desc) Popup_##name,
	UBA_VISUALIZER_FLAGS2
	#undef UBA_VISUALIZER_FLAG

	Popup_IncreaseFontSize,
	Popup_DecreaseFontSize,

	Popup_SaveAs,
	Popup_SaveSettings,
	Popup_OpenSettings,
	Popup_Quit,
};

namespace uba
{
	struct MessageBoxLogWriter : public LogWriter
	{
		NSWindow* _window;

		MessageBoxLogWriter(NSWindow* window) : _window(window) {}

		virtual void BeginScope() override
		{
		}

		virtual void EndScope() override
		{
		}

		virtual void Log(LogEntryType type, const tchar* str, u32 strLen, const tchar* prefix = nullptr, u32 prefixLen = 0) override
		{
			if (type > LogEntryType_Warning)
			{
				#if UBA_DEBUG
				StringBuffer<> buf;
				buf.Append(str, strLen).Append(TCV("\r\n"));
				//OutputDebugStringW(buf.data);
				#endif
				return;
			}
			NSString* text = [[NSString alloc] initWithBytes:str length:strLen encoding:NSUTF8StringEncoding];
			if (!text)
				text = @"Boo";

			auto createAlert = [&]()
			{
				NSAlert* alert = [[NSAlert alloc] init];
				alert.messageText = @"UbaVisualizer";
				alert.informativeText = text;
				[alert addButtonWithTitle:@"OK"];
				alert.alertStyle = NSAlertStyleInformational;
				return alert;
			};

			if ([NSThread isMainThread])
			{
				[NSApp activateIgnoringOtherApps:YES];
				[createAlert() runModal];
			}
			else
			{
				dispatch_sync(dispatch_get_main_queue(), ^{
					[createAlert() beginSheetModalForWindow:_window completionHandler:nil];
				});
			}

			if (type == LogEntryType_Error)
				_Exit(-1);

			/*
			HWND hwnd = NULL;
			if (m_visualizer)
			{
				hwnd = m_visualizer->GetHwnd();
				m_visualizer->Lock(true);
			}

			UINT flags = type == LogEntryType_Error ? MB_ICONERROR : MB_ICONWARNING;
			if (!hwnd)
				flags |= MB_TOPMOST;
			MessageBox(hwnd, str, TC("UbaVisualizer"), flags);
			if (type == LogEntryType_Error)
				ExitProcess(~0u);
			if (m_visualizer)
				m_visualizer->Lock(false);
			*/
		}
	};

	class VisualizerBackendMac final : public VisualizerBackend
	{
	public:
		static constexpr int BitmapCacheHeight = 16*1024;

		struct Bitmap
		{
			CGContextRef ctx;
			CGImageRef img;
			void* pixels;
		};

		struct PaintContextMac final : PaintContext
		{
			virtual SIZE GetClientSize() override
 			{
				return clientSize;
			}
			virtual void SetPen(PenHandle pen) override
			{
				CGContextSetStrokeColorWithColor(ctx, (CGColorRef)pen);
			}
			virtual void SetBrush(BrushHandle b) override
			{
				CGContextSetFillColorWithColor(ctx, (CGColorRef)b);
			}
			virtual void SetFont(const Font& font, bool underlined = false) override
			{
				activeFont = font;
			}
			virtual void SetTextColor(COLORREF color) override
			{
				activeTextColor = color;
				if (activeTextColorRef)
					CFRelease(activeTextColorRef);
				float r = float(GET_RED(color))/255.0f;
				float g = float(GET_GREEN(color))/255.0f;
				float b = float(GET_BLUE(color))/255.0f;
				CGFloat comps[4] = { r, g, b, 1.0 }; // black
				activeTextColorRef = CGColorCreate(owner.colorSpaceRef, comps);
			}
			virtual void Text(int x, int y, StringView str, const RECT* clipRect) override
			{
				// --- Convert UTF-8 to CFString ---
				CFStringRef cfStr = CFStringCreateWithBytes(
					kCFAllocatorDefault,
					reinterpret_cast<const UInt8*>(str.data),
					static_cast<CFIndex>(str.count),
					kCFStringEncodingUTF8,
					false //kCFStringEncodingUTF8 == kCFStringEncodingUTF8 ? false : false // just "false"
				);
				if (!cfStr)
					return;

				y -= 3; // Look in GetFontMetrics

				const void* keys[]   = { kCTFontAttributeName,			kCTForegroundColorAttributeName };
				const void* values[] = { (CTFontRef)activeFont.handle,	activeTextColorRef };
				CFDictionaryRef attrs = CFDictionaryCreate(kCFAllocatorDefault, keys, values, 2, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);

				CFAttributedStringRef attrStr = CFAttributedStringCreate(kCFAllocatorDefault, cfStr, attrs);
				CTLineRef line = CTLineCreateWithAttributedString(attrStr);

				// Logical top-left position in your coordinate system
				CGFloat topLeftX = (CGFloat)x;
				CGFloat topLeftY = (CGFloat)y;

				CGContextSaveGState(ctx);

				// Optional clip in your normal (0,0 top-left, y-down) coordinates
				CGFloat maxWidth = 0.0f;
				if (clipRect)
				{
					CGFloat clipW = (CGFloat)(clipRect->right  - clipRect->left);
					CGFloat clipH = (CGFloat)(clipRect->bottom - clipRect->top);
					CGRect cgClip = CGRectMake(topLeftX, topLeftY, clipW, clipH);
					CGContextClipToRect(ctx, cgClip);

					maxWidth = clipW;
				}

				// Start with an identity text matrix
				CGContextSetTextMatrix(ctx, CGAffineTransformIdentity);

				// Now make a local transform so Core Text sees y-up, but
				// the top of the text is at (topLeftX, topLeftY) in your coords.
				//
				// 1) Move origin to the *baseline*: (topLeftX, topLeftY + activeFont.height)
				// 2) Flip vertically so Core Text sees y-up.
				CGContextTranslateCTM(ctx, topLeftX, topLeftY + activeFont.height);
				CGContextScaleCTM(ctx, 1.0, -1.0);

				// Baseline at (0,0) in this local text space
				CGContextSetTextPosition(ctx, 0.0, 0.0);

				if (!clipRect)
				{
					CTLineDraw(line, ctx);
				}
				else
				{
					// Tail truncation to clip width
					CFStringRef ellipsisStr = CFSTR("\u2026");
					CFAttributedStringRef ellAttrStr = CFAttributedStringCreate(kCFAllocatorDefault, ellipsisStr, attrs);
					CTLineRef ellLine = CTLineCreateWithAttributedString(ellAttrStr);
					CTLineRef truncLine = CTLineCreateTruncatedLine(line, maxWidth, kCTLineTruncationEnd, ellLine);
					CTLineDraw(truncLine ? truncLine : line, ctx);
					CFRelease(ellLine);
					CFRelease(ellAttrStr);
					if (truncLine)
						CFRelease(truncLine);
				}

				CGContextRestoreGState(ctx);

				CFRelease(line);
				CFRelease(attrStr);
				CFRelease(attrs);
				CFRelease(cfStr);
			}

			virtual COLORREF GetTextColor() override
			{
				return activeTextColor;
			}
			virtual Font& GetFont() override
			{
				return activeFont;
			}
			virtual void FillRect(const RECT& r, BrushHandle b) override
			{
				CGContextSetFillColorWithColor(ctx, (CGColorRef)b);
				CGRect rect = CGRectMake(r.left, r.top, r.right - r.left, r.bottom - r.top);
				CGContextFillRect(ctx, rect);
			}
			virtual void Rectangle(int left, int top, int right, int bottom) override
			{
				CGRect rect = CGRectMake(left, top, right - left, bottom - top);
				CGContextSetLineWidth(ctx, 1.0);
				CGContextStrokeRect(ctx, rect);
			}
			virtual void Line(int fromX, int fromY, int toX, int toY) override
			{
				CGContextSetLineWidth(ctx, 1.0);
				CGContextMoveToPoint(ctx, fromX, fromY);
				CGContextAddLineToPoint(ctx, toX, toY);
				CGContextStrokePath(ctx);
			}
			virtual void Polyline(u32 cacheId, const Vector<POINT>& polyLine) override
			{
				Vector<CGPoint> points;
				points.resize(polyLine.size());
				u32 i = 0;
				for (auto& p : polyLine)
					points[i++] = CGPoint(p.x, p.y);
				CGContextBeginPath(ctx);
				CGContextAddLines(ctx, points.data(), points.size());
				CGContextStrokePath(ctx);
			}
			virtual void PolyPolyline(const POINT* points, const u32* lines, u32 lineCount) override
			{
				Vector<CGPoint> p;
				for (u32 i=0; i!=lineCount; ++i)
				{
					u32 pointCount = lines[i];
					p.resize(pointCount);
					for (u32 j=0; j!=pointCount; ++j, ++points)
						p[j] = CGPoint(points->x, points->y);
					CGContextBeginPath(ctx);
					CGContextAddLines(ctx, p.data(), p.size());
					CGContextStrokePath(ctx);
				}
			}
			virtual SIZE GetTextSize(StringView str) override
			{
				CFStringRef text = CFStringCreateWithBytes(kCFAllocatorDefault, reinterpret_cast<const UInt8*>(str.data), static_cast<CFIndex>(str.count), kCFStringEncodingUTF8, false);
				const void* keys[]   = { kCTFontAttributeName };
				const void* values[] = { activeFont.handle };
				CFDictionaryRef attrs = CFDictionaryCreate(kCFAllocatorDefault, keys, values, 1, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);
				CFAttributedStringRef attr = CFAttributedStringCreate(kCFAllocatorDefault, text, attrs);
				CTLineRef line = CTLineCreateWithAttributedString(attr);

				double ascent, descent, leading;
				double width = CTLineGetTypographicBounds(line, &ascent, &descent, &leading);
				double height = ascent + descent + leading;

				CFRelease(line);
				CFRelease(attr);
				CFRelease(attrs);
				CFRelease(text);
				return { int(width), int(height) };
			}
			virtual void DrawCached(TraceView::BitmapCache& process, RECT rect, int offsetY, int textHeight, Font& processFont, RECT clippingRect, const Function<void(PaintContext& textCacheContext)>& func) override
			{
				if (!process.bitmap || process.bitmapDirty)
				{
					if (!process.bitmap)
					{
						if (owner.m_lastBitmapOffset + processFont.height > BitmapCacheHeight)
						{
							if (owner.m_lastBitmap)
								owner.m_textBitmaps.push_back(owner.m_lastBitmap);

							CGFloat scale = NSScreen.mainScreen.backingScaleFactor;
							if (scale <= 0)
								scale = 1.0;

							auto& b = *new Bitmap;
							size_t w = 256;
							size_t h = BitmapCacheHeight;

							w = (size_t)lround((double)w * (double)scale);
							h = (size_t)lround((double)h * (double)scale);

							size_t bytesPerRow = w * 4;
							b.pixels = calloc(h, bytesPerRow);
							CGBitmapInfo info = kCGBitmapByteOrder32Little | kCGImageAlphaPremultipliedFirst;
							b.ctx = CGBitmapContextCreate(b.pixels, w, h, 8, bytesPerRow, owner.colorSpaceRef, info);
							CGContextScaleCTM(b.ctx, scale, scale);
							owner.m_lastBitmapOffset = 0;
							owner.m_lastBitmap = &b;
						}
						process.bitmap = owner.m_lastBitmap;
						process.bitmapOffset = owner.m_lastBitmapOffset;
						owner.m_lastBitmapOffset += processFont.height;
					}
					if (lastSelectedBitmap != (Bitmap*)process.bitmap)
						lastSelectedBitmap = (Bitmap*)process.bitmap;

					PaintContextMac textCacheContext{ owner, lastSelectedBitmap->ctx, { 256, BitmapCacheHeight } };
					textCacheContext.SetFont(activeFont);
					func(textCacheContext);

					if (lastSelectedBitmap->img)
						CGImageRelease(lastSelectedBitmap->img);
					lastSelectedBitmap->img = nullptr;
				}

				if (lastSelectedBitmap != (Bitmap*)process.bitmap)
					lastSelectedBitmap = (Bitmap*)process.bitmap;

				int processWidth = rect.right - rect.left;
				int width = Min(processWidth, 256);
				int bitmapOffsetY = process.bitmapOffset;// - processFont.height/4; // TODO: Revisit this.. something is going wrong with the bitmap offset
				int bltOffsetY = offsetY;
				if (bltOffsetY < 0)
				{
					bitmapOffsetY -= bltOffsetY;
					bltOffsetY = 0;
				}
				int height = Min(textHeight, processFont.height);
				if (bltOffsetY + height > textHeight)
					height = textHeight - bltOffsetY;

				int left = rect.left;
				if (left <= -256 || height <= 0)
					return;

				int bitmapOffsetX = rect.left - left;

				if (left < clippingRect.left)
				{
					int diff = clippingRect.left - left;
					rect.left = clippingRect.left;
					width -= diff;
					bitmapOffsetX += diff;
				}

				BlitInfo info
				{
					*lastSelectedBitmap,
					CGRectMake(float(rect.left), float(rect.top + bltOffsetY), float(width), float(height)), // dest
					CGRectMake(float(bitmapOffsetX), float(bitmapOffsetY), float(width), float(height)), // src
				};

				if (info.b.img)
					Blit(info);
				else
					m_deferredBlits.emplace_back(info);
			}

			struct BlitInfo
			{
				Bitmap& b;
				CGRect dest;
				CGRect source;
			};

			void Blit(BlitInfo& info)
			{
				CGImageRef img = info.b.img;
				CGContextSaveGState(ctx);
				CGContextClipToRect(ctx, info.dest);
				CGContextSetInterpolationQuality(ctx, kCGInterpolationNone);

				CGFloat sx = info.dest.size.width  / info.source.size.width;
				CGFloat sy = info.dest.size.height / info.source.size.height;

				// pixel-snap translation
				CGFloat scale = NSScreen.mainScreen.backingScaleFactor;
				CGFloat tx = info.dest.origin.x - info.source.origin.x * sx;
				CGFloat ty = info.dest.origin.y - info.source.origin.y * sy;
				tx = round(tx * scale) / scale;
				ty = round(ty * scale) / scale;

				CGContextTranslateCTM(ctx, tx, ty);
				CGContextScaleCTM(ctx, sx, sy);
				CGRect full = CGRectMake(0, 0, (CGFloat)256, (CGFloat)BitmapCacheHeight); // Draw whole image; clip ensures only desired region shows
				CGContextDrawImage(ctx, full, img);
				CGContextRestoreGState(ctx);
			}

			Vector<BlitInfo> m_deferredBlits;

			virtual void EndCached() override
			{
				for (auto& blit : m_deferredBlits)
				{
					if (!blit.b.img)
						blit.b.img = CGBitmapContextCreateImage(blit.b.ctx);
					Blit(blit);
				}
				m_deferredBlits.clear();
			}

			virtual POINT GetCursorClientPos() override
			{
				return owner.m_mousePos;
			}

			PaintContextMac(VisualizerBackendMac& o, CGContextRef c, SIZE s) : owner(o), ctx(c), clientSize(s)
			{
			}

			VisualizerBackendMac& owner;
			CGContextRef ctx;
			SIZE clientSize;
			Font activeFont;
			COLORREF activeTextColor;
			CGColorRef activeTextColorRef = nullptr;
			Bitmap* lastSelectedBitmap = 0;
		};

		VisualizerBackendMac()
		{
			colorSpaceRef = CGColorSpaceCreateDeviceRGB();
		}

		virtual const tchar* GetName() override
		{
			return "";
		}
		virtual void Init(void* windowHandle) override
		{
		}
		virtual void Reset() override
		{
			for (auto c : colors)
				CFRelease(c);
			colors.clear();

			for (auto b : m_textBitmaps)
			{
				CFRelease(b->ctx);
				CFRelease(b->img);
				free(b->pixels);
				delete b;
			}
			m_textBitmaps.clear();
			m_lastBitmap = 0;
			m_lastBitmapOffset = BitmapCacheHeight;
		}
		virtual void Resize(int width, int height) override
		{
		}
		virtual void Paint(const Function<void(PaintContext& context, RECT rect)>& func) override
		{
		}
		virtual PaintContext* CreateContext(int clientWidth, int clientHeight) override
		{
			return new PaintContextMac(*this, 0, {clientWidth, clientHeight});
		}
		virtual void DeleteContext(PaintContext* context) override
		{
			delete (PaintContextMac*)context;
		}
		virtual BrushHandle CreateSolidBrush(COLORREF color) override
		{
				float r = float(GET_RED(color))/255.0f;
				float g = float(GET_GREEN(color))/255.0f;
				float b = float(GET_BLUE(color))/255.0f;
				CGFloat comps[4] = { r, g, b, 1.0 }; // black
				CGColorRef ref = CGColorCreate(colorSpaceRef, comps);
				colors.push_back(ref);
				return ref;
		}
		virtual PenHandle CreatePen(COLORREF color) override
		{
			return CreateSolidBrush(color);
		}
		virtual FontHandle CreateFont(const tchar* font, int height, bool createUnderline) override
		{
			const CGFloat fontSize = height;
			CFStringRef fontStr = CFStringCreateWithCString(kCFAllocatorDefault, font, kCFStringEncodingUTF8);
			auto res = (FontHandle)CTFontCreateWithName(fontStr, fontSize-4, nullptr);
			UBA_ASSERT(res);
			CFRelease(fontStr);
			return res;
		}
		virtual void DeleteFont(FontHandle font) override
		{
			CFRelease((CTFontRef)font);
		}

		CGColorSpaceRef colorSpaceRef;
		List<CGColorRef> colors;

		Bitmap* m_lastBitmap = 0;
		int m_lastBitmapOffset = BitmapCacheHeight;
		Vector<Bitmap*> m_textBitmaps;

		POINT m_mousePos;
	};

	class VisualizerMac : public Visualizer
	{
	public:
		VisualizerMac(VisualizerConfig& config, Logger& logger, VisualizerBackend& backend, VisualizerCanvasView* view) : Visualizer(config, logger, backend)
		{
			m_view = view;
		}

		void Init()
		{
			//m_config.boxHeight = 20;

			UpdateDefaultFont();
			UpdateProcessFont();
			m_popupFont.handle = m_backend.CreateFont(TC("Menlo"), 16, false);
			m_popupFont.height = 14;

			NSArray<NSString *> *args = [[NSProcessInfo processInfo] arguments];
			if (args.count > 1)
			{
				auto filePath = [args[1] UTF8String];
				char fixedPath[512];
				GetFullPathNameW(filePath, 512, fixedPath, nullptr);

				m_fileName.Append(fixedPath);
				bool playFile = false;
				if (playFile)
				{
					m_trace.ReadFile(m_traceView, fixedPath, true);

					m_startTime = GetTime();

					bool changed = true;
					u64 time = 0;
					m_traceView.finished = false;
					m_trace.UpdateReadFile(m_traceView, time, changed);
					m_pauseStart = m_startTime + time;
					m_pauseTime = m_startTime - m_pauseStart;
					m_autoScroll = true;
					m_replay = 1;
				}
				else
					m_trace.ReadFile(m_traceView, fixedPath, false);
			}
			else
			{
				m_listenDone = false;
				m_listenThread.Start([this]() { ThreadListen(); return 0; });
			}
		}
		virtual void PaintEmptyScreen(PaintContext& context) override
		{
			//[[NSColor blackColor] setFill];
			//NSRect rect = NSMakeRect(50, 50, 200, 100);
			//NSRectFill(rect);
		}
		virtual void Redraw(bool now) override
		{
			[m_view setNeedsDisplay:YES];
		}
		virtual void StopUpdate() override
		{
			[m_view stopTimer];
		}
		void GetFontMetrics(int& outFh, int& outOffset, int height) override
		{
			outFh = height;
			outOffset = 3 - height/4;
			if (height <= 13)
				++outFh;
			if (height <= 11)
				++outFh;
			if (height <= 9)
				++outFh;
			if (height <= 8)
				++outFh;
			if (height <= 6)
				++outFh;
		}
		virtual void FinishIncompatible() override
		{
		}
		virtual void PostNewTitle(const StringView& title)
		{
			if (NSString* nsTitle = [[NSString alloc] initWithBytes:title.data length:title.count encoding:NSUTF8StringEncoding])
				dispatch_async(dispatch_get_main_queue(), ^{ [m_view.window setTitle:nsTitle]; });
		}
		bool UpdateSelection() override
		{
			return Visualizer::UpdateSelection(((VisualizerBackendMac&)m_backend).m_mousePos);
		}
		void MenuClicked(int id)
		{
			switch (id)
			{
			case Popup_ShowProcessText:
				m_config.ShowProcessText = !m_config.ShowProcessText;
				Redraw(true);
				break;
			case Popup_ShowReadWriteColors:
				m_config.ShowReadWriteColors = !m_config.ShowReadWriteColors;
				DirtyBitmaps(false);
				Redraw(true);
				break;
			case Popup_ScaleHorizontalWithScrollWheel:
				m_config.ScaleHorizontalWithScrollWheel = !m_config.ScaleHorizontalWithScrollWheel;
				break;
			case Popup_ShowAllTraces:
				m_config.ShowAllTraces = !m_config.ShowAllTraces;
				break;
			case Popup_SortActiveRemoteSessions:
				m_config.SortActiveRemoteSessions = !m_config.SortActiveRemoteSessions;
				Redraw(true);
				break;
			case Popup_AutoScaleHorizontal:
				m_config.AutoScaleHorizontal = !m_config.AutoScaleHorizontal;
				UpdateScrollbars(true);
				Redraw(true);
				break;
			case Popup_LockTimelineToBottom:
				m_config.LockTimelineToBottom = !m_config.LockTimelineToBottom;
				Redraw(true);
				break;
			case Popup_DarkMode:
			{
				m_config.DarkMode = !m_config.DarkMode;
				DirtyBitmaps(true);
				//UpdateTheme();
				Redraw(true);
				break;
			}
			case Popup_AutoSaveSettings:
				m_config.AutoSaveSettings = !m_config.AutoSaveSettings;
				break;

			}
		}

		void Paint(CGContextRef ctx, SIZE size)
		{
			auto& backend = (VisualizerBackendMac&)m_backend;

			if (backend.colors.empty())
				InitBrushes();

			m_clientWidth = size.cx;
			m_clientHeight = size.cy;

			uba::VisualizerBackendMac::PaintContextMac context(backend, ctx, size);
			RECT r { 0, 0, size.cx, size.cy };

			context.FillRect(r, m_backgroundBrush);
			PaintAll(context, r);
		}

		void PostNewTrace(const TString& newTrace)
		{
			m_newTraceName.Clear().Append(newTrace);
			m_usingNamed = true;
			NewTrace(0, false);
			[m_view startTimer];
		}

		bool Update()
		{
			return UpdateTrace();
		}

		void MouseMove(CGFloat x, CGFloat y)
		{
			((VisualizerBackendMac&)m_backend).m_mousePos =  { (int)x, (int)y };
			if (UpdateSelection() || m_config.showCursorLine)
				Redraw(false);
		}

		void MouseDown(CGFloat x, CGFloat y)
		{
			((VisualizerBackendMac&)m_backend).m_mousePos =  { (int)x, (int)y };

			if (m_buttonSelected != ~0u)
			{
				bool* values = &m_config.showProgress;
				values[m_buttonSelected] = !values[m_buttonSelected];
				SaveSettings();

				PaintContext* context = m_backend.CreateContext(m_clientWidth, m_clientHeight);
				HitTestResult res;
				HitTest(res, *context, { -1, -1 });
				m_backend.DeleteContext(context);
				UpdateScrollbars(true);
				Redraw(false);
			}
			else if (!m_hyperLinkSelected.empty())
			{
				NSString* s = [NSString stringWithUTF8String:m_hyperLinkSelected.c_str()];
				if (NSURL* url = [NSURL URLWithString:s])
					[[NSWorkspace sharedWorkspace] openURL:url];
			}
			else if (m_sessionSelectedIndex != ~0u)
			{
				auto& session = m_traceView.sessions[m_sessionSelectedIndex];
				for (auto& info : session.infos)
				{
					if (info.hyperlink.empty())
						continue;
					NSString* s = [NSString stringWithUTF8String:info.hyperlink.c_str()];
					if (NSURL* url = [NSURL URLWithString:s])
						[[NSWorkspace sharedWorkspace] openURL:url];
					break;
				}
			}
		}

		void Scroll(float dx, float dy)
		{
			m_scrollPosY = Min(0.0f, m_scrollPosY + float(dy));
			m_scrollPosX = Min(0.0f, m_scrollPosX + float(dx));
			UpdateSelection();

			int minScroll = m_clientWidth - m_contentWidth;
			if (!m_traceView.finished && m_scrollPosX <= float(minScroll))
				m_autoScroll = true;
			else
				m_autoScroll = false;
			//if (_visualizer->m_scrollPosX > _visualizer->)
		}

		bool ThreadListen()
		{
			const tchar* channelName = TC("Default");

			TraceChannel channel(m_logger);
			if (!channel.Init(channelName))
			{
				m_logger.Error(TC("TODO"));
				return false;
			}

			StringBuffer<> title;
			PostNewTitle(GetTitlePrefix(title).Appendf("Listening for new sessions on channel '%s'", channelName));

			m_listenTimeout.Create(EventResetType_Auto);
			
			TString newTrace;
			StringBuffer<256> traceName;
			while (!m_listenDone)
			{
				traceName.Clear();
				if (!channel.Read(traceName))
				{
					m_logger.Error(TC("TODO2"));
					return false;
				}
				if (traceName.count)
				{
					if (!traceName.Equals(newTrace))
					{
						newTrace = traceName.ToString();
						dispatch_sync(dispatch_get_main_queue(), ^{
							PostNewTrace(newTrace);
						});
					}
				}
				else if (!newTrace.empty())
				{
					dispatch_sync(dispatch_get_main_queue(), ^{
							PostNewTrace("");
					});
					newTrace.clear();
				}
				Sleep(500);
			}
			return true;
		}

		void SaveSettings()
		{
			m_config.Save(m_logger);
			m_settingsDirty = false;
		}

		VisualizerCanvasView* m_view;
		Event m_listenTimeout;
		Atomic<bool> m_listenDone;
		Thread m_listenThread;
		bool m_settingsDirty = false;
	};
}

uba::VisualizerMac* g_visualizerMac = nullptr;


@implementation VisualizerCanvasView


- (instancetype)initWithFrame:(NSRect)frame {
	self = [super initWithFrame:frame];
	if (self)
	{
		using namespace uba;
		char fixedPath[512];
		GetFullPathNameW("~/.epic/UbaVisualizerConfig.toml", 512, fixedPath, nullptr);
		VisualizerConfig config(fixedPath);
		config.boxHeight = 14;
		config.horizontalScaleValue = 1.0f;
		config.DarkMode = true;
		config.AutoScaleHorizontal = false;
		auto writer = new MessageBoxLogWriter(self.window);

		SetCustomAssertHandler([](const tchar* text, u32 textLen, void* userData)
		{
			((MessageBoxLogWriter*)userData)->Log(LogEntryType_Error, text, textLen);
		}, writer);

		auto logger = new LoggerWithWriter(*writer);
		auto backend = new VisualizerBackendMac;

		config.Load(*logger);

		_visualizer = new VisualizerMac(config, *logger, *backend, self);
		g_visualizerMac = _visualizer;
	}
	return self;
}

- (void)startTimer {
	if (_tickTimer) return;
	_tickTimer = [NSTimer scheduledTimerWithTimeInterval:(1.0/60.0) target:self selector:@selector(tick:) userInfo:nil repeats:YES];
}

- (void)stopTimer {
	if (!_tickTimer) return;
	[_tickTimer invalidate];
	_tickTimer = nil;
}

- (void)tick:(NSTimer *)timer {
	if (_visualizer->Update())
		[self setNeedsDisplay:YES];
}

- (BOOL)isFlipped { return YES; } // top-left is 0,0

- (void)drawRect:(NSRect)dirtyRect {
   [super drawRect:dirtyRect];
	CGContextRef ctx = [[NSGraphicsContext currentContext] CGContext];
	NSRect bounds = self.bounds;
	uba::SIZE size = { (int)bounds.size.width, (int)bounds.size.height };
	_visualizer->Paint(ctx, size);
}

- (void)updateTrackingAreas {
    [super updateTrackingAreas];

    if (_tracking)
	{
        [self removeTrackingArea:_tracking];
        _tracking = nil;
    }

    NSTrackingAreaOptions opts = NSTrackingMouseMoved | NSTrackingActiveInKeyWindow | NSTrackingInVisibleRect | NSTrackingMouseEnteredAndExited;

    _tracking = [[NSTrackingArea alloc] initWithRect:NSZeroRect options:opts owner:self userInfo:nil];
    [self addTrackingArea:_tracking];
}

- (BOOL)acceptsFirstResponder { return YES; }

- (void)viewDidMoveToWindow {
    [super viewDidMoveToWindow];
    [self.window setAcceptsMouseMovedEvents:YES];
}

- (void)mouseMoved:(NSEvent*)event {
	if (_mouseInside)
	{
		NSPoint p = [self convertPoint:event.locationInWindow fromView:nil];
		_visualizer->MouseMove(p.x, p.y);
	}
}

- (void)mouseEntered:(NSEvent*)event
{
    _mouseInside = true;
}

- (void)mouseExited:(NSEvent*)event
{
    _mouseInside = false;
	_visualizer->UnselectAndRedraw();
}

- (void)mouseDown:(NSEvent*)event
{
    NSPoint p = [self convertPoint:event.locationInWindow fromView:nil];
	_visualizer->MouseDown(p.x, p.y);
}

- (void)scrollWheel:(NSEvent *)event
{
	using namespace uba;
	if (event.modifierFlags & NSEventModifierFlagCommand)
	{
		float newValue = _visualizer->m_config.horizontalScaleValue + float(event.scrollingDeltaX*0.001);
		_visualizer->m_config.horizontalScaleValue = Max(newValue, 0.001f);
		_visualizer->m_settingsDirty = true;
		//_visualizer->m_zoomValue += float(event.scrollingDeltaX*0.001);
	}
	else
	{
		_visualizer->Scroll(float(event.scrollingDeltaX), float(event.scrollingDeltaY));
	}

	if (event.phase == NSEventPhaseEnded && _visualizer->m_settingsDirty)
		_visualizer->SaveSettings();

	[self setNeedsDisplay:YES];
}

- (void)magnifyWithEvent:(NSEvent *)event
{
	using namespace uba;

	if (event.phase == NSEventPhaseBegan)
	{
		_magnifyAccum = 0.0;
		_magnifyTotal = 0.0;
	}

	_magnifyAccum += event.magnification;
	_magnifyTotal += event.magnification;

	const CGFloat step = 0.10;

	bool changed = false;
	while (_magnifyAccum >= step) {
		_magnifyAccum -= step;
		_visualizer->m_config.boxHeight = u32(Min(int(_visualizer->m_config.boxHeight + 1), 20));
		changed = true;
	}
	while (_magnifyAccum <= -step) {
		_magnifyAccum += step;
		_visualizer->m_config.boxHeight = u32(Max(int(_visualizer->m_config.boxHeight - 1), 2));
		changed = true;
	}

	if (changed)
	{
		_visualizer->m_settingsDirty = true;
		_visualizer->UpdateProcessFont();
		_visualizer->DirtyBitmaps(true);
		_visualizer->UpdateSelection();
		[self setNeedsDisplay:YES];
	}

	if (event.phase == NSEventPhaseEnded || event.phase == NSEventPhaseCancelled)
	{
		_magnifyAccum = 0.0;
		if (_visualizer->m_settingsDirty)
			_visualizer->SaveSettings();
	}
}

- (NSMenu *)menuForEvent:(NSEvent *)event
{
    NSMenu *menu = [[NSMenu alloc] initWithTitle:@"Context"];
	#define UBA_VISUALIZER_FLAG(name, defaultValue, desc) \
		{ NSMenuItem* item = [menu addItemWithTitle:@desc action:@selector(onMenuItem:) keyEquivalent:@""]; \
			item.tag = Popup_##name; \
			item.state = _visualizer->m_config.name ? NSControlStateValueOn : NSControlStateValueOff; \
		}
				UBA_VISUALIZER_FLAGS2
	#undef UBA_VISUALIZER_FLAG

	menu.autoenablesItems = NO;
	return menu;
}

- (void)onMenuItem:(NSMenuItem*)sender {
	_visualizer->MenuClicked(sender.tag);
	_visualizer->SaveSettings();
}

- (void)onAction2:(id)sender {

}

@end

@interface AppDelegate : NSObject <NSApplicationDelegate, NSWindowDelegate>
@end

@implementation AppDelegate
- (void)windowWillClose:(NSNotification *)notification {
	[NSApp terminate:nil];
}

- (BOOL)application:(NSApplication *)sender openFile:(NSString *)filename
{
	using namespace uba;
	const char* pendingFile = [filename UTF8String];
	auto& visualizer = *g_visualizerMac;
	//m_newTraceName.Clear();//.Append(newTrace);

	visualizer.m_listenDone = true;
	visualizer.m_listenThread.Wait();

	visualizer.m_usingNamed = false;
	char fixedPath[512];
	GetFullPathNameW([filename UTF8String], 512, fixedPath, nullptr);
	visualizer.m_fileName.Append(fixedPath);
	visualizer.NewTrace(0, false);
    return YES;
}

- (NSApplicationTerminateReply)applicationShouldTerminate:(NSApplication *)sender
{
	using namespace uba;
	auto& visualizer = *g_visualizerMac;
	LoggerWithWriter nullLogger(g_nullLogWriter);
	visualizer.m_config.Save(nullLogger);
    return NSTerminateNow;
}
@end

int main(int argc, const char * argv[]) {
	@autoreleasepool {
		NSApplication* app = [NSApplication sharedApplication];
		[NSApp setActivationPolicy:NSApplicationActivationPolicyRegular];

		AppDelegate* delegate = [[AppDelegate alloc] init];
		app.delegate = delegate; 


		// --- Minimal menu bar with "Quit" (Cmd+Q) ---
		NSMenu *menubar = [[NSMenu alloc] init];
		NSMenuItem *appMenuItem = [[NSMenuItem alloc] init];
		[menubar addItem:appMenuItem];
		[NSApp setMainMenu:menubar];

		NSMenu *appMenu = [[NSMenu alloc] initWithTitle:@""];
		NSString *appName = [[NSProcessInfo processInfo] processName];
		NSMenuItem *quitItem =
			[[NSMenuItem alloc] initWithTitle:[NSString stringWithFormat:@"Quit %@", appName]
									   action:@selector(terminate:)
								keyEquivalent:@"q"];
		// Make sure it's Cmd+Q, not just 'q'
		[quitItem setKeyEquivalentModifierMask:NSEventModifierFlagCommand];

		[appMenu addItem:quitItem];
		[appMenuItem setSubmenu:appMenu];
		// --- end menu setup ---

		NSRect frame = NSMakeRect(200, 200, 800, 600);
		NSWindowStyleMask style = NSWindowStyleMaskTitled | NSWindowStyleMaskClosable | NSWindowStyleMaskResizable | NSWindowStyleMaskMiniaturizable;
		NSWindow* window = [[NSWindow alloc] initWithContentRect:frame styleMask:style backing:NSBackingStoreBuffered defer:NO];

		[window setDelegate:delegate];
		[window setTitle:@"UbaVisualizer"];

		VisualizerCanvasView *view = [[VisualizerCanvasView alloc] initWithFrame:frame];
		[window setContentView:view];
		[window makeKeyAndOrderFront:nil];
		[NSApp activateIgnoringOtherApps:YES]; // Make sure app shows in Dock and gets focus
		g_visualizerMac->Init();
		[NSApp run];
	}
	return 0;
}
