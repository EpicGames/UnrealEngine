// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaTraceReader.h"
#include "UbaFile.h"
#include "UbaFileAccessor.h"
#include "UbaNetworkClient.h"
#include "UbaNetworkMessage.h"

namespace uba
{
	const TraceView::Process& TraceView::GetProcess(const ProcessLocation& loc)
	{
		static TraceView::Process emptyProcess;
		if (loc.sessionIndex >= sessions.size())
			return emptyProcess;
		auto& session = sessions[loc.sessionIndex];
		if (loc.processorIndex >= session.processors.size())
			return emptyProcess;
		auto& processor = session.processors[loc.processorIndex];
		if (loc.processIndex >= processor.processes.size())
			return emptyProcess;
		return processor.processes[loc.processIndex];
	}

	const TraceView::Session& TraceView::GetSession(const ProcessLocation& loc)
	{
		if (loc.sessionIndex < sessions.size())
			return sessions[loc.sessionIndex];
		static TraceView::Session emptySession;
		return emptySession;
	}

	void TraceView::Session::GetFullName(StringBufferBase& out)
	{
		out.Append(name);
		u32 infoIndex = 0;
		for (auto& info : infos)
		{
			if (info.text.empty())
				continue;
			if (!infoIndex)
				out.Append(TCV(" ("));
			else
				out.Append(' ');
			out.Append(info.text);
			++infoIndex;
		}
		#if UBA_DEBUG
		out.Append(' ').Append(machineId);
		#endif

		if (infoIndex)
			out.Append(TCV(")"));
	}

	void TraceView::Clear()
	{
		version = 0;
		sessions.clear();
		workTracks.clear();
		cacheSummaries.clear();
		for (auto s : strings)
			free((void*)s);
		strings.clear();
		statusMap.clear();
		cacheWrites.clear();
		tables.clear();
		activeProcessCounts.clear();
		maxActiveProcessCount = 0;
		startTime = 0;
		totalProcessActiveCount = 0;
		totalProcessExitedCount = 0;
		activeSessionCount = 0;
		progressProcessesTotal = 0;
		progressProcessesDone = 0;
		progressErrorCount = 0;
		lastKillProcessTime = 0;
		lastKillReason = nullptr;
		lastSpawningDelayStartTime = 0;
		lastSpawningDelayEndTime = 0;
		remoteExecutionDisabled = false;
		finished = true;
	};


	TraceReader::TraceReader(Logger& logger)
	:	m_logger(logger)
	,	m_channel(logger)
	{
	}

	TraceReader::~TraceReader()
	{
		Unmap();
	}

	u64 ConvertTime(const TraceView& view, u64 time)
	{
		return u64(double(time) * view.timeMultiplier);
	}

	bool TraceReader::ReadFile(TraceView& out, const tchar* fileName, bool replay)
	{
		Reset(out);

		FileHandle readHandle;
		if (!OpenFileSequentialRead(m_logger, fileName, readHandle))
			return false;
		auto closeFile = MakeGuard([&]() { CloseFile(fileName, readHandle); });
		u64 fileSize;
		if (!uba::GetFileSizeEx(fileSize, readHandle))
			return false;

		m_memoryHandle = FileMapping_CreateFromFile(m_logger, readHandle, PAGE_READONLY, 0, fileName);
		if (!m_memoryHandle.IsValid())
			return false;
		auto closeFileMapping = MakeGuard([&]() { FileMapping_Close(m_logger, m_memoryHandle, fileName); m_memoryHandle = {}; });

		m_memoryBegin = FileMapping_Map(m_logger, m_memoryHandle, FILE_MAP_READ, 0, fileSize);
		if (!m_memoryBegin)
			return false;
		auto closeView = MakeGuard([&]() { FileMapping_Unmap(m_logger, m_memoryBegin, fileSize, TC("TraceReader")); m_memoryBegin = nullptr; m_memoryPos = nullptr; m_memoryEnd = nullptr; });

		m_memoryPos = m_memoryBegin;
		m_memoryEnd = m_memoryBegin + fileSize;

		BinaryReader reader(m_memoryBegin, 0, fileSize);

		u32 traceSize = reader.ReadU32(); (void)traceSize;
		u32 version = reader.ReadU32();
		if (version < TraceReadCompatibilityVersion || version > TraceVersion)
			return m_logger.Error(TC("Incompatible trace version (%u). Current executable supports version %u to %u."), version, TraceReadCompatibilityVersion, TraceVersion);

		out.version = version;
		reader.ReadU32(); // ProcessId
		u64 traceSystemStartTimeUs = 0;
		if (version >= 18)
			traceSystemStartTimeUs = reader.Read7BitEncoded();
		if (version >= 18)
			out.frequency = reader.Read7BitEncoded();
		else
			out.frequency = GetFrequency();
		out.timeMultiplier = double(GetFrequency()) / double(out.frequency);

		out.realStartTime = reader.Read7BitEncoded();

		out.startTime = out.realStartTime;
		if (replay)
			out.startTime = GetTime();
		else if (traceSystemStartTimeUs)
			out.startTime = GetTime() - UsToTime(GetSystemTimeUs() - traceSystemStartTimeUs);

		out.traceSystemStartTimeUs = traceSystemStartTimeUs;

		m_memoryPos += reader.GetPosition();
		out.finished = false;

		if (replay)
		{
			closeFileMapping.Cancel();
			closeView.Cancel();
			return true;
		}

		while (reader.GetPosition() < fileSize)
			if (!ReadTrace(out, reader, ~u64(0)))
				return false;
		out.finished = true;
		return true;
	}

	bool TraceReader::UpdateReadFile(TraceView& out, u64 maxTime, bool& outChanged)
	{
		bool res = true;
		BinaryReader traceReader(m_memoryPos, 0, m_memoryEnd - m_memoryPos);
		while (traceReader.GetLeft())
		{
			u64 pos = traceReader.GetPosition();
			if (!ReadTrace(out, traceReader, maxTime))
			{
				res = false;
				break;
			}
			if (pos == traceReader.GetPosition())
				break;
		}
		out.finished = !traceReader.GetLeft();
		outChanged = traceReader.GetPosition() != 0 || !m_activeProcesses.empty();
		m_memoryPos += traceReader.GetPosition();
		return res;
	}


	bool TraceReader::StartReadClient(TraceView& out, NetworkClient& client)
	{
		Reset(out);

		u32 traceMemSize = 128 * 1024 * 1024;
		m_memoryHandle = FileMapping_Create(m_logger, PAGE_READWRITE, traceMemSize, 0, TC("NetworkClient"));
		if (!m_memoryHandle.IsValid())
			return false;
		m_memoryBegin = FileMapping_Map(m_logger, m_memoryHandle, FILE_MAP_ALL_ACCESS, 0, traceMemSize);
		m_memoryPos = m_memoryBegin;
		m_memoryEnd = m_memoryBegin;

		out.finished = false;
		out.sessions.emplace_back();
		out.sessions.back().name = TC("LOCAL");
		bool changed;
		return UpdateReadClient(out, client, changed);
	}

	bool TraceReader::UpdateReadClient(TraceView& out, NetworkClient& client, bool& outChanged)
	{
		outChanged = false;
		if (!m_memoryHandle.IsValid())
			return true;

		if (!client.IsConnected() && !out.finished)
		{
			for (auto& session : out.sessions)
				for (auto& processor : session.processors)
				{
					auto& process = processor.processes.back();
					if (process.stop == ~u64(0))
					{
						process.stop = GetTime() - out.startTime;
						process.exitCode = ProcessCancelExitCode; // This is wrong but since we didn't get the final result we can't tell if it was success or not
						process.bitmapDirty = true;
					}
				}
			out.finished = true;
			return false;
		}

		SCOPED_FUTEX_READ(m_memoryFutex, lock);
		outChanged = !m_activeProcesses.empty() || m_memoryPos != m_memoryEnd;
		return ReadMemory(out, false, ~u64(0));
	}

	bool TraceReader::UpdateReceiveClient(NetworkClient& client)
	{
		if (!m_memoryBegin)
			return true;

		bool hasLock = false;
		auto lg = MakeGuard([&]() { if (hasLock) m_memoryFutex.Leave(); });

		while (true)
		{
			u32 pos = u32(m_memoryEnd - m_memoryBegin);

			StackBinaryWriter<32> writer;
			NetworkMessage msg(client, SystemServiceId, SystemMessageType_GetTraceInformation, writer);
			writer.WriteU32(pos);
			StackBinaryReader<SendDefaultSize> reader;
			if (!msg.Send(reader))
				return false;
			u64 remotePos = reader.ReadU32();(void)remotePos;
			u32 left = u32(reader.GetLeft());

			if (!hasLock)
			{
				m_memoryFutex.Enter();
				hasLock = true;
			}

			reader.ReadBytes(m_memoryEnd, left);
			pos += left;
			m_memoryEnd += left;
			if (remotePos == pos)
				break;
		}
		return true;
	}

	bool TraceReader::StartReadNamed(TraceView& out, const tchar* namedTrace, bool silentFail, bool replay)
	{
		Reset(out);

		if (!namedTrace)
			namedTrace = TC("");

		if (*namedTrace && m_namedTrace != namedTrace)
		{
#if PLATFORM_WINDOWS
			u64 traceCapacity = 0;
			m_memoryHandle.internal = u64(::OpenFileMappingW(PAGE_READWRITE, false, namedTrace));
#else
			u64 traceCapacity = 128 * 1024 * 1024;
			// Darwin's shm_open rejects O_NOFOLLOW with EINVAL — see matching comment in UbaFileMapping.inl FileMapping_Create.
			#if PLATFORM_MAC
			int oflags = O_RDONLY;
			#else
			int oflags = O_RDONLY | O_NOFOLLOW;
			#endif
			m_memoryHandle.shmFd = shm_open(namedTrace, oflags, S_IRUSR | S_IWUSR);
			m_memoryHandle.internal = ~0ull;

			// We don't want to unlink
			//if (m_memoryHandle.shmFd != -1)
			//	m_memoryHandle.uid = atoi(strchr(namedTrace, '_') + 1);
			//printf("%s: %i\n", namedTrace, m_memoryHandle.shmFd);
#endif

			if (!m_memoryHandle.IsValid())
			{
				if (!silentFail)
					m_logger.Error(TC("OpenFileMappingW - Failed to open file mapping %s (%s)"), namedTrace, LastErrorToText().data);
				return false;
			}
			m_memoryBegin = FileMapping_Map(m_logger, m_memoryHandle, FILE_MAP_READ, 0, traceCapacity);
			if (!m_memoryBegin)
			{
				if (!silentFail)
					m_logger.Error(TC("MapViewOfFile - Failed to map file mapping %s (%s)"), namedTrace, LastErrorToText().data);
				return false;
			}
		}

		m_namedTrace = namedTrace;

		m_memoryPos = m_memoryBegin;
		m_memoryEnd = m_memoryBegin;
		out.finished = false;
		out.sessions.emplace_back();
		bool changed;
		return UpdateReadNamed(out, replay ? 0ull : ~u64(0), changed);
	}

	bool TraceReader::UpdateReadNamed(TraceView& out, u64 maxTime, bool& outChanged)
	{
		outChanged = false;
		if (!m_memoryBegin)
			return true;
		m_memoryEnd = m_memoryBegin + *(u32*)m_memoryBegin;
		outChanged = !m_activeProcesses.empty() || m_memoryPos != m_memoryEnd;
		bool keepRunning = ReadMemory(out, true, maxTime);

#if PLATFORM_WINDOWS
		if (m_hostProcess && WaitForSingleObject(m_hostProcess, 0) != WAIT_TIMEOUT)
#else
		if (m_hostProcess && kill(m_hostProcess, 0) != 0)
#endif
		{
			StopAllActive(out, GetTime() - out.realStartTime);
			keepRunning = false;
		}

		if (keepRunning || m_namedTrace.empty())
			return true;

		// Move memory to local mapping in case we want to replay
		u64 traceMemSize = m_memoryEnd - m_memoryBegin;
		u8* memoryBegin = nullptr;
		FileMappingHandle memoryHandle = FileMapping_Create(m_logger, PAGE_READWRITE, traceMemSize, nullptr, TC("UpdateReadNamed"));
		if (memoryHandle.IsValid())
			if ((memoryBegin = FileMapping_Map(m_logger, memoryHandle, FILE_MAP_ALL_ACCESS, 0, traceMemSize)) != nullptr)
				memcpy(memoryBegin, m_memoryBegin, traceMemSize);

		u64 pos = m_memoryPos - m_memoryBegin;
		u64 end = m_memoryEnd - m_memoryBegin;
		Unmap();
		m_memoryHandle = memoryHandle;
		m_memoryPos = memoryBegin + pos;
		m_memoryEnd = memoryBegin + end;
		m_memoryBegin = memoryBegin;
		out.finished = true;

		return keepRunning;
	}

	bool TraceReader::ReadMemory(TraceView& out, bool trackHost, u64 maxTime)
	{
		if (m_memoryEnd == m_memoryPos)
			return true;
		
		if (out.version && (out.version < TraceReadCompatibilityVersion || out.version > TraceVersion))
			return true;

		u64 toRead = m_memoryEnd - m_memoryPos;
		BinaryReader traceReader(m_memoryPos);

		if (m_memoryPos == m_memoryBegin)
		{
			if (toRead < 128)
				return true;
			u32 traceSize = traceReader.ReadU32(); (void)traceSize;
			u32 version = traceReader.ReadU32();
			out.version = version;

			bool replay = maxTime != ~u64(0);

			u32 hostProcessId = traceReader.ReadU32();
			m_hostProcess = 0;
			if (trackHost && !replay)
			{
#if PLATFORM_WINDOWS
				m_hostProcess = OpenProcess(SYNCHRONIZE, FALSE, hostProcessId);
#else
				m_hostProcess = hostProcessId;
#endif
			}
			u64 traceSystemStartTimeUs = 0;

			if (version < TraceReadCompatibilityVersion || version > TraceVersion)
				return true;//m_logger.Error(TC("Incompatible trace version (%u). Current executable supports version %u to %u."), version, TraceReadCompatibilityVersion, TraceVersion);

			if (version >= 18)
				traceSystemStartTimeUs = traceReader.Read7BitEncoded();
			if (version >= 18)
				out.frequency = traceReader.Read7BitEncoded();
			else
				out.frequency = GetFrequency();
			out.timeMultiplier = double(GetFrequency()) / double(out.frequency);

			out.realStartTime = traceReader.Read7BitEncoded();
			if (traceSystemStartTimeUs)
				out.realStartTime = GetTime() - UsToTime(GetSystemTimeUs() - traceSystemStartTimeUs);

			out.traceSystemStartTimeUs = traceSystemStartTimeUs;

			out.startTime = out.realStartTime;
			if (replay)
				out.startTime = GetTime();
		}

		u64 lastPos = traceReader.GetPosition();
		while (lastPos != toRead)
		{
			if (!ReadTrace(out, traceReader, maxTime))
			{
				m_memoryPos += lastPos;
				return false;
			}
			u64 pos = traceReader.GetPosition();
			if (pos == lastPos)
				break;
			UBA_ASSERT(pos <= toRead);
			lastPos = pos;
		}
		m_memoryPos += lastPos;
		return true;
	}

	bool TraceReader::ReadTrace(TraceView& out, BinaryReader& reader, u64 maxTime)
	{
		u64 readPos = reader.GetPosition();
		u8 traceType = reader.ReadByte();
		u64 time = 0;
		if (out.version >= 15 && traceType != TraceType_String && traceType != TraceType_DriveUpdate)
		{
			time = ConvertTime(out, reader.Read7BitEncoded());
			if (time > maxTime)
			{
				reader.SetPosition(readPos);
				return true;
			}
		}

		#if 0
		static const tchar* traceTypeNames[] =
		{
			#define UBA_TRACE_TYPE(name) TC(#name),
			UBA_TRACE_TYPES
			#undef UBA_TRACE_TYPE
		};
		OutputDebugString(StringBuffer<>().Appendf(TC("TraceType: %s\r\n"), traceTypeNames[traceType]).data);
		#endif

		switch (traceType)
		{
		case TraceType_SessionAdded:
		{
			StringBuffer<128> sessionName;
			if (!reader.TryReadString(sessionName))
				return false;
			StringBuffer<> machineId;
			if (!reader.TryReadString(machineId))
				return false;
			Guid clientUid = ReadClientId(out, reader);
			u32 sessionIndex = reader.ReadU32();

			TraceView::SessionInfo info;

			if (out.version < 48)
				info = ReadSessionInfo(machineId);

			// Check if we can re-use existing session (same machine was disconnected and then reconnected)
			u32 virtualSessionIndex = sessionIndex;
			for (u32 i = 0; i != out.sessions.size(); ++i)
			{
				auto& oldSession = out.sessions[i];
				if (oldSession.machineId != machineId.data)
					continue;
				if (oldSession.disconnectTime == ~u64(0))
					break;
				u32 updateCount = u32(oldSession.updates.size());
				oldSession.networkSend.resize(updateCount);
				oldSession.networkRecv.resize(updateCount);
				oldSession.ping.resize(updateCount);
				oldSession.memAvail.resize(updateCount);
				oldSession.cpuLoad.resize(updateCount);
				oldSession.connectionCount.resize(updateCount);

				oldSession.reconnectIndices.push_back(updateCount);
				oldSession.isReset = true;
				oldSession.disconnectTime = ~u64(0);
				oldSession.proxyName.clear();
				oldSession.proxyCreated = false;
				oldSession.notification.clear();
				oldSession.infos.clear();
				//oldSession.fetchedFiles.clear();
				//oldSession.storedFiles.clear();
				virtualSessionIndex = i;
				break;
			}


			if (out.sessions.size() <= virtualSessionIndex)
				out.sessions.resize(virtualSessionIndex + 1);

			if (m_sessionIndexToSession.size() <= sessionIndex)
				m_sessionIndexToSession.resize(sessionIndex + 1);

			m_sessionIndexToSession[sessionIndex] = virtualSessionIndex;

			TraceView::Session& session = GetSession(out, sessionIndex);
			session.name = sessionName.data;
			session.machineId = machineId.data;
			if (!info.text.empty())
				session.infos.push_back(std::move(info));
			session.clientUid = clientUid;

			++out.activeSessionCount;
			break;
		}
		case TraceType_SessionInfo:
		{
			u32 sessionIndex = reader.ReadU32();
			StringBuffer<> info;
			reader.ReadString(info);
			TraceView::Session& session = GetSession(out, sessionIndex);
			session.infos.push_back(ReadSessionInfo(info));
			break;
		}

		case TraceType_SessionUpdate:
		{
			if (out.version < 15)
			{
				time = reader.Read7BitEncoded();
				if (time > maxTime)
				{
					reader.SetPosition(readPos);
					return true;
				}
			}

			u32 sessionIndex;
			u8 connectionCount = 0;
			u64 totalSend;
			u64 totalRecv;
			u64 lastPing;

			if (out.version >= 14)
			{
				sessionIndex = u32(reader.Read7BitEncoded());
				connectionCount = u8(reader.Read7BitEncoded());
				totalSend = reader.Read7BitEncoded();
				totalRecv = reader.Read7BitEncoded();
				lastPing = reader.Read7BitEncoded();
			}
			else
			{
				sessionIndex = reader.ReadU32();
				totalSend = reader.ReadU64();
				totalRecv = reader.ReadU64();
				lastPing = reader.Read7BitEncoded();
			}

			u64 memAvail = 0;
			u64 memTotal = 0;
			if (out.version >= 9)
			{
				memAvail = reader.Read7BitEncoded();
				memTotal = reader.Read7BitEncoded();
			}

			float cpuLoad = 0;
			if (out.version >= 13)
			{
				u32 value = reader.ReadU32();
				cpuLoad = *(float*)&value;
			}
			TraceView::Session& session = GetSession(out, sessionIndex);
			if (session.isReset)
			{
				session.isReset = false;
				session.prevUpdateTime = 0;
				session.prevSend = 0;
				session.prevRecv = 0;
				session.memTotal = 0;
				if (!session.updates.empty())
				{
					session.updates.push_back(time);
					session.networkSend.push_back(0);
					session.networkRecv.push_back(0);
					session.ping.push_back(lastPing);
					session.memAvail.push_back(memAvail);
					session.cpuLoad.push_back(cpuLoad);
					session.connectionCount.push_back(connectionCount);
				}
			}
			else
			{
				session.prevSend = session.networkSend.back();
				session.prevRecv = session.networkRecv.back();
				session.prevUpdateTime = session.updates.back();
			}

			if (totalSend < session.prevSend)
				totalSend = session.prevSend;
			if (totalRecv < session.prevRecv)
				totalRecv = session.prevRecv;

			//session.lastPing = lastPing;
			//session.memAvail = memAvail;
			session.memTotal = memTotal;
			session.updates.push_back(time);
			session.networkSend.push_back(totalSend);
			session.networkRecv.push_back(totalRecv);
			session.ping.push_back(lastPing);
			session.memAvail.push_back(memAvail);
			session.cpuLoad.push_back(cpuLoad);
			session.connectionCount.push_back(connectionCount);
			
			for (auto& pair : session.drives)
			{
				auto& drive = pair.second;
				drive.busyPercent.resize(session.updates.size());
				drive.readCount.resize(session.updates.size());
				drive.writeCount.resize(session.updates.size());
				drive.readBytes.resize(session.updates.size());
				drive.writeBytes.resize(session.updates.size());
			}

			if (session.prevUpdateTime)
			{
				session.highestSendPerS = Max(session.highestSendPerS, float(totalSend - session.prevSend)/TimeToS(time - session.prevUpdateTime));
				session.highestRecvPerS = Max(session.highestRecvPerS, float(totalRecv - session.prevRecv)/TimeToS(time - session.prevUpdateTime));
			}
			break;
		}
		case TraceType_SessionDisconnect:
		{
			u32 sessionIndex = reader.ReadU32();
			if (out.version < 15)
			{
				time = reader.Read7BitEncoded();
				if (time > maxTime)
				{
					reader.SetPosition(readPos);
					return true;
				}
			}

			TraceView::Session& session = GetSession(out, sessionIndex);
			session.disconnectTime = time;
			session.maxVisibleFiles = 0;
			for (auto it = session.fetchedFiles.begin(), end = session.fetchedFiles.end(); it != end; ++it)
				if (it->stop == ~u64(0))
					it->stop = time;
			for (auto it = session.storedFiles.begin(), end = session.storedFiles.end(); it != end; ++it)
				if (it->stop == ~u64(0))
					it->stop = time;

			--out.activeSessionCount;
			break;
		}
		case TraceType_SessionNotification:
		{
			u32 sessionIndex = reader.ReadU32();
			TraceView::Session& session = GetSession(out, sessionIndex);
			session.notification = reader.ReadString();
			break;
		}
		case TraceType_SessionSummary:
		{
			u32 sessionIndex = reader.ReadU32();
			u32 lineCount = reader.ReadU32();

			TraceView::Session& session = GetSession(out, sessionIndex);
			session.summary.reserve(lineCount);
			for (u32 i=0; i!=lineCount; ++i)
				session.summary.push_back(reader.ReadString());
			break;
		}
		case TraceType_ProcessAdded:
		{
			u32 sessionIndex = reader.ReadU32();
			u32 id = reader.ReadU32();
			TString desc = reader.ReadString();

			TString breadcrumbs;
			if (out.version >= 35)
			{
				if (out.version < 38)
					breadcrumbs = reader.ReadString();
				else if (out.version < 42)
				{
					if (reader.ReadBool())
						breadcrumbs = reader.ReadString();
					else
					{
						reader.Read7BitEncoded();
						reader.Skip(reader.Read7BitEncoded());
						breadcrumbs = TC("Upgrade your visualizer");
					}
				}
				else
					breadcrumbs = reader.ReadLongString();
			}

			if (out.version < 15)
			{
				time = reader.Read7BitEncoded();
				if (time > maxTime)
				{
					reader.SetPosition(readPos);
					return true;
				}
			}

			ProcessBegin(out, sessionIndex, id, time, std::move(desc), std::move(breadcrumbs), TraceView::ProcessType_Normal);
			break;
		}
		case TraceType_ProcessExited:
		{
			u32 id = reader.ReadU32();
			u32 exitCode = reader.ReadU32();
			if (out.version < 15)
			{
				time = reader.Read7BitEncoded();
				if (time > maxTime)
				{
					reader.SetPosition(readPos);
					return true;
				}
			}

			u32 sessionIndex;
			TraceView::Process& process = *ProcessEnd(out, sessionIndex, id, time);

			process.exitCode = exitCode;

			ProcessStats processStats;
			SessionStats sessionStats;
			StorageStats storageStats;
			KernelStats kernelStats;

			const u8* dataStart = reader.GetPositionData();
			processStats.Read(reader, out.version);

			UBA_ASSERT(process.isRemote == (sessionIndex != 0));

			if (process.isRemote)
			{
				if (out.version >= 7)
				{
					sessionStats.Read(reader, out.version);
					storageStats.Read(reader, out.version);
					kernelStats.Read(reader, out.version);
				}
			}
			else if (out.version >= 30)
			{
				if (out.version >= 36)
					sessionStats.Read(reader, out.version);
				storageStats.Read(reader, out.version);
				kernelStats.Read(reader, out.version);
			}
			const u8* dataEnd = reader.GetPositionData();
			process.stats.resize(dataEnd - dataStart);
			memcpy(process.stats.data(), dataStart, dataEnd - dataStart);

			if (out.version >= 34 && out.version < 35)
				process.breadcrumbs = reader.ReadString();

			process.createFilesTime = processStats.createFile.time;
			process.writeFilesTime = Max(processStats.writeFiles.time, processStats.sendFiles.time);

			if (out.version >= 22)
			{
				while (true)
				{
					auto type = (LogEntryType)reader.ReadByte();
					if (type == 255)
						break;
					process.logLines.emplace_back(reader.ReadString(), type);
				}
			}
			else if (out.version >= 20)
			{
				u64 logLineCount = reader.Read7BitEncoded();
				if (logLineCount >= 101)
					logLineCount = 101;
				process.logLines.reserve(logLineCount);
				while (logLineCount--)
				{
					auto type = (LogEntryType)reader.ReadByte();
					process.logLines.emplace_back(reader.ReadString(), type);
				}
			}

			break;
		}
		case TraceType_ProcessEnvironmentUpdated:
		{
			u32 processId = reader.ReadU32();
			auto findIt = m_activeProcesses.find(processId);
			if (findIt == m_activeProcesses.end())
				return false;
			StringBuffer<> reason;
			reader.ReadString(reason);
			if (out.version < 15)
				time = reader.Read7BitEncoded();
			TraceView::ProcessLocation& active = findIt->second;
			auto& session = GetSession(out, active.sessionIndex);

			auto& processes = session.processors[active.processorIndex].processes;
			TraceView::Process& process = processes[active.processIndex];

			const u8* dataStart = reader.GetPositionData();

			ProcessStats processStats;
			SessionStats sessionStats;
			StorageStats storageStats;
			KernelStats kernelStats;
			processStats.Read(reader, out.version);
			if (process.isRemote || out.version < 35)
				sessionStats.Read(reader, out.version);
			storageStats.Read(reader, out.version);
			kernelStats.Read(reader, out.version);

			const u8* dataEnd = reader.GetPositionData();
			process.stats.resize(dataEnd - dataStart);
			memcpy(process.stats.data(), dataStart, dataEnd - dataStart);

			TString breadcrumbs;
			if (out.version >= 35)
			{
				if (out.version < 38)
					breadcrumbs = reader.ReadString();
				else if (out.version < 42)
				{
					if (reader.ReadBool())
						breadcrumbs = reader.ReadString();
					else
					{
						reader.Read7BitEncoded();
						reader.Skip(reader.Read7BitEncoded());
						breadcrumbs = TC("Upgrade your visualizer");
					}
				}
				else
					breadcrumbs = reader.ReadLongString();
			}

			process.isReuse = true;
			process.exitCode = 0u;
			process.stop = time;
			process.bitmapDirty = true;
			process.createFilesTime = processStats.createFile.time;
			process.writeFilesTime = Max(processStats.writeFiles.time, processStats.sendFiles.time);

			bool isRemote = process.isRemote;

			processes.emplace_back();
			auto& process2 = processes.back();
			active.processIndex = u32(processes.size() - 1);
			process2.id = processId;
			process2.breadcrumbs = breadcrumbs;
			process2.start = time;
			process2.stop = ~u64(0);
			process2.exitCode = ~0u;
			process2.isRemote = isRemote;

			if (reason.Equals(TCV("IDLE")))
				process2.isIdle = true;
			else
				process2.description = reason.data;

			++session.processExitedCount;
			++out.totalProcessExitedCount;
			break;
		}
		case TraceType_ProcessReturned:
		{
			u32 id = reader.ReadU32();
			if (out.version < 15)
			{
				time = reader.Read7BitEncoded();
				if (time > maxTime)
				{
					reader.SetPosition(readPos);
					return true;
				}
			}
			TString reason;
			if (out.version >= 33)
				reason = reader.ReadString();
			if (reason.empty())
				reason = TC("Unknown");

			auto findIt = m_activeProcesses.find(id);
			if (findIt == m_activeProcesses.end())
				return false;
			TraceView::ProcessLocation active = findIt->second;
			m_activeProcesses.erase(findIt);

			auto& session = GetSession(out, active.sessionIndex);
			--session.processActiveCount;
			--out.totalProcessActiveCount;

			TraceView::Process& process = session.processors[active.processorIndex].processes[active.processIndex];
			process.exitCode = 0;
			process.stop = time;
			process.returnedReason = reason;
			process.bitmapDirty = true;
			break;
		}
		case TraceType_ProcessRace:
		{
			u32 id = reader.ReadU32();
			u32 remoteId = reader.ReadU32();(void)remoteId;

			auto findIt = m_activeProcesses.find(id);
			if (findIt != m_activeProcesses.end())
			{
				TraceView::ProcessLocation& active = findIt->second;
				auto& session = GetSession(out, active.sessionIndex);

				auto& processes = session.processors[active.processorIndex].processes;
				TraceView::Process& process = processes[active.processIndex];
				process.isRacing = true;
			}
			break;
		}
		case TraceType_FileFetchBegin:
		{
			Guid clientUid = ReadClientId(out, reader);
			CasKey key = reader.ReadCasKey();
			u64 size = 0;
			if (out.version < 36)
				size = reader.Read7BitEncoded();

			StringBuffer<> temp;
			const tchar* hint;
			if (out.version < 14)
			{
				reader.ReadString(temp);
				hint = temp.data;
			}
			else
				hint = out.strings[reader.Read7BitEncoded()];
				
			if (out.version < 15)
			{
				time = reader.Read7BitEncoded();
				if (time > maxTime)
				{
					reader.SetPosition(readPos);
					return true;
				}
			}

			if (auto session = GetSession(out, clientUid))
			{
				session->fetchedFilesActive.try_emplace(key, u32(session->fetchedFiles.size()));
				session->fetchedFiles.push_back({ key, size, hint, time, ~u64(0) });
			}
			break;
		}
		case TraceType_FileFetchLight:
		{
			Guid clientUid = ReadClientId(out, reader);
			u64 fileSize;
			if (!reader.TryRead7BitEncoded(fileSize))
				return false;
			if (auto session = GetSession(out, clientUid))
			{
				session->fetchedFilesBytes += fileSize;
				++session->fetchedFilesCount;
			}
			break;
		}
		case TraceType_ProxyCreated:
		{
			Guid clientUid = ReadClientId(out, reader);
			StringBuffer<> proxyName;
			reader.ReadString(proxyName);
			if (out.version < 15)
			{
				time = reader.Read7BitEncoded();
				if (time > maxTime)
				{
					reader.SetPosition(readPos);
					return true;
				}
			}

			if (auto session = GetSession(out, clientUid))
			{
				session->proxyName = proxyName.data;
				session->proxyCreated = true;
			}
			break;
		}
		case TraceType_ProxyUsed:
		{
			Guid clientUid = ReadClientId(out, reader);
			StringBuffer<> proxyName;
			reader.ReadString(proxyName);

			if (out.version < 15)
			{
				time = reader.Read7BitEncoded();
				if (time > maxTime)
				{
					reader.SetPosition(readPos);
					return true;
				}
			}

			if (auto session = GetSession(out, clientUid))
				session->proxyName = proxyName.data;
			break;
		}
		case TraceType_FileFetchSize:
		{
			Guid clientUid = ReadClientId(out, reader);
			CasKey key = reader.ReadCasKey();
			u64 fileSize = reader.Read7BitEncoded();
			if (auto session = GetSession(out, clientUid))
			{
				auto findIt = session->fetchedFilesActive.find(key);
				if (findIt != session->fetchedFilesActive.end())
				{
					auto& file = session->fetchedFiles[findIt->second];
					file.size = fileSize;
				}
			}
			break;
		}
		case TraceType_FileFetchEnd:
		{
			Guid clientUid = ReadClientId(out, reader);
			CasKey key = reader.ReadCasKey();

			if (out.version < 15)
			{
				time = reader.Read7BitEncoded();
				if (time > maxTime)
				{
					reader.SetPosition(readPos);
					return true;
				}
			}

			if (auto session = GetSession(out, clientUid))
			{
				auto findIt = session->fetchedFilesActive.find(key);
				if (findIt != session->fetchedFilesActive.end())
				{
					auto& file = session->fetchedFiles[findIt->second];
					file.stop = time;
					session->fetchedFilesBytes += file.size;
					++session->fetchedFilesCount;
					session->fetchedFilesActive.erase(findIt);
					break;
				}
			}
			break;
		}
		case TraceType_FileStoreBegin:
		{
			Guid clientUid = ReadClientId(out, reader);
			CasKey key = reader.ReadCasKey();
			u64 size = reader.Read7BitEncoded();
			StringBuffer<> temp;
			const tchar* hint;
			if (out.version < 14)
			{
				reader.ReadString(temp);
				hint = temp.data;
			}
			else
				hint = out.strings[reader.Read7BitEncoded()];
			if (out.version < 15)
			{
				time = reader.Read7BitEncoded();
				if (time > maxTime)
				{
					reader.SetPosition(readPos);
					return true;
				}
			}

			if (auto session = GetSession(out, clientUid))
			{
				session->storedFilesActive.try_emplace(key, u32(session->storedFiles.size()));
				session->storedFiles.push_back({ key, size, hint, time, ~u64(0) });
				session->storedFilesBytes += size;
				++session->storedFilesCount;
			}
			break;
		}
		case TraceType_FileStoreLight:
		{
			Guid clientUid = ReadClientId(out, reader);
			u64 fileSize = reader.Read7BitEncoded();
			if (auto session = GetSession(out, clientUid))
			{
				session->storedFilesBytes += fileSize;
				++session->storedFilesCount;
			}
			break;
		}
		case TraceType_FileStoreEnd:
		{
			Guid clientUid = ReadClientId(out, reader);
			CasKey key = reader.ReadCasKey();

			if (out.version < 15)
			{
				time = reader.Read7BitEncoded();
				if (time > maxTime)
				{
					reader.SetPosition(readPos);
					return true;
				}
			}

			if (auto session = GetSession(out, clientUid))
			{
				auto findIt = session->storedFilesActive.find(key);
				if (findIt != session->storedFilesActive.end())
				{
					auto& file = session->storedFiles[findIt->second];
					file.stop = time;
					session->storedFilesActive.erase(findIt);
					break;
				}
			}
			break;
		}
		case TraceType_Summary:
		{
			if (out.version < 15)
				time = reader.Read7BitEncoded();
			StopAllActive(out, time);
			return false;
		}
		case TraceType_WorkBegin:
		{
			u32 workIndex;
			if (out.version < 14)
				workIndex = reader.ReadU32();
			else
				workIndex = u32(reader.Read7BitEncoded());

			u32 trackIndex = 0;
			TraceView::WorkTrack* workTrack = nullptr;
			for (u32 i=0, e=u32(out.workTracks.size()); i!=e; ++i)
			{
				auto& it = out.workTracks[i];
				if (it.records.back().stop == ~u64(0))
					continue;
				trackIndex = i;
				workTrack = &it;
				break;
			}
			if (!workTrack)
			{
				trackIndex = u32(out.workTracks.size());
				out.workTracks.emplace_back();
				workTrack = &out.workTracks.back();
			}

			workTrack->records.emplace_back();
			auto& record = workTrack->records.back();

			m_activeWorkRecords.try_emplace(workIndex, WorkRecordLocation{trackIndex, u32(workTrack->records.size() - 1)});

			u64 stringIndex;
			if (out.version < 14)
				stringIndex = reader.ReadU32();
			else
				stringIndex = reader.Read7BitEncoded();
				
			if (out.version >= 38)
				record.color = reader.ReadU32();

			record.description = out.strings[stringIndex];
			if (out.version < 15)
				record.start = reader.Read7BitEncoded();
			else
				record.start = time;
			record.stop = ~u64(0);
			break;
		}
		case TraceType_WorkEnd:
		{
			u32 workIndex;
			if (out.version < 14)
				workIndex = reader.ReadU32();
			else
				workIndex = u32(reader.Read7BitEncoded());

			u64 stop;
			if (out.version < 15)
				stop = reader.Read7BitEncoded();
			else
				stop = time;

			auto findIt = m_activeWorkRecords.find(workIndex);
			if (findIt == m_activeWorkRecords.end())
				return true;

			WorkRecordLocation active = findIt->second;
			m_activeWorkRecords.erase(findIt);
			auto& record = out.workTracks[active.track].records[active.index];
			record.stop = stop;
			record.bitmapDirty = true;

			break;
		}
		case TraceType_TaskBegin:
		{
			u32 id = u32(reader.Read7BitEncoded());
			TString desc = reader.ReadString();
			TString details = reader.ReadString();
			u32 color =  reader.ReadU32();(void)color;
			ProcessBegin(out, 0, id, time, std::move(desc), std::move(details), TraceView::ProcessType_Task);
			break;
		}
		case TraceType_TaskHint:
		{
			u32 id = u32(reader.Read7BitEncoded());(void)id;
			TString hint = reader.ReadString();
			break;
		}
		case TraceType_TaskEnd:
		{
			u32 id = u32(reader.Read7BitEncoded());
			bool success = reader.ReadBool();

			u32 sessionIndex;
			if (TraceView::Process* process = ProcessEnd(out, sessionIndex, id, time))
				process->exitCode = success ? 0 : -1;
			break;
		}
		case TraceType_ProgressUpdate:
		{
			out.progressProcessesTotal = u32(reader.Read7BitEncoded());
			out.progressProcessesDone = u32(reader.Read7BitEncoded());
			out.progressErrorCount = u32(reader.Read7BitEncoded());
			break;
		}
		case TraceType_DriveUpdate:
		{
			u8 driveLetter = reader.ReadByte();
			u8 busyPercent = reader.ReadByte();
			u32 readCount = u32(reader.Read7BitEncoded());
			u64 readBytes = reader.Read7BitEncoded();
			u32 writeCount = u32(reader.Read7BitEncoded());
			u64 writeBytes = reader.Read7BitEncoded();

			if (out.sessions.empty())
				break;
			auto& session = out.sessions[0];
			auto& drive = session.drives[driveLetter];

			if (drive.busyPercent.empty())
			{
				u64 updatesCount = session.updates.size(); // We get these before update
				drive.busyPercent.resize(updatesCount);
				drive.readCount.resize(updatesCount);
				drive.writeCount.resize(updatesCount);
				drive.readBytes.resize(updatesCount);
				drive.writeBytes.resize(updatesCount);
			}

			drive.busyHighest = Max(drive.busyHighest, busyPercent);

			drive.busyPercent.push_back(busyPercent);
			drive.totalReadCount += readCount;
			drive.totalWriteCount += writeCount;
			drive.totalReadBytes += readBytes;
			drive.totalWriteBytes += writeBytes;
			drive.readCount.push_back(readCount);
			drive.readBytes.push_back(readBytes);
			drive.writeCount.push_back(writeCount);
			drive.writeBytes.push_back(writeBytes);
			break;
		}
		case TraceType_StatusUpdate:
		{
			if (out.version < 32)
			{
				reader.Read7BitEncoded();
				reader.Read7BitEncoded();
				reader.ReadString();
				reader.Read7BitEncoded();
				reader.ReadString();
				reader.ReadByte();
			}
			else
			{
				u64 row = reader.Read7BitEncoded();
				u64 column = reader.Read7BitEncoded();
				u64 key = (row << 32) | column;
				auto& status = out.statusMap[key];
				status.text = reader.ReadString();
				status.type = (LogEntryType)reader.ReadByte();
				status.link = reader.ReadString();
			}

			break;
		}
		case TraceType_StatusUpdate2:
		{
			u64 row = reader.Read7BitEncoded();
			u64 column = reader.Read7BitEncoded();
			u64 key = (row << 32) | column;
			auto& status = out.statusMap[key];
			u64 stringIndex = reader.Read7BitEncoded();
			status.text = out.strings[stringIndex];
			status.type = LogEntryType_Info;
			break;
		}
		case TraceType_TableUpdate:
		{
			u32 name = u32(reader.Read7BitEncoded());
			TraceView::Table& table = out.tables[name];
			table.rowCharWidth = u32(reader.Read7BitEncoded());
			table.rowMaxCount = u32(reader.Read7BitEncoded());
			u32 itemCount = u32(reader.Read7BitEncoded());
			table.items.resize(itemCount);
			for (u32 i = 0; i != itemCount; ++i)
				table.items[i] = out.strings[reader.Read7BitEncoded()];
			break;
		}
		case TraceType_ProcessBreadcrumbs:
		{
			u32 processId = reader.ReadU32();
			TString breadcrumbs;
			if (out.version < 38)
				breadcrumbs = reader.ReadString();
			else
				breadcrumbs = reader.ReadLongString();
			bool deleteOld = reader.ReadBool();

			auto writeBreadcrumb = [&](TraceView::Process& process)
				{
					if (deleteOld)
						process.breadcrumbs.clear();
					else if (!process.breadcrumbs.empty())
						process.breadcrumbs += '\n';
					process.breadcrumbs += breadcrumbs;
				};

			auto findIt = m_activeProcesses.find(processId);
			if (findIt != m_activeProcesses.end())
			{
				auto& active = findIt->second;
				writeBreadcrumb(GetSession(out, active.sessionIndex).processors[active.processorIndex].processes[active.processIndex]);
				break;
			}

			// Process is not active anymore, search for it the slow way
			for (auto& session : out.sessions)
				for (auto& processor : session.processors)
					for (auto& process : processor.processes)
						if (process.id == processId)
						{
							writeBreadcrumb(process);
							return true;
						}
			break;
		}
		case TraceType_RemoteExecutionDisabled:
		{
			out.remoteExecutionDisabled = true;
			if (!out.sessions.empty())
				out.sessions[0].notification = TC("(Remote scheduling finished)");
			break;
		}
		case TraceType_String:
		{
			out.strings.push_back(TStrdup(reader.ReadString().data()));
			break;
		}
		case TraceType_CacheBeginFetch:
		{
			u32 id = u32(reader.Read7BitEncoded());
			TString desc = reader.ReadString();
			ProcessBegin(out, 0, id, time, std::move(desc), {}, TraceView::ProcessType_CacheFetchTest);
			break;
		}
		case TraceType_CacheEndFetch:
		{
			u32 id = u32(reader.Read7BitEncoded());
			bool success = reader.ReadBool();

			u32 sessionIndex;
			TraceView::Process& process = *ProcessEnd(out, sessionIndex, id, time);
			process.exitCode = 0;//success ? 0 : -1;
			if (!success)
				process.returnedReason = TC("M");

			CacheFetchStats cacheStats;
			KernelStats kernelStats;
			StorageStats storageStats;
			const u8* dataStart = reader.GetPositionData();
			cacheStats.Read(reader, out.version);

			if (!cacheStats.fetchEntries.count)
				process.type = TraceView::ProcessType_CacheFetchDownload;

			if (success || out.version >= 29)
			{
				storageStats.Read(reader, out.version);
				kernelStats.Read(reader, out.version);
			}
			const u8* dataEnd = reader.GetPositionData();
			process.stats.resize(dataEnd - dataStart);
			memcpy(process.stats.data(), dataStart, dataEnd - dataStart);
			break;
		}
		case TraceType_CacheBeginWrite:
		{
			u32 processId = u32(reader.Read7BitEncoded());
			out.cacheWrites[processId].start = time;
			break;
		}
		case TraceType_CacheEndWrite:
		{
			u32 processId = u32(reader.Read7BitEncoded());
			TraceView::CacheWrite& write = out.cacheWrites[processId];
			write.success = reader.ReadBool();

			CacheSendStats stats;
			const u8* dataStart = reader.GetPositionData();
			if (out.version < 45)
				stats.sendFile.bytes = reader.Read7BitEncoded();
			else
				stats.Read(reader, out.version);

			const u8* dataEnd = reader.GetPositionData();
			write.stats.resize(dataEnd - dataStart);
			memcpy(write.stats.data(), dataStart, dataEnd - dataStart);

			write.end = time;
			break;
		}
		case TraceType_WorkHint:
		{
			u32 workIndex = u32(reader.Read7BitEncoded());
			u64 stringIndex = reader.Read7BitEncoded();
			u64 startTime = ConvertTime(out, reader.Read7BitEncoded());
			auto findIt = m_activeWorkRecords.find(workIndex);
			if (findIt == m_activeWorkRecords.end())
				return false;
			const tchar* text = out.strings[stringIndex];
			WorkRecordLocation active = findIt->second;
			TraceView::WorkRecord& record = out.workTracks[active.track].records[active.index];

			bool handled = false;
			// Collapse if already exists
			if (startTime)
			{
				for (auto rit=record.entries.rbegin(), rend=record.entries.rend(); rit!=rend; ++rit)
				{
					auto& entry = *rit;
					if (!entry.startTime)
						break;
					if (entry.text != text)
						continue;
					++entry.count;
					u64 entryTime = entry.time - entry.startTime;
					u64 newTime = time - startTime;
					if (newTime > entryTime)
					{
						entry.time = time;
						entry.startTime = startTime;
					}
					handled = true;
					break;
				}
			}

			if (!handled)
				record.entries.emplace_back(time, startTime, text); 
			break;
		}
		case TraceType_CacheSummary:
		{
			auto& cacheSummary = out.cacheSummaries.emplace_back();

			u32 lineCount = reader.ReadU32();
			cacheSummary.lines.reserve(lineCount);
			for (u32 i=0; i!=lineCount; ++i)
				cacheSummary.lines.push_back(reader.ReadString());
			break;
		}
		case TraceType_SchedulerUpdate:
		{
			if (reader.ReadBool())
			{
				out.lastSpawningDelayStartTime = time;
				out.lastSpawningDelayEndTime = ~0llu - MsToTime(10000);
			}
			else
				out.lastSpawningDelayEndTime = time;
			break;
		}
		case TraceType_SchedulerKillProcess:
		{
			u32 processId = reader.ReadU32();(void)processId;

			const tchar* killReason = nullptr;
			if (out.version >= 52)
				killReason = out.strings[reader.Read7BitEncoded()];
			out.lastKillProcessTime = time;
			out.lastKillReason = killReason;

			auto findIt = m_activeProcesses.find(processId);
			if (findIt != m_activeProcesses.end())
			{
				auto& active = findIt->second;
				TraceView::Process& process = GetSession(out, active.sessionIndex).processors[active.processorIndex].processes[active.processIndex];
				process.killReason = killReason;
			}
			break;
		}
		default:
		{
			return m_logger.Error(TC("Unknown trace type found in stream. UbaVisualizer too old?"));
		}
		}
		return true;
	}

	void TraceReader::StopAllActive(TraceView& out, u64 stopTime)
	{
		for (auto& pair : m_activeProcesses)
		{
			auto& active = pair.second;
			TraceView::Process& process = GetSession(out, active.sessionIndex).processors[active.processorIndex].processes[active.processIndex];
			process.exitCode = ~0u;
			process.stop = stopTime;
			process.bitmapDirty = true;
		}

		for (auto& pair : m_activeWorkRecords)
		{
			const WorkRecordLocation& active = pair.second;
			TraceView::WorkRecord& record = out.workTracks[active.track].records[active.index];
			record.stop = stopTime;
		}

		m_activeProcesses.clear();
		out.finished = true;
	}

	void TraceReader::Reset(TraceView& out)
	{
		out.Clear();
		m_activeProcesses.clear();
		m_activeWorkRecords.clear();
		m_sessionIndexToSession.clear();
	}

	void TraceReader::Unmap()
	{
#if PLATFORM_WINDOWS
		u64 unmapSize = 0;
		if (m_hostProcess)
			CloseHandle(m_hostProcess);
#else
		u64 unmapSize = 128 * 1024 * 1024;
#endif
		m_hostProcess = 0;

		if (m_memoryBegin)
			FileMapping_Unmap(m_logger, m_memoryBegin, unmapSize, TC("TraceReader"));
		m_memoryBegin = nullptr;
		if (m_memoryHandle.IsValid())
			FileMapping_Close(m_logger, m_memoryHandle, TC("TraceReader"));
		m_memoryHandle = {};
		m_namedTrace.clear();
	}

	bool TraceReader::SaveAs(const tchar* fileName)
	{
		if (!m_memoryBegin)
			return m_logger.Warning(TC("Can only save traces that are opened using -listen/-name."));

		FileAccessor file(m_logger, fileName);
		if (!file.CreateWrite())
			return false;
		if (!file.Write(m_memoryBegin, m_memoryPos - m_memoryBegin))
			return false;
		return file.Close();
	}

	Guid TraceReader::ReadClientId(TraceView& out, BinaryReader& reader)
	{
		if (out.version < 15)
			return reader.ReadGuid();
		Guid clientUid;
		clientUid.data1 = u32(reader.Read7BitEncoded());
		return clientUid;
	}

	TraceView::Session& TraceReader::GetSession(TraceView& out, u32 sessionIndex)
	{
		return out.sessions[m_sessionIndexToSession[sessionIndex]];
	}

	TraceView::Session* TraceReader::GetSession(TraceView& out, const Guid& clientUid)
	{
		for (auto& session : out.sessions)
			if (session.clientUid == clientUid)
				return &session;
		// First file can be retrieved here before session is connected... haven't figured out how this can happen but let's ignore that for now :-)
		//UBA_ASSERTF(false, TC("Failed to get session for clientUid %ls"), GuidToString(clientUid).str);
		return nullptr;
	}

	TraceView::Process* TraceReader::ProcessBegin(TraceView& out, u32 sessionIndex, u32 id, u64 time, TString&& description, TString&& breadcrumbs, TraceView::ProcessType type)
	{
		TraceView::Processor* processor = nullptr;
		TraceView::Session& session = GetSession(out, sessionIndex);

		u32 processorIndex = 0;
		for (u32 i=0, e=u32(session.processors.size()); i!=e; ++i)
		{
			auto& it = session.processors[i];
			if (it.processes.back().stop == ~u64(0))
				continue;
			processorIndex = i;
			processor = &it;
			break;
		}
		if (!processor)
		{
			processorIndex = u32(session.processors.size());
			session.processors.emplace_back();
			processor = &session.processors.back();
		}

		processor->processes.emplace_back();
		auto& process = processor->processes.back();

		m_activeProcesses.try_emplace(id, TraceView::ProcessLocation{sessionIndex, processorIndex, u32(processor->processes.size() - 1)});
		
		u16 activeCount = u16(m_activeProcesses.size());
		out.activeProcessCounts.emplace_back(time, activeCount);
		out.maxActiveProcessCount = Max(out.maxActiveProcessCount, activeCount);

		if (type == TraceView::ProcessType_Normal)
		{
			++session.processActiveCount;
			++out.totalProcessActiveCount;
		}

		process.id = id;
		process.type = type;
		process.description = std::move(description);
		process.breadcrumbs = std::move(breadcrumbs);
		process.start = time;
		process.stop = ~u64(0);
		process.exitCode = ~0u;
		process.isRemote = sessionIndex != 0;
		return &process;
	}

	TraceView::Process* TraceReader::ProcessEnd(TraceView& out, u32& outSessionIndex, u32 id, u64 time)
	{
		auto findIt = m_activeProcesses.find(id);
		if (findIt == m_activeProcesses.end())
			return nullptr;
		TraceView::ProcessLocation active = findIt->second;
		m_activeProcesses.erase(findIt);
		outSessionIndex = active.sessionIndex;
		auto& session = GetSession(out, active.sessionIndex);

		TraceView::Process& process = session.processors[active.processorIndex].processes[active.processIndex];

		if (process.type == TraceView::ProcessType_Normal)
		{
			++session.processExitedCount;
			--session.processActiveCount;
			++out.totalProcessExitedCount;
			--out.totalProcessActiveCount;
		}

		out.activeProcessCounts.emplace_back(time, u16(m_activeProcesses.size()));

		process.stop = time;
		process.bitmapDirty = true;
		return &process;
	}

	TraceView::SessionInfo TraceReader::ReadSessionInfo(StringBufferBase& info)
	{
		TString hyperlink;
		const tchar* hyperlinkPos = nullptr;
		if (!info.Contains(TC("http://"), true, &hyperlinkPos))
			info.Contains(TC("https://"), true, &hyperlinkPos);
		if (hyperlinkPos)
		{
			const tchar* hyperlinkEnd = info.data + info.count;
			if (const tchar* space = TStrchr(hyperlinkPos, ' '))
				hyperlinkEnd = space;

			hyperlink.assign(hyperlinkPos, hyperlinkEnd);
			if (*hyperlinkEnd == ' ')
				++hyperlinkEnd;
			while (hyperlinkPos[-1] == ' ')
				--hyperlinkPos;
			if (hyperlinkPos[-1] == ',')
				--hyperlinkPos;
			memmove(const_cast<tchar*>(hyperlinkPos), hyperlinkEnd, (info.data + info.count - hyperlinkEnd)*sizeof(tchar));
			info.Resize(info.count - (hyperlinkEnd - hyperlinkPos));
		}
		return { info.data, hyperlink };
	}
}
