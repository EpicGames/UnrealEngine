// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaMap.h"
#include "UbaProcessHandle.h"
#include "UbaStats.h"
#include "UbaTrace.h"

#if !PLATFORM_WINDOWS
typedef void* HBITMAP;
#endif

namespace uba
{
	class NetworkClient;

	class TraceView
	{
	public:
		enum ProcessType : u8 { ProcessType_Normal, ProcessType_CacheFetchTest, ProcessType_CacheFetchDownload, ProcessType_Task };

		struct BitmapCache
		{
			HBITMAP bitmap = 0;
			u32 bitmapOffset = 0;
			bool bitmapDirty = true;
		};

		struct Process : BitmapCache
		{
			u32 id = 0;
			u32 exitCode = ~0u;
			u64 start = 0;
			u64 stop = 0;
			TString description;
			TString returnedReason;
			TString breadcrumbs;
			ProcessType type = ProcessType_Normal;
			bool isRemote = false;
			bool isReuse = false;
			bool isRacing = false;
			bool isIdle = false;
			u64 createFilesTime = 0;
			u64 writeFilesTime = 0;
			Vector<u8> stats;
			Vector<ProcessLogLine> logLines;
			const tchar* killReason = nullptr;
		};

		struct Processor
		{
			Vector<Process> processes;
		};

		struct WorkRecordLogEntry
		{
			u64 time = 0;
			u64 startTime = 0;
			const tchar* text;
			u32 count = 1;
		};

		struct WorkRecord : BitmapCache
		{
			const tchar* description = nullptr;
			u64 start = 0;
			u64 stop = 0;
			Vector<WorkRecordLogEntry> entries;
			u32 color = ColorWork;
		};

		struct WorkTrack
		{
			Vector<WorkRecord> records;
		};


		struct FileTransfer
		{
			CasKey key;
			u64 size;
			TString hint;
			u64 start;
			u64 stop;
		};

		struct StatusUpdate
		{
			TString text;
			LogEntryType type;
			TString link;
		};

		struct Drive
		{
			u8 busyHighest = 0;
			u32 totalReadCount = 0;
			u32 totalWriteCount = 0;
			u64 totalReadBytes = 0;
			u64 totalWriteBytes = 0;
			Vector<u8> busyPercent;
			Vector<u32> readCount;
			Vector<u32> writeCount;
			Vector<u64> readBytes;
			Vector<u64> writeBytes;
		};

		struct SessionInfo
		{
			TString text;
			TString hyperlink;
		};

		struct Session
		{
			TString name;
			TString machineId;
			List<SessionInfo> infos;
			Guid clientUid;
			Vector<Processor> processors;
			Vector<u64> updates;
			Vector<u64> networkSend;
			Vector<u64> networkRecv;
			Vector<u64> ping;
			Vector<u64> memAvail;
			Vector<float> cpuLoad;
			Vector<u16> connectionCount;
			Vector<u32> reconnectIndices;

			Vector<TString> summary;
			UnorderedMap<CasKey, u32> fetchedFilesActive;
			Vector<FileTransfer> fetchedFiles;
			UnorderedMap<CasKey, u32> storedFilesActive;
			Vector<FileTransfer> storedFiles;
			Map<char, Drive> drives;
			TString notification;
			u64 fetchedFilesBytes = 0;
			u64 storedFilesBytes = 0;
			u32 fetchedFilesCount = 0;
			u32 storedFilesCount = 0;
			u32 maxVisibleFiles = 0;
			u32 fullNameWidth = 0;

			float highestSendPerS = 0;
			float highestRecvPerS = 0;

			bool isReset = true;
			u64 disconnectTime = ~u64(0);
			u64 prevUpdateTime = 0;
			u64 prevSend = 0;
			u64 prevRecv = 0;
			u64 memTotal = 0;
			u32 processActiveCount = 0;
			u32 processExitedCount = 0;

			TString proxyName;
			bool proxyCreated = false;

			void GetFullName(StringBufferBase& out);
		};

		struct ProcessLocation
		{
			u32 sessionIndex = 0;
			u32 processorIndex = 0;
			u32 processIndex = 0;
			bool operator==(const ProcessLocation& o) const { return sessionIndex == o.sessionIndex && processorIndex == o.processorIndex && processIndex == o.processIndex; }
		};

		struct CacheWrite
		{
			u64 start = 0;
			u64 end = 0;
			Vector<u8> stats;
			bool success = false;
		};

		const Process& GetProcess(const ProcessLocation& loc);
		const Session& GetSession(const ProcessLocation& loc);
		void Clear();

		struct ActiveProcessCount
		{
			u64 time;
			u16 count;
		};

		struct CacheSummary
		{
			Vector<TString> lines;
		};

		struct Table
		{
			Vector<const tchar*> items;
			u32 rowCharWidth;
			u32 rowMaxCount;
		};

		Vector<Session> sessions;
		Vector<CacheSummary> cacheSummaries;
		Vector<WorkTrack> workTracks;
		Vector<tchar*> strings;
		Vector<ActiveProcessCount> activeProcessCounts;
		Map<u64, StatusUpdate> statusMap;
		Map<u32, CacheWrite> cacheWrites;
		Map<u32, Table> tables;
		u64 realStartTime = 0;
		u64 traceSystemStartTimeUs = 0;
		u64 startTime = 0;
		u64 frequency = 0;
		double timeMultiplier = 0;
		u64 lastKillProcessTime = 0;
		const tchar* lastKillReason = nullptr;
		u64 lastSpawningDelayStartTime = 0;
		u64 lastSpawningDelayEndTime = 0;
		u32 totalProcessActiveCount = 0;
		u32 totalProcessExitedCount = 0;
		u32 activeSessionCount = 0;
		u32 version = 0;
		u32 progressProcessesTotal = 0;
		u32 progressProcessesDone = 0;
		u32 progressErrorCount = 0;
		u16 maxActiveProcessCount = 0;
		bool remoteExecutionDisabled = false;
		bool finished = true;
	};


	class TraceReader
	{
	public:
		TraceReader(Logger& logger);
		~TraceReader();

		// Use for file read
		bool ReadFile(TraceView& out, const tchar* fileName, bool replay);
		bool UpdateReadFile(TraceView& out, u64 maxTime, bool& outChanged);

		// Use for network
		bool StartReadClient(TraceView& out, NetworkClient& client);
		bool UpdateReadClient(TraceView& out, NetworkClient& client, bool& outChanged);
		bool UpdateReceiveClient(NetworkClient& client);

		// Use for local
		bool StartReadNamed(TraceView& out, const tchar* namedTrace, bool silentFail = false, bool replay = false);
		bool UpdateReadNamed(TraceView& out, u64 maxTime, bool& outChanged);

		bool ReadMemory(TraceView& out, bool trackHost, u64 maxTime);
		bool ReadTrace(TraceView& out, BinaryReader& reader, u64 maxTime);
		void StopAllActive(TraceView& out, u64 stopTime);
		void Reset(TraceView& out);
		void Unmap();

		bool SaveAs(const tchar* fileName);

		Guid ReadClientId(TraceView& out, BinaryReader& reader);
		TraceView::Session& GetSession(TraceView& out, u32 sessionIndex);
		TraceView::Session* GetSession(TraceView& out, const Guid& clientUid);

		TraceView::Process* ProcessBegin(TraceView& out, u32 sessionIndex, u32 id, u64 time, TString&& description, TString&& breadcrumbs, TraceView::ProcessType type);
		TraceView::Process* ProcessEnd(TraceView& out, u32& outSessionIndex, u32 id, u64 time);

		TraceView::SessionInfo ReadSessionInfo(StringBufferBase& info);

		UnorderedMap<u32, TraceView::ProcessLocation> m_activeProcesses;

		struct WorkRecordLocation { u32 track; u32 index; };
		UnorderedMap<u32, WorkRecordLocation> m_activeWorkRecords;

		Vector<u32> m_sessionIndexToSession;

		Logger& m_logger;
		TraceChannel m_channel;
		FileMappingHandle m_memoryHandle;
		TString m_namedTrace;
		u8* m_memoryBegin = nullptr;
		u8* m_memoryPos = nullptr;
		u8* m_memoryEnd = nullptr;
		Futex m_memoryFutex;

		#if PLATFORM_WINDOWS
		HANDLE m_hostProcess = NULL;
		#else
		pid_t m_hostProcess = 0;
		#endif
	};
}