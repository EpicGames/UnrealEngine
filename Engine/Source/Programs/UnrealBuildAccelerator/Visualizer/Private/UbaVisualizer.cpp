// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaVisualizer.h"
#include "UbaLogger.h"
#include <algorithm>
#include <string>

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	u64 ConvertTime(const TraceView& view, u64 time);

	#if !PLATFORM_WINDOWS
	void OffsetRect(RECT* r, int x, int y) { r->left += x; r->right += x; r->top += y; r->bottom += y; }
	#endif

	////////////////////////////////////////////////////////////////////////////////////////////////////

	struct Visualizer::DrawTextLogger : public Logger
	{
		DrawTextLogger(PaintContext& c, int fh, BrushHandle bb)
			:	context(c)
			,	fontHeight(fh)
			,	backgroundBrush(bb)
		{
			textColor = context.GetTextColor();
		}

		virtual void BeginScope() override {}
		virtual void EndScope() override {}
		virtual void Log(LogEntryType type, const tchar* str, u32 strLen) override
		{
			u32 textWidth = context.GetTextSize(StringView(str, strLen)).cx;

			lines.emplace_back(TString(str, strLen), textOffset, height, textColor);
			width = Max(width, int(textWidth + textOffset));
			height += fontHeight;
		}

		void AddSpace(int space = 5)
		{
			height += space;
		}

		void AddTextOffset(int offset)
		{
			textOffset += offset;
		}

		void AddWidth(int extra)
		{
			extraWidth += extra;
		}

		void DrawAtPos(int x, int y)
		{
			RECT r;
			r.left = x;
			r.top = y;
			r.right = r.left + width;
			r.bottom = r.top + height;

			SIZE clientSize = context.GetClientSize();

			if (r.right > clientSize.cx)
			{
				OffsetRect(&r, -width - 15, 0);
				if (r.left < 0)
					OffsetRect(&r, -r.left, 0);
			}
			if (r.bottom > clientSize.cy)
			{
				OffsetRect(&r, 0, clientSize.cy - r.bottom);
				if (r.top < 0)
					OffsetRect(&r, 0, -r.top);
			}

			RECT fillRect = r;
			fillRect.right += 2 + extraWidth;
			context.FillRect(fillRect, backgroundBrush);

			for (auto& line : lines)
			{
				RECT tr = r;
				tr.left += line.left;
				tr.top += line.top;
				context.SetTextColor(line.color);
				context.Text(tr.left, tr.top, line.str, nullptr);
			}
		}

		void DrawAtCursor()
		{
			POINT p = context.GetCursorClientPos();
			p.x += 3;
			p.y += 3;
			DrawAtPos(p.x, p.y);
		}

		DrawTextLogger& SetColor(COLORREF c) { textColor = c; return *this; }

		int width = 0;
		int height = 0;
		int textOffset = 2;
		int extraWidth = 0;
		struct Line { TString str; int left; int top; COLORREF color; };
		Vector<Line> lines;

		PaintContext& context;
		int fontHeight;
		BrushHandle backgroundBrush;
		COLORREF textColor;
		bool isFirst = true;
		using Logger::Log;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void Visualizer::DrawCenteredText(VisualizerBackend::PaintContext& context, const StringBuffer<>* rows, u32 rowCount, BrushHandle backgroundBrush)
	{
		SIZE clientSize = context.GetClientSize();
		for (u32 i=0; i!=rowCount; ++i)
		{ 
			SIZE textSize = context.GetTextSize(rows[i]);

			RECT rect = { 0, 0, 0, 0 };
			int top = (clientSize.cy - textSize.cy)/2 + i*20;
			rect.left = (clientSize.cx - textSize.cx)/2;
			rect.right = (clientSize.cx + textSize.cx)/2;
			rect.bottom = top + textSize.cy;
			rect.top = top;

			if (backgroundBrush)
			{
				RECT r = rect;
				r.left -= 4;
				r.right += 4;
				r.top -= 2;
				r.bottom += 2;
				context.FillRect(r, backgroundBrush);
			}
			context.Text(rect.left, rect.top, rows[i], nullptr);
		}
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	bool Visualizer::Stats::operator==(const Stats& o) const
	{
		if (memcmp(this, &o, offsetof(Stats, connectionCount)) != 0)
			return false;
		if (connectionCount != o.connectionCount)
			return false;
		if (drives.size() != o.drives.size())
			return false;
		for (auto i=drives.begin(), i2=o.drives.begin(); i!=drives.end(); ++i, ++i2)
		{
			if (i->first != i2->first)
				return false;
			const Drive& a = i->second;
			const Drive& b = i2->second;
			if (a.busyPercent != b.busyPercent || a.readPerSecond != b.readPerSecond || a.writePerSecond != b.writePerSecond)
				return false;
		}
		return true;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	Visualizer::Visualizer(VisualizerConfig& config, Logger& logger, VisualizerBackend& backend)
	:	m_logger(logger)
	,	m_config(config)
	,	m_trace(logger)
	,	m_backend(backend)
	{
	}

	void Visualizer::InitBrushes()
	{
		auto CreateBrush = [&](u8 r, u8 g, u8 b) { return m_backend.CreateSolidBrush(RGB(r, g, b)); };

		if (m_config.DarkMode)
		{
			m_textColor = RGB(190, 190, 190);
			m_textWarningColor = RGB(190, 190, 0);
			m_textErrorColor = RGB(190, 0, 0);

			m_processBrushes[0].inProgress = CreateBrush(70, 70, 160);
			m_processBrushes[1].inProgress = CreateBrush(80, 80, 170);
			m_processBrushes[0].inProgressRace = CreateBrush(60, 60, 120);
			m_processBrushes[1].inProgressRace = CreateBrush(80, 80, 140);

			m_processBrushes[0].error = CreateBrush(140, 0, 0);
			m_processBrushes[1].error = CreateBrush(190, 0, 0);

			m_processBrushes[0].returned = CreateBrush(70, 70, 70);
			m_processBrushes[1].returned = CreateBrush(130, 130, 130);

			m_processBrushes[0].recv = CreateBrush(10, 92, 10);
			m_processBrushes[1].recv = CreateBrush(10, 130, 10);
			m_processBrushes[0].success = CreateBrush(10, 100, 10);
			m_processBrushes[1].success = CreateBrush(10, 140, 10);
			m_processBrushes[0].send = CreateBrush(10, 115, 10);
			m_processBrushes[1].send = CreateBrush(10, 145, 10);
			m_processBrushes[0].cacheFetchTest = CreateBrush(24, 120, 110);
			m_processBrushes[1].cacheFetchTest = CreateBrush(31, 150, 138);
			m_processBrushes[0].cacheFetchDownload = CreateBrush(24, 112, 110);
			m_processBrushes[1].cacheFetchDownload = CreateBrush(31, 143, 138);
			m_processBrushes[0].cacheMiss = CreateBrush(90, 90, 90);
			m_processBrushes[1].cacheMiss = CreateBrush(130, 130, 130);
			m_processBrushes[0].warning = CreateBrush(100, 100, 0);
			m_processBrushes[1].warning = CreateBrush(100, 100, 0);
			m_processBrushes[0].idle = CreateBrush(45, 47, 47);
			m_processBrushes[1].idle = CreateBrush(55, 57, 57);

			m_backgroundBrush = CreateBrush(38, 37, 37);
			m_separatorPen = m_backend.CreatePen(RGB(50, 50, 50));
			m_tooltipBackgroundBrush = CreateBrush(64, 64, 64);
			m_checkboxPen = m_backend.CreatePen(RGB(130, 130, 130));

			m_sendColor = RGB(0, 170, 0);
			m_recvColor = RGB(0, 170, 255);
			m_cpuColor = RGB(170, 170, 0);
			m_memColor = RGB(170, 0, 255);
			m_driveColor = RGB(170, 65, 55);
			m_activeProcColor = RGB(0, 170, 170);
		}
		else
		{
			#if PLATFORM_WINDOWS
			m_textColor = GetSysColor(COLOR_INFOTEXT);
			COLORREF backgroundColor = GetSysColor(0);
			COLORREF tooltipBackgroundColor = GetSysColor(COLOR_INFOBK);
			#else
			m_textColor = RGB(0, 0, 0);
			COLORREF backgroundColor = RGB(220,220,220);
			COLORREF tooltipBackgroundColor = RGB(200,200,200);
			#endif

			m_textWarningColor = RGB(170, 130, 0);
			m_textErrorColor = RGB(190, 0, 0);

			m_processBrushes[0].inProgress = CreateBrush(100, 150, 200);
			m_processBrushes[1].inProgress = CreateBrush(120, 170, 200);
			m_processBrushes[0].inProgressRace = CreateBrush(100, 130, 180);
			m_processBrushes[1].inProgressRace = CreateBrush(130, 180, 210);

			m_processBrushes[0].error = CreateBrush(255, 70, 70);
			m_processBrushes[1].error = CreateBrush(255, 100, 70);

			m_processBrushes[0].returned = CreateBrush(150, 150, 150);
			m_processBrushes[1].returned = CreateBrush(180, 180, 180);

			m_processBrushes[0].recv = CreateBrush(70, 170, 70);
			m_processBrushes[1].recv = CreateBrush(20, 210, 20);
			m_processBrushes[0].success = CreateBrush(80, 180, 80);
			m_processBrushes[1].success = CreateBrush(20, 220, 20);
			m_processBrushes[0].send = CreateBrush(80, 190, 80);
			m_processBrushes[1].send = CreateBrush(90, 250, 90);
			m_processBrushes[0].cacheFetchTest = CreateBrush(150, 150, 200);
			m_processBrushes[1].cacheFetchTest = CreateBrush(170, 170, 200);
			m_processBrushes[0].cacheFetchDownload = CreateBrush(150, 150, 200);
			m_processBrushes[1].cacheFetchDownload = CreateBrush(170, 170, 200);
			m_processBrushes[0].cacheMiss = CreateBrush(150, 150, 150);
			m_processBrushes[1].cacheMiss = CreateBrush(180, 180, 180);
			m_processBrushes[0].warning = CreateBrush(170, 170, 0);
			m_processBrushes[1].warning = CreateBrush(170, 170, 0);
			m_processBrushes[0].idle = CreateBrush(70, 70, 160);
			m_processBrushes[1].idle = CreateBrush(80, 80, 170);

			m_separatorPen = m_backend.CreatePen(RGB(140, 140, 140));
			m_backgroundBrush = m_backend.CreateSolidBrush(backgroundColor);
			m_tooltipBackgroundBrush = m_backend.CreateSolidBrush(tooltipBackgroundColor);
			m_checkboxPen = m_backend.CreatePen(RGB(130, 130, 130));

			m_sendColor = RGB(0, 170, 0); // Green
			m_recvColor = RGB(63, 72, 204); // Blue
			m_cpuColor = RGB(200, 130, 0); // Orange
			m_memColor = RGB(170, 0, 255); // Purple
			m_driveColor = RGB(255, 115, 96);
			m_activeProcColor = RGB(0, 170, 170);
		}

		m_textPen = m_backend.CreatePen(m_textColor);
		m_sendPen = m_backend.CreatePen(m_sendColor);
		m_recvPen = m_backend.CreatePen(m_recvColor);
		m_cpuPen = m_backend.CreatePen(m_cpuColor);
		m_memPen = m_backend.CreatePen(m_memColor);
		m_drivePen = m_backend.CreatePen(m_driveColor);
		m_activeProcPen = m_backend.CreatePen(m_activeProcColor);

		m_coloredBrushes.clear(); // Needs to be recreated
	}

	u64 Visualizer::GetPlayTime()
	{
		u64 currentTime = m_paused ? m_pauseStart : GetTime();
		u64 playTime = 0;
		if (m_traceView.startTime && currentTime > (m_traceView.startTime + m_pauseTime))
			playTime = currentTime - m_traceView.startTime - m_pauseTime;
		if (m_replay)
			playTime *= m_replay;
		return playTime;
	}

	struct SessionRec
	{
		TraceView::Session* session;
		u32 index;
	};
	void Populate(SessionRec* recs, TraceView& traceView, bool sort)
	{
		u32 count = u32(traceView.sessions.size());
		for (u32 i = 0, e = count; i != e; ++i)
			recs[i] = { &traceView.sessions[i], i };
		if (count <= 1 || !sort)
			return;
		std::sort(recs + 1, recs + traceView.sessions.size(), [](SessionRec& a, SessionRec& b)
			{
				auto& as = *a.session;
				auto& bs = *b.session;
				if ((as.processActiveCount != 0) != (bs.processActiveCount != 0))
					return as.processActiveCount > bs.processActiveCount;
				if (as.processActiveCount && as.proxyCreated != bs.proxyCreated)
					return int(as.proxyCreated) > int(bs.proxyCreated);
				return a.index < b.index;
			});
	}

	int Visualizer::GetTimelineHeight()
	{
		return m_timelineFont.height + 8;
	}

	int Visualizer::GetTimelineTop()
	{
		int timelineHeight = GetTimelineHeight();
		int posY = m_contentHeight - timelineHeight;
		int maxY = int(m_clientHeight - timelineHeight);
		return m_config.LockTimelineToBottom ? maxY : Min(posY, maxY);
	}

	bool Visualizer::ActiveProcessesShouldFillHeight()
	{
		return !m_config.showDetailedData && !m_config.showTitleBars && !m_config.showCpuMemStats && !m_config.showNetworkStats && !m_config.showDriveStats && !m_config.showProcessBars;
	}

	StringBufferBase& Visualizer::GetTitlePrefix(StringBufferBase& out)
	{
		out.Clear();
		out.Append(TC("UbaVisualizer"));
		#if UBA_DEBUG
		out.Appendf(TC(" (DEBUG, %s)"), m_backend.GetName());
		#endif
		out.Append(TC(" - "));
		return out;
	}

	StringBuffer<128> Visualizer::GetWorldTime(u64 time)
	{
		return GetWorldTime(TimeToS(time));
	}

	StringBuffer<128> Visualizer::GetWorldTime(float seconds)
	{
		return StringBuffer<128>(DateToText(float(m_traceView.traceSystemStartTimeUs / 1'000'000) + seconds).str);
	}

	void Visualizer::HitTest(HitTestResult& outResult, PaintContext& context, const POINT& pos)
	{
		u64 playTime = GetPlayTime();

		context.SetFont(m_defaultFont);

		int posY = int(m_scrollPosY);
		int boxHeight = m_config.boxHeight;
		int processStepY = boxHeight + 1;
		float scaleX = 50.0f*m_zoomValue*m_config.horizontalScaleValue;

		RECT progressRect = { 0, 0, m_clientWidth, m_clientHeight };
		progressRect.left += m_progressRectLeft;
		progressRect.bottom -= 30;

		{
			int boxSide = 8;
			int boxStride = boxSide + 2;
			int top = 5;
			int bottom = top + boxSide;
			int left = progressRect.right - 7 - boxSide;
			int right = progressRect.right - 7;
			for (int i = VisualizerFlag_Count - 1; i >= 0; --i)
			{
				if (pos.x >= left && pos.x <= right && pos.y >= top && pos.y <= bottom)
				{
					outResult.buttonSelected = i;
					return;
				}
				left -= boxStride;
				right -= boxStride;
			}
		}

		outResult.section = 0;

		if (m_config.showTimeline && !m_traceView.sessions.empty())
		{
			int timelineTop = GetTimelineTop();
			if (pos.y >= timelineTop)// && pos.y < timelineTop + 40)
			{
				outResult.section = 3;
				float timeScale = (m_config.horizontalScaleValue * m_zoomValue)*50.0f;
				float startOffset = -(m_scrollPosX / timeScale);
				outResult.timelineSelected = startOffset + float(pos.x - m_progressRectLeft) / timeScale;
				return;
			}
		}

		u64 lastStop = 0;

		Font& activeFont = context.GetFont();

		if (m_config.showProgress && m_traceView.progressProcessesTotal)
			posY += activeFont.height + 2;

		if (m_config.showStatus && !m_traceView.statusMap.empty())
		{
			u32 lastRow = ~0u;
			u32 row = ~0u;
			for (auto& kv : m_traceView.statusMap)
			{
				if (kv.second.text.empty())
					continue;

				row = u32(kv.first >> 32);
				if (lastRow != ~0u && lastRow != row)
					posY += activeFont.height + 2;
				lastRow = row;

				if (!kv.second.link.empty())
				{
					if (pos.y >= posY && pos.y < posY + activeFont.height && pos.x > 20 && pos.x < 80) // TODO: x is just hard coded to fit horde for now
					{
						outResult.hyperLink = kv.second.link;
						return;
					}
				}
			}
			if (row != ~0u)
				posY += activeFont.height + 2;
			posY += 3;
		}

		if (pos.y >= posY)
			outResult.section = 1;

		if (m_config.showActiveProcessGraph)
		{
			if (pos.y > posY && pos.y < posY + GraphHeight)
			{
				float timeScale = (m_config.horizontalScaleValue * m_zoomValue)*50.0f;
				float startOffset = -(m_scrollPosX / timeScale);
				u64 selectedTimeMs = u64(1000.0f*(startOffset + float(pos.x - m_progressRectLeft) / timeScale));

				u64 lastTime = m_traceView.activeProcessCounts.empty() ? 0 : m_traceView.activeProcessCounts[m_traceView.activeProcessCounts.size()-1].time;
				if (selectedTimeMs < TimeToMs(lastTime))
				{
					u16 count = 0;
					for (auto& entry : m_traceView.activeProcessCounts)
					{
						count = entry.count;
						if (TimeToMs(entry.time) > selectedTimeMs)
							break;
					}

					outResult.activeProcessGraphSelected = true;
					outResult.activeProcessCount = count;
				}
			}
			posY += GraphHeight;
		}

		TraceView::ProcessLocation& outLocation = outResult.processLocation;

		if (m_config.showActiveProcesses && !m_trace.m_activeProcesses.empty())
		{
			PaintActiveProcesses(context, posY, [&](TraceView::ProcessLocation& processLocation, u32 boxHeight, bool firstWithHeight)
				{
					if (pos.y < posY || pos.y > posY + int(boxHeight))
						return;
					auto& session = m_trace.GetSession(m_traceView, processLocation.sessionIndex);
					TraceView::Process& process = session.processors[processLocation.processorIndex].processes[processLocation.processIndex];

					int posX = int(m_scrollPosX) + progressRect.left;
					u64 stop = process.stop;
					bool done = stop != ~u64(0);
					if (!done)
						stop = playTime;
					int left = posX + int(float(TimeToS(process.start)) * scaleX);
					int right = posX + int(float(TimeToS(stop)) * scaleX) - 1;

					if (pos.x >= left && pos.x <= right)
					{
						outLocation.sessionIndex = processLocation.sessionIndex;
						outLocation.processorIndex = processLocation.processorIndex;
						outLocation.processIndex = processLocation.processIndex;
						outResult.processSelected = true;
						return;
					}
				});
			if (outResult.processSelected)
				return;
		}

		if (pos.y >= posY)
			outResult.section = 2;

		Vector<SessionRec> sortedSessions;
		sortedSessions.resize(m_traceView.sessions.size());

		Populate(sortedSessions.data(), m_traceView, m_config.SortActiveRemoteSessions);

		for (u64 sessionIt = 0, sessionEnd = m_traceView.sessions.size(); sessionIt != sessionEnd; ++sessionIt)
		{
			bool isFirst = sessionIt == 0;
			auto& session = *sortedSessions[sessionIt].session;
			bool hasUpdates = !session.updates.empty();
			if (!isFirst)
			{
				if (!hasUpdates && session.processors.empty())
					continue;
				if (!m_config.showFinishedProcesses && session.disconnectTime != ~u64(0))
					continue;
			}

			u32 sessionIndex = sortedSessions[sessionIt].index;
			if (!isFirst)
				posY += 3;

			if (m_config.showTitleBars)
			{
				if (pos.y >= posY && pos.y < posY + m_sessionStepY)
				{
					if (pos.x < int(session.fullNameWidth) + 5)
					{
						outResult.sessionSelectedIndex = sessionIndex;
						return;
					}
				}

				posY += m_sessionStepY;
			}

			bool showGraph = m_config.showNetworkStats || m_config.showCpuMemStats || m_config.showDriveStats;
			if (showGraph && !session.updates.empty())
			{
				int posX = int(m_scrollPosX) + progressRect.left;

				if (pos.y >= posY && pos.y < posY + GraphHeight && pos.x >= posX)
				{
					bool loop = true;
					u32 reconnectIndex = 0;
					while (loop)
					{
						u64 i = 0;
						u64 e = 0;

						if (reconnectIndex)
							i = session.reconnectIndices[reconnectIndex - 1];

						if (reconnectIndex < session.reconnectIndices.size())
							e = session.reconnectIndices[reconnectIndex];
						else
						{
							e = session.updates.size();
							loop = false;
						}

						int prevX = -1;
						int prevCenterX = -1;
						for (; i!=e; ++i)
						{
							u64 updateTime = session.updates[i];
							auto updateSend = session.networkSend[i];
							auto updateRecv = session.networkRecv[i];

							int x = posX + int(float(TimeToS(updateTime)) * scaleX);
							int centerX = prevX != -1 ? (prevX + (x - prevX)/2) : x;

							if (prevCenterX != -1 && pos.x >= prevCenterX && pos.x < centerX)
							{
								u64 prevTime = 0;
								u64 prevSend = 0;
								u64 prevRecv = 0;
								if (i > 0)
								{
									prevTime = session.updates[i - 1];
									prevSend = Min(session.networkSend[i - 1], updateSend);
									prevRecv = Min(session.networkRecv[i - 1], updateRecv);
								}

								double duration = TimeToS(updateTime - prevTime);
								outResult.stats.recvBytes = updateRecv;
								outResult.stats.sendBytes = updateSend;
								outResult.stats.recvBytesPerSecond = u64(float(updateRecv - prevRecv) / duration);
								outResult.stats.sendBytesPerSecond = u64(float(updateSend - prevSend) / duration);
								outResult.stats.ping = session.ping[i];
								outResult.stats.memAvail = session.memAvail[i];
								outResult.stats.cpuLoad = session.cpuLoad[i];
								outResult.stats.memTotal = session.memTotal;
								outResult.stats.connectionCount = session.connectionCount[i];
								outResult.statsSelected = true;

								for (auto& pair : session.drives)
								{
									auto& updateBusy = pair.second.busyPercent[i];
									u64 updateRead = pair.second.readBytes[i];
									u64 updateWrite = pair.second.writeBytes[i];
									auto& statsDrive = outResult.stats.drives[pair.first];
									statsDrive.busyPercent = updateBusy;
									statsDrive.readPerSecond = u32(float(updateRead) / duration);
									statsDrive.writePerSecond = u32(float(updateWrite) / duration);
								}
								return;
							}

							prevCenterX = centerX;
							prevX = x;
						}
						++reconnectIndex;
					}
					posY += GraphHeight;
				}

				posY += GraphHeight;
			}

			if (m_config.showDetailedData)
			{
				auto drawText = [&](const StringBufferBase& text, RECT& rect, u32* outWidth)
					{
						if (pos.x >= rect.left && pos.x < rect.right && pos.y >= rect.top && pos.y < rect.bottom && text.StartsWith(TC("Fetched Files")))
							outResult.fetchedFilesSelected = sessionIndex;
					};
				PaintDetailedStats(context, posY, progressRect, session, sessionIt != 0, playTime, drawText);
			}

			if (m_config.showProcessBars)
			{
				u32 processorIndex = 0;
				for (auto& processor : session.processors)
				{
					bool drawProcessorIndex = m_config.showFinishedProcesses;

					if (pos.y < progressRect.bottom && posY + processStepY >= progressRect.top && posY <= progressRect.bottom && pos.y >= posY-1 && pos.y < posY-1 + processStepY)
					{
						u32 processIndex = 0;
						int posX = int(m_scrollPosX) + progressRect.left;
						for (auto& process : processor.processes)
						{
							int left = posX + int(float(TimeToS(process.start)) * scaleX);

							auto pig = MakeGuard([&]() { ++processIndex; });

							if (left >= progressRect.right)
								continue;

							if (left < progressRect.left)
								left = progressRect.left;

							u64 stopTime = process.stop;
							bool done = stopTime != ~u64(0);
							if (!done)
								stopTime = playTime;
							else if (!m_config.showFinishedProcesses)
								continue;

							drawProcessorIndex = true;

							RECT rect;
							rect.left = left;
							rect.right = posX + int(float(TimeToS(stopTime)) * scaleX);

							if (rect.right <= progressRect.left)
								continue;

							if (!m_filterString.empty() && !Contains(process.description.c_str(), m_filterString.c_str()) && !Contains(process.breadcrumbs.c_str(), m_filterString.c_str()))
								continue;

							rect.right = Max(int(rect.right), left + 1);
							rect.top = posY;
							rect.bottom = posY + int(float(18) * m_zoomValue);

							if (pos.x >= rect.left && pos.x <= rect.right)
							{
								outLocation.sessionIndex = sessionIndex;
								outLocation.processorIndex = processorIndex;
								outLocation.processIndex = processIndex;
								outResult.processSelected = true;
								return;
							}
						}
					}

					if (!processor.processes.empty())
						lastStop = Max(lastStop, processor.processes.rbegin()->stop);

					if (drawProcessorIndex)
						posY += processStepY;

					++processorIndex;
				}
			}
			else
			{
				for (auto& processor : session.processors)
					if (!processor.processes.empty())
						lastStop = Max(lastStop, processor.processes.rbegin()->stop);
			}

			if (m_config.showWorkers && isFirst)
			{
				int trackIndex = 0;
				for (auto& workTrack : m_traceView.workTracks)
				{
					if (pos.y < progressRect.bottom && posY + processStepY >= progressRect.top && posY <= progressRect.bottom && pos.y >= posY-1 && pos.y < posY-1 + processStepY)
					{
						u32 workIndex = 0;
						int posX = int(m_scrollPosX) + progressRect.left;
						for (auto& work : workTrack.records)
						{
							auto inc = MakeGuard([&](){ ++workIndex; });

							int left = posX + int(float(TimeToS(work.start)) * scaleX);

							if (left >= progressRect.right)
								continue;

							if (!m_filterString.empty())
							{
								bool keep = Contains(work.description, m_filterString.c_str());
								if (!keep)
									for (auto& en : work.entries)
										keep |= Contains(en.text, m_filterString.c_str());
								if (!keep)
									continue;
							}

							if (left < progressRect.left)
								left = progressRect.left;

							u64 stopTime = work.stop;
							bool done = stopTime != ~u64(0);
							if (!done)
								stopTime = playTime;

							RECT rect;
							rect.left = left;
							rect.right = posX + int(float(TimeToS(stopTime)) * scaleX);

							if (rect.right <= progressRect.left)
								continue;

							rect.right = Max(int(rect.right), left + 1);
							rect.top = posY;
							rect.bottom = posY + int(float(18) * m_zoomValue);

							if (pos.x >= rect.left && pos.x <= rect.right)
							{
								outResult.workTrack = trackIndex;
								outResult.workIndex = workIndex;
								outResult.workSelected = true;
								return;
							}
						}
					}
					++trackIndex;
					posY += processStepY;
				}
			}
		}

		m_contentWidth = m_progressRectLeft + Max(0, int(TimeToS((lastStop != 0 && lastStop != ~u64(0)) ? lastStop : playTime) * scaleX));
		m_contentHeight = posY - int(m_scrollPosY) + processStepY + 14;
	}

	void Visualizer::PaintAll(PaintContext& context, const RECT& clientRect)
	{
		u64 playTime = GetPlayTime();

		context.SetTextColor(m_textColor);
		context.SetPen(m_textPen);

		if (m_traceView.sessions.empty())
		{
			PaintEmptyScreen(context);
			return;
		}

		if (m_traceView.version && (m_traceView.version < TraceReadCompatibilityVersion || m_traceView.version > TraceVersion))
		{
			if (!m_traceView.finished)
			{
				StringBuffer<> str;
				str.Appendf(TC("Unsupported trace version %u (Versions supported are %u to %u)"), m_traceView.version, TraceReadCompatibilityVersion, TraceVersion);
				context.SetTextColor(m_textWarningColor);
				context.SetFont(m_popupFont);
				DrawCenteredText(context, &str, 1, nullptr);
			}
			else
			{
				m_traceView.Clear();
				FinishIncompatible();
				PaintEmptyScreen(context);
			}
			return;
		}

		int posY = int(m_scrollPosY);
		float scaleX = 50.0f*m_zoomValue*m_config.horizontalScaleValue;

		RECT progressRect = clientRect;
		progressRect.left += m_progressRectLeft;

		if (m_config.showTimeline)
		{
			progressRect.bottom -= m_defaultFont.height + 10;
		}

		u64 lastStop = 0;

		context.SetFont(m_defaultFont);

		auto drawStatusText = [&](StringView text, LogEntryType type, int posX, int endX, bool moveY, bool underlined = false, bool centered = false)
			{
				Font& activeFont = context.GetFont();
				RECT rect;
				rect.left = posX;
				rect.right = endX;
				rect.top = posY;// + activeFont.offset;
				rect.bottom = posY + activeFont.height + 2;
				context.SetTextColor(type == LogEntryType_Info ? m_textColor : (type == LogEntryType_Error ? m_textErrorColor : m_textWarningColor));

				if (centered)
					posX += ((endX - posX) - context.GetTextSize(text).cx)/2;
				if (underlined)
					context.SetFont(activeFont, true);
				context.Text(posX, posY, text, &rect);
				if (underlined)
					context.SetFont(activeFont);
				if (moveY)
					posY = rect.bottom;
			};

		auto drawIndentedText = [&](const StringView& text, LogEntryType type, int indent, bool moveY, bool underlined = false)
			{
				int posX = 5 + indent*m_defaultFont.height;
				drawStatusText(text, type, posX, clientRect.right, moveY, underlined);
			};

		if (m_config.showProgress && m_traceView.progressProcessesTotal)
		{
			drawIndentedText(TCV("Progress"), LogEntryType_Info, 1, false);

			Font& activeFont = context.GetFont();

			float progress = float(m_traceView.progressProcessesDone) / float(m_traceView.progressProcessesTotal);
			u32 start = 3 + 6*activeFont.height;
			u32 width = activeFont.height * 18;
			RECT rect;

			{
				rect.top = posY;
				rect.bottom = posY + activeFont.height;

				rect.left = start;
				rect.right = rect.left + int(progress*float(width));//durationMs/100;
				context.FillRect(rect, m_traceView.progressErrorCount ? m_processBrushes[0].error : m_processBrushes[0].success);

				rect.left = rect.right;
				rect.right = start + width;
				context.FillRect(rect, m_processBrushes[0].returned);
			}

			if (m_traceView.lastKillProcessTime && playTime >= m_traceView.lastKillProcessTime && playTime < m_traceView.lastKillProcessTime + MsToTime(800))
			{
				rect.left = rect.right + 20;
				rect.right = rect.left + activeFont.height * 7;
				const tchar* killReason = m_traceView.lastKillReason;
				if (!killReason)
					killReason = TC("mem");
				BrushHandle brush = TCV("mem").Equals(killReason) ? m_processBrushes[0].error : m_processBrushes[0].returned;
				context.FillRect(rect, brush);
				StringBuffer<128> text;
				text.Appendf(TC("Killing (%s)"), killReason);
				drawStatusText(text, LogEntryType_Info, rect.left, rect.right, 0, false, true);
			}
			else if (!m_traceView.finished && !m_traceView.sessions[0].memAvail.empty())
			{
				u64 memAvail = *m_traceView.sessions[0].memAvail.rbegin();
				u32 estPct = 100u - u32(memAvail * 100ull / m_traceView.sessions[0].memTotal);
				bool isThrottling = m_traceView.lastSpawningDelayStartTime && playTime >= m_traceView.lastSpawningDelayStartTime && playTime < m_traceView.lastSpawningDelayEndTime + MsToTime(500);

				u32 barWidth = activeFont.height * 8;
				rect.left = rect.right + 20;
				rect.right = rect.left + barWidth;

				// Gray background
				context.FillRect(rect, m_processBrushes[0].returned);

				// Fill proportional to estimated usage
				u32 fillPct = Min(estPct, 100u);
				RECT fillRect = rect;
				fillRect.right = fillRect.left + int(float(fillPct) / 100.0f * float(barWidth));
				BrushHandle fillBrush = isThrottling ? m_processBrushes[0].warning : m_processBrushes[0].success;
				if (fillPct > 0)
					context.FillRect(fillRect, fillBrush);

				StringBuffer<64> memText;
				if (isThrottling)
					memText.Appendf(TC("Throttling %u%%"), estPct);
				else
					memText.Appendf(TC("Mem %u%%"), estPct);
				drawStatusText(memText, LogEntryType_Info, rect.left, rect.right, 0, false, true);
			}
			else if (m_traceView.lastSpawningDelayStartTime && playTime >= m_traceView.lastSpawningDelayStartTime && playTime < m_traceView.lastSpawningDelayEndTime + MsToTime(500))
			{
				rect.left = rect.right + 20;
				rect.right = rect.left + activeFont.height * 7;
				context.FillRect(rect, m_processBrushes[0].warning);
				drawStatusText(TCV("Throttling (mem)"), LogEntryType_Info, rect.left, rect.right, 0, false, true);
			}

			#if 0
			if (!m_traceView.finished && !m_traceView.sessions[0].cpuLoad.empty())
			{
				u32 cpuLoad = u32(*m_traceView.sessions[0].cpuLoad.rbegin() * 100.0f);
				rect.left = rect.right + 20;
				u32 barWidth = activeFont.height * 8;
				rect.right = rect.left + barWidth;
				context.FillRect(rect, m_processBrushes[0].returned);
				StringBuffer<64> memText;
				memText.Appendf(TC("Cpu %u%%"), cpuLoad);
				drawStatusText(memText, LogEntryType_Info, rect.left, rect.right, 0, false, true);
			}
			#endif

			// This is last since drawIndentedText line feeds
			{
				StringBuffer<> str;
				str.Appendf(TC("%u%%    %u / %u"), u32(progress*100.0f), m_traceView.progressProcessesDone, m_traceView.progressProcessesTotal);

				const tchar* remoteDisabled = TC(""); // TODO: Only show this in "advanced mode" m_traceView.remoteExecutionDisabled ? TC("   (Remote spawn disabled)") : TC("");
				str.Append(remoteDisabled);

				if (u32 active = m_traceView.totalProcessActiveCount)
					str.Appendf(TC("   (%u active)"), active);

				drawIndentedText(str, LogEntryType_Info, 6, true);
			}
		}

		if (m_config.showStatus && !m_traceView.statusMap.empty())
		{
			u32 lastRow = ~0u;
			u32 row = ~0u;
			Font& activeFont = context.GetFont();
			for (auto& kv : m_traceView.statusMap)
			{
				auto& status = kv.second;
				if (status.text.empty())
					continue;
				row = u32(kv.first >> 32);
				if (lastRow != ~0u && lastRow != row)
					posY += activeFont.height + 2;
				lastRow = row;
				u32 column = u32(kv.first & ~0u);
				drawIndentedText(status.text, status.type, column, false, !status.link.empty());
			}
			if (row != ~0u)
				posY += activeFont.height + 2;

			context.SetTextColor(m_textColor);
			posY += 3;
		}

		if (m_config.showActiveProcessGraph)
		{
			if (posY + GraphHeight >= progressRect.top && posY + GraphHeight - 5 < progressRect.bottom)
			{
				int graphBaseY = posY + GraphHeight - 4;
				double graphHeight = double(GraphHeight - 2);

				Vector<POINT> line;

				context.SetPen(m_activeProcPen);
				bool isFirstUpdate = true;
				bool isFirstDraw = true;
				u64 prevValue = 0;
				int prevX = 0;
				int prevY = 0;

				float timeScale = (m_config.horizontalScaleValue*m_zoomValue)*50.0f;
				float time = 0;
				float playTimeS = TimeToS(playTime);
				u32 activeProcessCountIndex = 0;
				u16 count = 0;
				while (time < playTimeS && activeProcessCountIndex < m_traceView.activeProcessCounts.size())
				{
					int posX = progressRect.left + int(m_scrollPosX + time*timeScale);
					if (posX >= clientRect.right)
						break;

					while (activeProcessCountIndex < m_traceView.activeProcessCounts.size())
					{
						auto& entry = m_traceView.activeProcessCounts[activeProcessCountIndex];
						count = entry.count;
						if (float(TimeToMs(entry.time))/1000.0f > time)
							break;
						++activeProcessCountIndex;
					}

					int x = posX;
					int y = graphBaseY;

					double div = double(m_traceView.maxActiveProcessCount);
					y -= int(double(count) * graphHeight / div);

					if (prevValue <= 0)
						prevY = y;

					if (x > clientRect.left)
					{
						if (isFirstUpdate)
						{
							line.push_back({ x, y });
							isFirstDraw = false;
						}
						else
						{
							if (isFirstDraw)
								line.push_back({ prevX, prevY });
							line.push_back({ x, y });
							isFirstDraw = false;
						}
					}
					if (x > clientRect.right)
						break;
					isFirstUpdate = false;
					prevX = x;
					prevY = y;
					prevValue = count;

					time += 0.25f;
				}
				if (line.size() > 1)
					context.Polyline(0, line);
			}
			posY += GraphHeight;
		}

		if (m_config.showActiveProcesses && !m_trace.m_activeProcesses.empty())
		{
			auto drawBox = [&](u64 start, u64 stop, int height, bool selected, bool inProgress)
				{
					//int left = 3 + 6*m_activeFont.height;
					//int right = rect.left + durationMs/100;
					int posX = int(m_scrollPosX) + progressRect.left;
					bool done = stop != ~u64(0);
					if (!done)
						stop = playTime;
					int left = posX + int(float(TimeToS(start)) * scaleX);
					int right = posX + int(float(TimeToS(stop)) * scaleX) - 1;

					RECT rect;
					rect.left = left;
					rect.right = right;
					rect.top = posY;
					rect.bottom = posY + height;
					context.FillRect(rect, inProgress ? m_processBrushes[selected].inProgress : m_processBrushes[selected].success);
					return rect;
				};

			m_activeProcessCountHistory[m_activeProcessCountHistoryIterator++ % sizeof_array(m_activeProcessCountHistory)] = u32(m_trace.m_activeProcesses.size());

			PaintActiveProcesses(context, posY, [&](TraceView::ProcessLocation& processLocation, u32 boxHeight, bool firstWithHeight)
				{
					auto& session = m_trace.GetSession(m_traceView, processLocation.sessionIndex);
					TraceView::Process& process = session.processors[processLocation.processorIndex].processes[processLocation.processIndex];

					bool selected = m_processSelected && m_processSelectedLocation == processLocation;

					if (m_config.showFinishedProcesses)
					{
						u32 index = processLocation.processIndex;
						while (index > 0)
						{
							--index;
							TraceView::Process& process2 = session.processors[processLocation.processorIndex].processes[index];
							drawBox(process2.start, process2.stop, boxHeight, false, false);
						}
					}

					RECT boxRect = drawBox(process.start, process.stop, boxHeight, selected, true);

					u32 v = boxHeight;// - 1;

					if (v > 4)
					{
						u32 fontIndex = Min(v, u32(sizeof_array(m_activeProcessFont) - 1));
						Font& font = m_activeProcessFont[fontIndex];
						if (!font.handle)
							UpdateFont(font, fontIndex - 1, false);
						if (firstWithHeight)
							context.SetFont(font);

						StringBuffer<> str;
						auto firstPar = -1;//process.description.find_first_of('(');
						if (firstPar != -1 && process.description.back() == ')')
						{
							str.Append(process.description.c_str() + firstPar + 1).Resize(str.count-1);
							str.Append(TCV("  ")).Append(process.description.c_str(), firstPar);
						}
						else
							str.Append(process.description);

						if (process.isRemote)
							str.Append(TCV(" [")).Append(session.name).Append(']');
						else if (process.type == TraceView::ProcessType_CacheFetchTest || process.type == TraceView::ProcessType_CacheFetchDownload)
							str.Append(TCV(" [cache]"));
						if (boxRect.left < 0)
							str.Appendf(TC("   %s"), TimeToText(playTime - process.start, true).str);
						drawStatusText(str, LogEntryType_Info, Max(int(boxRect.left+1), 1), boxRect.right, false);
					}
				});
		}

		int boxHeight = m_config.boxHeight;
		int stepY = int(boxHeight) + 2;
		int processStepY = boxHeight + 1;

		TraceView::WorkRecord selectedWork;

		Vector<SessionRec> sortedSessions;
		sortedSessions.resize(m_traceView.sessions.size());

		Populate(sortedSessions.data(), m_traceView, m_config.SortActiveRemoteSessions);

		u32 visibleBoxes = 0;

		TraceView::ProcessLocation processLocation { 0, 0, 0 };
		for (u32 sessionIt = 0, sessionEnd = u32(m_traceView.sessions.size()); sessionIt != sessionEnd; ++sessionIt)
		{
			bool isFirst = sessionIt == 0;
			auto& session = *sortedSessions[sessionIt].session;
			bool hasUpdates = !session.updates.empty();
			if (!isFirst)
			{
				if (!hasUpdates && session.processors.empty())
					continue;

				if (!m_config.showFinishedProcesses && session.disconnectTime != ~u64(0))
					continue;
			}

			processLocation.sessionIndex = sortedSessions[sessionIt].index;
			if (!isFirst)
				posY += 3;

			if (m_config.showTitleBars)
			{
				if (posY + stepY >= progressRect.top && posY <= progressRect.bottom)
				{
					context.SetPen(m_separatorPen);
					context.Line(0, posY, clientRect.right, posY);

					StringBuffer<> text;
					session.GetFullName(text);

					if (hasUpdates && session.disconnectTime == ~u64(0))
					{
						u64 ping = session.ping.back();
						//u64 memAvail = session.updates.back().memAvail;
						//float cpuLoad = session.updates.back().cpuLoad;

						//text.Appendf(TC(" - Cpu: %.1f%%"), cpuLoad * 100.0f);
						//if (memAvail)
						//	text.Appendf(TC(" Mem: %s/%s"), BytesToText(session.memTotal - memAvail).str, BytesToText(session.memTotal).str);
						if (ping)
							text.Appendf(TC(" Ping: %s"), TimeToText(ping, false, m_traceView.frequency).str);
						if (!session.notification.empty())
							text.Append(TCV(" - ")).Append(session.notification);
					}
					else if (!isFirst)
					{
						text.Append(TCV(" - Disconnected"));
						if (!session.notification.empty())
							text.Append(TCV(" (")).Append(session.notification).Append(')');
					}

					bool selected = m_sessionSelectedIndex == processLocation.sessionIndex;

					int textBottom = Min(posY + m_sessionStepY, int(progressRect.bottom));

					RECT rect;
					rect.left = 5;
					rect.right = clientRect.right;
					rect.top = posY;
					rect.bottom = textBottom;

					context.SetTextColor(m_textColor);
					if (selected)
					{
						RECT r2 = rect;
						r2.right = r2.left + context.GetTextSize(text).cx;
						context.FillRect(r2, m_tooltipBackgroundBrush);
					}

					SIZE textSize = context.GetTextSize(text);
					if (textSize.cx)
						session.fullNameWidth = u32(textSize.cx);
					context.Text(5, posY+2, text, &rect);
				}
				posY += m_sessionStepY;
			}

			bool showGraph = m_config.showNetworkStats || m_config.showCpuMemStats || m_config.showDriveStats;
			if (showGraph && hasUpdates)
			{
				if (posY + GraphHeight >= progressRect.top && posY + GraphHeight - 5 < progressRect.bottom)
				{
					int posX = int(m_scrollPosX) + progressRect.left;
					int graphBaseY = posY + GraphHeight - 4;
					double graphHeight = double(GraphHeight - 2);

					Vector<POINT> line;

					auto drawGraph = [&](u32 id, auto& values, auto maxValue, double scale, PenHandle pen, bool accumulatingValue, int offsetY = 0)
						{
							bool loop = true;
							u32 reconnectIndex = 0;
							while (loop)
							{
								context.SetPen(pen);
								bool isFirstUpdate = true;
								bool isFirstDraw = true;
								decltype(maxValue) prevValue = 0;
								u64 prevTime = 0;
								int prevX = 0;
								int prevPrevX = 0;
								int prevY = 0;

								u64 i = 0;
								u64 e = 0;

								if (reconnectIndex)
									i = session.reconnectIndices[reconnectIndex - 1];

								if (reconnectIndex < session.reconnectIndices.size())
									e = session.reconnectIndices[reconnectIndex];
								else
								{
									e = Min(session.updates.size(), values.size());
									loop = false;
								}

								line.clear();
								for (; i!=e; ++i)
								{
									u64 updateTime = session.updates[i];
									auto value = values[i];
									int x = posX + int(float(TimeToS(updateTime)) * scaleX);
									int y = graphBaseY;

									double duration = TimeToS(updateTime - prevTime);
									if (updateTime == 0)
										isFirstUpdate = true;

									if (accumulatingValue)
									{
										double invScaleY = duration * scale;
										if (invScaleY != 0 && prevValue != 0)
											y -= int(double(value - prevValue) / invScaleY) + offsetY;
									}
									else
									{
										double div = 1.0f;
										if (maxValue != 0)
											div = double(maxValue);
										y -= int(double((double)maxValue - (double)value*scale)*graphHeight/div) + offsetY;
									}

									if (prevValue <= 0)
										prevY = y;

									if (x > clientRect.left)
									{
										if (isFirstUpdate)
										{
											isFirstDraw = false;
										}
										else
										{
											if (isFirstDraw)
												line.push_back({prevPrevX, prevY});
											line.push_back({prevX, y});
											isFirstDraw = false;
										}
									}
									if (prevX > clientRect.right)
										break;
									isFirstUpdate = false;
									prevPrevX = prevX;
									prevX = x;
									prevY = y;
									prevValue = value;
									prevTime = updateTime;
								}
								if (line.size() > 1)
									context.Polyline(id + sessionIt*10, line);
								++reconnectIndex;
							}
						};

					if (m_config.showNetworkStats && session.highestSendPerS != 0 && session.highestRecvPerS != 0)
					{
						drawGraph(1, session.networkSend, 100000000000000ull, double(session.highestSendPerS) / graphHeight, m_sendPen, true);
						drawGraph(2, session.networkRecv, 100000000000000ull, double(session.highestRecvPerS) / graphHeight, m_recvPen, true, 1);
					}

					if (m_config.showCpuMemStats)
					{
						drawGraph(3, session.cpuLoad, 0.0f, -1.0, m_cpuPen, false);
						drawGraph(4, session.memAvail, session.memTotal, 1.0, m_memPen, false);
					}

					if (m_config.showDriveStats)
						for (auto& pair : session.drives)
							if (pair.second.busyHighest)
								drawGraph(5, pair.second.busyPercent, 0, -0.01, m_drivePen, false);

				}
				posY += GraphHeight;
			}

			if (m_config.showDetailedData)
			{
				auto drawText = [&](const StringBufferBase& text, RECT& rect, u32* outWidth)
					{
						if (rect.top > progressRect.bottom)
							return;
						bool selected = m_fetchedFilesSelected == processLocation.sessionIndex && text.StartsWith(TC("Fetched Files"));
						if (selected)
						{
							RECT r2 = rect;
							r2.right = r2.left + context.GetTextSize(text).cx;
							context.FillRect(r2, m_tooltipBackgroundBrush);
						}
						if (rect.bottom > progressRect.bottom)
						{
							rect.bottom = progressRect.bottom;
							context.Text(rect.left, rect.top, text, &rect);
						}
						else
							context.Text(rect.left, rect.top, text, nullptr);
						if (outWidth)
							*outWidth = u32(context.GetTextSize(text).cx);
					};
				bool isRemote = processLocation.sessionIndex != 0;
				PaintDetailedStats(context, posY, progressRect, session, isRemote, playTime, drawText);
			}

			context.SetFont(m_processFont);

			bool shouldDrawText = m_processFont.height > 4;

			if (m_config.showProcessBars)
			{
				processLocation.processorIndex = 0;
				for (auto& processor : session.processors)
				{
					bool drawProcessorIndex = m_config.showFinishedProcesses;

					if (posY + m_sessionStepY >= progressRect.top && posY < progressRect.bottom)
					{
						int barHeight = boxHeight;
						int textOffsetY = 0;
						if (posY + boxHeight > progressRect.bottom)
						{
							int newBarHeight = Min(barHeight, int(progressRect.bottom - posY));
							textOffsetY = int(barHeight - newBarHeight);
							barHeight = newBarHeight;
						}

						const int textHeight = barHeight;
						const int rectBottom = posY + textHeight;
						const int offsetY = (textHeight - m_processFont.height + textOffsetY) / 2;

						processLocation.processIndex = 0;
						int posX = int(m_scrollPosX) + progressRect.left;
						for (auto& process : processor.processes)
						{
							int left = posX + int(float(TimeToS(process.start)) * scaleX);

							auto pig = MakeGuard([&]() { ++processLocation.processIndex; });

							if (left >= progressRect.right)
								continue;

							u64 stop = process.stop;
							bool done = stop != ~u64(0);
							if (!done)
								stop = playTime;
							else if (!m_config.showFinishedProcesses)
								continue;

							drawProcessorIndex = true;

							RECT rect;
							rect.left = left;
							rect.right = Max(posX + int(float(TimeToS(stop)) * scaleX) - 1, left + 1);
							rect.top = posY;
							rect.bottom = rectBottom - 1;

							if (rect.right <= progressRect.left)
								continue;

							if (!m_filterString.empty() && !Contains(process.description.c_str(), m_filterString.c_str()) && !Contains(process.breadcrumbs.c_str(), m_filterString.c_str()))
								continue;
							++visibleBoxes;

							bool selected = m_processSelected && m_processSelectedLocation == processLocation;
							if (selected)
								process.bitmapDirty = true;

							--rect.top;
							PaintProcessRect(context, process, rect, progressRect, selected, false);
							++rect.top;

							int processWidth = rect.right - rect.left;
							if (shouldDrawText && m_config.ShowProcessText && processWidth > 3)
							{
								context.DrawCached(process, rect, offsetY, textHeight, m_processFont, progressRect, [&](PaintContext& textCacheContext)
									{
										RECT rect2{ 0, int(process.bitmapOffset), 256, int(process.bitmapOffset) + m_processFont.height };
										RECT rect3 = rect2;
										if (done)
											rect3.right = processWidth;

										PaintProcessRect(textCacheContext, process, rect3, rect2, selected, true);

										rect2.left += 3; // Move in text a bit

										int textY = rect2.top + m_processFont.offset;

										bool dropShadow = m_config.DarkMode;
										if (dropShadow)
										{
											textCacheContext.SetTextColor(RGB(5, 60, 5));
											++rect2.left;
											++textY;
											textCacheContext.Text(rect2.left, textY, process.description, &rect2);
											--rect2.left;
											--textY;
										}
										textCacheContext.SetTextColor(m_textColor);
										textCacheContext.Text(rect2.left, textY, process.description, &rect2);

										if (!selected)
											process.bitmapDirty = false;
									});
							}
						}

						if (drawProcessorIndex)
						{
							RECT rect;
							rect.left = 5;
							rect.right = progressRect.left - 2;
							rect.top = posY;
							rect.bottom = rectBottom;

							StringBuffer<> buf;
							buf.AppendValue(u64(processLocation.processorIndex) + 1);
							context.Text(5, posY + offsetY, buf, &rect);
						}
					}

					lastStop = Max(lastStop, processor.processes.rbegin()->stop);

					++processLocation.processorIndex;

					if (drawProcessorIndex)
						posY += processStepY;
				}
			}
			else
			{
				for (auto& processor : session.processors)
					if (!processor.processes.empty())
						lastStop = Max(lastStop, processor.processes.rbegin()->stop);
			}

			if (m_config.showWorkers && isFirst)
			{
				u32 trackIndex = 0;
				for (auto& workTrack : m_traceView.workTracks)
				{
					if (posY + m_sessionStepY >= progressRect.top && posY <= progressRect.bottom)
					{
						int textOffsetY = 0;
						int barHeight = boxHeight;
						if (posY + int(boxHeight) > progressRect.bottom)
						{
							int newBarHeight = Min(barHeight, int(progressRect.bottom - posY));
							textOffsetY = barHeight - newBarHeight;
							barHeight = newBarHeight;
						}

						const int textHeight = barHeight;
						const int rectBottom = posY + textHeight;
						const int offsetY = (textHeight - m_processFont.height + textOffsetY) / 2;

						if (shouldDrawText)
						{
							RECT rect;
							rect.left = 5;
							rect.right = progressRect.left - 5;
							rect.top = posY;
							rect.bottom = rectBottom;

							StringBuffer<> buf;
							buf.AppendValue(u64(trackIndex) + 1);
							context.Text(5, posY + offsetY, buf, &rect);
						}

						int lastDrawnRight = 0;

						u32 workIndex = 0;
						int posX = int(m_scrollPosX) + progressRect.left;
						for (auto& work : workTrack.records)
						{
							auto inc = MakeGuard([&](){ ++workIndex; });

							if (work.start == work.stop)
								continue;

							float startTime = TimeToS(work.start);

							int left = posX + int(startTime * scaleX);

							if (left >= progressRect.right)
								continue;

							if (!m_filterString.empty())
							{
								bool keep = Contains(work.description, m_filterString.c_str());
								if (!keep)
									for (auto& en : work.entries)
										keep |= Contains(en.text, m_filterString.c_str());
								if (!keep)
									continue;
							}

							BrushHandle brush;

							u64 stop = work.stop;

							float stopTime = TimeToS(stop);

							RECT rect;
							rect.left = left;
							rect.right = posX + int(stopTime * scaleX) - 1;
							rect.top = posY;
							rect.bottom = rectBottom - 1;

							if (rect.right <= progressRect.left)
								continue;

							++visibleBoxes;

							rect.right = Max(int(rect.right), left + 1);

							Color color = work.color;
							bool selected = m_workSelected && m_workTrack == trackIndex && m_workIndex == workIndex;
							if (selected)
							{
								selectedWork = work;
								work.bitmapDirty = true;
								auto c = (u8*)&color;
								for (u32 j=0; j!=3; ++j)
									c[j] = u8(Min(int(c[j]) + 40, 255));
							}
							else
							{
								if (rect.left + 1 == rect.right)
									if (lastDrawnRight == rect.right)
										continue;
							}

							lastDrawnRight = rect.right;

							bool done = stop != ~u64(0);
							if (done && m_config.DarkMode) // Color only work in dark mode.. hard to see in light mode)
							{
								auto insres = m_coloredBrushes.try_emplace(color);
								if (insres.second)
								{
									auto c = (u8*)&color;
									insres.first->second = m_backend.CreateSolidBrush(RGB(c[2], c[1], c[0]));
								}
								brush = insres.first->second;
							}
							else
							{
								//stop = playTime;
								brush = m_processBrushes[selected].returned;
							}

							--rect.top;

							auto clampRect = [&](RECT& r) { r.left = Min(Max(r.left, progressRect.left), progressRect.right); r.right = Max(Min(r.right, progressRect.right), progressRect.left); };

							clampRect(rect);
							context.FillRect(rect, brush);
							++rect.top;

							int processWidth = rect.right - rect.left;
							if (shouldDrawText && m_config.ShowProcessText && processWidth > 3)
							{
								context.DrawCached(work, rect, offsetY, textHeight, m_processFont, progressRect, [&](PaintContext& textCacheContext)
									{
										RECT rect2{ 0,int(work.bitmapOffset), 256, int(work.bitmapOffset) + m_processFont.height };
										textCacheContext.FillRect(rect2, brush);
										textCacheContext.Text(rect2.left, rect2.top, ToView(work.description), &rect2);
										if (!selected)
											work.bitmapDirty = false;
									});
							}
						}
					}
					++trackIndex;
					posY += processStepY;
				}
			}

			context.SetFont(m_defaultFont);
		}

		context.EndCached();

		if (true)
		{
			Font& activeFont = context.GetFont();
			for (auto& kv : m_traceView.tables)
			{
				auto& table = kv.second;
				u32 tableOffsetX = activeFont.height*40;
				u32 tableOffsetY = 30;

				u32 rowOffset = activeFont.height + 2;
				u32 columnOffset = table.rowCharWidth * activeFont.height;

				u32 columnIndex = 0;
				u32 rowIndex = 0;
				for (const tchar* item : table.items)
				{
					u32 y = tableOffsetY + rowIndex*rowOffset;
					u32 x = tableOffsetX + columnIndex*columnOffset;
					if (int(x) >= progressRect.right)
						break;

					context.Text(x, y, ToView(item), nullptr);

					++rowIndex;
					if (rowIndex > table.rowMaxCount || int(y) >= progressRect.bottom)
					{
						rowIndex = 0;
						++columnIndex;
					}
				}
			}
		}

		m_contentWidth = m_progressRectLeft + Max(0, int(TimeToS((lastStop != 0 && lastStop != ~u64(0)) ? lastStop : playTime) * scaleX));


		m_contentHeight = posY - int(m_scrollPosY) + stepY + 14;

		float timelineSelected = m_timelineSelected;

		if (m_config.showTimeline && !m_traceView.sessions.empty())
			PaintTimeline(context);

		if (!m_filterString.empty())
		{
			StringBuffer<> str[2];
			str[0].Append(TCV("Box Filter: ")).Append(m_filterString);
			str[1].Appendf(TC("Box Count: %u"), visibleBoxes);

			context.SetFont(m_popupFont);
			DrawCenteredText(context, str, 2, m_tooltipBackgroundBrush);
		}

		if (m_config.showCursorLine && m_mouseOverWindow)
		{
			float timeScale = (m_config.horizontalScaleValue * m_zoomValue)*50.0f;
			float startOffset = -(m_scrollPosX / timeScale);
			POINT pos = context.GetCursorClientPos();;
			timelineSelected = startOffset + float(pos.x - m_progressRectLeft) / timeScale;
		}

		if (timelineSelected != 0)
		{
			int posX = int(m_scrollPosX) + progressRect.left;
			int left = posX + int(timelineSelected * scaleX);
			int timelineTop = GetTimelineTop();

			// TODO: Draw line up
			context.Line(left, 2, left, timelineTop);

			if (timelineSelected >= 0)
			{
				StringBuffer<> b;
				u32 milliseconds = u32(timelineSelected * 1000.0f);
				u32 seconds = milliseconds / 1000;
				milliseconds -= seconds * 1000;
				u32 minutes = seconds / 60;
				seconds -= minutes * 60;
				u32 hours = minutes / 60;
				minutes -= hours * 60;
				if (hours)
				{
					b.AppendValue(hours).Append('h');
					if (minutes < 10)
						b.Append('0');
				}
				if (minutes || hours)
				{
					b.AppendValue(minutes).Append('m');
					if (seconds < 10)
						b.Append('0');
				}
				b.AppendValue(seconds).Append('.');
				if (milliseconds < 100)
					b.Append('0');
				if (milliseconds < 10)
					b.Append('0');
				b.AppendValue(milliseconds);

				StringBuffer<128> wt = GetWorldTime(timelineSelected);

				context.SetFont(m_popupFont);
				DrawTextLogger logger(context, m_popupFont.height, m_tooltipBackgroundBrush);

				logger.Info(TC("%s (%s)"), b.data, wt.data);
				logger.DrawAtPos(left + 4, timelineTop - 20);
			}
		}

		{
			int boxSide = 8;
			int boxStride = boxSide + 2;
			int top = 5;
			int bottom = top + boxSide;
			int left = progressRect.right - 7 - boxSide;
			int right = progressRect.right - 7;
			bool* values = &m_config.showProgress;
			for (int i = VisualizerFlag_Count - 1; i >= 0; --i)
			{
				context.SetPen(m_buttonSelected == u32(i) ? m_textPen : m_checkboxPen);
				context.Rectangle(left, top, right, bottom);

				if (values[i])
				{
					context.Line(left + 2, top + 2, right - 2, bottom - 2);
					context.Line(right - 3, top + 2, left + 1, bottom - 2);
				}

				left -= boxStride;
				right -= boxStride;
			}

			top -= 2;

			auto drawText = [&](StringView text, COLORREF color)
				{
					context.SetTextColor(color);
					left -= (context.GetTextSize(text).cx) + 5;
					context.Text(left, top, text, nullptr);
				};

			if (m_config.showDriveStats)
			{
				context.SetFont(m_defaultFont);
				drawText(TCV("DRV"), m_driveColor);
			}

			if (m_config.showNetworkStats)
			{
				context.SetFont(m_defaultFont);
				drawText(TCV("SND"), m_sendColor);
				drawText(TCV("RCV"), m_recvColor);
			}

			if (m_config.showCpuMemStats)
			{
				context.SetFont(m_defaultFont);
				drawText(TCV("CPU"), m_cpuColor);
				drawText(TCV("MEM"), m_memColor);
			}

			context.SetTextColor(m_textColor);
		}

		if (m_processSelected)
		{
			const TraceView::Process& process = m_traceView.GetProcess(m_processSelectedLocation);
			u64 duration = 0;

			Vector<TString> logLines;
			u32 maxCharCount = 50u;

			bool hasExited = process.stop != ~u64(0);
			if (hasExited)
			{
				duration = process.stop - process.start;

				if (!process.logLines.empty())
				{
					u32 lineMaxCount = 0;
					for (auto& line : process.logLines)
					{
						u32 offset = 0;
						u32 left = u32(line.text.size());
						while (left)
						{
							u32 toCopy = Min(left, maxCharCount);
							lineMaxCount = Max(lineMaxCount, toCopy);
							logLines.push_back(line.text.substr(offset, toCopy));
							offset += toCopy;
							left -= toCopy;
						}
					}
				}
			}
			else
			{
				duration = playTime - process.start;
			}

			context.SetFont(m_popupFont);
			DrawTextLogger logger(context, m_popupFont.height, m_tooltipBackgroundBrush);

			logger.AddTextOffset(-10); // Remove spaces in the front
			logger.AddWidth(3);

			logger.AddSpace(2);
			const tchar* desc = process.isIdle ? TC("Waiting for process") : process.description.c_str();
			logger.Info(TC("  %s"), desc);
			if (process.type != TraceView::ProcessType_Task)
			{
				logger.Info(TC("  Host:        %s"), m_processSelectedLocation.sessionIndex == 0 ? TC("local") : m_traceView.GetSession(m_processSelectedLocation).name.c_str());
				logger.Info(TC("  ProcessId:  %6u"), process.id);
			}
			logger.Info(TC("  Start:     %7s (%s)"), TimeToText(process.start, true).str, GetWorldTime(process.start).data);
			logger.Info(TC("  Duration:  %7s"), TimeToText(duration, true).str);
			if (!process.returnedReason.empty())
				logger.Info(TC("  Returned: %7s"), process.returnedReason.data());
			if (hasExited && process.exitCode != 0)
			{
				if (process.exitCode == ProcessCancelExitCode)
					logger.Info(TC("  ExitCode: Cancelled%s"), process.isRacing ? TC(" (Lost race)") : TC(""));
				else if (process.exitCode >= 0xC0000000u)
					logger.Info(TC("  ExitCode: 0x%x"), process.exitCode);
				else
					logger.Info(TC("  ExitCode: %7u"), process.exitCode);
			}

			const auto& breadcrumbs = process.breadcrumbs;
			if (!breadcrumbs.empty())
			{
				constexpr TString::size_type maxLineLen = 50;
				logger.Info(TC(""));
				logger.Info(TC("  ------------ Breadcrumbs ------------"));
				for (TString::size_type lineStart = 0, lineEnd = 0; lineEnd < breadcrumbs.size(); lineStart = lineEnd + 1)
				{
					// Log each individual line
					lineEnd = breadcrumbs.find(L'\n', lineStart);
					TString line = (lineEnd == TString::npos ? breadcrumbs.substr(lineStart) : breadcrumbs.substr(lineStart, lineEnd - lineStart));

					// Break each line down into smaller section if they are longer than the maximum allowed length
					if (line.size() > maxLineLen)
					{
						for (TString::size_type sectionStart = 0, sectionEnd = 0; sectionStart < line.size(); sectionStart = sectionEnd)
						{
							const TString::size_type maxSectionLen = sectionStart == 0 ? maxLineLen : maxLineLen - 2;
							sectionEnd = std::min<TString::size_type>(sectionEnd + maxSectionLen, line.size());
							TString section = (sectionStart == 0 ? TC("  ") : TC("    ")) + line.substr(sectionStart, sectionEnd - sectionStart);
							logger.Info(section.c_str());
						}
					}
					else
					{
						line = TC("  ") + line;
						logger.Info(line.c_str());
					}
				}
			}

			if (process.stop != ~u64(0) && !process.stats.empty())
			{
				BinaryReader reader(process.stats.data(), 0, process.stats.size());
				ProcessStats processStats;
				SessionStats sessionStats;
				StorageStats storageStats;
				KernelStats kernelStats;
				CacheFetchStats cacheStats;

				if (process.type == TraceView::ProcessType_CacheFetchTest || process.type == TraceView::ProcessType_CacheFetchDownload)
				{
					if (!process.returnedReason.empty())
						logger.Info(TC("  Cache:       Miss"));
					else
						logger.Info(TC("  Cache:        Hit"));
					cacheStats.Read(reader, m_traceView.version);
					if (reader.GetLeft())
					{
						storageStats.Read(reader, m_traceView.version);
						kernelStats.Read(reader, m_traceView.version);
					}
				}
				else
				{
					processStats.Read(reader, m_traceView.version);

					if (reader.GetLeft())
					{
						if (process.isRemote || (m_traceView.version >= 36 && !process.isReuse))
							sessionStats.Read(reader, m_traceView.version);
						storageStats.Read(reader, m_traceView.version);
						kernelStats.Read(reader, m_traceView.version);
					}
				}

				if (processStats.hostTotalTime)
				{
					logger.Info(TC(""));
					logger.Info(TC("  ----------- Detours stats -----------"));
					processStats.Print(logger, m_traceView.frequency);
				}
				else if (processStats.peakMemory)
				{
					logger.Info(TC(""));
					logger.Info(TC("  ----------- Process stats -----------"));
					processStats.Print(logger, m_traceView.frequency);
				}

				if (!sessionStats.IsEmpty())
				{
					logger.Info(TC(""));
					logger.Info(TC("  ----------- Session stats -----------"));
					sessionStats.Print(logger, m_traceView.frequency);
				}

				if (!cacheStats.IsEmpty())
				{
					logger.Info(TC(""));
					logger.Info(TC("  ------------ Cache stats ------------"));
					cacheStats.Print(logger, m_traceView.frequency);
				}

				if (!storageStats.IsEmpty())
				{
					logger.Info(TC(""));
					logger.Info(TC("  ----------- Storage stats -----------"));
					storageStats.Print(logger, m_traceView.frequency);
				}

				if (!kernelStats.IsEmpty())
				{
					logger.Info(TC(""));
					logger.Info(TC("  ----------- Kernel stats ------------"));
					kernelStats.Print(logger, false, m_traceView.frequency);
				}

				PrintCacheWriteStats(logger, process.id);

				if (!logLines.empty())
				{
					logger.Info(TC(""));
					logger.Info(TC("  ---------------- Log ----------------"));
					logger.AddTextOffset(14);
					for (auto line : logLines)
					{
						if (line.empty())
						{
							logger.Log(LogEntryType_Info, line);
							continue;
						}
						const tchar* segStart = line.c_str();
						for (const tchar* it=segStart;; ++it)
						{
							if (*it != '\n' && *it != 0)
								continue;
							const tchar* segEnd = it;
							if (it != segStart && it[-1] == '\r')
								--segEnd;
							std::basic_string<tchar> str(segStart, u32(segEnd - segStart));
							while (true)
							{
								auto escapeStart = str.find_first_of(TC("\x1b["));
								if (escapeStart == -1)
									break;
								auto escapeEnd = str.find_first_of(TC("m"), escapeStart);
								if(escapeEnd == -1)
									break;
								str.erase(escapeStart, escapeEnd + 1);
							}
							logger.Log(LogEntryType_Info, str.data(), str.size());
							if (!*it)
								break;
							segStart = it + 1;
						}
					}
				}
			}
			logger.AddSpace(3);
			logger.DrawAtCursor();
		}
		else if (m_workSelected && selectedWork.description)
		{
			u64 duration;
			if (selectedWork.stop != ~u64(0))
				duration = selectedWork.stop - selectedWork.start;
			else
				duration = playTime - selectedWork.start;

			context.SetFont(m_popupFont);
			DrawTextLogger logger(context, m_popupFont.height, m_tooltipBackgroundBrush);

			logger.AddSpace();
			logger.Info(TC("  %s"), selectedWork.description);
			logger.Info(TC("  Start:     %s (%s)"), TimeToText(selectedWork.start, true).str, GetWorldTime(selectedWork.start).data);
			logger.Info(TC("  Duration:  %s"), TimeToText(duration, true).str);
			if (!selectedWork.entries.empty())
				logger.AddSpace();
			for (u32 i=0,e=u32(selectedWork.entries.size()); i!=e; ++i)
			{
				auto& entry = selectedWork.entries[i];
				u64 time = 0;
				if (i == 0)
				{
					u64 start = entry.startTime ? entry.startTime : entry.time;
					if (TimeToMs(start - selectedWork.start) > 1)
						logger.Info(TC("   Start (%s)"), TimeToText(start - selectedWork.start).str);
				}

				if (entry.startTime)
					time = entry.time - entry.startTime;

				if (!time)
					for (u32 j=i+1;j<e;++j)
					{
						auto& next = selectedWork.entries[j];
						if (next.startTime)
							continue;
						time = next.time - entry.time;
						break;
					}

				if (!time && selectedWork.stop != ~0u)
					time = selectedWork.stop - entry.time;
				if (entry.count == 1)
					logger.Info(TC("%s  %s (%s)"), (entry.startTime ? TC(" ") : TC("")), entry.text, TimeToText(time).str);
				else
					logger.Info(TC("%s  %s (%s %u)"), (entry.startTime ? TC(" ") : TC("")), entry.text, TimeToText(time).str, entry.count);
			}
			logger.AddSpace();
			logger.DrawAtCursor();
		}
		else if (m_sessionSelectedIndex != ~0u)
		{
			context.SetFont(m_popupFont);
			DrawTextLogger logger(context, m_popupFont.height, m_tooltipBackgroundBrush);
			logger.AddWidth(3);
			logger.AddSpace(2);

			auto& session = m_traceView.sessions[m_sessionSelectedIndex];
			Vector<TString>& summary = session.summary;
			if (summary.empty())
			{
				if (m_traceView.finished)
					logger.Info(TC(" Session summary was never received"));
				else
					logger.Info(TC(" Session summary not available until session is done"));
			}

			logger.AddTextOffset(-10); // Remove spaces in the front
			for (auto& line : summary)
				logger.Log(LogEntryType_Info, line);
			logger.AddTextOffset(0);

			if (m_sessionSelectedIndex == 0)
			{
				for (auto& cacheSummary : m_traceView.cacheSummaries)
					for (auto& line : cacheSummary.lines)
						logger.Log(LogEntryType_Info, line);
			}

			for (auto& pair : session.drives)
			{
				auto& d = pair.second;
				logger.Info(TC("  %c: Read: %s (%u) Write: %s (%u)"), pair.first, BytesToText(d.totalReadBytes).str, d.totalReadCount, BytesToText(d.totalWriteBytes).str, d.totalWriteCount);
			}

			logger.AddSpace(3);
			logger.DrawAtCursor();
		}
		else if (m_statsSelected)
		{
			context.SetFont(m_popupFont);
			DrawTextLogger logger(context, m_popupFont.height, m_tooltipBackgroundBrush);
			logger.AddSpace(3);
			logger.SetColor(m_cpuColor).Info(TC("  Cpu: %.1f%%"), m_stats.cpuLoad * 100.0f);
			logger.SetColor(m_memColor).Info(TC("  Mem: %s/%s"), BytesToText(m_stats.memTotal - m_stats.memAvail).str, BytesToText(m_stats.memTotal).str);
			if (m_stats.recvBytes || m_stats.sendBytes)
			{
				logger.SetColor(m_recvColor).Info(TC("  Recv: %sit/s"), BytesToText(m_stats.recvBytesPerSecond*8).str);
				logger.SetColor(m_sendColor).Info(TC("  Send: %sit/s"), BytesToText(m_stats.sendBytesPerSecond*8).str);
			}
			logger.SetColor(m_textColor);

			if (m_stats.ping)
				logger.Info(TC("  Ping: %s"), TimeToText(m_stats.ping, false, m_traceView.frequency).str);
			if (m_stats.connectionCount)
				logger.Info(TC("  Conn: %u"), u32(m_stats.connectionCount));

			for (auto& pair : m_stats.drives)
				logger.SetColor(m_driveColor).Info(TC("  %c: %u%% R:%s/s W:%s/s"), pair.first, pair.second.busyPercent, BytesToText(pair.second.readPerSecond).str, BytesToText(pair.second.writePerSecond).str);

			logger.AddSpace(3);
			logger.DrawAtCursor();
		}
		else if (m_activeProcessGraphSelected)
		{
			context.SetFont(m_popupFont);
			DrawTextLogger logger(context, m_popupFont.height, m_tooltipBackgroundBrush);
			logger.AddSpace(3);
			logger.AddSpace(3);
			logger.SetColor(m_activeProcColor).Info(TC("  Active Processes: %u"), m_activeProcessCount);
			logger.DrawAtCursor();
		}
		else if (m_buttonSelected != ~0u)
		{
			const tchar* tooltip[] =
			{
#define UBA_VISUALIZER_FLAG(name, defaultValue, desc) TC(desc),
				UBA_VISUALIZER_FLAGS1
#undef UBA_VISUALIZER_FLAG
			};

			context.SetFont(m_popupFont);
			DrawTextLogger logger(context, m_popupFont.height, m_tooltipBackgroundBrush);
			logger.Info(TC("%s %s"), TC("Show"), tooltip[m_buttonSelected]);
			logger.DrawAtCursor();
		}
		else if (m_fetchedFilesSelected != ~0u)
		{
			auto& session = m_traceView.sessions[m_fetchedFilesSelected];
			auto& fetchedFiles = session.fetchedFiles;
			if (!fetchedFiles.empty() && !fetchedFiles[0].hint.empty())
			{
				/*
				int colWidth = 500;
				int width = colWidth * 2;
				int height = Min(int(clientRect.bottom), int(fetchedFiles.size() * m_popupFont.height));

				context.SetFont(m_defaultFont);
				DrawTextLogger logger(context, r, m_font.height, m_tooltipBackgroundBrush);
				for (auto& f : fetchedFiles)
				{
				if (f.hint == TC("KnownInput"))
				continue;
				if (logger.rect.top >= r.bottom - m_font.height)
				{
				if (logger.rect.left + colWidth >= r.right)
				{
				logger.Info(TC("..."));
				break;
				}
				logger.rect.top = r.top;
				logger.rect.left += colWidth;
				}
				logger.Info(TC("%s"), f.hint.c_str());
				}
				logger.DrawAtCursor();
				*/
			}
		}
		context.SetTextColor(m_textColor);
	}

	void Visualizer::PaintProcessRect(PaintContext& context, TraceView::Process& process, RECT rect, const RECT& progressRect, bool selected, bool writingBitmap)
	{
		auto clampRect = [&](RECT& r) { r.left = Min(Max(r.left, progressRect.left), progressRect.right); r.right = Max(Min(r.right, progressRect.right), progressRect.left); };
		bool done = process.stop != ~u64(0);

		BrushHandle brush = m_processBrushes[selected].success;
		if (!process.returnedReason.empty())
			brush = process.type == TraceView::ProcessType_CacheFetchTest ? m_processBrushes[selected].cacheMiss : m_processBrushes[selected].returned;
		else if (process.isIdle)
			brush = m_processBrushes[selected].idle;
		else if (!done)
			brush = process.isRacing ? m_processBrushes[selected].inProgressRace : m_processBrushes[selected].inProgress;
		else if (process.type == TraceView::ProcessType_CacheFetchTest)
			brush = m_processBrushes[selected].cacheFetchTest;
		else if (process.type == TraceView::ProcessType_CacheFetchDownload)
			brush = m_processBrushes[selected].cacheFetchDownload;
		else if (process.exitCode == ProcessCancelExitCode)
			brush = m_processBrushes[selected].returned;
		else if (process.exitCode != 0)
			brush = m_processBrushes[selected].error;

		u64 writeFilesTime = process.writeFilesTime;

		if (!done || process.exitCode != 0 || !m_config.ShowReadWriteColors)
		{
			if (writingBitmap)
				rect.right = 256;
			clampRect(rect);
			context.FillRect(rect, brush);
			return;
		}

		double duration = double(process.stop - process.start);

		RECT main = rect;
		int width = rect.right - rect.left;

		double recvPart = (double(ConvertTime(m_traceView, process.createFilesTime)) / duration);
		if (int headSize = int(recvPart * width))
		{
			main.left += headSize;
			RECT r2 = rect;
			r2.right = r2.left + headSize;
			clampRect(r2);
			if (r2.left != r2.right)
				context.FillRect(r2, m_processBrushes[selected].recv);
		}

		double sendPart = (double(ConvertTime(m_traceView, writeFilesTime)) / duration);
		if (int tailSize = int(sendPart * width))
		{
			main.right -= tailSize;
			RECT r2 = rect;
			r2.left = r2.right - tailSize;
			clampRect(r2);
			if (r2.left != r2.right)
				context.FillRect(r2, m_processBrushes[selected].send);
		}

		clampRect(main);
		if (main.left != main.right)
			context.FillRect(main, brush);
	}

	void Visualizer::PaintActiveProcesses(PaintContext& context, int& posY, const Function<void(TraceView::ProcessLocation&, u32, bool)>& drawProcess)
	{
		context.SetFont(m_processFont);
		int startPosY = posY;

		Map<u64, TraceView::ProcessLocation*> activeProcesses;
		u32 remoteCount = 0;
		for (auto& kv : m_trace.m_activeProcesses)
		{
			auto& active = kv.second;
			TraceView::Process& process = m_trace.GetSession(m_traceView, active.sessionIndex).processors[active.processorIndex].processes[active.processIndex];
			u64 start = process.start; //~0llu - process.start;
			activeProcesses.try_emplace(start, &active);
			if (process.isRemote)
				++remoteCount;
		}

		Font& activeFont = context.GetFont();

		u32 maxHeight = m_clientHeight;

		bool fillHeight = ActiveProcessesShouldFillHeight();
		if (fillHeight)
		{
			maxHeight = m_clientHeight - posY;
			if (m_config.showTimeline)
				maxHeight -= GetTimelineHeight();
		}
		else
		{
			u32 maxHeight2 = m_config.maxActiveVisible * (activeFont.height + 2);
			maxHeight = Min(maxHeight, maxHeight2);
		}

		u32 maxSize = Min(m_config.maxActiveProcessHeight, 32u);
		u32 maxSizeMinusOne = maxSize - 1;

		u32 counts[128] = { 0 };

		u32 highestHistoryCount = 0;
		for (u32 i=0; i!= sizeof_array(m_activeProcessCountHistory) - 1; ++i)
			highestHistoryCount = Max(highestHistoryCount, m_activeProcessCountHistory[i]);

		u32 activeProcessCount = highestHistoryCount;
		counts[0] = activeProcessCount;
		u32 totalHeight = counts[0]*2;
		while (totalHeight < maxHeight && counts[maxSizeMinusOne] != activeProcessCount)
		{
			bool changed = false;
			for (u32 i=0; i!=maxSizeMinusOne; ++i)
			{
				if (counts[i] && counts[i] > counts[i+1]*2 + 1)
				{
					--counts[i];
					++counts[i+1];
					++totalHeight;
					changed = true;
				}
			}
			if (!changed)
			{
				for (u32 j=0; j!=maxSizeMinusOne; ++j)
				{
					if (!counts[j])
						continue;
					++counts[j + 1];
					--counts[j];
					++totalHeight;
					break;
				}
			}
		}

		auto it = activeProcesses.begin();
		auto itEnd = activeProcesses.end();
		int endY = int(startPosY + maxHeight);
		for (u32 i=0;i!=maxSize;++i)
		{
			u32 v = maxSizeMinusOne - i;
			u32 boxHeight = v + 1;

			for (u32 j=0;j!=counts[v];++j)
			{
				if (it == itEnd || posY >= endY)
					break;
				auto& active = *it->second;
				++it;
				drawProcess(active, boxHeight, j == 0);
				posY += boxHeight + 1;
			}
		}

		//if (fillHeight)
		//	posY = clientRect.bottom-1;//startPosY; // To prevent vertical scrollbar
		if (fillHeight || counts[maxSizeMinusOne] != activeProcessCount)
			posY = startPosY + maxHeight;
		else
			posY += 3;

		context.SetFont(m_defaultFont);
	}

	void Visualizer::PaintTimeline(PaintContext& context)
	{
		context.SetFont(m_timelineFont);
		int top = GetTimelineTop();
		float timeScale = (m_config.horizontalScaleValue*m_zoomValue)*50.0f;
		float startOffset = ((m_scrollPosX/timeScale) - float(int(m_scrollPosX/timeScale))) * timeScale;
		int index = -int(startOffset/timeScale);

		int number = -int(float(m_scrollPosX)/timeScale);

		int textStepSize = int((5.0f / timeScale) + 1) * 5;
		if (textStepSize > 150)
			textStepSize = 600;
		else if (textStepSize > 120)
			textStepSize = 300;
		else if (textStepSize > 90)
			textStepSize = 240;
		else if (textStepSize > 45)
			textStepSize = 120;
		else if (textStepSize > 30)
			textStepSize = 60;
		else if (textStepSize > 10)
			textStepSize = 30;

		int lineStepSize = textStepSize / 5;

		RECT progressRect = { 0, 0, m_clientWidth, m_clientHeight };
		progressRect.left += m_progressRectLeft;

		context.SetPen(m_textPen);
		Vector<u32> lines;
		Vector<POINT> points;

		while (true)
		{
			int pos = progressRect.left + int(startOffset + float(index)*timeScale);
			if (pos >= m_clientWidth)
				break;

			int lineBottom = top + 5;
			if (!(number % textStepSize))
			{
				bool shouldDraw = true;
				int seconds = number;
				StringBuffer<> buffer;
				if (seconds >= 60)
				{
					int min = seconds / 60;
					seconds -= min * 60;
					if (!seconds)
					{
						buffer.Appendf(TC("%um"), min);
						lineBottom += 4;
					}
				}
				if (!number || seconds)
					buffer.Appendf(TC("%u"), seconds);

				if (shouldDraw)
				{
					SIZE s = context.GetTextSize(buffer);
					RECT textRect;
					textRect.top = top + 8;
					textRect.bottom = textRect.top + m_timelineFont.height;
					textRect.right = pos + s.cx/2;
					textRect.left = pos - s.cx/2;
					context.Text(textRect.left, textRect.top, buffer, nullptr);
				}
			}
			if (!(number % lineStepSize))
			{
				points.push_back({pos, top});
				points.push_back({pos, lineBottom});
				lines.push_back(2);
			}

			++number;
			++index;
		}

		points.push_back({m_contentWidth, top - 25});
		points.push_back({m_contentWidth, top});
		lines.push_back(2);

		context.PolyPolyline(points.data(), lines.data(), int(lines.size()));
	}

	void Visualizer::PaintDetailedStats(PaintContext& context, int& posY, const RECT& progressRect, TraceView::Session& session, bool isRemote, u64 playTime, const DrawTextFunc& drawTextFunc)
	{
		int stepY = context.GetFont().height;
		int startPosY = posY;
		int posX = progressRect.left + 5;
		int maxY = posY + stepY;
		u32 maxTextWidth = 0;

		RECT textRect;
		auto drawText = [&](const tchar* format, ...)
			{
				textRect.top = posY;
				textRect.bottom = posY + 20;
				textRect.left = posX;
				textRect.right = posX + 1000;
				posY += stepY;
				StringBuffer<> str;
				va_list arg;
				va_start(arg, format);
				str.Append(format, arg);
				va_end(arg);
				u32 lastTextWidth = 0;
				drawTextFunc(str, textRect, &lastTextWidth);
				maxTextWidth = Max(maxTextWidth, lastTextWidth);
				maxY = Max(maxY, posY);
			};

		if (isRemote)
		{
			drawText(TC("Finished Processes: %u"), session.processExitedCount);
			drawText(TC("Active Processes: %u"), session.processActiveCount);
			drawText(TC("ClientId: %u  TcpCount: %u"), session.clientUid.data1, session.connectionCount.empty() ? 1 : session.connectionCount.back());

			if (session.disconnectTime == ~u64(0))
			{
				if (session.proxyCreated)
					drawText(TC("Proxy(HOSTED): %s"), session.proxyName.c_str());
				else if (!session.proxyName.empty())
					drawText(TC("Proxy: %s"), session.proxyName.c_str());
				else
					drawText(TC("Proxy: None"));
			}

			bool hasFileDetails = !session.fetchedFiles.empty() || !session.storedFiles.empty();

			if (!hasFileDetails)
			{
				posY = startPosY;
				posX += maxTextWidth + 15;
			}

			if (!session.updates.empty())
			{
				auto updateTime = session.updates.back();
				auto updateSend = session.networkSend.back();
				auto updateRecv = session.networkRecv.back();
				u64 sendPerS = 0;
				u64 recvPerS = 0;
				float duration = TimeToS(updateTime - session.prevUpdateTime);
				if (duration != 0)
				{
					sendPerS = u64(float(updateSend - session.prevSend) / duration);
					recvPerS = u64(float(updateRecv - session.prevRecv) / duration);
				}
				drawText(TC("Recv: %s (%sit/s)"), BytesToText(updateRecv), BytesToText(recvPerS*8));
				drawText(TC("Send: %s (%sit/s)"), BytesToText(updateSend), BytesToText(sendPerS*8));
			}

			int fileWidth = 700;

			auto drawFiles = [&](const tchar* fileType, Vector<TraceView::FileTransfer>& files, u64 count, u64 bytes, u64 activeCount, u32& maxVisibleFiles)
				{
					textRect.left = posX;
					textRect.right = posX + fileWidth;
					drawText(TC("%s Files: %u (%u) %s"), fileType, u32(count), u32(activeCount), BytesToText(bytes));
					u32 fileCount = 0;
					for (auto rit = files.rbegin(), rend = files.rend(); rit != rend; ++rit)
					{
						TraceView::FileTransfer& file = *rit;
						if (file.stop != ~u64(0))
							continue;
						u64 time = 0;
						if (file.start < playTime)
							time = playTime - file.start;

						if ((((u8*)&file.key)[19] & IsProxyMask) != 0)
							drawText(TC("%s (proxy) %s"), file.hint.c_str(), TimeToText(time, true).str);
						else if (file.size == 0)
							drawText(TC("%s (calc) %s"), file.hint.c_str(), TimeToText(time, true).str);
						else
							drawText(TC("%s (%s) %s"), file.hint.c_str(), BytesToText(file.size).str, TimeToText(time, true).str);

						if (fileCount++ > 5)
							break;
					}
					posY += stepY * (maxVisibleFiles - fileCount);
					maxVisibleFiles = Max(maxVisibleFiles, fileCount);
				};

			Vector<TraceView::FileTransfer> fetchedFiles;
			for (auto& kv : session.fetchedFilesActive)
				fetchedFiles.push_back(session.fetchedFiles[kv.second]);
			std::sort(fetchedFiles.begin(), fetchedFiles.end(), [](const TraceView::FileTransfer& a, const TraceView::FileTransfer& b) { return a.start > b.start; });

			if (hasFileDetails)
			{
				posY = startPosY;
				posX += maxTextWidth + 15;
			}
			drawFiles(TC("Fetched"), fetchedFiles, session.fetchedFilesCount, session.fetchedFilesBytes, session.fetchedFilesActive.size(), session.maxVisibleFiles);
			if (hasFileDetails)
			{
				posY = startPosY;
				posX += fileWidth;
			}
			drawFiles(TC("Stored"), session.storedFiles, session.storedFilesCount, session.storedFilesBytes, session.storedFilesActive.size(), session.maxVisibleFiles);
		}
		else
		{
			if (m_traceView.totalProcessActiveCount || m_traceView.totalProcessExitedCount)
			{
				drawText(TC("Finished Processes: %u (local: %u)"), m_traceView.totalProcessExitedCount, session.processExitedCount);
				drawText(TC("Active Processes: %u (local: %u)"), m_traceView.totalProcessActiveCount, session.processActiveCount);
				drawText(TC("Active Helpers: %u"), Max(1u, m_traceView.activeSessionCount) - 1);
			}

			if (session.highestSendPerS > 0.0f || session.highestRecvPerS > 0.0f)
			{
				auto updateTime = session.updates.back();
				auto updateSend = session.networkSend.back();
				auto updateRecv = session.networkRecv.back();
				if (updateSend || updateRecv)
				{
					u64 sendPerS = 0;
					u64 recvPerS = 0;
					float duration = TimeToS(updateTime - session.prevUpdateTime);
					if (duration != 0)
					{
						sendPerS = u64(float(updateSend - session.prevSend) / duration);
						recvPerS = u64(float(updateRecv - session.prevRecv) / duration);
					}

					drawText(TC("Recv: %s (%sit/s)"), BytesToText(updateRecv), BytesToText(recvPerS*8));
					drawText(TC("Send: %s (%sit/s)"), BytesToText(updateSend), BytesToText(sendPerS*8));
				}
			}

			if (!session.updates.empty())
			{
				posY = startPosY;
				posX += maxTextWidth + 10;

				for (auto& pair : session.drives)
				{
					auto& drive = pair.second;

					u64 readPerS = 0;
					u64 writePerS = 0;

					u64 updateCount = session.updates.size();
					float duration = TimeToS(session.updates.back() - session.prevUpdateTime);
					if (duration != 0 && updateCount > 1)
					{
						readPerS = u64(float(drive.readBytes.back()) / duration);
						writePerS = u64(float(drive.writeBytes.back()) / duration);
					}

					drawText(TC("%c: Rd %s (%s/s) Wr %s (%s/s) "), pair.first, BytesToText(drive.totalReadBytes).str, BytesToText(readPerS).str, BytesToText(drive.totalWriteBytes).str, BytesToText(writePerS).str);
				}
			}
		}

		posY = maxY;
	}

	void Visualizer::PrintCacheWriteStats(Logger& logger, u32 processId)
	{
		auto findIt = m_traceView.cacheWrites.find(processId);
		if (findIt == m_traceView.cacheWrites.end())
			return;
		TraceView::CacheWrite& write = findIt->second;
		logger.Info(TC(""));
		logger.Info(TC("  -------- Cache write stats ----------"));
		if (write.end)
		{
			logger.Info(TC("  Duration                    %9s"), TimeToText(write.end - write.start).str);
			logger.Info(TC("  Success                     %9s"), write.success ? TC("true") : TC("false"));

			BinaryReader reader(write.stats.data(), 0, write.stats.size());
			CacheSendStats stats;
			if (m_traceView.version < 45)
			{
				stats.sendFile.bytes = reader.Read7BitEncoded();
				stats.sendFile.count = 1;
			}
			else
				stats.Read(reader, m_traceView.version);
			stats.Print(logger, m_traceView.frequency);
		}
		else
		{
			logger.Info(TC("  In progress..."));
		}
	}

	void Visualizer::WriteProcessStats(Logger& out, const TraceView::Process& process)
	{
		bool hasExited = process.stop != ~u64(0);
		out.Info(TC("  %s"), process.description.c_str());
		out.Info(TC("  ProcessId: %u"), process.id);
		out.Info(TC("  Start:     %s"), TimeToText(process.start, true).str);
		if (hasExited)
			out.Info(TC("  Duration:  %s"), TimeToText(process.stop - process.start, true).str);
		if (hasExited && process.exitCode != 0)
			out.Info(TC("  ExitCode:  %u"), process.exitCode);

		if (process.stop != ~u64(0) && process.stats.size())
		{
			out.Info(TC(""));

			BinaryReader reader(process.stats.data(), 0, process.stats.size());
			ProcessStats processStats;
			SessionStats sessionStats;
			StorageStats storageStats;
			KernelStats kernelStats;

			processStats.Read(reader, m_traceView.version);
			if (reader.GetLeft())
			{
				if (process.isRemote || (m_traceView.version >= 36 && !process.isReuse))
					sessionStats.Read(reader, m_traceView.version);
				storageStats.Read(reader, m_traceView.version);
				kernelStats.Read(reader, m_traceView.version);
			}

			out.Info(TC("  ----------- Detours stats -----------"));
			processStats.Print(out, m_traceView.frequency);

			if (!sessionStats.IsEmpty())
			{
				out.Info(TC(""));
				out.Info(TC("  ----------- Session stats -----------"));
				sessionStats.Print(out, m_traceView.frequency);
			}

			if (!storageStats.IsEmpty())
			{
				out.Info(TC(""));
				out.Info(TC("  ----------- Storage stats -----------"));
				storageStats.Print(out, m_traceView.frequency);
			}

			if (!kernelStats.IsEmpty())
			{
				out.Info(TC(""));
				out.Info(TC("  ----------- Kernel stats ------------"));
				kernelStats.Print(out, false, m_traceView.frequency);
			}

			PrintCacheWriteStats(out, process.id);
		}
	}

	void Visualizer::WriteWorkStats(Logger& out, const TraceView::WorkRecord& record)
	{
		out.Info(TC("  %s"), record.description);
		out.Info(TC("  Start:     %s"), TimeToText(record.start, true).str);
		if (record.stop != ~u64(0))
			out.Info(TC("  Duration:  %s"), TimeToText(record.stop - record.start, true).str);
		for (auto& e : record.entries)
			out.Info(TC("   %s (%s)"), e.text, TimeToText(e.time - record.start).str);
	}

	void Visualizer::PaintEmptyScreen(PaintContext& context)
	{
		StringBuffer<> str[2];
		if (!m_fileName.IsEmpty())
		{
			str[0].Append(TCV("Loading trace file"));
		}
		else
		{
			str[1].Append(TCV("Click here to open trace file"));
			if (m_listenChannel.count)
			{
				str[0].Appendf(TC("Listening for new sessions on channel '%s'"), m_listenChannel.data);
				str[1].Append(TCV(" instead"));
			}
			else
				str[0].Append(TCV("No trace active"));
		}
		context.SetFont(m_popupFont);
		DrawCenteredText(context, str, 2, nullptr);
	}

	void Visualizer::FinishIncompatible()
	{
	}

	void Visualizer::Redraw(bool now)
	{
	}

	void Visualizer::UpdateScrollbars(bool redraw)
	{
	}

	void Visualizer::PostNewTitle(const StringView& title)
	{
	}

	void Visualizer::StopUpdate()
	{
	}

	void Visualizer::GetFontMetrics(int& outFh, int& outOffset, int height)
	{
		outFh = height;
		outOffset = 0;
		if (height <= 13)
		{
			++outFh;
			--outOffset;
		}
		if (height <= 11)
			++outFh;
		if (height <= 9)
			++outFh;
		if (height <= 8)
			++outFh;
		if (height <= 6)
		{
			++outFh;
			--outOffset;
		}
		if (height <= 4)
			--outOffset;
	}

	bool Visualizer::NewTrace(u32 replay, bool paused)
	{
		m_replay = replay;
		m_paused = paused;
		m_autoScroll = true;
		m_scrollPosX = 0;
		m_scrollPosY = 0;
		Reset();
		StringBuffer<> title;
		GetTitlePrefix(title);

		auto g = MakeGuard([&]()
			{
				Redraw(true);
				UpdateScrollbars(true);
			});

		if (m_client)
		{
			if (!m_trace.StartReadClient(m_traceView, *m_client))
			{
				m_clientDisconnect.Set();
				return false;
			}
			m_namedTrace.Clear().Append(m_newTraceName);
			m_traceView.finished = false;
		}
		else if (!m_fileName.IsEmpty())
		{
			m_trace.ReadFile(m_traceView, m_fileName.data, m_replay != 0);
			m_traceView.finished = m_replay == 0;
			PostNewTitle(GetTitlePrefix(title).Appendf(TC("%s (v%u) - %s"), m_fileName.data, m_traceView.version, GetWorldTime((u64)0).data));
		}
		else if (m_usingNamed)
		{
			if (!m_trace.StartReadNamed(m_traceView, m_newTraceName.data, true, m_replay != 0))
				return false;
			m_namedTrace.Clear().Append(m_newTraceName);
			m_traceView.finished = false;
			PostNewTitle(GetTitlePrefix(title).Appendf(TC("%s (Listening for new sessions on channel '%s')"), m_namedTrace.data, m_listenChannel.data));
		}

		return true;
	}

	bool Visualizer::UpdateTrace()
	{
		bool changed = false;
		if (!m_paused)
		{
			u64 timeOffset = (GetTime() - m_startTime - m_pauseTime) * m_replay;
			if (!m_fileName.IsEmpty())
			{
				if (m_replay)
					m_trace.UpdateReadFile(m_traceView, timeOffset, changed);
			}
			else if (m_client)
			{
				if (!m_trace.UpdateReadClient(m_traceView, *m_client, changed))
					m_clientDisconnect.Set();
			}
			else if (m_usingNamed)
			{
				if (!m_trace.UpdateReadNamed(m_traceView, m_replay ? timeOffset : ~u64(0), changed))
					if (m_listenTimeout.IsCreated())
						m_listenTimeout.Set();
			}
		}

		if (m_traceView.finished)
		{
			m_autoScroll = false;
			StopUpdate();
			changed = true;
		}

		changed = UpdateAutoscroll() || changed;
		changed = UpdateSelection() || changed;

		return changed;
	}

	bool Visualizer::UpdateAutoscroll()
	{
		if (!m_autoScroll)
			return false;

		u64 playTime = GetPlayTime();

		int right = m_clientWidth;
		if (right == 0)
			return false;

		float timeS = TimeToS(playTime);

		if (m_config.AutoScaleHorizontal)
		{
			m_scrollPosX = 0;
			timeS = Max(timeS, 20.0f/m_zoomValue);
			m_config.horizontalScaleValue = Max(float(right - m_progressRectLeft - 2)/(m_zoomValue*timeS*50.0f), 0.001f);
			return true;
		}
		else
		{
			float oldScrollPosX = m_scrollPosX;
			m_scrollPosX = Min(0.0f, (float)right - timeS*50.0f*m_config.horizontalScaleValue*m_zoomValue - float(m_progressRectLeft));
			return oldScrollPosX != m_scrollPosX;
		}
	}

	bool Visualizer::UpdateSelection()
	{
		return false;
	}

	void Visualizer::Reset()
	{
		m_backend.Reset();
		m_contentWidth = 0;
		m_contentHeight = 0;
		//m_autoScroll = true;
		//m_scrollPosX = 0;
		//m_scrollPosY = 0;
		//m_zoomValue = 0.75f;
		//m_horizontalScaleValue = 1.0f;

		m_startTime = GetTime();
		m_pauseTime = 0;

		Unselect();
	}

	bool Visualizer::Unselect()
	{
		if (m_processSelected || m_sessionSelectedIndex != ~0u || m_statsSelected || m_timelineSelected != 0 || m_fetchedFilesSelected != ~0u || m_workSelected || !m_hyperLinkSelected.empty())
		{
			m_processSelected = false;
			m_sessionSelectedIndex = ~0u;
			m_statsSelected = false;
			m_activeProcessGraphSelected = false;
			m_buttonSelected = ~0u;
			m_timelineSelected = 0;
			m_fetchedFilesSelected = ~0u;
			m_workSelected = false;
			m_hyperLinkSelected.clear();
			return true;
		}
		return false;
	}

	void Visualizer::UpdateFont(Font& font, int height, bool createUnderline)
	{
		font.height = height;
		int fh;
		GetFontMetrics(fh, font.offset, height);

		if (font.handle)
			m_backend.DeleteFont(font.handle);
		font.handle = 0;
		if (font.handleUnderlined)
			m_backend.DeleteFont(font.handleUnderlined);
		font.handleUnderlined = 0;

		font.handle = m_backend.CreateFont(m_config.fontName.c_str(), fh, false);
		if (createUnderline)
			font.handleUnderlined = m_backend.CreateFont(m_config.fontName.c_str(), fh, true);
	}

	void Visualizer::UpdateDefaultFont()
	{
		UpdateFont(m_defaultFont, m_config.fontSize, true);
		m_sessionStepY = m_defaultFont.height + 4;
		m_timelineFont = m_defaultFont;
	}

	void Visualizer::UpdateProcessFont()
	{
		m_zoomValue = 1.0f + float(m_config.boxHeight) / 30.0f;
		int fontHeight = Max(int(m_config.boxHeight) - 2, 1);
		UpdateFont(m_processFont, fontHeight, false);
		m_progressRectLeft = int(13 + float(m_processFont.height) * 1.5f);
	}

	void Visualizer::DirtyBitmaps(bool full)
	{
		for (auto& session : m_traceView.sessions)
			for (auto& processor : session.processors)
				for (auto& process : processor.processes)
				{
					process.bitmapDirty = true;
					if (full)
						process.bitmap = 0;
				}

		for (auto& workTrack : m_traceView.workTracks)
			for (auto& work : workTrack.records)
			{
				work.bitmapDirty = true;
				if (full)
					work.bitmap = 0;
			}

		if (!full)
			return;

		m_backend.Reset();
	}

	bool Visualizer::UpdateSelection(POINT pos)
	{
		HitTestResult res;
		{
			PaintContext* context = m_backend.CreateContext(m_clientWidth, m_clientHeight);
			HitTest(res, *context, pos);
			m_backend.DeleteContext(context);
		}
		m_activeSection = res.section;

		if (res.processSelected == m_processSelected && res.processLocation == m_processSelectedLocation &&
			res.sessionSelectedIndex == m_sessionSelectedIndex &&
			res.statsSelected == m_statsSelected && res.stats == m_stats &&
			res.buttonSelected == m_buttonSelected && res.timelineSelected == m_timelineSelected &&
			res.activeProcessGraphSelected == m_activeProcessGraphSelected && 
			res.activeProcessCount == m_activeProcessCount && 
			res.fetchedFilesSelected == m_fetchedFilesSelected &&
			res.workSelected == m_workSelected && res.workTrack == m_workTrack && res.workIndex == m_workIndex &&
			res.hyperLink == m_hyperLinkSelected)
			return false;
		m_processSelected = res.processSelected;
		m_processSelectedLocation = res.processLocation;
		m_sessionSelectedIndex = res.sessionSelectedIndex;
		m_statsSelected = res.statsSelected;
		m_stats = res.stats;
		m_activeProcessGraphSelected = res.activeProcessGraphSelected;
		m_activeProcessCount = res.activeProcessCount;
		m_buttonSelected = res.buttonSelected;
		m_timelineSelected = res.timelineSelected;
		m_fetchedFilesSelected = res.fetchedFilesSelected;
		m_workSelected = res.workSelected;
		m_workTrack = res.workTrack;
		m_workIndex = res.workIndex;
		m_hyperLinkSelected = res.hyperLink;
		return true;
	}

	void Visualizer::UnselectAndRedraw()
	{
		if (Unselect() || m_config.showCursorLine)
			Redraw(false);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
