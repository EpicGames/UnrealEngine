// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaTraceReader.h"
#include "UbaVisualizerBackend.h"
#include "UbaVisualizerConfig.h"

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	class Visualizer
	{
	public:
		Visualizer(VisualizerConfig& config, Logger& logger, VisualizerBackend& backend);
		void InitBrushes();

		u64 GetPlayTime();
		int GetTimelineHeight();
		int GetTimelineTop();
		StringBufferBase& GetTitlePrefix(StringBufferBase& out);
		StringBuffer<128> GetWorldTime(u64 time);
		StringBuffer<128> GetWorldTime(float seconds);
		bool ActiveProcessesShouldFillHeight();

		using Font = VisualizerBackend::Font;
		using PaintContext = VisualizerBackend::PaintContext;
		struct HitTestResult;

		void HitTest(HitTestResult& outResult, PaintContext& context, const POINT& pos);
		void PaintAll(PaintContext& context, const RECT& clientRect);
		void PaintProcessRect(PaintContext& context, TraceView::Process& process, RECT rect, const RECT& progressRect, bool selected, bool writingBitmap);
		void PaintActiveProcesses(PaintContext& context, int& posY, const Function<void(TraceView::ProcessLocation&, u32, bool)>& drawProcess);
		void PaintTimeline(PaintContext& context);
		using DrawTextFunc = Function<void(const StringBufferBase& text, RECT& rect, u32* outWidth)>;
		void PaintDetailedStats(PaintContext& context, int& posY, const RECT& progressRect, TraceView::Session& session, bool isRemote, u64 playTime, const DrawTextFunc& drawTextFunc);
		void PrintCacheWriteStats(Logger& logger, u32 processId);
		void WriteProcessStats(Logger& out, const TraceView::Process& process);
		void WriteWorkStats(Logger& out, const TraceView::WorkRecord& record);

		virtual void PaintEmptyScreen(PaintContext& context);
		virtual void FinishIncompatible();
		virtual void Redraw(bool now);
		virtual void UpdateScrollbars(bool redraw);
		virtual void PostNewTitle(const StringView& title);
		virtual bool UpdateSelection();
		virtual void StopUpdate();
		virtual void GetFontMetrics(int& outFh, int& outOffset, int height);

		bool NewTrace(u32 replay, bool paused);
		bool UpdateTrace();
		bool UpdateAutoscroll();
		void Reset();
		bool Unselect();
		void UpdateFont(Font& font, int height, bool createUnderline);
		void UpdateDefaultFont();
		void UpdateProcessFont();
		void DirtyBitmaps(bool full);
		bool UpdateSelection(POINT pos);
		void UnselectAndRedraw();

		enum VisualizerFlag
		{
			#define UBA_VISUALIZER_FLAG(name, defaultValue, desc) VisualizerFlag_##name,
			UBA_VISUALIZER_FLAGS1
			#undef UBA_VISUALIZER_FLAG
			VisualizerFlag_Count
		};

		struct Stats
		{
			u64 recvBytesPerSecond = 0;
			u64 sendBytesPerSecond = 0;
			u64 ping = 0;
			u64 memAvail = 0;
			u64 memTotal = 0;
			u64 recvBytes = 0;
			u64 sendBytes = 0;
			u64 procActive = 0;
			float cpuLoad = 0;
			u16 connectionCount = 0;

			struct Drive
			{
				u8 busyPercent;
				u32 readPerSecond = 0;
				u32 writePerSecond = 0;
			};
			Map<char, Drive> drives;
			bool operator==(const Stats& o) const;
		};

		struct HitTestResult
		{
			u32 section = ~0u;
			TraceView::ProcessLocation processLocation;
			bool processSelected = false;
			u32 sessionSelectedIndex = ~0u;
			bool statsSelected = false;
			Stats stats;
			u32 buttonSelected = ~0u;
			float timelineSelected = 0;
			u32 fetchedFilesSelected = ~0u;
			bool workSelected = false;
			u32 workTrack = ~0u;
			u32 workIndex = ~0u;
			bool activeProcessGraphSelected = false;
			u16 activeProcessCount = 0u;
			TString hyperLink;
		};

		struct ProcessBrushes
		{
			BrushHandle inProgress = 0;
			BrushHandle inProgressRace = 0;
			BrushHandle success = 0;
			BrushHandle error = 0;
			BrushHandle returned = 0;
			BrushHandle recv = 0;
			BrushHandle send = 0;
			BrushHandle cacheFetchTest = 0;
			BrushHandle cacheFetchDownload = 0;
			BrushHandle cacheMiss = 0;
			BrushHandle warning = 0;
			BrushHandle idle = 0;
		};

		Logger& m_logger;
		VisualizerConfig m_config;
		TraceReader m_trace;
		TraceView m_traceView;
		VisualizerBackend& m_backend;

		StringBuffer<256> m_namedTrace;
		StringBuffer<256> m_fileName;
		NetworkClient* m_client = nullptr;
		Event m_clientDisconnect;
		Event m_listenTimeout;
		StringBuffer<256>m_listenChannel;
		StringBuffer<256> m_newTraceName;
		bool m_usingNamed = false;


		int m_clientWidth;
		int m_clientHeight;

		ProcessBrushes m_processBrushes[2]; // Non-selected and selected

		static constexpr int GraphHeight = 30;

		COLORREF m_textColor = {};
		COLORREF m_textWarningColor = {};
		COLORREF m_textErrorColor = {};
		COLORREF m_sendColor = {};
		COLORREF m_recvColor = {};
		COLORREF m_cpuColor = {};
		COLORREF m_memColor = {};
		COLORREF m_driveColor = {};
		COLORREF m_activeProcColor = {};
		BrushHandle m_backgroundBrush = 0;
		BrushHandle m_tooltipBackgroundBrush = 0;
		PenHandle m_textPen = 0;
		PenHandle m_separatorPen = 0;
		PenHandle m_sendPen = 0;
		PenHandle m_recvPen = 0;
		PenHandle m_cpuPen = 0;
		PenHandle m_memPen = 0;
		PenHandle m_drivePen = 0;
		PenHandle m_activeProcPen = 0;
		PenHandle m_processUpdatePen = 0;
		PenHandle m_checkboxPen = 0;
		int m_sessionStepY = 0;

		int m_processFontOffsetY = 0;

		Font m_defaultFont;
		Font m_processFont;
		Font m_timelineFont;
		Font m_popupFont;

		Font m_activeProcessFont[32];
		u32 m_activeProcessCountHistory[5];
		u32 m_activeProcessCountHistoryIterator = 0;

		bool m_autoScroll = true;
		bool m_paused = false;
		u32 m_replay = 0;
		u64 m_startTime = 0;
		u64 m_pauseTime = 0;
		u64 m_pauseStart = 0;

		bool m_mouseOverWindow = false;

		int m_progressRectLeft = 30;
		int m_contentWidth = 0;
		int m_contentHeight = 0;

		float m_scrollPosX = 0;
		float m_scrollPosY = 0;
		float m_zoomValue = 0.5f;
		float m_horizontalScaleValue = 0.5f;

		u32 m_activeSection = ~0u;
		TraceView::ProcessLocation m_processSelectedLocation;
		bool m_processSelected = false;
		u32 m_sessionSelectedIndex = ~0u;
		bool m_statsSelected = false;
		bool m_activeProcessGraphSelected = false;
		u64 m_activeProcessCount = 0u;
		Stats m_stats;
		u32 m_buttonSelected = ~0u;
		float m_timelineSelected = 0;
		u32 m_fetchedFilesSelected = ~0u;
		TString m_hyperLinkSelected;

		TString m_filterString;

		UnorderedMap<Color, BrushHandle> m_coloredBrushes;

		bool m_workSelected = false;
		u32 m_workTrack = ~0u;
		u32 m_workIndex = ~0u;

		struct DrawTextLogger;
		void DrawCenteredText(VisualizerBackend::PaintContext& context, const StringBuffer<>* rows, u32 rowCount, BrushHandle backgroundBrush);
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////
}