// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaTraceReader.h"

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	#if !PLATFORM_WINDOWS
	struct POINT { int x; int y; };
	struct SIZE { int cx; int cy; };
	struct RECT { int left; int top; int right; int bottom; };
	using COLORREF = u32;
	#define RGB(r,g,b) ((COLORREF)((u8(r)|(u16(u8(g))<<8))|(u32(u8(b))<<16)))
	#define GET_RED(c) u8(c & 0xff)
	#define GET_GREEN(c) u8((c >> 8) & 0xff)
	#define GET_BLUE(c) u8((c >> 16) & 0xff)
	#endif

	using BrushHandle = void*;
	using PenHandle = void*;
	using FontHandle = void*;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class VisualizerBackend
	{
	public:
		struct Font
		{
			FontHandle handle = 0;
			FontHandle handleUnderlined = 0;
			int height = 0;
			int offset = 0;
		};

		struct PaintContext
		{
			virtual SIZE GetClientSize() = 0;
			virtual void SetPen(PenHandle pen) = 0;
			virtual void SetBrush(BrushHandle b) = 0;
			virtual void SetFont(const Font& font, bool underlined = false) = 0;
			virtual void SetTextColor(COLORREF color) = 0;
			virtual void Text(int x, int y, StringView str, const RECT* clipRect) = 0;
			virtual COLORREF GetTextColor() = 0;
			virtual Font& GetFont() = 0;
			virtual void FillRect(const RECT& r, BrushHandle b) = 0;
			virtual void Rectangle(int left, int top, int right, int bottom) = 0;
			virtual void Line(int fromX, int fromY, int toX, int toY) = 0;
			virtual void Polyline(u32 cacheId, const Vector<POINT>& polyLine) = 0;
			virtual void PolyPolyline(const POINT* points, const u32* lines, u32 lineCount) = 0;
			virtual SIZE GetTextSize(StringView str) = 0;
			virtual void DrawCached(TraceView::BitmapCache& process, RECT rect, int offsetY, int textHeight, Font& processFont, RECT clippingRect, const Function<void(PaintContext& textCacheContext)>& func) = 0;
			virtual void EndCached() {}

			virtual POINT GetCursorClientPos() = 0;
		};

		virtual const tchar* GetName() = 0;
		virtual void Init(void* windowHandle) = 0;
		virtual void Reset() = 0;

		virtual void Resize(int width, int height) = 0;
		virtual void Paint(const Function<void(PaintContext& context, RECT rect)>& func) = 0;

		virtual PaintContext* CreateContext(int clientWidth, int clientHeight) = 0;
		virtual void DeleteContext(PaintContext*) = 0;

		virtual BrushHandle CreateSolidBrush(COLORREF color) = 0;
		virtual PenHandle CreatePen(COLORREF color) = 0;

		virtual FontHandle CreateFont(const tchar* font, int height, bool createUnderline) = 0;
		virtual void DeleteFont(FontHandle font) = 0;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
