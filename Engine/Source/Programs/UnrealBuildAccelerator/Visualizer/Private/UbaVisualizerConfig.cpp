// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaVisualizerConfig.h"
#include "UbaConfig.h"
#include "UbaPlatform.h"

namespace uba
{
	VisualizerConfig::VisualizerConfig(const tchar* fn) : filename(fn)
	{
		fontName = TC("Arial");
	}

	bool VisualizerConfig::Load(Logger& logger)
	{
		Config config;
		if (!config.LoadFromFile(logger, filename.c_str()))
		{
			#if PLATFORM_WINDOWS
			DWORD value = 1;
			DWORD valueSize = sizeof(value);
			if (RegGetValueW(HKEY_CURRENT_USER, L"Software\\Microsoft\\Windows\\CurrentVersion\\Themes\\Personalize", L"AppsUseLightTheme", RRF_RT_REG_DWORD, NULL, &value, &valueSize) == ERROR_SUCCESS)
				DarkMode = value == 0;
			#endif
			return false;
		}
		config.GetValueAsInt(x, TC("X"));
		config.GetValueAsInt(y, TC("Y"));
		config.GetValueAsU32(width, TC("Width"));
		config.GetValueAsU32(height, TC("Height"));
		config.GetValueAsU32(fontSize, TC("FontSize"));
		config.GetValueAsString(fontName, TC("FontName"));
		config.GetValueAsU32(maxActiveVisible, TC("MaxActiveVisible"));
		config.GetValueAsU32(maxActiveProcessHeight, TC("MaxActiveProcessHeight"));

		config.GetValueAsU32(boxHeight, TC("BoxHeight"));
		u64 temp;
		if (config.GetValueAsU64(temp, TC("HorizontalScrollValue")))
			horizontalScaleValue = float(double(temp) / 10000000.0);

#define UBA_VISUALIZER_FLAG(name, defaultValue, desc) config.GetValueAsBool(show##name, TC("Show" #name));
		UBA_VISUALIZER_FLAGS1
		#undef UBA_VISUALIZER_FLAG
		#define UBA_VISUALIZER_FLAG(name, defaultValue, desc) config.GetValueAsBool(name, TC(#name));
		UBA_VISUALIZER_FLAGS2
		#undef UBA_VISUALIZER_FLAG

		config.GetValueAsBool(useGDI, TC("UseGDI"));

		fontSize = Min(fontSize, 30u);
		return true;
	}

	bool VisualizerConfig::Save(Logger& logger)
	{
		Config config;
		config.AddValue(TC("X"), x);
		config.AddValue(TC("Y"), y);
		config.AddValue(TC("Width"), width);
		config.AddValue(TC("Height"), height);
		config.AddValue(TC("FontSize"), fontSize);
		config.AddValue(TC("FontName"), fontName.c_str());
		config.AddValue(TC("MaxActiveVisible"), maxActiveVisible);
		config.AddValue(TC("MaxActiveProcessHeight"), maxActiveProcessHeight);

		config.AddValue(TC("BoxHeight"), boxHeight);
		config.AddValue(TC("HorizontalScrollValue"), u64(double(horizontalScaleValue)*10000000.0));

		#define UBA_VISUALIZER_FLAG(name, defaultValue, desc) config.AddValue(TC("Show" #name), show##name);
		UBA_VISUALIZER_FLAGS1
		#undef UBA_VISUALIZER_FLAG
		#define UBA_VISUALIZER_FLAG(name, defaultValue, desc) config.AddValue(TC(#name), name);
		UBA_VISUALIZER_FLAGS2
		#undef UBA_VISUALIZER_FLAG

		config.AddValue(TC("UseGDI"), useGDI);

		return config.SaveToFile(logger, filename.c_str());
	}
}