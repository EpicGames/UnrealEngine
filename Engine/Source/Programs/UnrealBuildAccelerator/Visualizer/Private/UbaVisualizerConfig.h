// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaMemory.h"
#include "UbaString.h"

namespace uba
{
	class Config;
	class Logger;

	#define UBA_VISUALIZER_FLAGS1 \
		UBA_VISUALIZER_FLAG(Progress, true, "progress") \
		UBA_VISUALIZER_FLAG(Status, true, "status") \
		UBA_VISUALIZER_FLAG(ActiveProcesses, false, "active processes") \
		UBA_VISUALIZER_FLAG(TitleBars, true, "instance title bars") \
		UBA_VISUALIZER_FLAG(DetailedData, false, "detailed data (use -UbaDetailedTrace for even more)") \
		UBA_VISUALIZER_FLAG(CpuMemStats, true, "cpu/mem stats") \
		UBA_VISUALIZER_FLAG(NetworkStats, true, "network stats") \
		UBA_VISUALIZER_FLAG(DriveStats, true, "drive stats") \
		UBA_VISUALIZER_FLAG(ActiveProcessGraph, false, "graph of active processes over time") \
		UBA_VISUALIZER_FLAG(ProcessBars, true, "process bars") \
		UBA_VISUALIZER_FLAG(FinishedProcesses, true, "finished process bars") \
		UBA_VISUALIZER_FLAG(Timeline, true, "timeline") \
		UBA_VISUALIZER_FLAG(Workers, false, "workers (threads on host taking care of requests from helpers)") \
		UBA_VISUALIZER_FLAG(CursorLine, false, "cursor (vertical line)") \

	#define UBA_VISUALIZER_FLAGS2 \
		UBA_VISUALIZER_FLAG(ShowProcessText, true, "Show text in process bars") \
		UBA_VISUALIZER_FLAG(ShowReadWriteColors, true, "Show colors for read/write times in process bars") \
		UBA_VISUALIZER_FLAG(ScaleHorizontalWithScrollWheel, false, "Use scroll wheel to scale horizontally") \
		UBA_VISUALIZER_FLAG(DarkMode, false, "Use dark mode to draw visualizer") \
		UBA_VISUALIZER_FLAG(AutoSaveSettings, true, "Auto save Position/Settings on close") \
		UBA_VISUALIZER_FLAG(ShowAllTraces, true, "Show all traces started on channel") \
		UBA_VISUALIZER_FLAG(SortActiveRemoteSessions, true, "Sort active sessions on top") \
		UBA_VISUALIZER_FLAG(AutoScaleHorizontal, true, "Automatically scale horizontally to fit processes") \
		UBA_VISUALIZER_FLAG(LockTimelineToBottom, true, "Lock timeline to always paint at bottom") \

	struct VisualizerConfig
	{
		VisualizerConfig(const tchar* filename);

		bool Load(Logger& logger);
		bool Save(Logger& logger);

		TString filename;

		int x = 100;
		int y = 100;
		u32 width = 1500;
		u32 height = 1500;
		u32 fontSize = 13;
		TString fontName;
		u32 maxActiveVisible = 5;
		u32 maxActiveProcessHeight = 16;
		u32 boxHeight = 12;
		float horizontalScaleValue = 0.5f;
		bool useGDI = false;

		#define UBA_VISUALIZER_FLAG(name, defaultValue, desc) bool show##name = defaultValue;
		UBA_VISUALIZER_FLAGS1
		#undef UBA_VISUALIZER_FLAG

		#define UBA_VISUALIZER_FLAG(name, defaultValue, desc) bool name = defaultValue;
		UBA_VISUALIZER_FLAGS2
		#undef UBA_VISUALIZER_FLAG

		u64 parent = 0;
	};
}
