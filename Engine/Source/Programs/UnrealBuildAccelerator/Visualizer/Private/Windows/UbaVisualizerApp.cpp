// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaConfig.h"
#include "UbaNetworkBackendTcp.h"
#include "UbaVersion.h"
#include "Windows/UbaVisualizerWin.h"
#include <tlhelp32.h>
#include <psapi.h>
#include <imm.h>

#pragma comment (lib, "Imm32.lib")

using namespace uba;

static int PrintHelp(const tchar* message = nullptr)
{
	StringBuffer<64*1024> s;
	if (message && *message)
		s.Appendf(TC("%s\r\n\r\n"), message);

	s.Appendf(TC("\r\n"));
	s.Appendf(TC("------------------------\r\n"));
	s.Appendf(TC("   UbaVisualizer v%s\r\n"), GetVersionString());
	s.Appendf(TC("------------------------\r\n"));
	s.Appendf(TC("\r\n"));
	s.Appendf(TC("  When started UbaVisualizer will keep trying to connect to provided host address or named memory buffer.\r\n"));
	s.Appendf(TC("  Once connected it will start visualizing. Nothing else is needed :)\r\n"));
	s.Appendf(TC("\r\n"));
	s.Appendf(TC("  -host=<host>[:<port>] The ip/name and port (default: %u) of the machine we want to connect to\r\n"), DefaultPort);
	s.Appendf(TC("  -named=<name>        Name of named memory to connect to\r\n"));
	s.Appendf(TC("  -file=<name>         Name of file to parse\r\n"));
	s.Appendf(TC("  -listen[=<channel>]  Listen for announcements of new sessions. Defaults to channel '%s'\r\n"), TC("Default"));
	s.Appendf(TC("  -replay              Visualize the data as if it was running right now\r\n"));
	s.Appendf(TC("  -config=<file>       Specify config file to use\r\n"));
	s.Appendf(TC("  -parent=<hwnd>       Specify hwnd this window should be a child of\r\n"));
	s.Appendf(TC("  -nocopy              Will prevent UbaVisualizer.exe from being copied to temp and executed from there\r\n"));
	s.Appendf(TC("\r\n"));
	MessageBox(NULL, s.data, TC("UbaVisualizer"), 0);
	//wprintf(s.data);
	return -1;
}

struct MessageBoxLogWriter : public LogWriter
{
	virtual void BeginScope() override
	{
	}

	virtual void EndScope() override
	{
	}

	virtual void Log(LogEntryType type, const tchar* str, u32 strLen, const tchar* prefix = nullptr, u32 prefixLen = 0) override
	{
		if (type > LogEntryType_Warning)
		{
			#if UBA_DEBUG
			StringBuffer<> buf;
			buf.Append(str, strLen).Append(TCV("\r\n"));
			OutputDebugStringW(buf.data);
			#endif
			return;
		}

		HWND hwnd = NULL;
		if (m_visualizer)
		{
			hwnd = m_visualizer->GetHwnd();
			m_visualizer->Lock(true);
		}

		UINT flags = type == LogEntryType_Error ? MB_ICONERROR : MB_ICONWARNING;
		if (!hwnd)
			flags |= MB_TOPMOST;
		MessageBox(hwnd, str, TC("UbaVisualizer"), flags);
		if (type == LogEntryType_Error)
			ExitProcess(~0u);
		if (m_visualizer)
			m_visualizer->Lock(false);
	}
	
	VisualizerWin* m_visualizer = nullptr;
};

int WINAPI wWinMain(_In_ HINSTANCE hInstance, _In_opt_ HINSTANCE hPrevInstance, _In_ PWSTR pCmdLine, _In_ int nShowCmd)
{
	// Disable IME process-wide before any window is created.
	ImmDisableIME(static_cast<DWORD>(-1));

	StringBuffer<> host; // 192.168.86.49
	StringBuffer<> named;
	StringBuffer<> file;
	StringBuffer<> channel;
	StringBuffer<> configPath;
	u32 port = DefaultPort;
	u32 replay = 0;
	u64 parent = 0;
	bool copyAndLaunch = true;
	bool useGDI = false;
	bool useD2D = false;

	int argc;
	auto argv = CommandLineToArgvW(GetCommandLine(), &argc);

	for (int i=1; i!=argc; ++i)
	{
		StringBuffer<> name;
		StringBuffer<> value;

		if (i == 1 && argv[i][0] != '-')
		{
			file.Append(argv[i]);
			continue;
		}

		if (const tchar* equals = wcschr(argv[i],'='))
		{
			name.Append(argv[i], equals - argv[i]);
			value.Append(equals+1);
		}
		else
		{
			name.Append(argv[i]);
		}

		if (name.Equals(TCV("-help")))
		{
			return PrintHelp();
		}
		else if (name.Equals(TCV("-host")))
		{
			if (const tchar* portIndex = value.First(':'))
			{
				StringBuffer<> portStr(portIndex + 1);
				if (!portStr.Parse(port))
					return PrintHelp(TC("Invalid value for port in -host"));
				value.Resize(portIndex - value.data);
			}
			if (value.IsEmpty())
				return PrintHelp(TC("-host needs a name/ip"));
			host.Append(value);
		}
		else if (name.Equals(TCV("-named")))
		{
			if (value.IsEmpty())
				return PrintHelp(TC("-named needs a value"));
			named.Append(value);
		}
		else if (name.Equals(TCV("-file")))
		{
			if (value.IsEmpty())
				return PrintHelp(TC("-file needs a value"));
			file.Append(value);
		}
		else if (name.Equals(TCV("-port")))
		{
			if (!value.Parse(port))
				return PrintHelp(TC("Invalid value for -port"));
		}
		else if (name.Equals(TCV("-listen")))
		{
			if (!value.IsEmpty())
				channel.Append(value.data);
			else
				channel.Append(TCV("Default"));
		}
		else if (name.Equals(TCV("-replay")))
		{
			replay = 1;
			if (!value.IsEmpty())
				value.Parse(replay);
		}
		else if (name.Equals(TCV("-config")))
		{
			if (value.IsEmpty())
				return PrintHelp(TC("-config needs a value"));
			configPath.Append(value);
		}
		else if (name.Equals(TCV("-parent")))
		{
			if (value.IsEmpty())
				return PrintHelp(TC("-parent needs a value"));
			if (value.count > 8)
				return PrintHelp(TC("-parent has invalid value"));
			value.MakeLower();
			if (value.count & 1) // uneven char, add 0 in the front
			{
				memmove(value.data + 1, value.data, (value.count+1)*sizeof(tchar));
				value.data[0] = '0';
				++value.count;
			}
			parent = StringToValue2(value.data, value.count);
			//value.ParseHex(parent);
		}
		else if (name.Equals(TCV("-nocopy")))
		{
			copyAndLaunch = false;
		}
		else if (name.Equals(TCV("-gdi")))
		{
			useGDI = true;
		}
		else if (name.Equals(TCV("-d2d")))
		{
			useD2D = true;
		}
		else if (name.Equals(TCV("-ownerPid")))
		{
			u32 ownerPid;
			if (value.Parse(ownerPid))
				const_cast<OwnerInfo&>(GetOwnerInfo()).pid = ownerPid;
		}
		else if (name.Equals(TCV("-ownerId")))
		{
			TStrcpy_s(const_cast<tchar*>(GetOwnerInfo().id), 260, value.data);
		}
		else
		{
			StringBuffer<> msg;
			msg.Appendf(TC("Unknown argument '%s'"), name.data);
			return PrintHelp(msg.data);
		}
	}

	MessageBoxLogWriter logWriter;
	LoggerWithWriter logger(logWriter);

	if (copyAndLaunch)
	{
		StringBuffer<> tempPath;
		tempPath.count = GetTempPathW(tempPath.capacity, tempPath.data);
		if (!tempPath.count)
		{
			logger.Error(TC("GetTempPathW failed"));
			return -1;
		}

		StringBuffer<> thisExe;
		thisExe.count = GetModuleFileNameW(NULL, thisExe.data, thisExe.capacity);
		if (!thisExe.count)
		{
			logger.Error(TC("GetModuleFileNameW failed"));
			return -1;
		}

		WIN32_FILE_ATTRIBUTE_DATA data;
		if (!GetFileAttributesExW(thisExe.data, GetFileExInfoStandard, &data))
		{
			logger.Error(TC("GetFileAttributesExW failed"));
			return -1;
		}

		u64 thisLastWriteTime = *(u64*)&data.ftLastWriteTime;

		StringBuffer<> ubaFileName;
		for (u32 i=0; i!=10; ++i)
		{
			ubaFileName.Append(tempPath).EnsureEndsWithSlash().Appendf(TC("UbaVisualizer%u.exe"), i);
			if (GetFileAttributesExW(ubaFileName.data, GetFileExInfoStandard, &data))
			{
				u64 lastWriteTime = *(u64*)&data.ftLastWriteTime;
				if (thisLastWriteTime == lastWriteTime)
					break;
			}
			if (CopyFileW(thisExe.data, ubaFileName.data, false))
				break;
			ubaFileName.Clear();
		}

		if (!ubaFileName.count)
		{
			logger.Error(TC("Failed to create temporary UbaVisualizer.exe to launch."));
			return -1;
		}

		StringBuffer<> args;
		args.Append(ubaFileName);
		for (int i = 1; i != argc; ++i)
		{
			args.Append(' ');
			if (TStrchr(argv[i], ' '))
				args.Append('\"').Append(argv[i]).Append('\"');
			else
				args.Append(argv[i]);
		}
		args.Append(" -nocopy");

		OwnerInfo ownerInfo = GetOwnerInfo();
		if (ownerInfo.pid)
			args.Appendf(TC(" -ownerPid=%u -ownerId=%s"), ownerInfo.pid, ownerInfo.id);

		STARTUPINFOW si;
		ZeroMemory(&si, sizeof(si));
		si.cb = sizeof(si);
		PROCESS_INFORMATION pi;
		ZeroMemory(&pi, sizeof(pi));
		if (!CreateProcessW(NULL, args.data, NULL, NULL, false, 0, NULL, NULL, &si, &pi))
		{
			logger.Error(TC("Failed to launch process %s"), ubaFileName.data);
			return -1;
		}
		CloseHandle(pi.hProcess);
		CloseHandle(pi.hThread);
		return 0;
	}

	LocalFree(argv);

	if (host.IsEmpty() && named.IsEmpty() && file.IsEmpty() && !channel.count)
		channel.Append(TCV("Default")); // return PrintHelp(TC("No host/named/file provided. Add -host=<host> or -file=<file> or -named=<name>"));

	bool showAllTraces = true;
	StringBuffer<> legacyConfigPath;
	if (!configPath.count)
	{
		StringBuffer<> fileLeaf;
		fileLeaf.Append(L"\\UbaVisualizer");

		OwnerInfo ownerInfo = GetOwnerInfo();
		if (ownerInfo.pid)
		{
			fileLeaf.Append('_').Append(ownerInfo.id);
			//showAllTraces = false; // It seems like showAllTraces is actually what most people want regardless of where a trace is initiated.
		}
		fileLeaf.Append(L".toml");

		DWORD localAppDataLen = ExpandEnvironmentStringsW(L"%LOCALAPPDATA%", configPath.data, configPath.capacity);
		if (localAppDataLen && localAppDataLen <= configPath.capacity)
		{
			configPath.count = localAppDataLen - 1;
			configPath.Append(L"\\Epic Games\\UbaVisualizer").Append(fileLeaf);
		}

		// Legacy machine-global location used to migrate old path
		DWORD programDataLen = ExpandEnvironmentStringsW(L"%PROGRAMDATA%", legacyConfigPath.data, legacyConfigPath.capacity);
		if (programDataLen && programDataLen <= legacyConfigPath.capacity)
		{
			legacyConfigPath.count = programDataLen - 1;
			legacyConfigPath.Append(L"\\Epic\\UbaVisualizer").Append(fileLeaf);
		}
	}

	VisualizerConfig visualizerConfig(configPath.data);
	visualizerConfig.parent = parent;
	visualizerConfig.ShowAllTraces = showAllTraces;

	if (!visualizerConfig.Load(logger) && configPath.count && legacyConfigPath.count)
	{
		// No per-user settings yet; migrate legacy machine-global settings forward and persist them.
		visualizerConfig.filename = legacyConfigPath.data;
		bool migrated = visualizerConfig.Load(logger);
		visualizerConfig.filename = configPath.data;
		if (migrated)
			visualizerConfig.Save(logger);
	}

	if (useGDI)
		visualizerConfig.useGDI = true;
	if (useD2D)
		visualizerConfig.useGDI = false;

	NetworkBackendTcp networkBackend(logWriter);
	VisualizerWin visualizer(visualizerConfig, logger);
	logWriter.m_visualizer = &visualizer;

	if (channel.count)
	{
		if (!visualizer.ShowUsingListener(channel.data))
			logger.Error(TC("Failed listening to named pipe"));
	}
	else if (!named.IsEmpty())
	{
		if (!visualizer.ShowUsingNamedTrace(named.data))
			logger.Error(TC("Failed reading from mapped memory %s"), named.data);
	}
	else if (!host.IsEmpty())
	{
		if (!visualizer.ShowUsingSocket(networkBackend, host.data, u16(port)))
			logger.Error(TC("Failed to connect to %s:%u"), host.data, port);
	}
	else
	{
		if (!visualizer.ShowUsingFile(file.data, replay))
			logger.Error(TC("Failed to read trace file '%s'"), file.data);
	}

	while (true)
	{
		if (!visualizer.HasWindow())
			break;
		uba::Sleep(500);
	}
	return 0;
}
