// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaVisualizerBackend.h"
#include "Windows/UbaVisualizerBackendGDI.h"

#include <d2d1.h>
#include <dwrite.h>

#pragma comment(lib, "d2d1.lib")
#pragma comment(lib, "dwrite.lib")

namespace uba
{
	class VisualizerWin;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class VisualizerBackendD2D final : public VisualizerBackend
	{
	public:
		static constexpr int BitmapCacheHeight = 16*1024;

		static inline D2D1::ColorF ToRgbD2D(COLORREF c)
		{
			return  ((c & 0x000000FF) << 16) | (c & 0x0000FF00) | ((c & 0x00FF0000) >> 16);
		}

		struct PaintContextD2D final : public PaintContext
		{
			PaintContextD2D(VisualizerBackendD2D& o, ID2D1RenderTarget& rt, SIZE cs)
			:	owner(o)
			,	renderTarget(rt)
			,	clientSize(cs)
			{
			}

			virtual SIZE GetClientSize() override
			{
				return clientSize;
			}
			virtual void SetPen(PenHandle pen) override
			{
				activePen = pen;
			}
			virtual void SetBrush(BrushHandle b) override
			{
				activeBrush = b;
			}
			virtual void SetFont(const Font& font, bool underlined = false) override
			{
				activeFont = font;
				activeFontHandle = (IDWriteTextFormat*)(underlined ? font.handleUnderlined : font.handle);
			}
			virtual void SetTextColor(COLORREF color) override
			{
				if (activeTextColor != color)
				{
					activeTextColor = color;
					owner.m_textBrush->SetColor(ToRgbD2D(color));
				}
			}
			virtual void Text(int x, int y, StringView str, const RECT* clipRect) override
			{
				if (!activeFontHandle)
					return;

				D2D1_RECT_F layoutRect = D2D1::RectF(float(x), float(y), float(x + 10000), float(y + 100));

				if (clipRect)
				{
					//UBA_ASSERT(clipRect->left <= x);
					//UBA_ASSERT(clipRect->top <= y);
					D2D1_RECT_F cr = layoutRect;
					cr.right = float(clipRect->right);
					cr.bottom = float(clipRect->bottom);
					if (cr.bottom < 0)
						return;
					renderTarget.PushAxisAlignedClip(cr, D2D1_ANTIALIAS_MODE_ALIASED);
				}
				auto g = MakeGuard([&]() { if (clipRect) renderTarget.PopAxisAlignedClip(); });

				if ((FontHandle)activeFontHandle == activeFont.handleUnderlined)
				{
					IDWriteTextLayout* layout = nullptr;
					HRESULT hr = owner.m_dwFactory->CreateTextLayout(str.data, str.count, activeFontHandle, 10000.0f, 1000.0f, &layout);
					if (SUCCEEDED(hr) && layout)
					{
						DWRITE_TEXT_METRICS m{};
						DWRITE_TEXT_RANGE range{ 0, str.count };
						layout->SetUnderline(TRUE, range);
						D2D1_POINT_2F origin = D2D1::Point2F(layoutRect.left, layoutRect.top);
						renderTarget.DrawTextLayout(origin, layout, owner.m_textBrush);
						layout->Release();
						return;
					}
				}

				renderTarget.DrawText(str.data, str.count, activeFontHandle, &layoutRect, owner.m_textBrush);
			}
			virtual COLORREF GetTextColor() override
			{
				return activeTextColor;
			}
			virtual Font& GetFont() override
			{
				return activeFont;
			}
			virtual void FillRect(const RECT& r, BrushHandle b) override
			{
				D2D1_RECT_F rect = D2D1::RectF(float(r.left), float(r.top), float(r.right), float(r.bottom));
				renderTarget.FillRectangle(rect, (ID2D1SolidColorBrush*)b);
			}
			virtual void Rectangle(int left, int top, int right, int bottom) override
			{
				D2D1_RECT_F rect = D2D1::RectF(float(left)+0.5f, float(top)+0.5f, float(right)-0.5f, float(bottom)-0.5f);
				float strokeWidth = 1.0f;
				renderTarget.DrawRectangle(rect, (ID2D1SolidColorBrush*)activePen, strokeWidth);
			}
			virtual void Line(int fromX, int fromY, int toX, int toY) override
			{
				D2D1_POINT_2F p0 = D2D1::Point2F(float(fromX)+0.5f, float(fromY)+0.5f);
				D2D1_POINT_2F p1 = D2D1::Point2F(float(toX)+0.5f, float(toY)+0.5f);
				float strokeWidth = 1.0f;
				renderTarget.DrawLine(p0, p1, (ID2D1SolidColorBrush*)activePen, strokeWidth, nullptr);
			}
			virtual void Polyline(u32 cacheId, const Vector<POINT>& polyLine) override
			{
				float strokeWidth = 1.0f;
				D2D1_POINT_2F prev = {};
				bool first = true;
				for (auto& p : polyLine)
				{
					D2D1_POINT_2F pf = D2D1::Point2F(float(p.x), float(p.y));
					if (!first)
						renderTarget.DrawLine(prev, pf, (ID2D1SolidColorBrush*)activePen, strokeWidth, nullptr);
					first = false;
					prev = pf;
				}
			}
			virtual void PolyPolyline(const POINT* points, const u32* lines, u32 lineCount) override
			{
				float strokeWidth = 1.0f;
				const POINT* pointsIt = points;
				for (u32 i=0; i!=lineCount; ++i)
				{
					D2D1_POINT_2F prev = {};
					bool first = true;
					for (u32 j=0; j!=lines[i]; ++j)
					{
						D2D1_POINT_2F pf = D2D1::Point2F(float(pointsIt->x), float(pointsIt->y));
						if (!first)
							renderTarget.DrawLine(prev, pf, (ID2D1SolidColorBrush*)activePen, strokeWidth, nullptr);
						first = false;
						prev = pf;
						++pointsIt;
					}
				}
			}
			virtual SIZE GetTextSize(StringView str) override
			{
				IDWriteTextLayout* layout = nullptr;
				HRESULT hr = owner.m_dwFactory->CreateTextLayout(str.data, str.count, activeFontHandle, 10000.0f, 1000.0f, &layout);
				if (FAILED(hr) || !layout)
					return { 0, 0 };
				DWRITE_TEXT_METRICS m{};
				layout->GetMetrics(&m);
				layout->Release();
				return { int(m.width), int(m.height) }; // m.widthIncludingTrailingWhitespace
			}
			virtual void DrawCached(TraceView::BitmapCache& process, RECT rect, int offsetY, int textHeight, Font& processFont, RECT clippingRect, const Function<void(PaintContext& textCacheContext)>& func) override
			{
				if (!process.bitmap || process.bitmapDirty)
				{
					if (!process.bitmap)
					{
						if (owner.m_lastBitmapOffset + processFont.height > owner.m_bitmapCacheHeight)
						{
							ID2D1BitmapRenderTarget* textRT = nullptr;
							HRESULT hr = renderTarget.CreateCompatibleRenderTarget(D2D1::SizeF(float(256), float(owner.m_bitmapCacheHeight)), &textRT);
							// Skip the cached text path this frame on failure rather than storing a null render target into m_lastBitmap.
							if (FAILED(hr) || !textRT)
								return;

							if (owner.m_lastBitmap)
								owner.m_textBitmaps.push_back(owner.m_lastBitmap);

							owner.m_lastBitmapOffset = 0;
							owner.m_lastBitmap = textRT;
						}
						process.bitmap = (HBITMAP)owner.m_lastBitmap;
						process.bitmapOffset = owner.m_lastBitmapOffset;
						owner.m_lastBitmapOffset += processFont.height;
					}
					if (lastSelectedBitmap != (ID2D1BitmapRenderTarget*)process.bitmap)
						lastSelectedBitmap = (ID2D1BitmapRenderTarget*)process.bitmap;

					lastSelectedBitmap->BeginDraw();
					PaintContextD2D textCacheContext{ owner, *lastSelectedBitmap, { 256, owner.m_bitmapCacheHeight } };
					textCacheContext.SetFont(activeFont);
					func(textCacheContext);
					lastSelectedBitmap->EndDraw();
				}

				if (lastSelectedBitmap != (ID2D1BitmapRenderTarget*)process.bitmap)
					lastSelectedBitmap = (ID2D1BitmapRenderTarget*)process.bitmap;

				int processWidth = rect.right - rect.left;
				int width = Min(processWidth, 256);
				int bitmapOffsetY = process.bitmapOffset;
				int bltOffsetY = offsetY;
				if (bltOffsetY < 0)
				{
					bitmapOffsetY -= bltOffsetY;
					bltOffsetY = 0;
				}
				int height = Min(textHeight, processFont.height);
				if (bltOffsetY + height > textHeight)
					height = textHeight - bltOffsetY;

				int left = rect.left;
				if (left <= -256 || height <= 0)
					return;

				int bitmapOffsetX = rect.left - left;

				if (left < clippingRect.left)
				{
					int diff = clippingRect.left - left;
					rect.left = clippingRect.left;
					width -= diff;
					bitmapOffsetX += diff;
				}

				D2D1_RECT_F destRect = D2D1::RectF(float(rect.left), float(rect.top + bltOffsetY), float(rect.left + width), float(rect.top + bltOffsetY + height));
				D2D1_RECT_F sourceRect = D2D1::RectF(float(bitmapOffsetX), float(bitmapOffsetY), float(bitmapOffsetX + width), float(bitmapOffsetY + height));

				ID2D1Bitmap* textBitmap = nullptr;
				lastSelectedBitmap->GetBitmap(&textBitmap);
				renderTarget.DrawBitmap(textBitmap, &destRect, 1.0f, D2D1_BITMAP_INTERPOLATION_MODE_NEAREST_NEIGHBOR, &sourceRect);
				textBitmap->Release();
			}
			virtual POINT GetCursorClientPos() override
			{
				POINT p;
				GetCursorPos(&p);
				ScreenToClient(owner.m_owner.m_hwnd, &p);
				return p;
			}

			VisualizerBackendD2D& owner;
			ID2D1RenderTarget& renderTarget;
			SIZE clientSize;
			PenHandle activePen = 0;
			BrushHandle activeBrush = 0;
			Font activeFont;
			IDWriteTextFormat* activeFontHandle = nullptr;
			COLORREF activeTextColor = 0;
			ID2D1BitmapRenderTarget* lastSelectedBitmap = 0;
		};

		VisualizerBackendD2D(VisualizerWin& o) : m_owner(o)
		{
		}

		virtual const tchar* GetName() override { return TC("D2D"); }

		bool Create()
		{
			D2D1_FACTORY_OPTIONS options{};
			// options.debugLevel = D2D1_DEBUG_LEVEL_INFORMATION;
			if (D2D1CreateFactory(D2D1_FACTORY_TYPE_SINGLE_THREADED, options, &m_d2dFactory) != S_OK)
				return false;
			if (DWriteCreateFactory(DWRITE_FACTORY_TYPE_SHARED, __uuidof(IDWriteFactory), reinterpret_cast<IUnknown**>(&m_dwFactory)) != S_OK)
				return false;
			return true;
		}

		virtual void Init(void* windowHandle) override
		{
		}

		virtual void Reset() override
		{
			m_dirty = true;
			for (auto b : m_brushes)
				b->Release();
			m_brushes.clear();
			for (auto bm : m_textBitmaps)
				bm->Release();
			m_textBitmaps.clear();
			m_lastBitmap = 0;
			m_lastBitmapOffset = m_bitmapCacheHeight;
		}

		virtual void Resize(int width, int height) override
		{
			if (m_d2dRenderTarget)
				m_d2dRenderTarget->Resize(D2D1::SizeU(width, height));
		}

		virtual void Paint(const Function<void(PaintContext& context, RECT rect)>& func) override
		{
			if (!m_d2dRenderTarget)
			{
				D2D1_SIZE_U size = D2D1::SizeU(m_owner.m_clientWidth, m_owner.m_clientHeight);
				auto prop = D2D1::RenderTargetProperties();
				auto hwndProp = D2D1::HwndRenderTargetProperties((HWND)m_owner.m_hwnd, size);
				if (m_d2dFactory->CreateHwndRenderTarget(prop, hwndProp, &m_d2dRenderTarget) != S_OK)
					return;
				//m_d2dRenderTarget->SetAntialiasMode(D2D1_ANTIALIAS_MODE_ALIASED);

				// Clamp the text cache height to the device's max bitmap size (lower on RDP / software fallbacks). It's in pixels,
				// but the cache is sized in DIPs and inherits our DPI, so convert it - else a DPI-aware (embedded) session requests >16k px.
				float dpiX, dpiY;
				m_d2dRenderTarget->GetDpi(&dpiX, &dpiY);
				int maxCacheHeightDip = int(float(m_d2dRenderTarget->GetMaximumBitmapSize()) * 96.0f / dpiY);
				m_bitmapCacheHeight = Min<int>(BitmapCacheHeight, maxCacheHeightDip);
				m_lastBitmapOffset = m_bitmapCacheHeight;
			}

			if (m_dirty)
			{
				m_owner.InitBrushes();
				m_d2dRenderTarget->CreateSolidColorBrush(ToRgbD2D(m_owner.m_textColor), &m_textBrush);
				m_dirty = false;
			}
			PAINTSTRUCT ps;
			BeginPaint(m_owner.m_hwnd, &ps);

			m_d2dRenderTarget->BeginDraw();

			COLORREF bgColor = m_owner.m_config.DarkMode ? RGB(37, 37, 38) : RGB(180, 180, 180);
			m_d2dRenderTarget->Clear(ToRgbD2D(bgColor));

			int clientWidth = m_owner.m_clientWidth;
			int clientHeight = m_owner.m_clientHeight;
			PaintContextD2D context(*this, *m_d2dRenderTarget, { clientWidth, clientHeight });
			func(context, { 0, 0, clientWidth, clientHeight });

			HRESULT hr = m_d2dRenderTarget->EndDraw();
			if (hr == D2DERR_RECREATE_TARGET)
			{
				Reset();
				m_d2dRenderTarget->Release();
				m_d2dRenderTarget = nullptr;
			}

			EndPaint(m_owner.m_hwnd, &ps);
		}

		virtual PaintContext* CreateContext(int clientWidth, int clientHeight) override
		{
			return new PaintContextD2D(*this, *m_d2dRenderTarget, { m_owner.m_clientWidth, m_owner.m_clientHeight });
		}

		virtual void DeleteContext(PaintContext* context) override
		{
			delete (PaintContextD2D*)context;
		}

		virtual BrushHandle CreateSolidBrush(COLORREF color) override
		{
			ID2D1SolidColorBrush* brush;
			m_d2dRenderTarget->CreateSolidColorBrush(ToRgbD2D(color), &brush);
			m_brushes.push_back(brush);
			return (BrushHandle)brush;
		}

		virtual PenHandle CreatePen(COLORREF color) override
		{
			return (PenHandle)CreateSolidBrush(color);
		}

		virtual FontHandle CreateFont(const tchar* font, int height, bool createUnderline) override
		{
			auto weight = DWRITE_FONT_WEIGHT_NORMAL;
			auto style = DWRITE_FONT_STYLE_NORMAL;
			auto stretch = DWRITE_FONT_STRETCH_NORMAL;
			IDWriteTextFormat* textFormat = 0;
			if (m_dwFactory->CreateTextFormat(font, nullptr, weight, style, stretch, float(height - 4), L"en-us", &textFormat) != S_OK)
				return 0;
			textFormat->SetTextAlignment(DWRITE_TEXT_ALIGNMENT_LEADING);
			textFormat->SetParagraphAlignment(DWRITE_PARAGRAPH_ALIGNMENT_NEAR);
			return (FontHandle)textFormat;
		}

		virtual void DeleteFont(FontHandle font)
		{
			((IDWriteTextFormat*)font)->Release();
		}

		VisualizerWin& m_owner;
		ID2D1Factory* m_d2dFactory = nullptr;
		ID2D1HwndRenderTarget* m_d2dRenderTarget = nullptr;
		List<ID2D1SolidColorBrush*> m_brushes;

		IDWriteFactory* m_dwFactory = nullptr;
		ID2D1SolidColorBrush* m_textBrush = nullptr;

		ID2D1BitmapRenderTarget* m_lastBitmap = 0;
		int m_bitmapCacheHeight = BitmapCacheHeight;
		int m_lastBitmapOffset = BitmapCacheHeight;
		Vector<ID2D1BitmapRenderTarget*> m_textBitmaps;

		bool m_dirty = true;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
