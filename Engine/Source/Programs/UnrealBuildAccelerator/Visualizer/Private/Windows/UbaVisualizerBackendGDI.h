// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaVisualizerBackend.h"

namespace uba
{
	class VisualizerWin;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class VisualizerBackendGDI final : public VisualizerBackend
	{
	public:
		static constexpr int BitmapCacheHeight = 1024*1024;

		struct PaintContextGDI final : public PaintContext
		{
			virtual SIZE GetClientSize() override { return clientSize; }
			virtual void SetTextColor(COLORREF color) override { ::SetTextColor(hdc, color); }
			virtual COLORREF GetTextColor() override { return ::GetTextColor(hdc); }
			virtual void SetPen(PenHandle pen) override { ::SelectObject(hdc, pen); }
			virtual void Text(int x, int y, StringView str, const RECT* clipRect) override
			{
				::ExtTextOutW(hdc, x, y, clipRect ? ETO_CLIPPED : 0, clipRect, str.data, str.count, NULL);
			}

			virtual void SetFont(const Font& font, bool underlined) override
			{
				if (underlined)
				{
					if (font.handleUnderlined != activeFontHandle)
						::SelectObject(hdc, font.handleUnderlined);
					activeFontHandle = font.handleUnderlined;
				}
				else
				{
					if (font.handle != activeFontHandle)
						::SelectObject(hdc, font.handle);
					activeFontHandle = font.handle;
				}
				activeFont = font;
			}

			virtual Font& GetFont() override
			{
				return activeFont;
			}

			virtual void SetBrush(BrushHandle b) override
			{
				if (lastSelectedBrush != b)
				{
					SelectObject(hdc, b);
					lastSelectedBrush = b;
				}
			}

			virtual void FillRect(const RECT& r, BrushHandle b) override
			{
				SetBrush(b);
				PatBlt(hdc, r.left, r.top, r.right - r.left, r.bottom - r.top, PATCOPY);
			}

			virtual void Rectangle(int left, int top, int right, int bottom) override
			{
				SetBrush((BrushHandle)GetStockObject(NULL_BRUSH));
				::Rectangle(hdc, left, top, right, bottom);
			}

			virtual void Line(int fromX, int fromY, int toX, int toY) override
			{
				::MoveToEx(hdc, fromX, fromY, NULL);
				::LineTo(hdc, toX, toY);
			}

			virtual void Polyline(u32 cacheId, const Vector<POINT>& polyLine) override
			{
				::Polyline(hdc, polyLine.data(), int(polyLine.size()));
			}

			virtual void PolyPolyline(const POINT* points, const u32* lines, u32 lineCount) override
			{
				::PolyPolyline(hdc, points, (const DWORD*)lines, int(lineCount));
			}

			virtual SIZE GetTextSize(StringView str) override
			{
				SIZE s;
				return GetTextExtentPoint32W(hdc, str.data, str.count, &s) ? s : SIZE();
			}

			virtual void DrawCached(TraceView::BitmapCache& process, RECT rect, int offsetY, int textHeight, Font& processFont, RECT clippingRect, const Function<void(PaintContext& textCacheContext)>& func) override
			{
				if (!process.bitmap || process.bitmapDirty)
				{
					if (!process.bitmap)
					{
						if (owner.m_lastBitmapOffset == BitmapCacheHeight)
						{
							if (owner.m_lastBitmap)
								owner.m_textBitmaps.push_back(owner.m_lastBitmap);

							owner.m_lastBitmapOffset = 0;
							owner.m_lastBitmap = CreateCompatibleBitmap(hdc, 256, BitmapCacheHeight);
						}
						process.bitmap = owner.m_lastBitmap;
						process.bitmapOffset = owner.m_lastBitmapOffset;
						owner.m_lastBitmapOffset += processFont.height;
					}
					if (lastSelectedBitmap != process.bitmap)
					{
						SelectObject(textDC, process.bitmap);
						lastSelectedBitmap = process.bitmap;
					}

					PaintContextGDI textCacheContext{owner, NULL, textDC, clientSize };
					func(textCacheContext);
				}

				if (lastSelectedBitmap != process.bitmap)
				{
					SelectObject(textDC, process.bitmap);
					lastSelectedBitmap = process.bitmap;
				}

				int processWidth = rect.right - rect.left;
				int width = Min(processWidth, 256);
				int bitmapOffsetY = process.bitmapOffset;
				int bltOffsetY = offsetY;
				if (bltOffsetY < 0)
				{
					bitmapOffsetY -= bltOffsetY;
					bltOffsetY = 0;
				}
				int height = Min(textHeight, processFont.height);
				if (bltOffsetY + height > textHeight)
					height = textHeight - bltOffsetY;

				int left = rect.left;
				if (left <= -256 || height <= 0)
					return;

				int bitmapOffsetX = rect.left - left;

				if (left < clippingRect.left)
				{
					int diff = clippingRect.left - left;
					rect.left = clippingRect.left;
					width -= diff;
					bitmapOffsetX += diff;
				}
				BitBlt(hdc, rect.left, rect.top + bltOffsetY, width, height, textDC, bitmapOffsetX, bitmapOffsetY, SRCCOPY);
			}

			virtual POINT GetCursorClientPos() override
			{
				POINT p;
				GetCursorPos(&p);
				ScreenToClient(hwnd, &p);
				return p;
			}

			PaintContextGDI(VisualizerBackendGDI& o, HWND hw, HDC dc, SIZE cs, HDC tdc = 0) : owner(o), hwnd(hw), hdc(dc), clientSize(cs), textDC(tdc) {}


			VisualizerBackendGDI& owner;
			HWND hwnd;
			HDC hdc;
			SIZE clientSize;

			Font activeFont;
			FontHandle activeFontHandle;

			HDC textDC = NULL;
			BrushHandle lastSelectedBrush = 0;
			HBITMAP lastSelectedBitmap = 0;
		};

		VisualizerBackendGDI(VisualizerWin& owner) : m_owner(owner)
		{
		}

		virtual const tchar* GetName() override { return TC("GDI"); }

		virtual void Init(void* windowHandle) override
		{
			m_owner.InitBrushes();
		}

		virtual void Reset() override
		{
			for (HBITMAP bm : m_textBitmaps)
				DeleteObject(bm);
			m_textBitmaps.clear();
			DeleteObject(m_lastBitmap);
			m_lastBitmap = 0;
			m_lastBitmapOffset = BitmapCacheHeight;
			m_owner.InitBrushes();
		}

		virtual void Resize(int width, int height) override
		{
		}

		virtual void Paint(const Function<void(PaintContext& context, RECT rect)>& func) override
		{
			HWND hwnd = m_owner.m_hwnd;
			int clientWidth = m_owner.m_clientWidth;
			int clientHeight = m_owner.m_clientHeight;

			HDC hdc = GetDC(hwnd);

			HDC memDC = CreateCompatibleDC(hdc);

			if (m_cachedBitmapSize.cx != clientWidth || m_cachedBitmapSize.cy != clientHeight)
			{
				if (m_cachedBitmap)
					DeleteObject(m_cachedBitmap);
				m_cachedBitmap = CreateCompatibleBitmap(hdc, clientWidth, clientHeight);
				m_cachedBitmapSize = { clientWidth, clientHeight };
			}
			HGDIOBJ oldBmp = SelectObject(memDC, m_cachedBitmap);

			RECT rect = { 0, 0, clientWidth, clientHeight };
			FillRect(memDC, &rect, (HBRUSH)m_owner.m_backgroundBrush);
			SetBkColor(memDC, m_owner.m_config.DarkMode ? RGB(70, 70, 70) : RGB(180, 180, 180));
			SetBkMode(memDC, TRANSPARENT);

			HDC textDC = CreateCompatibleDC(memDC);
			SetTextColor(textDC, m_owner.m_textColor);
			SelectObject(textDC, m_owner.m_processFont.handle);
			SelectObject(textDC, GetStockObject(NULL_BRUSH));
			SetBkMode(textDC, TRANSPARENT);

			HBITMAP nullBmp = CreateCompatibleBitmap(memDC, 1, 1);
			HBITMAP oldBmp2 = (HBITMAP)SelectObject(textDC, nullBmp);

			VisualizerBackendGDI::PaintContextGDI context(*this, hwnd, memDC, { clientWidth, clientHeight }, textDC);
			
			func(context, rect);

			SelectObject(textDC, oldBmp2);
			DeleteObject(nullBmp);
			DeleteDC(textDC);

			BitBlt(hdc, 0, 0, rect.right - rect.left, rect.bottom - rect.top, memDC, 0, 0, SRCCOPY);

			SelectObject(memDC, oldBmp);
			DeleteDC(memDC);

			ReleaseDC(hwnd, hdc);
		}

		virtual PaintContext* CreateContext(int clientWidth, int clientHeight) override
		{
			return new PaintContextGDI(*this, NULL, NULL, { clientWidth, clientHeight });
		}

		virtual void DeleteContext(PaintContext* context) override
		{
			delete (PaintContextGDI*)context;
		}

		virtual BrushHandle CreateSolidBrush(COLORREF color) override
		{
			return ::CreateSolidBrush(color);
		}

		virtual PenHandle CreatePen(COLORREF color) override
		{
			return ::CreatePen(PS_SOLID, 1, color);
		}

		FontHandle CreateFont(const tchar* font, int height, bool createUnderline) override
		{
			DWORD underline = createUnderline ? 1 : 0;
			return ::CreateFontW(4 - height, 0, 0, 0, FW_NORMAL, 0, underline, 0, ANSI_CHARSET, OUT_DEFAULT_PRECIS, CLIP_DEFAULT_PRECIS, CLEARTYPE_QUALITY, DEFAULT_PITCH, font);
		}

		void DeleteFont(FontHandle font)
		{
			::DeleteObject(font);
		}

		VisualizerWin& m_owner;

		HBITMAP m_cachedBitmap = 0;
		SIZE m_cachedBitmapSize = { INT_MIN, INT_MIN };

		HBITMAP m_lastBitmap = 0;
		int m_lastBitmapOffset = BitmapCacheHeight;
		Vector<HBITMAP> m_textBitmaps;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
