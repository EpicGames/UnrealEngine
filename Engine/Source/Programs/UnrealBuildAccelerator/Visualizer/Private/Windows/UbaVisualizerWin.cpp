// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaVisualizerWin.h"
#include "UbaConfig.h"
#include "UbaVisualizerBackendD2D.h"

#include <algorithm>
#include <uxtheme.h>
#include <dwmapi.h>
#include <imm.h>
#include <shellscalingapi.h>
#include <shlobj_core.h>

#pragma comment (lib, "Shcore.lib")
#pragma comment (lib, "UxTheme.lib")
#pragma comment (lib, "Dwmapi.lib")
#pragma comment (lib, "Imm32.lib")
#define WM_NEWTRACE WM_USER+1
#define WM_SETTITLE WM_USER+2

namespace uba
{
	// Returns true if any window in the parent chain belongs to a Java AWT host
	static bool IsAWTHost(HWND hwnd)
	{
		wchar_t className[256];
		HWND cur = hwnd;
		HWND desktop = GetDesktopWindow();
		while (cur && cur != desktop)
		{
			if (GetClassNameW(cur, className, sizeof(className) / sizeof(wchar_t)) > 0 &&
				wcsncmp(className, L"SunAwt", 6) == 0)
				return true;

			HWND next = GetWindow(cur, GW_OWNER);
			if (!next)
				next = GetAncestor(cur, GA_PARENT);
			cur = next;
		}
		return false;
	}

#if PLATFORM_WINDOWS
	extern bool g_isInAssertMessageBox;
#endif

	enum
	{
		Popup_CopySessionInfo = 3,
		Popup_CopyProcessInfo,
		Popup_CopyProcessLog,
		Popup_CopyProcessBreadcrumbs,
		Popup_CopyWorkInfo,
		Popup_Replay,
		Popup_Pause,
		Popup_Play,
		Popup_JumpToEnd,

#define UBA_VISUALIZER_FLAG(name, defaultValue, desc) Popup_##name,
		UBA_VISUALIZER_FLAGS2
#undef UBA_VISUALIZER_FLAG

		Popup_IncreaseFontSize,
		Popup_DecreaseFontSize,

		Popup_SaveAs,
		Popup_SaveSettings,
		Popup_OpenSettings,
		Popup_Quit,
	};

	class WriteTextLogger : public Logger
	{
	public:
		WriteTextLogger(TString& out) : m_out(out) {}
		virtual void BeginScope() override {}
		virtual void EndScope() override {}
		virtual void Log(LogEntryType type, const wchar_t* str, u32 strLen) override { m_out.append(str, strLen).append(TC("\n")); }
		TString& m_out;
	};

	VisualizerBackend& CreateBackend(VisualizerConfig& config, VisualizerWin& v)
	{
		if (!config.useGDI)
		{
			auto d2d = new VisualizerBackendD2D(v);
			if (d2d->Create())
				return *d2d;
			delete d2d;
		}
		return *new VisualizerBackendGDI(v);
	}

	VisualizerWin::VisualizerWin(VisualizerConfig& config, Logger& logger)
	:	Visualizer(config, logger, CreateBackend(config, *this))
	{
		memset(m_activeProcessFont, 0, sizeof(m_activeProcessFont));
		memset(m_activeProcessCountHistory, 0, sizeof(m_activeProcessCountHistory));
	}

	VisualizerWin::~VisualizerWin()
	{
		m_looping = false;

		// Make sure GetMessage is triggered out of its slumber
		PostMessage(m_hwnd, WM_QUIT, 0, 0);

		m_thread.Wait();
		delete m_client;
	}

	bool VisualizerWin::ShowUsingListener(const wchar_t* channelName)
	{
		TraceChannel channel(m_logger);
		if (!channel.Init(channelName))
		{
			m_logger.Error(L"TODO");
			return false;
		}

		channel.StartListening();

		m_listenTimeout.Create(EventResetType_Auto);

		m_listenChannel.Append(channelName);
		m_looping = true;
		m_autoScroll = false;
		if (!StartHwndThread())
			return true;

		{
			StringBuffer<> title;
			PostNewTitle(GetTitlePrefix(title).Appendf(L"Listening for new sessions on channel '%s'", m_listenChannel.data));
		}

		StringBuffer<256> traceName;
		while (m_hwnd)
		{
			if (m_locked)
			{
				m_listenTimeout.IsSet(1000);
				continue;
			}

			if (m_parentHwnd && !IsWindow(m_parentHwnd))
				PostQuit();

			traceName.Clear();
			if (!channel.Read(traceName))
			{
				m_logger.Error(L"TODO2");
				return false;
			}

			if (traceName.count)
			{
				StringBuffer<128> filter;
				if (!m_config.ShowAllTraces)
				{
					OwnerInfo ownerInfo = GetOwnerInfo();
					if (ownerInfo.pid)
						filter.Appendf(L"_%s%u", ownerInfo.id, ownerInfo.pid);
				}

				if (!traceName.Equals(m_newTraceName.data) && traceName.EndsWith(filter))
				{
					m_newTraceName.Clear().Append(traceName);
					m_usingNamed = true;
					PostNewTrace(0, false);
				}
			}
			else
				m_newTraceName.Clear();

			m_listenTimeout.IsSet(1000);
		}

		return true;
	}

	bool VisualizerWin::ShowUsingNamedTrace(const wchar_t* namedTrace)
	{
		m_looping = true;
		if (!StartHwndThread())
			return true;
		m_newTraceName.Append(namedTrace);
		m_usingNamed = true;
		PostNewTrace(0, false);
		return true;
	}

	bool VisualizerWin::ShowUsingSocket(NetworkBackend& backend, const wchar_t* host, u16 port)
	{
		NetworkClient* activeClient = nullptr;
		NetworkClient* oldClient = nullptr;

		auto destroyClient = MakeGuard([&]() { m_client = nullptr; delete activeClient; });
		m_looping = true;
		m_autoScroll = false;
		if (!StartHwndThread())
			return true;

		m_clientDisconnect.Create(EventResetType_Manual);

		wchar_t dots[] = TC("....");
		u32 dotsCounter = 0;

		StringBuffer<256> traceName;
		while (m_hwnd)
		{
			if (!activeClient)
			{
				bool ctorSuccess = true;
				NetworkClientCreateInfo ncci;
				ncci.workerCount = 0;
				activeClient = new NetworkClient(ctorSuccess, ncci);
				if (!ctorSuccess)
					return false;
			}

			StringBuffer<> title;
			PostNewTitle(GetTitlePrefix(title).Appendf(L"Trying to connect to %s:%u%s", host, port, dots + ((dotsCounter--) % 4)));

			bool timedOut;
			if (!activeClient->Connect(backend, host, port, &timedOut))
				continue;
			m_client = activeClient;

			PostNewTitle(GetTitlePrefix(title).Appendf(L"Connected to %s:%u", host, port));
			PostNewTrace(0, false);

			if (oldClient)
			{
				delete oldClient;
				oldClient = nullptr;
			}

			while (m_hwnd && !m_clientDisconnect.IsSet(100) && activeClient->IsConnected())
			{
				m_trace.UpdateReceiveClient(*activeClient);
			}

			activeClient->Disconnect();

			while (activeClient->IsConnected())
			{
				Sleep(500);
				if (!activeClient->SendKeepAlive())
					break;
			}
			PostNewTitle(GetTitlePrefix(title).Appendf(L"Disconnected..."));

			oldClient = activeClient;
			activeClient = nullptr;

			m_clientDisconnect.Reset();
		}
		return true;
	}

	bool VisualizerWin::ShowUsingFile(const wchar_t* fileName, u32 replay)
	{
		m_looping = true;
		m_autoScroll = false;
		m_fileName.Append(fileName);
		if (!StartHwndThread())
			return true;
		PostNewTrace(replay, 0);
		return true;
	}

	bool VisualizerWin::StartHwndThread()
	{
		m_thread.Start([this]() { ThreadLoop(); return 0;}, TC("UbaHwnd"));
		while (!m_hwnd)
			if (m_thread.Wait(10))
				return false;
		return true;
	}

	bool VisualizerWin::HasWindow()
	{
		return m_looping == true;
	}

	HWND VisualizerWin::GetHwnd()
	{
		return m_hwnd;
	}

	void VisualizerWin::Lock(bool lock)
	{
		m_locked = lock;
	}

	void VisualizerWin::ThreadLoop()
	{
		if (m_config.parent)
		{
			SetProcessDpiAwareness(PROCESS_SYSTEM_DPI_AWARE);
		}

		HINSTANCE hInstance = GetModuleHandle(NULL);
		int winPosX = m_config.x;
		int winPosY = m_config.y;
		int winWidth = m_config.width;
		int winHeight = m_config.height;

		RECT rectCombined;
		SetRectEmpty(&rectCombined);
		EnumDisplayMonitors(0, 0, [](HMONITOR hMon,HDC hdc,LPRECT lprcMonitor,LPARAM pData)
			{
				RECT* rectCombined = reinterpret_cast<RECT*>(pData);
				UnionRect(rectCombined, rectCombined, lprcMonitor);
				return TRUE;
			}, (LPARAM)&rectCombined);

		winPosX = Max(int(rectCombined.left), winPosX);
		winPosY = Max(int(rectCombined.top), winPosY);
		winPosX = Min(int(rectCombined.right) - winWidth, winPosX);
		winPosY = Min(int(rectCombined.bottom) - winHeight, winPosY);

		WNDCLASSEX wndClassEx;
		ZeroMemory(&wndClassEx, sizeof(wndClassEx));
		wndClassEx.cbSize = sizeof(wndClassEx);
		wndClassEx.style = CS_HREDRAW | CS_VREDRAW;
		wndClassEx.lpfnWndProc = &StaticWinProc;
		wndClassEx.hIcon = LoadIcon(hInstance, MAKEINTRESOURCE(123));
		wndClassEx.hCursor = 0;
		wndClassEx.hInstance = hInstance;
		wndClassEx.hbrBackground = NULL;
		wndClassEx.lpszClassName = TEXT("UbaVisualizer");
		ATOM wndClassAtom = RegisterClassEx(&wndClassEx);
		const TCHAR* windowClassName = MAKEINTATOM(wndClassAtom);

		auto unreg = MakeGuard([&]() { UnregisterClass(windowClassName, hInstance); });

		UpdateDefaultFont();

		UpdateProcessFont();

		//const TCHAR* fontName = TEXT("Consolas");
		//m_popupFont.handle = CreateFontW(-12, 0, 0, 0, FW_NORMAL, false, false, false, ANSI_CHARSET, OUT_DEFAULT_PRECIS, CLIP_DEFAULT_PRECIS, CLEARTYPE_QUALITY, FIXED_PITCH | FF_MODERN, fontName);

		m_popupFont.handle = m_backend.CreateFont(TEXT("Consolas"), 16, false);
		m_popupFont.height = 14;
		//m_popupFont = (FontHandle)GetStockObject(SYSTEM_FIXED_FONT);

		DWORD scrollbarFlags = 0;
		m_verticalScrollBarEnabled = !ActiveProcessesShouldFillHeight();
		if (m_verticalScrollBarEnabled)
			scrollbarFlags |= WS_VSCROLL;
		m_horizontalScrollBarEnabled = !m_config.AutoScaleHorizontal;
		if (m_horizontalScrollBarEnabled)
			scrollbarFlags |= WS_HSCROLL;

		DWORD windowStyle = WS_CAPTION | WS_SYSMENU | WS_THICKFRAME | WS_MAXIMIZEBOX | WS_MINIMIZEBOX | WS_CLIPCHILDREN | scrollbarFlags;

		DWORD exStyle = 0;
		if (m_config.parent)
			windowStyle = WS_POPUP | scrollbarFlags;

		StringBuffer<> title;
		GetTitlePrefix(title).Append(L"Initializing...");

		HWND hwnd = CreateWindowEx(exStyle, windowClassName, title.data, windowStyle, winPosX, winPosY, winWidth, winHeight, NULL, NULL, hInstance, this);
		m_hwnd = hwnd;

		if (hwnd)
			ImmAssociateContextEx(hwnd, NULL, IACE_CHILDREN);

		auto destroyWin = [&]()
			{
				if (m_hwnd)
				{
					if (m_config.AutoSaveSettings)
						SaveSettings();
					DestroyWindow(m_hwnd);
					m_hwnd = 0;
				}
			};

		BOOL cloak = TRUE;
		DwmSetWindowAttribute(hwnd, DWMWA_CLOAK, &cloak, sizeof(cloak));
		auto exitCloak = MakeGuard([&]() { cloak = FALSE; DwmSetWindowAttribute(hwnd, DWMWA_CLOAK, &cloak, sizeof(cloak)); });

		if (m_config.DarkMode)
			UpdateTheme();

		// Initialize OLE for Drag and Drop support.
		HRESULT dragDropRes = ::OleInitialize(NULL);
		UBA_ASSERT(SUCCEEDED(dragDropRes));
		dragDropRes = ::RegisterDragDrop(hwnd, this);
		UBA_ASSERT(SUCCEEDED(dragDropRes));


		RECT r;
		GetClientRect(m_hwnd, &r);
		m_clientWidth = r.right;
		m_clientHeight = r.bottom;

		m_backend.Init(m_hwnd);

		LOGBRUSH br = { 0 };
		GetObject(m_backgroundBrush, sizeof(br), &br);
		m_processUpdatePen = CreatePen(PS_SOLID, 2, RGB(GetRValue(br.lbColor), GetGValue(br.lbColor), GetBValue(br.lbColor)));

		{
			PaintContext* context = m_backend.CreateContext(m_clientWidth, m_clientHeight);
			HitTestResult res;
			HitTest(res, *context, { -1, -1 });
			m_backend.DeleteContext(context);
		}

		if (m_config.parent)
		{
			exitCloak.Execute();

			m_parentHwnd = (HWND)(uintptr_t)m_config.parent;

			if (IsAWTHost(m_parentHwnd))
			{
				// AWT host  use OWNED top-level window.
				LONG curExStyle = GetWindowLong(hwnd, GWL_EXSTYLE);
				SetWindowLong(hwnd, GWL_EXSTYLE, curExStyle | WS_EX_NOACTIVATE | WS_EX_TOOLWINDOW);
				SetWindowLongPtr(hwnd, GWLP_HWNDPARENT, (LONG_PTR)m_parentHwnd);
			}
			else
			{
				// Non-AWT host (UnrealVS / WPF, plain Win32, …): WS_CHILD path.
				// If not child it will not propagate keyboard presses etc to parent.
				SetWindowLong(hwnd, GWL_STYLE, WS_CHILD | scrollbarFlags);
				if (!SetParent(hwnd, m_parentHwnd))
					m_logger.Error(L"SetParent failed using parentHwnd 0x%llx", m_parentHwnd);
			}

			UpdateWindow(m_hwnd);
			UpdateScrollbars(true);
			PostMessage(m_parentHwnd, 0x0444, 0, (LPARAM)hwnd);
		}
		else
		{
			ShowWindow(hwnd, SW_SHOW);
			UpdateWindow(m_hwnd);
			UpdateScrollbars(true);
			exitCloak.Execute();
		}

		m_startTime = GetTime();

		while (m_looping)
		{
			DWORD timeoutMs = 2000;
			DWORD result = MsgWaitForMultipleObjects(0, NULL, FALSE, timeoutMs, QS_ALLINPUT);
			if (result == WAIT_TIMEOUT)
				continue;
			if (result != WAIT_OBJECT_0)
				break;
			MSG msg;
			while (PeekMessage(&msg, NULL, 0, 0, PM_REMOVE))
			{
				TranslateMessage(&msg);
				DispatchMessage(&msg);

				// It may happen that we receive the WM_DESTOY message from within the DistpachMessage above and handle it directly in WndProc.
				// So before trying to call GetMessage again, we just need to validate that m_looping is still true otherwise we could end
				// up waiting forever for this thread to exit.
				if (!m_looping || msg.message == WM_QUIT || msg.message == WM_DESTROY || msg.message == WM_CLOSE)
				{
					destroyWin();
					m_looping = false;
					if (m_listenTimeout.IsCreated())
						m_listenTimeout.Set();
					break;
				}
			}
		}

		destroyWin();
	}

	void VisualizerWin::Pause(bool pause)
	{
		if (m_paused == pause)
			return;

		m_paused = pause;
		if (pause)
		{
			m_pauseStart = GetTime();
		}
		else
		{
			m_replay = 1;
			m_pauseTime += GetTime() - m_pauseStart;
			m_traceView.finished = false;
			SetTimer(m_hwnd, 0, 200, NULL);
		}
	}

	void VisualizerWin::StartDragToScroll(const POINT& anchor)
	{
		// Uses reference-counter method since multiple input events (left and middle mouse button) can trigger the drag-to-scroll mechansim
		if (m_dragToScrollCounter == 0)
		{
			m_processSelected = false;
			m_sessionSelectedIndex = ~0u;
			m_statsSelected = false;
			m_activeProcessGraphSelected = false;
			m_buttonSelected = ~0u;
			m_timelineSelected = 0;
			m_fetchedFilesSelected = ~0u;
			m_workSelected = false;
			m_hyperLinkSelected.clear();
			m_autoScroll = false;
			m_mouseAnchor = { anchor.x, anchor.y };
			m_scrollAtAnchorX = m_scrollPosX;
			m_scrollAtAnchorY = m_scrollPosY;
			SetCapture(m_hwnd);
			Redraw(false);
		}
		++m_dragToScrollCounter;
	}

	void VisualizerWin::StopDragToScroll()
	{
		if (m_dragToScrollCounter > 0)
			--m_dragToScrollCounter;
		if (m_dragToScrollCounter != 0)
			return;

		ReleaseCapture();
		if (UpdateSelection())
			Redraw(false);
	}

	void VisualizerWin::SaveSettings()
	{
		RECT rect;
		GetWindowRect(m_hwnd, &rect);

		m_config.x = rect.left;
		m_config.y = rect.top;
		m_config.width = rect.right - rect.left;
		m_config.height = rect.bottom - rect.top;
		m_config.Save(m_logger);
		m_settingsDirty = false;
	}

	void VisualizerWin::ChangeFontSize(int offset)
	{
		m_config.fontSize += offset;
		m_config.fontSize = Max(m_config.fontSize, 10u);
		m_settingsDirty = true;
		UpdateDefaultFont();
		Redraw(true);
	}

	void VisualizerWin::Redraw(bool now)
	{
		u32 flags = RDW_INVALIDATE;
		if (now)
			flags |= RDW_UPDATENOW;
		RedrawWindow(m_hwnd, NULL, NULL, flags);

		u32 activeProcessCount = u32(m_trace.m_activeProcesses.size());
		for (u32 i=0; i!=sizeof_array(m_activeProcessCountHistory); ++i)
			m_activeProcessCountHistory[i] = activeProcessCount;
	}

	void VisualizerWin::StopUpdate()
	{
		KillTimer(m_hwnd, 0);
	}

	void VisualizerWin::PaintEmptyScreen(PaintContext& context)
	{
		if (!m_parentHwnd)
			Visualizer::PaintEmptyScreen(context);
		else
		{
			StringBuffer<> str[2];
			if (m_listenChannel.count)
			{
				str[0].Appendf(TC("Listening for new sessions on channel '%s'"), m_listenChannel.data);
				if (!m_config.ShowAllTraces)
				{
					OwnerInfo ownerInfo = GetOwnerInfo();
					if (ownerInfo.pid)
						str[1].Appendf(TC("Filtering by '_%s%u'"), ownerInfo.id, ownerInfo.pid);
				}
			}
			context.SetFont(m_popupFont);
			Visualizer::DrawCenteredText(context, str, 2, nullptr);
		}
	}

	void VisualizerWin::FinishIncompatible()
	{
		StringBuffer<> title;
		SetWindowTextW(m_hwnd, GetTitlePrefix(title).Appendf(L" (Listening for new sessions on channel '%s')", m_listenChannel.data).data);
	}

	void VisualizerWin::CopyTextToClipboard(const TString& str)
	{
		if (!OpenClipboard(m_hwnd))
			return;
		if (auto hglbCopy = GlobalAlloc(GMEM_MOVEABLE, (str.size() + 1) * sizeof(TCHAR)))
		{
			if (auto lptstrCopy = GlobalLock(hglbCopy))
			{
				memcpy(lptstrCopy, str.data(), (str.size() + 1) * sizeof(TCHAR));
				GlobalUnlock(hglbCopy);
				EmptyClipboard();
				SetClipboardData(CF_UNICODETEXT, hglbCopy);
			}
		}
		CloseClipboard();
	}

	bool VisualizerWin::UpdateSelection()
	{
		if (!m_mouseOverWindow || m_dragToScrollCounter > 0)
			return false;
		POINT pos;
		GetCursorPos(&pos);
		ScreenToClient(m_hwnd, &pos);
		return Visualizer::UpdateSelection(pos);
	}

	void VisualizerWin::UpdateScrollbars(bool redraw)
	{
		SCROLLINFO si;
		si.cbSize = sizeof(SCROLLINFO);
		si.fMask = SIF_ALL | SIF_DISABLENOSCROLL;
		si.nMin = 0;
		si.nMax = m_contentHeight;
		si.nPage = int(m_clientHeight);
		si.nPos = -int(m_scrollPosY);
		si.nTrackPos = 0;

		bool updateFrame = false;
		if (ActiveProcessesShouldFillHeight())
		{
			if (m_verticalScrollBarEnabled)
			{
				SetWindowLong(m_hwnd, GWL_STYLE, GetWindowLong(m_hwnd, GWL_STYLE) & ~WS_VSCROLL);
				m_verticalScrollBarEnabled = false;
				updateFrame = true;
			}
		}
		else
		{
			if (!m_verticalScrollBarEnabled)
			{
				SetWindowLong(m_hwnd, GWL_STYLE, GetWindowLong(m_hwnd, GWL_STYLE) | WS_VSCROLL);
				m_verticalScrollBarEnabled = true;
				updateFrame = true;
			}
			SetScrollInfo(m_hwnd, SB_VERT, &si, redraw);
		}

		if (m_config.AutoScaleHorizontal)
		{
			if (m_horizontalScrollBarEnabled)
			{
				SetWindowLong(m_hwnd, GWL_STYLE, GetWindowLong(m_hwnd, GWL_STYLE) & ~WS_HSCROLL);
				m_horizontalScrollBarEnabled = false;
				updateFrame = true;
			}
		}
		else
		{
			if (!m_horizontalScrollBarEnabled)
			{
				SetWindowLong(m_hwnd, GWL_STYLE, GetWindowLong(m_hwnd, GWL_STYLE) | WS_HSCROLL);
				m_horizontalScrollBarEnabled = true;
				updateFrame = true;
			}

			si.nMax = m_contentWidth;
			si.nPage = m_clientWidth;
			si.nPos = -int(m_scrollPosX);
			SetScrollInfo(m_hwnd, SB_HORZ, &si, redraw);
		}

		if (updateFrame)
			SetWindowPos(m_hwnd, NULL, 0, 0, 0, 0, SWP_NOSIZE | SWP_NOMOVE | SWP_NOZORDER | SWP_FRAMECHANGED);
	}

	void VisualizerWin::UpdateTheme()
	{
		SetWindowTheme(m_hwnd, m_config.DarkMode ? L"DarkMode_Explorer" : L"Explorer", NULL);
		SendMessageW(m_hwnd, WM_THEMECHANGED, 0, 0);
		BOOL useDarkMode = m_config.DarkMode;
		u32 attribute = 20; // DWMWA_USE_IMMERSIVE_DARK_MODE
		DwmSetWindowAttribute(m_hwnd, attribute, &useDarkMode, sizeof(useDarkMode));
	}

	HRESULT VisualizerWin::QueryInterface(REFIID riid, void** ppvObject)
	{
		if (IID_IDropTarget == riid || IID_IUnknown == riid)
		{
			AddRef();
			*ppvObject = (IDropTarget*)(this);
			return S_OK;
		}
		else
		{
			*ppvObject = NULL;
			return E_NOINTERFACE;
		}
	}

	ULONG VisualizerWin::AddRef(void)
	{
		return InterlockedIncrement(&m_OLEReferenceCount);
	}

	ULONG VisualizerWin::Release(void)
	{
		InterlockedDecrement(&m_OLEReferenceCount);
		return m_OLEReferenceCount;
	}

	HRESULT VisualizerWin::DragEnter(IDataObject* pDataObj, ::DWORD grfKeyState, POINTL pt, ::DWORD* pdwEffect)
	{
		if (pDataObj == nullptr)
			return E_FAIL;
		if (pdwEffect == nullptr)
			return E_POINTER;
		*pdwEffect = DROPEFFECT_COPY;
		return S_OK;
	}

	HRESULT VisualizerWin::DragOver(::DWORD grfKeyState, POINTL pt, ::DWORD* pdwEffect)
	{
		if (pdwEffect == nullptr)
			return E_POINTER;
		*pdwEffect = DROPEFFECT_COPY;
		return S_OK;
	}

	HRESULT VisualizerWin::DragLeave(void)
	{
		return S_OK;
	}

	HRESULT VisualizerWin::Drop(IDataObject* pDataObj, ::DWORD grfKeyState, POINTL pt, ::DWORD* pdwEffect)
	{
		if (pDataObj == nullptr)
			return E_FAIL;
		if (pdwEffect == nullptr)
			return E_POINTER;

		// Utility to ensure resource release
		struct FOLEResourceGuard
		{
			STGMEDIUM& StorageMedium;
			LPVOID DataPointer;
			FOLEResourceGuard(STGMEDIUM& InStorage) : StorageMedium(InStorage), DataPointer(GlobalLock(InStorage.hGlobal)) {}
			~FOLEResourceGuard() { GlobalUnlock(StorageMedium.hGlobal); ReleaseStgMedium(&StorageMedium); }
		};

		// Attempt to get plain text or unicode text from the data being dragged in
		STGMEDIUM storageMedium;
		FORMATETC FormatEtc_Ansii = { CF_TEXT, 0, DVASPECT_CONTENT, -1, TYMED_HGLOBAL };
		const bool bHaveAnsiText = (pDataObj->QueryGetData(&FormatEtc_Ansii) == S_OK) ? true : false;

		FORMATETC FormatEtc_UNICODE = { CF_UNICODETEXT, 0, DVASPECT_CONTENT, -1, TYMED_HGLOBAL };
		const bool bHaveUnicodeText = (pDataObj->QueryGetData(&FormatEtc_UNICODE) == S_OK) ? true : false;

		FORMATETC FormatEtc_File = { CF_HDROP, 0, DVASPECT_CONTENT, -1, TYMED_HGLOBAL };
		const bool bHaveFiles = (pDataObj->QueryGetData(&FormatEtc_File) == S_OK) ? true : false;

		StringBuffer<MaxPath> filePath;
		if (bHaveUnicodeText && S_OK == pDataObj->GetData(&FormatEtc_UNICODE, &storageMedium))
		{
			FOLEResourceGuard ResourceGuard(storageMedium);
			filePath.Append(static_cast<TCHAR*>(ResourceGuard.DataPointer));
		}
		if (bHaveAnsiText && S_OK == pDataObj->GetData(&FormatEtc_Ansii, &storageMedium))
		{
			// TODO: support ansi text
			// FOLEResourceGuard ResourceGuard(storageMedium);
			// filePath.Append(static_cast<TCHAR*>(ResourceGuard.DataPointer));
		}
		if (bHaveFiles && S_OK == pDataObj->GetData(&FormatEtc_File, &storageMedium))
		{
			FOLEResourceGuard ResourceGuard(storageMedium);
			const DROPFILES* DropFiles = static_cast<DROPFILES*>(ResourceGuard.DataPointer);

			// pFiles is the offset to the beginning of the file list, in bytes
			LPVOID FileListStart = (BYTE*)ResourceGuard.DataPointer + DropFiles->pFiles;

			if (DropFiles->fWide) // TODO: support ansi filenames
			{
				// Unicode filenames
				// The file list is NULL delimited with an extra NULL character at the end.
				TCHAR* Pos = static_cast<TCHAR*>(FileListStart);
				while (Pos[0] != 0)
				{
					filePath.Append(Pos);
					break; // only care about the first one
				}
			}
		}

		if (!filePath.IsEmpty())
		{
			m_fileName.Clear();
			m_fileName.Append(filePath.data);
			PostNewTrace(false, 0);
		}

		return S_OK;
	}

	void VisualizerWin::PostNewTrace(u32 replay, bool paused)
	{
		KillTimer(m_hwnd, 0);
		PostMessage(m_hwnd, WM_NEWTRACE, replay, paused);
	}

	void VisualizerWin::PostNewTitle(const StringView& title)
	{
#if !defined( __clang_analyzer__ )
		PostMessage(m_hwnd, WM_SETTITLE, 0, (LPARAM)_wcsdup(title.data));
#endif
	}

	void VisualizerWin::PostQuit()
	{
		m_looping = false;
		PostMessage(m_hwnd, WM_USER+666, 0, 0);
	}

	LRESULT VisualizerWin::WinProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam)
	{
		switch (Msg)
		{
		case WM_SETTITLE:
		{
			auto title = (wchar_t*)lParam;
			SetWindowTextW(hWnd, title);
			free(title);
			break;
		}
		case WM_NEWTRACE:
		{
			if (NewTrace(u32(wParam), lParam != 0))
				SetTimer(m_hwnd, 0, 200, NULL);
			return 0;
		}

		case WM_SYSCOMMAND:
			// Don't send this message through DefWindowProc as it will destroy the window 
			// and GetMessage will stay stuck indefinitely.
			if (wParam == SC_CLOSE)
			{
				PostQuit();
				return 0;
			}
			break;

		case WM_DESTROY:
			PostQuit();
			return 0;

		case WM_ERASEBKGND:
			return 1;

		case WM_PAINT:
		{
			u64 start = GetTime();
			m_backend.Paint([&](PaintContext& context, RECT rect)
				{
					PaintAll(context, rect);
#if UBA_DEBUG
					StringBuffer<32> buf;
					buf.Appendf(TC("%llums"), m_lastPaintTimeMs);
					context.SetFont(m_defaultFont);
					context.SetTextColor(m_textColor);
					context.Text(m_clientWidth - 50, m_clientHeight - 50, buf, nullptr);
#endif
				});
			m_lastPaintTimeMs = TimeToMs(GetTime() - start);
			break;
		}
		case WM_SIZE:
		{
			int height = HIWORD(lParam);
			if (m_contentHeight && m_contentHeight + int(m_scrollPosY) < height)
				m_scrollPosY = float(Min(0, height - m_contentHeight));
			int width = LOWORD(lParam);
			if (m_contentWidth && m_contentWidth + int(m_scrollPosX) < width)
				m_scrollPosX = float(Min(0, width - m_contentWidth));
			RECT r;
			GetClientRect(m_hwnd, &r);
			m_clientWidth = r.right;
			m_clientHeight = r.bottom;
			m_backend.Resize(m_clientWidth, m_clientHeight);
			UpdateScrollbars(true);
			break;
		}
		case WM_TIMER:
		{
			if (m_handlingTimer) // This is to prevent hangs caused by message boxes from asserts
				break;
			m_handlingTimer = true;

			bool changed = UpdateTrace();
			if (changed && !IsIconic(m_hwnd))
			{
				UpdateScrollbars(true);

				RedrawWindow(m_hwnd, NULL, NULL, RDW_INVALIDATE); // Don't use RDW_UPDATENOW.. it will cause stutter
				u32 waitTime = 60u;//u32(Min(m_lastPaintTimeMs * 5, 200ull));
				if (!m_traceView.finished)
					SetTimer(m_hwnd, 0, waitTime, NULL);
			}

			m_handlingTimer = false;
			break;
		}
		case WM_MOUSEWHEEL:
		{
			if (m_dragToScrollCounter > 0)
				break;

			int delta = GET_WHEEL_DELTA_WPARAM(wParam);
			bool controlDown = GetAsyncKeyState(VK_CONTROL) & (1<<15);
			bool shiftDown = GetAsyncKeyState(VK_LSHIFT) & (1<<15);

			if (m_config.ScaleHorizontalWithScrollWheel || controlDown || shiftDown)
			{
				if (m_activeSection == 2 || !controlDown) // process bars
				{
					// Use mouse cursor as scroll anchor point
					POINT cursorPos = {};
					GetCursorPos(&cursorPos);
					ScreenToClient(m_hwnd, &cursorPos);

					float horizontalScaleValue = m_config.horizontalScaleValue;
					float newScaleValue = horizontalScaleValue;
					int oldBoxHeight = m_config.boxHeight;
					int newBoxHeight = m_config.boxHeight;

					if (controlDown)
					{
						if (delta < 0)
						{
							if (newBoxHeight > 1)
								--newBoxHeight;
						}
						else if (delta > 0)
							++newBoxHeight;
					}
					else
						newScaleValue = Max(horizontalScaleValue + horizontalScaleValue*float(delta)*0.0006f, 0.001f);

					// TODO: m_progressRectLeft changes with zoom so anchor logic is wrong
					const float scrollAnchorOffsetX = float(cursorPos.x) - float(m_progressRectLeft);
					const float scrollAnchorOffsetY = 0;//float(cursorPos.y)*m_zoomValue;// - m_progressRectLeft;

					float oldZoomValue = m_zoomValue;
					if (newBoxHeight != int(m_config.boxHeight))
					{
						m_config.boxHeight = newBoxHeight;
						UpdateProcessFont();
					}

					m_scrollPosY = Min(0.0f, float(m_scrollPosY - scrollAnchorOffsetY)*(m_zoomValue/oldZoomValue) + scrollAnchorOffsetY);
					m_scrollPosX = Min(0.0f, float(m_scrollPosX - scrollAnchorOffsetX)*(m_zoomValue/oldZoomValue)*(newScaleValue/horizontalScaleValue) + scrollAnchorOffsetX);//LOWORD(lParam);

					bool wider = false;
					if (horizontalScaleValue != newScaleValue)
					{
						wider = horizontalScaleValue < newScaleValue;
						m_config.horizontalScaleValue = newScaleValue;
					}

					UpdateAutoscroll();
					UpdateSelection();

					int minScroll = m_clientWidth - m_contentWidth;
					m_scrollPosX = Min(0.0f, Max(m_scrollPosX, float(minScroll)));
					m_scrollPosY = Min(0.0f, Max(m_scrollPosY, float(m_clientHeight - m_contentHeight)));

					UpdateAutoscroll();
					//if (!m_traceView.finished && m_scrollPosX <= minScroll)
					//	m_autoScroll = true;

					if (oldBoxHeight != newBoxHeight || wider)
						DirtyBitmaps(true);
					else if (m_config.ShowReadWriteColors)
						for (auto& session : m_traceView.sessions)
							for (auto& processor : session.processors)
								for (auto& process : processor.processes)
									if (process.type == TraceView::ProcessType_Normal && (process.createFilesTime || process.writeFilesTime))
										process.bitmapDirty = true;
				}
				else if (m_activeSection == 1) // active processes
				{
					if (delta < 0)
						m_config.maxActiveProcessHeight = Max(m_config.maxActiveProcessHeight - 1u, 5u);
					else if (delta > 0)
						m_config.maxActiveProcessHeight = Min(m_config.maxActiveProcessHeight + 1u, 32u);
				}
				else if (m_activeSection == 0 || m_activeSection == 3) // status/timeline
				{
					if (delta < 0)
						m_config.fontSize -= 1;
					else if (delta > 0)
						m_config.fontSize += 1;
					UpdateDefaultFont();
				}
				UpdateScrollbars(true);
				Redraw(false);
			}
			else
			{
				float oldScrollY = m_scrollPosY;
				m_scrollPosY = m_scrollPosY + float(delta);
				m_scrollPosY = Min(Max(m_scrollPosY, float(m_clientHeight - m_contentHeight)), 0.0f);
				if (oldScrollY != m_scrollPosY)
				{
					UpdateScrollbars(true);
					Redraw(false);
				}
			}
			break;
		}
		case WM_NCHITTEST:
			if (m_parentHwnd)
				return HTCLIENT;
			break;
		case WM_MOUSEMOVE:
		{
			//m_logger.Info(TC("Section: %u"), m_activeSection);

			POINTS p = MAKEPOINTS(lParam);
			POINT pos{ p.x, p.y };
			if (m_dragToScrollCounter > 0)
			{
				if (m_contentHeight <= m_clientHeight)
					m_scrollPosY = 0;
				else
					m_scrollPosY = Max(Min(m_scrollAtAnchorY + float(pos.y - m_mouseAnchor.y), 0.0f), float(m_clientHeight - m_contentHeight));

				if (m_contentWidth <= m_clientWidth)
					m_scrollPosX = 0;
				else
				{
					int minScroll = m_clientWidth - m_contentWidth;
					m_scrollPosX = Max(Min(m_scrollAtAnchorX + float(pos.x - m_mouseAnchor.x), 0.0f), float(minScroll));
					if (!m_traceView.finished && m_scrollPosX <= float(minScroll))
						m_autoScroll = true;
				}
				UpdateScrollbars(true);
				Redraw(false);
			}
			else
			{
				if (UpdateSelection() || m_config.showCursorLine)
					Redraw(false);
			}

			TRACKMOUSEEVENT tme;
			tme.cbSize = sizeof(tme);
			tme.dwFlags = TME_LEAVE;
			tme.hwndTrack = hWnd;
			TrackMouseEvent(&tme);
			m_mouseOverWindow = true;
			break;
		}
		case WM_MOUSELEAVE:
			m_mouseOverWindow = false;
			TRACKMOUSEEVENT tme;
			tme.cbSize = sizeof(tme);
			tme.dwFlags = TME_CANCEL;
			tme.hwndTrack = hWnd;
			TrackMouseEvent(&tme);

			if (!m_showPopup)
				UnselectAndRedraw();
			break;

		case WM_MBUTTONDOWN:
		{
			POINTS p = MAKEPOINTS(lParam);
			StartDragToScroll(POINT{ p.x, p.y });
			break;
		}

		case WM_LBUTTONDOWN:
		{
			if (!m_parentHwnd && m_traceView.sessions.empty())
			{
				int centerHrz = m_clientWidth/2;
				int centerVrt = m_clientHeight/2;
				RECT r = { centerHrz, centerVrt, centerHrz, centerVrt };
				InflateRect(&r, 180, 40);
				POINTS p = MAKEPOINTS(lParam);
				if (PtInRect(&r, { p.x, p.y }))
				{
					OPENFILENAME ofn;
					tchar szFile[MAX_PATH] = TC("");
					ZeroMemory(&ofn, sizeof(ofn));
					ofn.lStructSize = sizeof(ofn);
					ofn.hwndOwner = m_hwnd;
					ofn.lpstrFilter = L"Uba Files\0*.uba\0All Files\0*.*\0";
					ofn.lpstrFile = szFile;
					ofn.nMaxFile = MAX_PATH;
					ofn.Flags = OFN_FILEMUSTEXIST | OFN_PATHMUSTEXIST;

					if (GetOpenFileName(&ofn))
					{
						m_fileName.Append(szFile);
						PostNewTrace(0, false);
					}
				}
				break;
			}

			if (m_buttonSelected != ~0u)
			{
				bool* values = &m_config.showProgress;
				values[m_buttonSelected] = !values[m_buttonSelected];
				m_settingsDirty = true;

				PaintContext* context = m_backend.CreateContext(m_clientWidth, m_clientHeight);
				HitTestResult res;
				HitTest(res, *context, { -1, -1 });
				m_backend.DeleteContext(context);
				UpdateScrollbars(true);
				Redraw(false);
			}
			else if (m_timelineSelected != 0)
			{
				if (!m_client) // Does not work for network streams
				{
					float timelineSelected = Max(m_timelineSelected, 0.0f);
					Reset();

					bool changed;
					u64 time = MsToTime(u64(timelineSelected * 1000.0));

					if (!m_fileName.IsEmpty())
					{
						if (!m_trace.ReadFile(m_traceView, m_fileName.data, true))
							return false;
					}
					else
					{
						if (!m_trace.StartReadNamed(m_traceView, nullptr, true, true))
							return false;
						if (m_traceView.realStartTime + time > m_startTime)
							time = m_startTime - m_traceView.realStartTime;
					}

					m_traceView.finished = false;
					if (!m_fileName.IsEmpty())
						m_trace.UpdateReadFile(m_traceView, time, changed);
					else if (m_usingNamed)
						m_trace.UpdateReadNamed(m_traceView, time, changed);

					m_pauseStart = m_startTime + time;
					m_pauseTime = m_startTime - m_pauseStart;

					if (!m_paused)
					{
						m_autoScroll = true;
						m_replay = 1;
						SetTimer(m_hwnd, 0, 200, NULL);
					}
					else
					{
						m_pauseTime = 0;
					}

					HitTestResult res;
					PaintContext* context = m_backend.CreateContext(m_clientWidth, m_clientHeight);
					HitTest(res, *context, { -1, -1 });
					m_backend.DeleteContext(context);

					m_scrollPosX = Min(Max(m_scrollPosX, float(m_clientWidth - m_contentWidth)), 0.0f);
					m_scrollPosY = Min(0.0f, Max(m_scrollPosY, float(m_clientHeight - m_contentHeight)));
					UpdateAutoscroll();

					UpdateScrollbars(true);
					Redraw(true);
				}
			}
			else if (!m_hyperLinkSelected.empty())
			{
				ShellExecuteW(NULL, L"open", m_hyperLinkSelected.c_str(), NULL, NULL, SW_SHOW);
			}
			else if (m_sessionSelectedIndex != ~0u)
			{
				auto& session = m_traceView.sessions[m_sessionSelectedIndex];
				for (auto& info : session.infos)
				{
					if (info.hyperlink.empty())
						continue;
					ShellExecuteW(NULL, L"open", info.hyperlink.c_str(), NULL, NULL, SW_SHOW);
					break;
				}
			}
			else
			{
				POINTS p = MAKEPOINTS(lParam);
				StartDragToScroll(POINT{ p.x, p.y });
			}
			break;
		}

		case WM_SETCURSOR:
		{
			static HCURSOR arrow = LoadCursorW(NULL, IDC_ARROW);
			static HCURSOR hand = LoadCursorW(NULL, IDC_HAND);
			bool useHand = false;
			if (!m_parentHwnd && m_traceView.sessions.empty())
			{
				int centerHrz = m_clientWidth/2;
				int centerVrt = m_clientHeight/2;
				RECT r = { centerHrz, centerVrt, centerHrz, centerVrt };
				InflateRect(&r, 180, 40);
				POINT pt;
				GetCursorPos(&pt);
				ScreenToClient(hWnd, &pt);
				useHand = PtInRect(&r, { pt.x, pt.y });
			}
			else if (!m_hyperLinkSelected.empty())
			{
				useHand = true;
			}
			else if (m_sessionSelectedIndex != ~0u)
			{
				auto& session = m_traceView.sessions[m_sessionSelectedIndex];
				for (auto& info : session.infos)
					if (!info.hyperlink.empty())
						useHand = true;
			}
			SetCursor(useHand ? hand : arrow);
			break;
		}
		case WM_LBUTTONUP:
		{
			if (!(m_buttonSelected != ~0u || m_timelineSelected != 0))
			{
				StopDragToScroll();
			}
			break;
		}

		case WM_RBUTTONUP:
		{
			POINT point;
			point.x = LOWORD(lParam);
			point.y = HIWORD(lParam);

			HMENU hMenu = CreatePopupMenu();
			ClientToScreen(hWnd, &point);

#define UBA_VISUALIZER_FLAG(name, defaultValue, desc) \
				AppendMenuW(hMenu, MF_STRING | (m_config.name ? MF_CHECKED : 0), Popup_##name, TC(desc));
			UBA_VISUALIZER_FLAGS2
#undef UBA_VISUALIZER_FLAG

				AppendMenuW(hMenu, MF_STRING, Popup_IncreaseFontSize, L"&Increase Font Size");
			AppendMenuW(hMenu, MF_STRING, Popup_DecreaseFontSize, L"&Decrease Font Size");

			AppendMenuW(hMenu, MF_SEPARATOR, 0, NULL);

			if (m_sessionSelectedIndex != ~0u)
			{
				AppendMenuW(hMenu, MF_STRING, Popup_CopySessionInfo, L"&Copy Session Info");
				AppendMenuW(hMenu, MF_SEPARATOR, 0, NULL);
			}
			else if (m_processSelected)
			{
				const TraceView::Process& process = m_traceView.GetProcess(m_processSelectedLocation);

				AppendMenuW(hMenu, MF_STRING, Popup_CopyProcessInfo, L"&Copy Process Info");
				if (!process.logLines.empty())
					AppendMenuW(hMenu, MF_STRING, Popup_CopyProcessLog, L"Copy Process &Log");
				if (!process.breadcrumbs.empty())
					AppendMenuW(hMenu, MF_STRING, Popup_CopyProcessBreadcrumbs, L"Copy Process &Breadcrumbs");
				AppendMenuW(hMenu, MF_SEPARATOR, 0, NULL);
			}
			else if (m_workSelected)
			{
				AppendMenuW(hMenu, MF_STRING, Popup_CopyWorkInfo, L"&Copy Work Info");
			}

			if (!m_traceView.sessions.empty())
			{
				if (!m_client)
				{
					if (!m_replay || m_traceView.finished)
						AppendMenuW(hMenu, MF_STRING, Popup_Replay, L"&Replay Trace");
					else
					{
						if (m_paused)
							AppendMenuW(hMenu, MF_STRING, Popup_Play, L"&Play");
						else
							AppendMenuW(hMenu, MF_STRING, Popup_Pause, L"&Pause");
						AppendMenuW(hMenu, MF_STRING, Popup_JumpToEnd, L"&Jump To End");
					}
				}

				if (m_fileName.IsEmpty())
					AppendMenuW(hMenu, MF_STRING, Popup_SaveAs, L"&Save Trace");

				AppendMenuW(hMenu, MF_SEPARATOR, 0, NULL);
			}

			AppendMenuW(hMenu, MF_STRING, Popup_SaveSettings, L"Save Position/Settings");
			AppendMenuW(hMenu, MF_STRING, Popup_OpenSettings, L"Open Settings file");
			AppendMenuW(hMenu, MF_STRING, Popup_Quit, L"&Quit");
			m_showPopup = true;
			switch (TrackPopupMenu(hMenu, TPM_RETURNCMD | TPM_RIGHTBUTTON, point.x, point.y, 0, hWnd, NULL))
			{
			case Popup_SaveAs:
			{
				OPENFILENAME ofn;
				TCHAR szFile[260] = { 0 };

				// Initialize OPENFILENAME
				ZeroMemory(&ofn, sizeof(ofn));
				ofn.lStructSize = sizeof(ofn);
				ofn.hwndOwner = hWnd;
				ofn.lpstrFile = szFile;
				ofn.nMaxFile = sizeof(szFile);
				ofn.lpstrDefExt = TC("uba");
				ofn.lpstrFilter = TC("Uba\0*.uba\0All\0*.*\0");
				ofn.nFilterIndex = 1;
				ofn.lpstrFileTitle = NULL;
				ofn.nMaxFileTitle = 0;
				ofn.lpstrInitialDir = NULL;
				//ofn.Flags = OFN_PATHMUSTEXIST | OFN_FILEMUSTEXIST;
				if (GetSaveFileName(&ofn))
					m_trace.SaveAs(ofn.lpstrFile);
				break;
			}
			case Popup_ShowProcessText:
				m_config.ShowProcessText = !m_config.ShowProcessText;
				m_settingsDirty = true;
				Redraw(true);
				break;
			case Popup_ShowReadWriteColors:
				m_config.ShowReadWriteColors = !m_config.ShowReadWriteColors;
				m_settingsDirty = true;
				DirtyBitmaps(false);
				Redraw(true);
				break;
			case Popup_ScaleHorizontalWithScrollWheel:
				m_config.ScaleHorizontalWithScrollWheel = !m_config.ScaleHorizontalWithScrollWheel;
				m_settingsDirty = true;
				break;
			case Popup_ShowAllTraces:
				m_config.ShowAllTraces = !m_config.ShowAllTraces;
				m_settingsDirty = true;
				break;
			case Popup_SortActiveRemoteSessions:
				m_config.SortActiveRemoteSessions = !m_config.SortActiveRemoteSessions;
				m_settingsDirty = true;
				Redraw(true);
				break;
			case Popup_AutoScaleHorizontal:
				m_config.AutoScaleHorizontal = !m_config.AutoScaleHorizontal;
				m_settingsDirty = true;
				UpdateScrollbars(true);
				Redraw(true);
				break;
			case Popup_LockTimelineToBottom:
				m_config.LockTimelineToBottom = !m_config.LockTimelineToBottom;
				m_settingsDirty = true;
				Redraw(true);
				break;
			case Popup_DarkMode:
			{
				m_config.DarkMode = !m_config.DarkMode;
				m_settingsDirty = true;
				DirtyBitmaps(true);
				UpdateTheme();
				Redraw(true);
				break;
			}
			case Popup_AutoSaveSettings:
				m_config.AutoSaveSettings = !m_config.AutoSaveSettings;
				m_settingsDirty = true;
				break;

			case Popup_Replay:
				PostNewTrace(1, false);
				break;

			case Popup_Play:
				Pause(false);
				break;

			case Popup_Pause:
				Pause(true);
				break;

			case Popup_JumpToEnd:
				m_traceView.finished = true;
				PostNewTrace(0, false);
				break;

			case Popup_SaveSettings:
				SaveSettings();
				break;

			case Popup_OpenSettings:
				SaveSettings();
				ShellExecuteW(NULL, L"open", m_config.filename.c_str(), NULL, NULL, SW_SHOW);
				break;

			case Popup_Quit: // Quit
				PostQuit();
				break;

			case Popup_IncreaseFontSize:
				ChangeFontSize(1);
				break;
			case Popup_DecreaseFontSize:
				ChangeFontSize(-1);
				break;

			case Popup_CopySessionInfo:
			{
				TString str;
				auto& session = m_traceView.sessions[m_sessionSelectedIndex];
				StringBuffer<> fullName;
				session.GetFullName(fullName);
				str.append(fullName.data).append(TC("\n"));
				for (auto& line : session.summary)
					str.append(line).append(TC("\n"));
				CopyTextToClipboard(str);
				break;
			}

			case Popup_CopyProcessInfo:
			{
				TString str;
				WriteTextLogger logger(str);
				const TraceView::Process& process = m_traceView.GetProcess(m_processSelectedLocation);
				WriteProcessStats(logger, process);
				CopyTextToClipboard(str);
				break;
			}
			case Popup_CopyProcessLog:
			{
				TString str;
				const TraceView::Process& process = m_traceView.GetProcess(m_processSelectedLocation);
				bool isFirst = true;
				for (auto line : process.logLines)
				{
					if (!isFirst)
						str += '\n';
					isFirst = false;
					str += line.text;
				}
				CopyTextToClipboard(str);
				break;
			}
			case Popup_CopyProcessBreadcrumbs:
			{
				const TraceView::Process& process = m_traceView.GetProcess(m_processSelectedLocation);
				CopyTextToClipboard(process.breadcrumbs);
				break;
			}
			case Popup_CopyWorkInfo:
			{
				TString str;
				WriteTextLogger logger(str);
				auto& record = m_traceView.workTracks[m_workTrack].records[m_workIndex];
				WriteWorkStats(logger, record);
				CopyTextToClipboard(str);
				break;
			}
			}

			DestroyMenu(hMenu);
			m_showPopup = false;
			if (m_settingsDirty && m_config.AutoSaveSettings)
				SaveSettings();
			UnselectAndRedraw();
			break;
		}

		case WM_MBUTTONUP:
		{
			StopDragToScroll();
			//m_processSelected = false;
			break;
		}
		case WM_KEYDOWN:
		{
			if (wParam == VK_SPACE)
				Pause(!m_paused);
			if (wParam == VK_ADD)
				++m_replay;
			if (wParam == VK_SUBTRACT)
				--m_replay;
			if (wParam == VK_BACK)
				if (!m_filterString.empty())
				{
					m_filterString.resize(m_filterString.size() -1);
					Redraw(true);
				}
			break;
		}
		case WM_CHAR:
		{
			tchar c = static_cast<tchar>(wParam);
			if (c > 32 && c != '\t' && c != '\n')
				m_filterString += static_cast<tchar>(wParam);
			Redraw(true);
		}
		case WM_VSCROLL:
		{
			float oldScrollY = m_scrollPosY;

			// HIWORD(wParam) only carries 16-bits, so use GetScrollInfo for larger scroll areas
			SCROLLINFO scrollInfo = {};
			scrollInfo.cbSize = sizeof(scrollInfo);
			scrollInfo.fMask = SIF_TRACKPOS;
			GetScrollInfo(m_hwnd, SB_VERT, &scrollInfo);

			switch (LOWORD(wParam))
			{
			case SB_THUMBTRACK:
			case SB_THUMBPOSITION:
				m_scrollPosY = -float(scrollInfo.nTrackPos);
				break;
			case SB_PAGEDOWN:
				m_scrollPosY = m_scrollPosY - float(m_clientHeight);
				break;
			case SB_PAGEUP:
				m_scrollPosY = m_scrollPosY + float(m_clientHeight);
				break;
			case SB_LINEDOWN:
				m_scrollPosY -= 30;
				break;
			case SB_LINEUP:
				m_scrollPosY += 30;
				break;
			}
			m_scrollPosY = Min(Max(m_scrollPosY, float(m_clientHeight - m_contentHeight)), 0.0f);

			if (oldScrollY != m_scrollPosY)
			{
				UpdateScrollbars(true);
				Redraw(false);
			}
			return 0;
		}
		case WM_HSCROLL:
		{
			float oldScrollX = m_scrollPosX;
			bool autoScroll = false;

			// HIWORD(wParam) only carries 16-bits, so use GetScrollInfo for larger scroll areas
			SCROLLINFO scrollInfo = {};
			scrollInfo.cbSize = sizeof(scrollInfo);
			scrollInfo.fMask = SIF_TRACKPOS;
			GetScrollInfo(m_hwnd, SB_HORZ, &scrollInfo);

			int right = m_clientWidth;

			switch (LOWORD(wParam))
			{
			case SB_THUMBTRACK:
				m_scrollPosX = -float(scrollInfo.nTrackPos);
				if (m_contentWidthWhenThumbTrack == 0)
					m_contentWidthWhenThumbTrack = m_contentWidth;
				break;
			case SB_THUMBPOSITION:
				autoScroll = m_contentWidthWhenThumbTrack - right <= HIWORD(wParam) + 10;
				m_contentWidthWhenThumbTrack = 0;
				m_scrollPosX = -float(scrollInfo.nTrackPos);
				break;
			case SB_PAGEDOWN:
				m_scrollPosX = m_scrollPosX - float(right);
				break;
			case SB_PAGEUP:
				m_scrollPosX = m_scrollPosX + float(right);
				break;
			case SB_LINEDOWN:
				m_scrollPosX -= 30;
				break;
			case SB_LINEUP:
				m_scrollPosX += 30;
				break;
			case SB_ENDSCROLL:
				return 0;
			}

			int minScroll = right - m_contentWidth;
			m_autoScroll = !m_traceView.finished && (m_scrollPosX <= float(minScroll) || autoScroll);
			m_scrollPosX = Min(Max(m_scrollPosX, float(right - m_contentWidth)), 0.0f);

			if (oldScrollX != m_scrollPosX)
			{
				UpdateScrollbars(true);
				Redraw(false);
			}
			return 0;
		}
		}
		return DefWindowProc(hWnd, Msg, wParam, lParam);
	}

	LRESULT CALLBACK VisualizerWin::StaticWinProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam)
	{
		VisualizerWin* thisPtr;
		if (Msg == WM_CREATE)
		{
			LPCREATESTRUCT pcs = (LPCREATESTRUCT)lParam;
			thisPtr = (VisualizerWin*)pcs->lpCreateParams;
			SetWindowLongPtr(hWnd, GWLP_USERDATA, (LONG_PTR)thisPtr);
			//EnableNonClientDpiScaling(hWnd);
		}
		else
			thisPtr = (VisualizerWin*)GetWindowLongPtr(hWnd, GWLP_USERDATA);

		if (thisPtr && hWnd == thisPtr->m_hwnd)
			return thisPtr->WinProc(hWnd, Msg, wParam, lParam);
		else
			return DefWindowProc(hWnd, Msg, wParam, lParam);
	}
}