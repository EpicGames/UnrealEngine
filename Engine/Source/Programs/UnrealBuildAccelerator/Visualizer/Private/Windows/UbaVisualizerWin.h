// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UbaNetworkClient.h"
#include "UbaTraceReader.h"
#include "UbaVisualizer.h"
#include <Ole2.h>

namespace uba
{
	////////////////////////////////////////////////////////////////////////////////////////////////////

	class VisualizerWin : public Visualizer, public IDropTarget
	{
	public:
		VisualizerWin(VisualizerConfig& config, Logger& logger);
		~VisualizerWin();

		bool ShowUsingListener(const wchar_t* channelName);
		bool ShowUsingNamedTrace(const wchar_t* namedTrace);
		bool ShowUsingSocket(NetworkBackend& backend, const wchar_t* host, u16 port = DefaultPort);
		bool ShowUsingFile(const wchar_t* fileName, u32 replay);

		bool HasWindow();
		HWND GetHwnd();
		void Lock(bool lock);

	//private:
		bool StartHwndThread();
		virtual void PaintEmptyScreen(PaintContext& context) override;
		virtual void FinishIncompatible() override;

		void CopyTextToClipboard(const TString& str);
		virtual bool UpdateSelection() override;
		virtual void UpdateScrollbars(bool redraw) override;
		void ThreadLoop();
		void Pause(bool pause);
		void StartDragToScroll(const POINT& anchor);
		void StopDragToScroll();
		void SaveSettings();

		void ChangeFontSize(int offset);
		virtual void Redraw(bool now) override;
		virtual void StopUpdate() override;
		void UpdateTheme();

		// IUnknown interface
		virtual HRESULT STDMETHODCALLTYPE QueryInterface(REFIID riid, void** ppvObject) override;
		virtual ULONG STDMETHODCALLTYPE AddRef(void) override;
		virtual ULONG STDMETHODCALLTYPE Release(void) override;
		// IDropTarget interface
		virtual HRESULT STDMETHODCALLTYPE DragEnter(__RPC__in_opt IDataObject* pDataObj, ::DWORD grfKeyState, POINTL pt, __RPC__inout::DWORD* pdwEffect) override;
		virtual HRESULT STDMETHODCALLTYPE DragOver(::DWORD grfKeyState, POINTL pt, __RPC__inout::DWORD* pdwEffect) override;
		virtual HRESULT STDMETHODCALLTYPE DragLeave(void) override;
		virtual HRESULT STDMETHODCALLTYPE Drop(__RPC__in_opt IDataObject* pDataObj, ::DWORD grfKeyState, POINTL pt, __RPC__inout::DWORD* pdwEffect) override;

		ULONG m_OLEReferenceCount = 0;

		u64 m_lastPaintTimeMs = 0;

		Atomic<bool> m_looping = false;
		HWND m_hwnd = 0;
		HWND m_parentHwnd = 0;

		int m_contentWidthWhenThumbTrack = 0;

		bool m_isInPaint = false;

		bool m_showPopup = false;
		bool m_locked = false;
		bool m_handlingTimer = false;
		bool m_settingsDirty = false;

		POINT m_mouseAnchor = {};
		float m_scrollAtAnchorX = 0;
		float m_scrollAtAnchorY = 0;
		int m_dragToScrollCounter = 0;

		bool m_horizontalScrollBarEnabled = true;
		bool m_verticalScrollBarEnabled = true;

		Thread m_thread;

		void PostNewTrace(u32 replay, bool paused);
		virtual void PostNewTitle(const StringView& title);
		void PostQuit();
		LRESULT WinProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam);
		static LRESULT CALLBACK StaticWinProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam);

		struct PaintContextWin;
		friend class VisualizerBackendGDI;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////
}
