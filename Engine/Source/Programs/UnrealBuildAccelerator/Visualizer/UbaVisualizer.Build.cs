// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaVisualizer : ModuleRules
{
	public UbaVisualizer(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivatePCHHeaderFile = "../Core/Public/UbaCorePch.h";

		bRequiresPlatformSDK = true;

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		StaticAnalyzerDisabledCheckers.Clear();

		PrivateDependencyModuleNames.AddRange(new string[] {
			"UbaCommon",
		});

		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PublicFrameworks.AddRange([
				"Cocoa",
				"AppKit"
			]);
		}
	}
}
