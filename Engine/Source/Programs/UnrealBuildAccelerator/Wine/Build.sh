#!/bin/sh
cd "$(dirname "$0")"

EXTRA_FLAGS="-O2 -DNDEBUG"

if [ -d "/usr/include/wine/wine/windows" ]; then
    EXTRA_FLAGS="$EXTRA_FLAGS -I/usr/include/wine/wine/windows"
fi

winegcc -shared -m64 -Wno-changes-meaning $EXTRA_FLAGS UbaWineWindows.cpp UbaWineLinux.cpp UbaWineWindows.def -o ../../../../Binaries/Win64/UnrealBuildAccelerator/x64/UbaWine.dll.so