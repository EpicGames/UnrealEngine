This folder contains code to create a custom .dll.so file that can be loaded in wine to be able to access native Linux api directly

To build the Wine dll.so file, just run Build.sh from wsl or linux machine

Note, you might need to run these first

sudo apt update
sudo apt install --yes mingw-w64
sudo apt install wine64-tools
