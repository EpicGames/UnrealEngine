// Copyright Epic Games, Inc. All Rights Reserved.


#include <netinet/tcp.h>
#include <netinet/in.h>
#include <dirent.h>
#include <errno.h>
#include <pthread.h>
#include <signal.h>
#include <stdarg.h>
#include <stdio.h>
#include <stdlib.h>
#include <string.h>
#include <sys/file.h>
#include <sys/mman.h>
#include <sys/stat.h>
#include <atomic>

//#include "../Common/Public/UbaLinuxNetworkWrappers.h"

int UnixGetsockopt(int fd, int level, int optname, char* optval, int* optlen)
{
	return getsockopt(fd, level, optname, optval, (socklen_t*)optlen);
}

int UnixSetsockopt(int fd, int level, int optname, char* optval, int optlen)
{
	return setsockopt(fd, level, optname, optval, (socklen_t)optlen);
}

int UnixGetTcpInfo(int fd, void* buf, int* len)
{
	socklen_t l = *len;
	if (getsockopt(fd, IPPROTO_TCP, TCP_INFO, buf, &l) == -1)
		return -errno;

	*len = (int)l;
	return 0;
}

int UnixErrnoToWSA(int e)
{
	switch (e)
	{
		case 0:               return 0;
		case EINTR:           return 10004;
		case EBADF:           return 10009;
		case EACCES:          return 10013;
		case EFAULT:          return 10014;
		case EINVAL:          return 10022;
		case EMFILE:          return 10024;
		case EAGAIN:          return 10035;
		case EINPROGRESS:     return 10036;
		case ENOTSOCK:        return 10038;
		case EMSGSIZE:        return 10040;
		case ENOPROTOOPT:     return 10042;
		case EPROTONOSUPPORT: return 10043;
		case EOPNOTSUPP:      return 10045;
		case EAFNOSUPPORT:    return 10047;
		case EADDRINUSE:      return 10048;
		case EADDRNOTAVAIL:   return 10049;
		case ENETDOWN:        return 10050;
		case ENETUNREACH:     return 10051;
		case ENETRESET:       return 10052;
		case ECONNABORTED:    return 10053;
		case ECONNRESET:      return 10054;
		case ENOBUFS:         return 10055;
		case EISCONN:         return 10056;
		case ENOTCONN:        return 10057;
		case ESHUTDOWN:       return 10058;
		case ETIMEDOUT:       return 10060;
		case ECONNREFUSED:    return 10061;
		case EHOSTDOWN:       return 10064;
		case EHOSTUNREACH:    return 10065;
		case EPIPE:           return 10054;
		default:              return e;
	}
}

int UnixKillTcpHogs(unsigned short port)
{
	char lsofCommand[1024];
	snprintf(lsofCommand, sizeof(lsofCommand), "lsof -i :%u -sTCP:LISTEN -Pn -t", (unsigned int)port);

	FILE* lsof = popen(lsofCommand, "r");
	if (!lsof)
	{
		printf("Failed run lsof while trying to kill processes holding port %u (not installed?)", (unsigned int)port);
		return -1;
	}
	char pidStr[16];
	while (fgets(pidStr, sizeof(pidStr), lsof))
	{
		pid_t pid = (pid_t)atoi(pidStr);
		if (pid <= 0)
			continue;
		if (kill(pid, SIGKILL) != 0)
		{
			printf("Failed to kill process %d while trying to kill processes holding port %u", pid, (unsigned int)port);
			return -1;
		}
		printf("Process %d killed successfully", pid);
	}
	pclose(lsof);
	return 0;
}

#define PLATFORM_WINDOWS 0
#define PLATFORM_MAC 0
#define PLATFORM_LINUX 1

#include <unistd.h>
#include <fcntl.h>
#include <poll.h>
#include <sys/uio.h>
#include <sys/syscall.h>  // SYS_openat, SYS_mmap, etc.
#include "../Core/Public/UbaBase.h"
#include "../Common/Public/UbaSocket.h"

#define UBA_ASSERT(...)
#define UBA_ASSERTF(...)
#define UBA_FM_STRING_TYPE wchar_t
#define UBA_USE_MEMFD 0 // Disabled because issues with distros/containers not having ptrace allowed

#include "../Core/Public/UbaFileHandle.h"
#include "../Core/Public/UbaFileMappingHandle.h"
#include "../Core/Public/UbaFileMapping.h"
#include "../Core/Public/UbaLinuxRawsysCall.h"

namespace uba
{
	class Logger { public: void Logf(...) {}
		void Info(const char* fmt, ...) { fputs("UbaWine - ", stderr); va_list a; va_start(a, fmt); vfprintf(stderr, fmt, a); va_end(a); fputc('\n', stderr); fflush(stderr); }
		bool Warning(...) { return false; } bool Error(...) { return false; } };
	template<typename Type> using Atomic = std::atomic<Type>;
	thread_local int t_lastError; // shm_open path (UBA_USE_MEMFD 0) captures mmap errno via Set/GetLastError
	void SetLastError(int err) { t_lastError = err; }
	int GetLastError() { return t_lastError; }
	enum LogEntryType : u8 { LogEntryType_Error = 0, LogEntryType_Warning = 1, };
	void FatalError(u32 code, const tchar* format, ...) { _exit(code); }

	#include "../Core/Public/UbaFileMapping.inl"

	static constexpr u32 KeepAliveProbeCount = 10;
	#include "../Common/Public/UbaSocket.inl"
}
