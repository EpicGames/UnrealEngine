// Copyright Epic Games, Inc. All Rights Reserved.

#define PLATFORM_WINDOWS 1
#define PLATFORM_MAC 0
#define PLATFORM_LINUX 0

#include <winsock2.h>
#include <stdint.h>
#include <unistd.h>
#include <stdbool.h>
#include <stdio.h>
#include "../Core/Public/UbaBase.h"
#include "../Common/Public/UbaSocket.h"
#include "../Core/Public/UbaFileMapping.h"

// NOTE, README
// Must use "WINAPI" on function and add Windows.def


extern "C" NTSYSAPI NTSTATUS CDECL wine_server_handle_to_fd( HANDLE handle, unsigned int access, int *unix_fd, unsigned int *options );

extern int UnixGetsockopt(int fd, int level, int optname, char* optval, int* optlen);
extern int UnixSetsockopt(int fd, int level, int optname, char* optval, int optlen);
extern int UnixGetTcpInfo(int fd, void* buf, int* len);
//extern bool UnixGetTcpAutoTuning(int* outReadMin, int* outReadDefault, int* outReadMax, int* outWriteMin, int* outWriteDefault, int* outWriteMax);
extern int UnixKillTcpHogs(unsigned short port);
extern int UnixErrnoToWSA(int e);

extern "C" INT WINAPI GetLinuxTcpInfo(SOCKET s, void* buf, INT* len)
{
	int fd = -1;
	if (wine_server_handle_to_fd((HANDLE)s, 0, &fd, NULL) != 0)
		return -1;
	int res = UnixGetTcpInfo(fd, buf, len);
	close(fd);
	return res;
}

int UNIX_IPPROTO_TCP = 6;
int TCP_CONGESTION = 13;

extern "C" bool WINAPI GetCongestionAlgorithm(SOCKET s, char* out, int outCapacity)
{
	int fd = -1;
	if (wine_server_handle_to_fd((HANDLE)s, 0, &fd, NULL) != 0)
		return -1;
	int res = UnixGetsockopt(fd, UNIX_IPPROTO_TCP, TCP_CONGESTION, out, &outCapacity);
	close(fd);
	return res == 0;
}

extern "C" bool WINAPI SetCongestionAlgorithm(SOCKET s, char* value, int valuelen)
{
	int fd = -1;
	if (wine_server_handle_to_fd((HANDLE)s, 0, &fd, NULL) != 0)
		return -1;
	int res = UnixSetsockopt(fd, UNIX_IPPROTO_TCP, TCP_CONGESTION, value, valuelen);
	close(fd);
	return res == 0;
}


extern "C" bool WINAPI GetTcpAutoTuning(int* outReadMin, int* outReadDefault, int* outReadMax, int* outWriteMin, int* outWriteDefault, int* outWriteMax)
{
	return false;//UnixGetTcpAutoTuning(outReadMin, outReadDefault, outReadMax, outWriteMin, outWriteDefault, outWriteMax);
}

extern "C" bool WINAPI KillTcpHogs(unsigned short port)
{
	return UnixKillTcpHogs(port) == 0;
}

extern "C" uba::Socket WINAPI SocketCreate(int family, int type, int protocol)
{
	return uba::SocketCreate(family, type, protocol);
}
extern "C" int WINAPI SocketConnect(uba::Socket s, const sockaddr* addr, unsigned int addrSize, bool isBlocking)
{
	return uba::SocketConnect(s, addr, addrSize, isBlocking);
}
extern "C" int WINAPI SocketClose(uba::Socket s)
{
	return uba::SocketClose(s);
}
extern "C" int WINAPI SocketSetBlocking(uba::Socket s, bool blocking)
{
	return uba::SocketSetBlocking(s, blocking);
}
extern "C" int WINAPI SocketReuseAddr(uba::Socket s)
{
	return uba::SocketReuseAddr(s);
}
extern "C" int WINAPI SocketBind(uba::Socket s, const sockaddr* addr, unsigned int addrSize)
{
	return uba::SocketBind(s, addr, addrSize);
}
extern "C" int WINAPI SocketListen(uba::Socket s)
{
	return uba::SocketListen(s);
}
extern "C" int WINAPI SocketSetKeepAlive(uba::Socket s, int keepAliveTime, int keepAliveTimeInterval)
{
	return uba::SocketSetKeepAlive(s, keepAliveTime, keepAliveTimeInterval);
}
extern "C" int WINAPI SocketSetUserTimeout(uba::Socket s, int timeoutMs)
{
	return uba::SocketSetUserTimeout(s, timeoutMs);
}
extern "C" int WINAPI SocketGetLastError()
{
	return UnixErrnoToWSA(uba::SocketGetLastError());
}
extern "C" int WINAPI SocketSetNoDelay(uba::Socket s)
{
	return uba::SocketSetNoDelay(s);
}
extern "C" int WINAPI SocketSetLinger(uba::Socket s, int lingerSeconds)
{
	return uba::SocketSetLinger(s, lingerSeconds);
}
extern "C" int WINAPI SocketSetTimeout(uba::Socket s, int timeoutMs)
{
	return uba::SocketSetTimeout(s, timeoutMs);
}
extern "C" int WINAPI SocketSetRecvBuf(uba::Socket s, int size)
{
	return uba::SocketSetRecvBuf(s, size);
}
extern "C" int WINAPI SocketSetSendBuf(uba::Socket s, int size)
{
	return uba::SocketSetSendBuf(s, size);
}
extern "C" int WINAPI SocketSetPriority(uba::Socket s)
{
	return uba::SocketSetPriority(s);
}
extern "C" int WINAPI SocketPoll(uba::Socket s, int events, int timeoutMs, int* outRevents)
{
	return uba::SocketPoll(s, events, timeoutMs, outRevents);
}
extern "C" uba::Socket WINAPI SocketAccept(uba::Socket s, sockaddr* outAddr, int addrLen)
{
	return uba::SocketAccept(s, outAddr, addrLen);
}
extern "C" int WINAPI SocketRecv(uba::Socket s, void* data, int len)
{
	return uba::SocketRecv(s, data, len);
}
extern "C" int WINAPI SocketRecv2(uba::Socket s, uba::RecvBuf* bufs, int bufCount)
{
	return uba::SocketRecv2(s, bufs, bufCount);
}
extern "C" int WINAPI SocketSend(uba::Socket s, uba::SendBuf* bufs, int bufCount)
{
	return uba::SocketSend(s, bufs, bufCount);
}
extern "C" bool WINAPI SocketShouldPoll(bool* outRetry)
{
	return uba::SocketShouldPoll(outRetry);
}
extern "C" int WINAPI SocketCheckConnect(uba::Socket s, bool* outTimedOut)
{
	return uba::SocketCheckConnect(s, outTimedOut);
}
extern "C" int WINAPI SocketShutdown(uba::Socket s)
{
	return uba::SocketShutdown(s);
}
extern "C" int WINAPI SocketGetCongestionAlgorithm(uba::Socket s, char* out, int outCapacity)
{
	return uba::SocketGetCongestionAlgorithm(s, out, outCapacity);
}
extern "C" int WINAPI SocketGetRecvBuf(uba::Socket s, int& outSize)
{
	return uba::SocketGetRecvBuf(s, outSize);
}
extern "C" int WINAPI SocketGetSendBuf(uba::Socket s, int& outSize)
{
	return uba::SocketGetSendBuf(s, outSize);
}
extern "C" int WINAPI SocketPoll2(uba::Socket s, int events, int timeoutMs, uba::Socket wakeupSocket, int* outRevents)
{
	return uba::SocketPoll2(s, events, timeoutMs, wakeupSocket, outRevents);
}
extern "C" uba::Socket WINAPI SocketCreateWakeupSocket()
{
	return uba::SocketCreateWakeupSocket();
}
extern "C" int WINAPI SocketWakeup(uba::Socket wakeupSocket)
{
	return uba::SocketWakeup(wakeupSocket);
}

using FileMappingHandle = uba::FileMappingHandle;
using tchar = uba::tchar;
using u8 = uba::u8;
using MemoryMapType = uba::MemoryMapType;
using u32 = uba::u32;
using u64 = uba::u64;
using FileHandle = uba::FileHandle;

extern "C" FileMappingHandle WINAPI FileMapping_Create(uba::Logger& logger, u32 flProtect, u64 maxSize, const tchar* name, const tchar* hint)
{
	return uba::FileMapping_Create(logger, flProtect, maxSize, name, hint);
}
extern "C" FileMappingHandle WINAPI FileMapping_CreateFromFile(uba::Logger& logger, FileHandle file, u32 flProtect, u64 maxSize, const tchar* hint)
{
	return uba::FileMapping_CreateFromFile(logger, file, flProtect, maxSize, hint);
}
extern "C" u8* WINAPI FileMapping_Map(uba::Logger& logger, FileMappingHandle fileMappingObject, u32 dwDesiredAccess, u64 offset, u64 dwNumberOfBytesToMap, bool allowDiscard)
{
	return uba::FileMapping_Map(logger, fileMappingObject, dwDesiredAccess, offset, dwNumberOfBytesToMap, allowDiscard);
}
extern "C" bool WINAPI FileMapping_Commit(uba::Logger& logger, void* address, u64 size, bool allowDiscard)
{
	return uba::FileMapping_Commit(logger, address, size, allowDiscard);
}
extern "C" bool WINAPI FileMapping_Unmap(uba::Logger& logger, const void* lpBaseAddress, u64 bytesToUnmap, const tchar* hint, bool allowDiscard)
{
	return uba::FileMapping_Unmap(logger, lpBaseAddress, bytesToUnmap, hint, allowDiscard);
}
extern "C" bool WINAPI FileMapping_Close(uba::Logger& logger, FileMappingHandle h, const tchar* hint)
{
	return uba::FileMapping_Close(logger, h, hint);
}
extern "C" void* WINAPI FileMapping_ReservePlaceholder(void* baseAddress, u64 capacity)
{
	return uba::FileMapping_ReservePlaceholder(baseAddress, capacity);
}
extern "C" void* WINAPI FileMapping_MapPlaceholder(FileMappingHandle handle, MemoryMapType mapType, void* targetAddress, u64 targetOffset, u64 targetCapacity, u64 handleOffset, u64 size)
{
	return uba::FileMapping_MapPlaceholder(handle, mapType, targetAddress, targetOffset, targetCapacity, handleOffset, size);
}
extern "C" bool WINAPI FileMapping_UnmapPlaceholder(void* memory, u64 capacity, u64 mappedSize, u64* subMappings, u64 subMappingsCount)
{
	return uba::FileMapping_UnmapPlaceholder(memory, capacity, mappedSize, subMappings, subMappingsCount);
}
extern "C" FileMappingHandle WINAPI FileMapping_DuplicateFromHost(FileMappingHandle handle, const char* hint)
{
	return uba::FileMapping_DuplicateFromHost(handle, hint);
}
extern "C" u8* WINAPI FileMapping_MapFromHost(FileMappingHandle handle, u64 size, u64 offset, bool writable, const char* hint)
{
	return uba::FileMapping_MapFromHost(handle, size, offset, writable, hint);
}
extern "C" u8* WINAPI FileMapping_MapFromHost2(FileMappingHandle handle, u64 size, u64 offset, MemoryMapType mapType, void* targetAddress, const char* hint)
{
	return uba::FileMapping_MapFromHost2(handle, size, offset, mapType, targetAddress, hint);
}

extern "C" char* CDECL wine_get_unix_file_name(const wchar_t* dosW); // wine kernelbase
namespace uba { bool FileMapping_ResolveBackingUnix(uba::Logger&, const char* preferredDirUnix, u64 shmFloorBytes, char* outEnv, int outEnvCap); }

extern "C" bool WINAPI FileMapping_ResolveBacking(uba::Logger& logger, const tchar* preferredDir, u64 shmFloorBytes, char* outEnv, int outEnvCap)
{
	char* unixDir = preferredDir ? wine_get_unix_file_name(preferredDir) : nullptr; // HeapAlloc'd, leaked once per session
	return uba::FileMapping_ResolveBackingUnix(logger, unixDir, shmFloorBytes, outEnv, outEnvCap);
}
