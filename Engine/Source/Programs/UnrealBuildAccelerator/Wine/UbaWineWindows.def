LIBRARY libtcpinfo.dll
EXPORTS
	GetLinuxTcpInfo
	GetCongestionAlgorithm
	SetCongestionAlgorithm
	GetTcpAutoTuning
	KillTcpHogs
	SocketCreate
	SocketConnect
	SocketClose
	SocketSetBlocking
	SocketReuseAddr
	SocketBind
	SocketListen
	SocketSetKeepAlive
	SocketSetUserTimeout
	SocketGetLastError
	SocketPoll
	SocketAccept
	SocketSetNoDelay
	SocketSetLinger
	SocketSetTimeout
	SocketSetRecvBuf
	SocketSetSendBuf
	SocketSetPriority
	SocketRecv
	SocketRecv2
	SocketSend
	SocketCheckConnect
	SocketShouldPoll
	SocketShutdown
	SocketGetCongestionAlgorithm
	SocketGetRecvBuf
	SocketGetSendBuf
	SocketPoll2
	SocketCreateWakeupSocket
	SocketWakeup
	FileMapping_Create
	FileMapping_CreateFromFile
	FileMapping_Map
	FileMapping_Commit
	FileMapping_Unmap
	FileMapping_Close
	FileMapping_ReservePlaceholder
	FileMapping_MapPlaceholder
	FileMapping_UnmapPlaceholder
	FileMapping_DuplicateFromHost
	FileMapping_MapFromHost
	FileMapping_MapFromHost2
	FileMapping_ResolveBacking