// Hint files help the Visual Studio IDE interpret Visual C++ identifiers
// such as names of functions and macros.
// For more information see https://go.microsoft.com/fwlink/?linkid=865984
#define __analysis_noreturn __declspec(noreturn)
#define API
