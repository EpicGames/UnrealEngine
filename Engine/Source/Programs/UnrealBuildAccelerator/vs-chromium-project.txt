[SourceExplorer.ignore]
.git/
.vs/
.vscode/
*.sdf
*.opensdf
*.suo
*.csproj
*.vcxproj
*.udd
*.uasset
**/Cooked/
**/DerivedDataCache/0
**/DerivedDataCache/1
**/DerivedDataCache/2
**/DerivedDataCache/3
**/DerivedDataCache/4
**/DerivedDataCache/5
**/DerivedDataCache/6
**/DerivedDataCache/7
**/DerivedDataCache/8
**/DerivedDataCache/9
**/Intermediate/
**/Saved/
**/Staged/
**/Content/
enc_temp_folder/
LocalBuilds/

[SearchableFiles.include]
*.bat
*.c
*.cc
*.cpp
*.cs
*.cxx
*.h
*.hh
*.hpp
*.ini
*.inl
*.ispc
*.isph
*.txt
*.uplugin
*.uproject
*.usf
*.ush
*.json
*.xml

[SearchableFiles.ignore]
.git/
.vs/
.vscode/
Engine/Programs/UnrealBuildTool/Log*
*.bak
*.cpp.txt
*.gen.cpp
*.gen.h
*.udd
*.uasset
**/Binaries/
**/Cooked/
**/obj/
**/bin/
**/DerivedDataCache/0
**/DerivedDataCache/1
**/DerivedDataCache/2
**/DerivedDataCache/3
**/DerivedDataCache/4
**/DerivedDataCache/5
**/DerivedDataCache/6
**/DerivedDataCache/7
**/DerivedDataCache/8
**/DerivedDataCache/9
**/Intermediate/
**/Logs/
**/Private_Projects/
**/Saved/
**/Staged/
**/ThirdParty/**/*.json
**/Content/
enc_temp_folder/
LocalBuilds/